{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Geometry Processing",
	"Description": "Data Structures and Algorithms for Processing 2D and 3D Geometry",
	"Category": "Geometry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "GeometryAlgorithms",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DynamicMesh",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshFileUtils",
			"Type": "DeveloperTool",
			"LoadingPhase": "Default"
		}
	],
	"SupportedPrograms": ["ChaosVisualDebugger"]
}