// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DynamicMesh : ModuleRules
{	
	public DynamicMesh(ReadOnlyTargetRules Target) : base(Target)
	{
		NumIncludedBytesPerUnityCPPOverride = 589824; // best unity size found from using UBT ProfileUnitySizes mode

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"GeometryCore",
				"GeometryAlgorithms",
			}
		);

		// Note: The module purposefully doesn't have a dependency on CoreUObject.
		// If possible, we would like avoid having UObjects in GeometryProcessing
		// modules to keep the door open for writing standalone command-line programs
		// (which won't have UObject garbage collection).
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Eigen",
				"MeshConversion"
			}
		);
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
