// Copyright Epic Games, Inc. All Rights Reserved.

#include "FaceGroupUtil.h"
#include "Selections/MeshConnectedComponents.h"
#include "Async/ParallelFor.h"
#include "Polygroups/GroupSetAdapter.h"

using namespace UE::Geometry;

void FaceGroupUtil::SetGroupID(FDynamicMesh3& Mesh, int32 to)
{
	if (Mesh.HasTriangleGroups() == false)
	{
		return;
	}
	for (int32 tid : Mesh.TriangleIndicesItr())
	{
		Mesh.SetTriangleGroup(tid, to);
	}
}


void FaceGroupUtil::SetGroupID(FDynamicMesh3& Mesh, const TArrayView<const int>& triangles, int32 to)
{
	if (Mesh.HasTriangleGroups() == false)
	{
		return;
	}
	for (int32 tid : triangles)
	{
		Mesh.SetTriangleGroup(tid, to);
	}
}


void FaceGroupUtil::SetGroupToGroup(FDynamicMesh3& Mesh, int32 from, int32 to)
{
	if (Mesh.HasTriangleGroups() == false)
	{
		return;
	}

	int32 NT = Mesh.MaxTriangleID();
	for (int32 tid = 0; tid < NT; ++tid)
	{
		if (Mesh.IsTriangle(tid))
		{
			int32 gid = Mesh.GetTriangleGroup(tid);
			if (gid == from)
			{
				Mesh.SetTriangleGroup(tid, to);
			}
		}
	}
}


bool FaceGroupUtil::HasMultipleGroups(const FDynamicMesh3& Mesh)
{
	if (Mesh.HasTriangleGroups())
	{
		int32 CurGroupID = -1;
		int32 GroupsCounter = 0;
		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			int32 GroupID = Mesh.GetTriangleGroup(tid);
			if (GroupID != CurGroupID)
			{
				CurGroupID = GroupID;
				if (GroupsCounter++ > 1)
				{
					return true;
				}
			}
		}
	}
	return false;
}


void FaceGroupUtil::FindAllGroups(const FDynamicMesh3& Mesh, TSet<int32>& GroupsOut)
{
	if (Mesh.HasTriangleGroups())
	{
		int32 NT = Mesh.MaxTriangleID();
		for (int32 tid = 0; tid < NT; ++tid)
		{
			if (Mesh.IsTriangle(tid))
			{
				int32 gid = Mesh.GetTriangleGroup(tid);
				GroupsOut.Add(gid);
			}
		}
	}
}


void FaceGroupUtil::CountAllGroups(const FDynamicMesh3& Mesh, TArray<int32>& GroupCountsOut)
{
	GroupCountsOut.SetNum(Mesh.MaxGroupID());

	if (Mesh.HasTriangleGroups())
	{
		int32 NT = Mesh.MaxTriangleID();
		for (int32 tid = 0; tid < NT; ++tid)
		{
			if (Mesh.IsTriangle(tid))
			{
				int32 gid = Mesh.GetTriangleGroup(tid);
				GroupCountsOut[gid]++;
			}
		}
	}
}


void FaceGroupUtil::FindTriangleSetsByGroup(const FDynamicMesh3& Mesh, TArray<TArray<int32>>& GroupTrisOut, int32 IgnoreGID)
{
	if (!Mesh.HasTriangleGroups())
	{
		return;
	}

	// find # of groups and triangle count for each
	TArray<int32> Counts;
	CountAllGroups(Mesh, Counts);
	TArray<int32> GroupIDs;
	for (int32 CountIdx = 0; CountIdx < Counts.Num(); CountIdx++) 
	{
		int32 Count = Counts[CountIdx];
		if (CountIdx != IgnoreGID && Count > 0)
		{
			GroupIDs.Add(CountIdx);
		}
	}
	TArray<int32> GroupMap; 
	GroupMap.SetNum(Mesh.MaxGroupID());

	// allocate sets
	GroupTrisOut.SetNum(GroupIDs.Num());
	for (int32 i = 0; i < GroupIDs.Num(); ++i)
	{
		int32 GID = GroupIDs[i];
		GroupTrisOut[i].Reserve(Counts[GID]);
		GroupMap[GID] = i;
	}

	// accumulate triangles
	int32 NT = Mesh.MaxTriangleID();
	for (int32 tid = 0; tid < NT; ++tid)
	{
		if (Mesh.IsTriangle(tid))
		{
			int32 GID = Mesh.GetTriangleGroup(tid);
			int32 i = GroupMap[GID];
			if (i >= 0)
			{
				GroupTrisOut[i].Add(tid);
			}
		}
	}
}



bool FaceGroupUtil::FindTrianglesByGroup(FDynamicMesh3& Mesh, int32 FindGroupID, TArray<int32>& TrianglesOut)
{
	int32 NumAdded = 0;
	TArray<int32> tris;
	if (Mesh.HasTriangleGroups() == false)
	{
		return false;
	}
	for (int32 tid : Mesh.TriangleIndicesItr())
	{
		if (Mesh.GetTriangleGroup(tid) == FindGroupID)
		{
			TrianglesOut.Add(tid);
			NumAdded++;
		}
	}
	return (NumAdded > 0);
}


void FaceGroupUtil::SeparateMeshByGroups(FDynamicMesh3& Mesh, TArray<FDynamicMesh3>& SplitMeshes)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDynamicMeshEditor::SplitMesh(&Mesh, SplitMeshes, [&Mesh](int32 TID)
	{
		return Mesh.GetTriangleGroup(TID);
	});
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FaceGroupUtil::SeparateMeshByGroups(FDynamicMesh3& Mesh, TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes)
{
	FDynamicMeshEditor::SplitMesh(&Mesh, SplitMeshes, true, [&Mesh](int32 TID)
	{
		return Mesh.GetTriangleGroup(TID);
	});
}

void FaceGroupUtil::SeparateMeshByGroups(FDynamicMesh3& Mesh, TArray<FDynamicMesh3>& SplitMeshes, TArray<int32>& GroupIDs)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// build split meshes
	SeparateMeshByGroups(Mesh, SplitMeshes);

	// build array of per-mesh group id
	GroupIDs.Reset();
	for (const FDynamicMesh3& M : SplitMeshes)
	{
		check(M.TriangleCount() > 0); // SplitMesh should never add an empty mesh
		GroupIDs.Add(M.GetTriangleGroup(0));
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FaceGroupUtil::SeparateMeshByGroups(FDynamicMesh3& Mesh, TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes, TArray<int32>& GroupIDs)
{
	FDynamicMeshEditor::SplitMesh(&Mesh, SplitMeshes, true, [&Mesh](int32 TID)
	{
		return Mesh.GetTriangleGroup(TID);
	}, INDEX_NONE, &GroupIDs);
}



void FGroupVisualizationCache::UpdateGroupInfo_ConnectedComponents(
	const FDynamicMesh3& SourceMesh,
	const FPolygroupSet& GroupSet,
	bool bParallel)
{
	const FPolygroupLayerAdapter Adapter(GroupSet.Mesh, GroupSet.PolygroupAttrib);
	UpdateGroupInfo_ConnectedComponents(SourceMesh, Adapter, bParallel);
}

void FGroupVisualizationCache::UpdateGroupInfo_ConnectedComponents(
	const FDynamicMesh3& SourceMesh,
	const ITriangleGroupSetAdapter& Adapter,
	bool bParallel)
{
	// find connected group components
	FMeshConnectedComponents Components(&SourceMesh);
	Components.FindConnectedTriangles([&](int32 t1, int32 t2) { return Adapter.GetGroup(t1) == Adapter.GetGroup(t2); });

	GroupInfo.SetNum(Components.Num());

	EParallelForFlags UseParallelFlags = (bParallel) ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;
	ParallelFor(Components.Num(), [&](int32 ci) {
		const FMeshConnectedComponents::FComponent& Component = Components[ci];
		GroupInfo[ci].GroupID = Adapter.GetGroup(Component.Indices[0]);

		// compute bounds
		GroupInfo[ci].Bounds = FAxisAlignedBox3d::Empty();
		for (int32 tid : Component.Indices)
		{
			GroupInfo[ci].Bounds.Contain(SourceMesh.GetTriBounds(tid));
		}

		if (Component.Indices.Num() == 1)
		{
			GroupInfo[ci].Center = SourceMesh.GetTriCentroid(Component.Indices[0]);
			GroupInfo[ci].CenterTris = FIndex2i(Component.Indices[0], Component.Indices[0]);
		}
		else if (Component.Indices.Num() == 2 &&
			SourceMesh.GetTriNeighbourTris(Component.Indices[0]).Contains(Component.Indices[1]))
		{
			int32 eid = SourceMesh.FindEdgeFromTriPair(Component.Indices[0], Component.Indices[1]);
			if (eid != IndexConstants::InvalidID)
			{
				GroupInfo[ci].Center = SourceMesh.GetEdgePoint(eid, 0.5);
				GroupInfo[ci].CenterTris = SourceMesh.GetEdgeT(eid);
			}
			else
			{
				GroupInfo[ci].Center = SourceMesh.GetTriCentroid(Component.Indices[0]);
				GroupInfo[ci].CenterTris = FIndex2i(Component.Indices[0], Component.Indices[0]);
			}
		}
		else
		{
			int32 CenterTriID = -1;
			TSet<int32> TrisInGroup(Component.Indices);
			TArray<int32> BorderTris;
			while (CenterTriID == -1)
			{
				BorderTris.Reset();
				for (int32 tid : TrisInGroup)
				{
					FIndex3i NbrTris = SourceMesh.GetTriNeighbourTris(tid);
					bool bIsBorder = (TrisInGroup.Contains(NbrTris.A) == false || TrisInGroup.Contains(NbrTris.B) == false || TrisInGroup.Contains(NbrTris.C) == false);
					if (bIsBorder)
					{
						BorderTris.Add(tid);
					}
				}
				if (BorderTris.IsEmpty())
				{
					CenterTriID = *TrisInGroup.begin();
					break;
				}
				for (int32 tid : BorderTris)
				{
					TrisInGroup.Remove(tid);
					if (TrisInGroup.Num() == 1)
					{
						CenterTriID = *TrisInGroup.begin();
						break;
					}
				}
			}
			GroupInfo[ci].Center = SourceMesh.GetTriCentroid(CenterTriID);
			GroupInfo[ci].CenterTris = FIndex2i(CenterTriID, CenterTriID);
		}

		if (bStorePerGroupTriangleIDs)
		{
			GroupInfo[ci].TriangleIDs = MoveTemp(Components[ci].Indices);
		}

	}, UseParallelFlags);
}