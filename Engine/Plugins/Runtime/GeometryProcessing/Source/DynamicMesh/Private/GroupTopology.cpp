// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroupTopology.h"

#include "Algo/MinElement.h"
#include "MeshRegionBoundaryLoops.h"

using namespace UE::Geometry;



bool FGroupTopology::FGroupEdge::IsConnectedToVertices(const TSet<int>& Vertices) const
{
	for (int VertexID : Span.Vertices)
	{
		if (Vertices.Contains(VertexID))
		{
			return true;
		}
	}
	return false;
}



FGroupTopology::FGroupTopology(const FDynamicMesh3* MeshIn, bool bAutoBuild)
{
	this->Mesh = MeshIn;
	this->GroupLayer = nullptr;
	if (bAutoBuild)
	{
		RebuildTopology();
	}
}


FGroupTopology::FGroupTopology(const FDynamicMesh3* MeshIn, const FDynamicMeshPolygroupAttribute* GroupLayerIn, bool bAutoBuild)
{
	this->Mesh = MeshIn;
	this->GroupLayer = GroupLayerIn;
	if (bAutoBuild)
	{
		RebuildTopology();
	}
}


bool FGroupTopology::RebuildTopology()
{
	Groups.Reset();
	Edges.Reset();
	Corners.Reset();

	CurrentExtraCornerVids.Reset();

	int32 MaxGroupID = 0;
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		MaxGroupID = FMath::Max(GetGroupID(tid), MaxGroupID);
	}
	MaxGroupID++;

	// initialize groups map first to avoid resizes
	GroupIDToGroupIndexMap.Reset();
	GroupIDToGroupIndexMap.Init(-1, MaxGroupID);
	TArray<int> GroupFaceCounts;
	GroupFaceCounts.Init(0, MaxGroupID);
	for (int tid : Mesh->TriangleIndicesItr())
	{
		int GroupID = GetGroupID(tid);
		if (GroupIDToGroupIndexMap[GroupID] == -1)
		{
			FGroup NewGroup;
			NewGroup.GroupID = GroupID;
			GroupIDToGroupIndexMap[GroupID] = Groups.Add(NewGroup);
		}
		GroupFaceCounts[GroupID]++;
	}
	for (FGroup& Group : Groups)
	{
		Group.Triangles.Reserve(GroupFaceCounts[Group.GroupID]);
	}


	// sort faces into groups
	for (int tid : Mesh->TriangleIndicesItr())
	{
		int GroupID = GetGroupID(tid);
		Groups[GroupIDToGroupIndexMap[GroupID]].Triangles.Add(tid);
	}

	VertexIDToCornerIDMap.Reset();
	TMap<int32, int32> GroupEdgeMinEidToGroupEdgeID;
	TBitArray<> VertCheckedForCorner;
	VertCheckedForCorner.Init(false, Mesh->MaxVertexID());

	// construct boundary loops
	for (FGroup& Group : Groups)
	{
		bool bOK = GenerateBoundaryAndGroupEdges(Group, GroupEdgeMinEidToGroupEdgeID, VertCheckedForCorner);
		if (!bOK)
		{
			return false;
		}

		// collect up .NeighbourGroupIDs and set .bIsOnBoundary
		for (FGroupBoundary& Boundary : Group.Boundaries)
		{
			Boundary.bIsOnBoundary = false;
			for (int EdgeIndex : Boundary.GroupEdges)
			{
				FGroupEdge& Edge = Edges[EdgeIndex];

				int OtherGroupID = (Edge.Groups.A == Group.GroupID) ? Edge.Groups.B : Edge.Groups.A;
				if (OtherGroupID != FDynamicMesh3::InvalidID)
				{
					Boundary.NeighbourGroupIDs.AddUnique(OtherGroupID);
				}
				else
				{
					Boundary.bIsOnBoundary = true;
				}
			}
		}

		// make all-neighbour-groups list at group level
		for (FGroupBoundary& Boundary : Group.Boundaries)
		{
			for (int NbrGroupID : Boundary.NeighbourGroupIDs)
			{
				Group.NeighbourGroupIDs.AddUnique(NbrGroupID);
			}
		}
	}

	return true;
}


void FGroupTopology::RetargetOnClonedMesh(const FDynamicMesh3* NewMesh)
{
	Mesh = NewMesh;
	for (FGroupEdge& Edge : Edges)
	{
		Edge.Span.Mesh = NewMesh;
	}
}

bool FGroupTopology::IsCornerVertex(int VertexID) const
{ 
	return ShouldVertBeCorner(VertexID);
}

bool FGroupTopology::ShouldVertBeCorner(int VertexID, bool* bIsAnExtraCornerOut) const
{
	if (bIsAnExtraCornerOut)
	{
		*bIsAnExtraCornerOut = false;
	}

	// If the vertex has at least three attached groupedge edges (ie three edges that separate two groups
	// or are boundary edges), then it has to be a corner. Otherwise, we let the user supply a function that
	// determines whether the vert should be a corner anyway, passing in the group edges that do pass through the
	// vertex.

	// only ends up being used if there are less than 3, so that we can pass in the
	// relevant edges to the user function.
	FIndex2i AttachedGroupEdgeEids(IndexConstants::InvalidID, IndexConstants::InvalidID);

	for (int32 Eid : Mesh->VtxEdgesItr(VertexID))
	{
		FIndex2i EdgeTris = Mesh->GetEdgeT(Eid);
		if (EdgeTris.B == IndexConstants::InvalidID
			|| GetGroupID(EdgeTris.A) != GetGroupID(EdgeTris.B))
		{
			// This is a group edge.
			if (AttachedGroupEdgeEids[0] == IndexConstants::InvalidID)
			{
				AttachedGroupEdgeEids[0] = Eid;
			}
			else if (AttachedGroupEdgeEids[1] == IndexConstants::InvalidID)
			{
				AttachedGroupEdgeEids[1] = Eid;
			}
			else
			{
				// This is the third such edge. Has to be corner.
				return true;
			}
		}//end if part of group edge
	}//end looking through attached edges.

	// If we got to here, then the vert isn't a corner just based on group topology, but can be made
	// into a corner anyway by user function (for instance, at sharp corners, or at user-defined points).
	if (ShouldAddExtraCornerAtVert && ShouldAddExtraCornerAtVert(*this, VertexID, AttachedGroupEdgeEids))
	{
		if (bIsAnExtraCornerOut)
		{
			*bIsAnExtraCornerOut = true;
		}
		return true;
	}
	return false;
}




int FGroupTopology::GetCornerVertexID(int CornerID) const
{
	check(CornerID >= 0 && CornerID < Corners.Num());
	return Corners[CornerID].VertexID;
}

int32 FGroupTopology::GetCornerIDFromVertexID(int32 VertexID) const
{
	check(Mesh->IsVertex(VertexID));
	const int32* Found = VertexIDToCornerIDMap.Find(VertexID);
	return (Found == nullptr) ? IndexConstants::InvalidID : *Found;
}


const FGroupTopology::FGroup* FGroupTopology::FindGroupByID(int GroupID) const
{
	if (GroupID < 0 || GroupID >= GroupIDToGroupIndexMap.Num() || GroupIDToGroupIndexMap[GroupID] == -1)
	{
		return nullptr;
	}
	return &Groups[GroupIDToGroupIndexMap[GroupID]];
}


const TArray<int>& FGroupTopology::GetGroupTriangles(int GroupID) const
{
	const FGroup* Found = FindGroupByID(GroupID);
	ensure(Found != nullptr);
	return (Found != nullptr) ? Found->Triangles : EmptyArray;
}

const TArray<int>& FGroupTopology::GetGroupNbrGroups(int GroupID) const
{
	const FGroup* Found = FindGroupByID(GroupID);
	ensure(Found != nullptr);
	return (Found != nullptr) ? Found->NeighbourGroupIDs : EmptyArray;
}


bool FGroupTopology::IsGroupEdge(FMeshTriEdgeID TriEdgeID, bool bIncludeMeshBoundary) const
{
	int32 EdgeID = Mesh->GetTriEdge((int32)TriEdgeID.TriangleID, (int32)TriEdgeID.TriEdgeIndex);
	if (EdgeID != IndexConstants::InvalidID)
	{
		FIndex2i EdgeT = Mesh->GetEdgeT(EdgeID);
		if (EdgeT.B == IndexConstants::InvalidID)
		{
			return bIncludeMeshBoundary;
		}
		int32 GroupA = GetGroupID(EdgeT.A);
		int32 GroupB = GetGroupID(EdgeT.B);
		return GroupA != GroupB;
	}
	return false;
}


int FGroupTopology::FindGroupEdgeID(int MeshEdgeID) const
{
	int GroupID = GetGroupID(Mesh->GetEdgeT(MeshEdgeID).A);
	const FGroup* Group = FindGroupByID(GroupID);
	ensure(Group != nullptr);
	if (Group != nullptr)
	{
		for (const FGroupBoundary& Boundary : Group->Boundaries)
		{
			for (int EdgeID : Boundary.GroupEdges)
			{
				const FGroupEdge& Edge = Edges[EdgeID];
				if (Edge.Span.Edges.Contains(MeshEdgeID))
				{
					return EdgeID;
				}
			}
		}
	}
	return -1;
}

int FGroupTopology::FindGroupEdgeID(FMeshTriEdgeID TriEdgeID) const
{
	int32 EdgeID = Mesh->GetTriEdge((int32)TriEdgeID.TriangleID, (int32)TriEdgeID.TriEdgeIndex);
	return (EdgeID != IndexConstants::InvalidID) ? FindGroupEdgeID(EdgeID) : -1;
}

const TArray<int>& FGroupTopology::GetGroupEdgeVertices(int GroupEdgeID) const
{
	check(GroupEdgeID >= 0 && GroupEdgeID < Edges.Num());
	return Edges[GroupEdgeID].Span.Vertices;
}

const TArray<int>& FGroupTopology::GetGroupEdgeEdges(int GroupEdgeID) const
{
	check(GroupEdgeID >= 0 && GroupEdgeID < Edges.Num());
	return Edges[GroupEdgeID].Span.Edges;
}

bool FGroupTopology::IsSimpleGroupEdge(int32 GroupEdgeID) const
{
	check(GroupEdgeID >= 0 && GroupEdgeID < Edges.Num());
	return Edges[GroupEdgeID].Span.Edges.Num() == 1;
}

void FGroupTopology::FindEdgeNbrGroups(int GroupEdgeID, TArray<int>& GroupsOut) const
{
	check(GroupEdgeID >= 0 && GroupEdgeID < Edges.Num());
	const TArray<int> & Vertices = GetGroupEdgeVertices(GroupEdgeID);
	FindVertexNbrGroups(Vertices[0], GroupsOut);
	FindVertexNbrGroups(Vertices[Vertices.Num() - 1], GroupsOut);
}

void FGroupTopology::FindEdgeNbrGroups(const TArray<int>& GroupEdgeIDs, TArray<int>& GroupsOut) const
{
	for (int GroupEdgeID : GroupEdgeIDs)
	{
		FindEdgeNbrGroups(GroupEdgeID, GroupsOut);
	}
}

void FGroupTopology::FindEdgeNbrEdges(int GroupEdgeID, TArray<int>& EdgesOut) const
{
	check(GroupEdgeID >= 0 && GroupEdgeID < Edges.Num());
	const FGroupEdge& Edge = Edges[GroupEdgeID];
	if ( Edge.EndpointCorners.A != IndexConstants::InvalidID )
	{
		FindCornerNbrEdges(Edge.EndpointCorners.A, EdgesOut);
	}
	if ( Edge.EndpointCorners.B != IndexConstants::InvalidID )
	{
		FindCornerNbrEdges(Edge.EndpointCorners.B, EdgesOut);
	}
}



bool FGroupTopology::IsBoundaryEdge(int32 GroupEdgeID) const
{
	return Mesh->IsBoundaryEdge(Edges[GroupEdgeID].Span.Edges[0]);
}

bool FGroupTopology::IsIsolatedLoop(int GroupEdgeID) const
{
	const FGroupEdge& Edge = Edges[GroupEdgeID];
	return Edge.EndpointCorners[0] == IndexConstants::InvalidID;
}



double FGroupTopology::GetEdgeArcLength(int32 GroupEdgeID, TArray<double>* PerVertexLengthsOut) const
{
	check(GroupEdgeID >= 0 && GroupEdgeID < Edges.Num());
	const TArray<int32>& Vertices = GetGroupEdgeVertices(GroupEdgeID);
	int32 NumV = Vertices.Num();
	if (PerVertexLengthsOut != nullptr)
	{
		PerVertexLengthsOut->SetNum(NumV);
		(*PerVertexLengthsOut)[0] = 0.0;
	}
	double AccumLength = 0;
	for (int32 k = 1; k < NumV; ++k)
	{
		AccumLength += Distance(Mesh->GetVertex(Vertices[k]), Mesh->GetVertex(Vertices[k-1]));
		if (PerVertexLengthsOut != nullptr)
		{
			(*PerVertexLengthsOut)[k] = AccumLength;
		}
	}
	return AccumLength;
}


FVector3d FGroupTopology::GetEdgeAveragePosition(int32 GroupEdgeID) const
{
	const TArray<int32>& Vertices = GetGroupEdgeVertices(GroupEdgeID);

	double WtSum = 0;
	FVector3d Sum = FVector3d::ZeroVector;
	FVector LastV = Mesh->GetVertex(Vertices[0]);
	for (int32 Idx = 0; Idx + 1 < Vertices.Num(); ++Idx)
	{
		FVector3d A = LastV;
		FVector3d B = Mesh->GetVertex(Vertices[Idx + 1]);
		// weight edge centers by edge lengths
		double Wt = FVector3d::Distance(A, B);
		Sum += Wt * .5 * (A + B);
		WtSum += Wt;
		LastV = B;
	}
	if (WtSum < FMathd::Epsilon) // if edges were all ~ zero length, just use one of the vertices
	{
		return LastV;
	}
	return Sum / WtSum;
}


FVector3d FGroupTopology::GetEdgeMidpoint(int32 GroupEdgeID, double* ArcLengthOut, TArray<double>* PerVertexLengthsOut) const
{
	check(GroupEdgeID >= 0 && GroupEdgeID < Edges.Num());
	const TArray<int32>& Vertices = GetGroupEdgeVertices(GroupEdgeID);
	int32 NumV = Vertices.Num();

	// trivial case
	if (NumV == 2)
	{
		FVector3d A(Mesh->GetVertex(Vertices[0])), B(Mesh->GetVertex(Vertices[1]));
		if (ArcLengthOut)
		{
			*ArcLengthOut = Distance(A, B);
		}
		if (PerVertexLengthsOut)
		{
			(*PerVertexLengthsOut).SetNum(2);
			(*PerVertexLengthsOut)[0] = 0;
			(*PerVertexLengthsOut)[1] = Distance(A, B);
		}
		return (A + B) * 0.5;
	}

	// if we want lengths anyway we can avoid second loop
	if (PerVertexLengthsOut)
	{
		double Len = GetEdgeArcLength(GroupEdgeID, PerVertexLengthsOut);
		if (ArcLengthOut)
		{
			*ArcLengthOut = Len;
		}
		Len /= 2;
		int32 k = 0;
		while ((*PerVertexLengthsOut)[k] < Len)
		{
			k++;
		}
		int32 kprev = k - 1;
		double a = (*PerVertexLengthsOut)[k-1], b = (*PerVertexLengthsOut)[k];
		double t = (Len - a) / (b - a);
		FVector3d A(Mesh->GetVertex(Vertices[k-1])), B(Mesh->GetVertex(Vertices[k]));
		return Lerp(A, B, t);
	}

	// compute arclen and then walk forward until we get halfway
	double Len = GetEdgeArcLength(GroupEdgeID);
	if (ArcLengthOut)
	{
		*ArcLengthOut = Len;
	}
	Len /= 2;
	double AccumLength = 0;
	for (int32 k = 1; k < NumV; ++k)
	{
		double NewLen = AccumLength + Distance(Mesh->GetVertex(Vertices[k]), Mesh->GetVertex(Vertices[k-1]));
		if ( NewLen > Len )
		{
			double t = (Len - AccumLength) / (NewLen - AccumLength);
			FVector3d A(Mesh->GetVertex(Vertices[k - 1])), B(Mesh->GetVertex(Vertices[k]));
			return Lerp(A, B, t);
		}
		AccumLength = NewLen;
	}

	// somehow failed?
	return (Mesh->GetVertex(Vertices[0]) + Mesh->GetVertex(Vertices[NumV-1])) * 0.5;
}


void FGroupTopology::FindCornerNbrGroups(int CornerID, TArray<int>& GroupsOut) const
{
	check(CornerID >= 0 && CornerID < Corners.Num());
	for (int GroupID : Corners[CornerID].NeighbourGroupIDs)
	{
		GroupsOut.AddUnique(GroupID);
	}
}

void FGroupTopology::FindCornerNbrGroups(const TArray<int>& CornerIDs, TArray<int>& GroupsOut) const
{
	for (int cid : CornerIDs)
	{
		FindCornerNbrGroups(cid, GroupsOut);
	}
}


void FGroupTopology::FindCornerNbrEdges(int CornerID, TArray<int>& EdgesOut) const
{
	ForCornerNbrEdges(CornerID, [&EdgesOut](int32 EdgeID)
	{
		EdgesOut.Add(EdgeID);
		return true;
	});
}

void UE::Geometry::FGroupTopology::ForCornerNbrEdges(int CornerID, TFunctionRef<bool(int32 EdgeId)>ReturnTrueToContinue) const
{
	check(CornerID >= 0 && CornerID < Corners.Num());
	TSet<int32> ProcessedEdges;
	for (int GroupID : Corners[CornerID].NeighbourGroupIDs)
	{
		const FGroup* Group = FindGroupByID(GroupID);
		for (const FGroupBoundary& Boundary : Group->Boundaries)
		{
			for (int32 EdgeID : Boundary.GroupEdges)
			{
				const FGroupEdge& Edge = Edges[EdgeID];
				if (Edge.EndpointCorners.A != CornerID && Edge.EndpointCorners.B != CornerID)
				{
					continue;
				}

				bool bAlreadyProcessed = false;
				ProcessedEdges.Add(EdgeID, &bAlreadyProcessed);
				if (bAlreadyProcessed)
				{
					continue;
				}

				if (!ReturnTrueToContinue(EdgeID))
				{
					return;
				}
			}
		}
	}
}

void FGroupTopology::FindCornerNbrCorners(int CornerID, TArray<int>& CornersOut) const
{
	check(CornerID >= 0 && CornerID < Corners.Num());
	for (int GroupID : Corners[CornerID].NeighbourGroupIDs)
	{
		const FGroup* Group = FindGroupByID(GroupID);
		for (const FGroupBoundary& Boundary : Group->Boundaries)
		{
			for (int32 EdgeID : Boundary.GroupEdges)
			{
				const FGroupEdge& Edge = Edges[EdgeID];
				if (Edge.EndpointCorners.A == CornerID || Edge.EndpointCorners.B == CornerID)
				{
					int32 OtherCornerID = Edge.EndpointCorners.OtherElement(CornerID);
					CornersOut.AddUnique(OtherCornerID);
				}
			}
		}

	}
}




void FGroupTopology::FindVertexNbrGroups(int VertexID, TArray<int>& GroupsOut) const
{
	for (int tid : Mesh->VtxTrianglesItr(VertexID))
	{
		int GroupID = GetGroupID(tid);
		GroupsOut.AddUnique(GroupID);
	}
}

void FGroupTopology::FindVertexNbrGroups(const TArray<int>& VertexIDs, TArray<int>& GroupsOut) const
{
	for (int vid : VertexIDs)
	{
		for (int tid : Mesh->VtxTrianglesItr(vid))
		{
			int GroupID = GetGroupID(tid);
			GroupsOut.AddUnique(GroupID);
		}
	}
}



void FGroupTopology::CollectGroupVertices(int GroupID, TSet<int>& Vertices) const
{
	const FGroup* Found = FindGroupByID(GroupID);
	ensure(Found != nullptr);
	if (Found != nullptr)
	{
		for (int TriID : Found->Triangles)
		{
			FIndex3i TriVerts = Mesh->GetTriangle(TriID);
			Vertices.Add(TriVerts.A);
			Vertices.Add(TriVerts.B);
			Vertices.Add(TriVerts.C);
		}
	}
}



void FGroupTopology::CollectGroupBoundaryVertices(int GroupID, TSet<int>& Vertices) const
{
	const FGroup* Group = FindGroupByID(GroupID);
	ensure(Group != nullptr);
	if (Group != nullptr)
	{
		for (const FGroupBoundary& Boundary : Group->Boundaries)
		{
			for (int EdgeIndex : Boundary.GroupEdges)
			{
				const FGroupEdge& Edge = Edges[EdgeIndex];
				for (int vid : Edge.Span.Vertices)
				{
					Vertices.Add(vid);
				}
			}
		}
	}
}



bool FGroupTopology::GenerateBoundaryAndGroupEdges(FGroup& Group, 
	TMap<int32, int32>& GroupEdgeMinEidToGroupEdgeID, TBitArray<>& VertCheckedForCorner)
{
	FMeshRegionBoundaryLoops BdryLoops(Mesh, Group.Triangles, true);

	if (BdryLoops.bFailed)
	{
		// Unrecoverable error when trying to find the group boundary loops 
		return false;
	}

	int NumLoops = BdryLoops.Loops.Num();

	// Function we use to see if a vertex is a corner, creating a corner object if so
	auto CheckForCornerAndCreateIfNeeded = [this, &GroupEdgeMinEidToGroupEdgeID, &VertCheckedForCorner](int32 Vid)
	{
		// See if we have a cached result
		if (VertCheckedForCorner[Vid])
		{
			return VertexIDToCornerIDMap.Contains(Vid);
		}
		else
		{
			VertCheckedForCorner[Vid] = true;
			bool bIsExtraCorner = false;
			if (ShouldVertBeCorner(Vid, &bIsExtraCorner))
			{
				// New corner vertex found. Create a corner for this vertex.
				int32 CornerID = Corners.Emplace();
				FCorner& Corner = Corners[CornerID];
				Corner.VertexID = Vid;
				GetAllVertexGroups(Vid, Corner.NeighbourGroupIDs);

				VertexIDToCornerIDMap.Add(Vid, CornerID);

				if (bIsExtraCorner)
				{
					CurrentExtraCornerVids.Add(Vid);
				}
				return true;
			}
			else
			{
				return false;
			}
		}
	};

	// Go through the boundary, find the corners, and use the intervening edges to create
	// group edges.
	Group.Boundaries.SetNum(NumLoops);
	for ( int li = 0; li < NumLoops; ++li )
	{
		FEdgeLoop& Loop = BdryLoops.Loops[li];
		FGroupBoundary& Boundary = Group.Boundaries[li];

		// find indices of corners of group polygon
		TArray<int> CornerIndices;
		int NumV = Loop.Vertices.Num();
		for (int i = 0; i < NumV; ++i)
		{
			if (CheckForCornerAndCreateIfNeeded(Loop.Vertices[i]))
			{
				CornerIndices.Add(i);
			}
		}

		// if we had no indices then this is like the cap of a cylinder, just one single long edge
		if ( CornerIndices.Num() == 0 )
		{ 
			int32 MinEid = *Algo::MinElement(Loop.Edges);
			int32* ExistingGroupEdgeID = GroupEdgeMinEidToGroupEdgeID.Find(MinEid);
			int EdgeIndex = ExistingGroupEdgeID ? *ExistingGroupEdgeID : IndexConstants::InvalidID;
			if (EdgeIndex == IndexConstants::InvalidID)
			{
				FGroupEdge Edge = { MakeEdgeGroupsPair(Loop.Edges[0]) };
				Edge.Span = FEdgeSpan(Mesh);
				Edge.Span.InitializeFromEdges(Loop.Edges);
				Edge.EndpointCorners = FIndex2i::Invalid();
				EdgeIndex = Edges.Add(Edge);
				GroupEdgeMinEidToGroupEdgeID.Add(MinEid, EdgeIndex);
			}
			Boundary.GroupEdges.Add(EdgeIndex);
			continue;
		}

		// duplicate first corner vertex so that we can just loop back around to it w/ modulo count
		int NumSpans = CornerIndices.Num();
		int FirstIdx = CornerIndices[0];
		CornerIndices.Add(FirstIdx);

		// add each span
		for (int k = 0; k < NumSpans; ++k)
		{
			int32 StartIndex = CornerIndices[k];
			int32 EndIndex = CornerIndices[k + 1];		// note: StartIndex == EndIndex on a closed loop, ie NumSpans == 1

			// This calculation of number of edges is correct as long as we're not starting and stopping in the same place.
			int32 NumSpanEdges = (EndIndex + NumV - StartIndex) % NumV;

			// If StartIndex == EndIndex, then NumSpanEdges should actually be NumV, not 0, because we are on a loop with
			// a single corner (either a bowtie or user-generated).
			if (NumSpanEdges == 0)
			{
				ensure(NumSpans == 1);
				NumSpanEdges = NumV;
			}

			// Find the min eid in this subsection so that we can see if we already have a group edge for it.
			int32 MinEid = Loop.Edges[StartIndex];
			for (int32 i = 1; i < NumSpanEdges; ++i)
			{
				int32 Index = (StartIndex + i) % NumV;
				MinEid = FMath::Min(MinEid, Loop.Edges[Index]);
			}

			int32* ExistingGroupEdgeID = GroupEdgeMinEidToGroupEdgeID.Find(MinEid);
			int EdgeIndex = ExistingGroupEdgeID ? *ExistingGroupEdgeID : IndexConstants::InvalidID;
			if (EdgeIndex != IndexConstants::InvalidID)
			{
				FGroupEdge& Existing = Edges[EdgeIndex];
				Boundary.GroupEdges.Add(EdgeIndex);
				continue;
			}

			FGroupEdge Edge = { MakeEdgeGroupsPair(Loop.Edges[StartIndex]) };

			TArray<int> SpanVertices;
			int32 NumSpanVertices = NumSpanEdges + 1;
			for (int32 i = 0; i < NumSpanVertices; ++i)
			{
				int32 Index = (StartIndex + i) % NumV;
				SpanVertices.Add(Loop.Vertices[Index]);
			}

			Edge.Span = FEdgeSpan(Mesh);
			Edge.Span.InitializeFromVertices(SpanVertices);
			Edge.EndpointCorners = FIndex2i(GetCornerIDFromVertexID(SpanVertices[0]), GetCornerIDFromVertexID(SpanVertices.Last()));
			check(Edge.EndpointCorners.A != IndexConstants::InvalidID && Edge.EndpointCorners.B != IndexConstants::InvalidID);
			EdgeIndex = Edges.Add(Edge);
			Boundary.GroupEdges.Add(EdgeIndex);
			GroupEdgeMinEidToGroupEdgeID.Add(MinEid, EdgeIndex);
		}
	}

	return true;
}


bool UE::Geometry::FGroupTopology::RebuildTopologyWithSpecificExtraCorners(const TSet<int32>& ExtraCornerVids)
{
	// We temporarily replace the vert corner-forcing function with one that checks against the given vids,
	// then we revert after doing the build.
	TFunction<bool(const FGroupTopology& GroupTopology, int32 Vid, const FIndex2i& AttachedGroupEdgeEids)> OldCornerFunction = MoveTemp(ShouldAddExtraCornerAtVert);

	ShouldAddExtraCornerAtVert = [&ExtraCornerVids](const FGroupTopology&, int32 Vid, const FIndex2i&) { return ExtraCornerVids.Contains(Vid); };

	bool bSuccess = RebuildTopology();

	ShouldAddExtraCornerAtVert = MoveTemp(OldCornerFunction);
	return bSuccess;
}


bool FGroupTopology::GetGroupEdgeTangent(int GroupEdgeID, FVector3d& TangentOut) const
{
	check(GroupEdgeID >= 0 && GroupEdgeID < Edges.Num());
	const FGroupEdge& Edge = Edges[GroupEdgeID];
	FVector3d StartPos = Mesh->GetVertex(Edge.Span.Vertices[0]);
	FVector3d EndPos = Mesh->GetVertex(Edge.Span.Vertices[Edge.Span.Vertices.Num()-1]);
	if (DistanceSquared(StartPos, EndPos) > 100 * FMathd::ZeroTolerance)
	{
		TangentOut = Normalized(EndPos - StartPos);
		return true;
	}
	else
	{
		TangentOut = FVector3d::UnitX();
		return false;
	}
}



FFrame3d FGroupTopology::GetGroupFrame(int32 GroupID) const
{
	FVector3d Centroid = FVector3d::Zero();
	FVector3d Normal = FVector3d::Zero();
	const FGroup& Face = Groups[GroupIDToGroupIndexMap[GroupID]];
	for (int32 tid : Face.Triangles)
	{
		Centroid += Mesh->GetTriCentroid(tid);
		Normal += Mesh->GetTriNormal(tid);
	}
	Centroid /= (double)Face.Triangles.Num();
	Normalize(Normal);
	return FFrame3d(Centroid, Normal);
}


FFrame3d FGroupTopology::GetSelectionFrame(const FGroupTopologySelection& Selection, FFrame3d* InitialLocalFrame) const
{
	int32 NumCorners = Selection.SelectedCornerIDs.Num();
	int32 NumEdges = Selection.SelectedEdgeIDs.Num();
	int32 NumFaces = Selection.SelectedGroupIDs.Num();

	FFrame3d StartFrame = (InitialLocalFrame) ? (*InitialLocalFrame) : FFrame3d();
	if (NumEdges == 1)
	{
		int32 GroupEdgeID = Selection.GetASelectedEdgeID();

		const TArray<int32>& Vertices = GetGroupEdgeVertices(GroupEdgeID);
		bool bIsSelfLoop = Vertices[0] == Vertices.Last();

		if (!bIsSelfLoop)
		{
			// align Z axis of frame to face normal of one of the connected faces. 
			int32 MeshEdgeID = GetGroupEdgeEdges(GroupEdgeID)[0];
			FIndex2i EdgeTris = Mesh->GetEdgeT(MeshEdgeID);
			int32 UseFace = (EdgeTris.B != IndexConstants::InvalidID) ? FMath::Min(EdgeTris.A, EdgeTris.B)
				: EdgeTris.A;
			FVector3d FaceNormal = Mesh->GetTriNormal(UseFace);
			if (!FaceNormal.IsZero())
			{
				StartFrame.AlignAxis(2, FaceNormal);
			}

			// align X axis along the edge, around the aligned Z axis
			FVector3d Tangent;
			if (GetGroupEdgeTangent(GroupEdgeID, Tangent))
			{
				StartFrame.ConstrainedAlignAxis(0, Tangent, StartFrame.Z());
			}
			StartFrame.Origin = GetEdgeMidpoint(GroupEdgeID);
		}
		else
		{
			StartFrame.Origin = GetEdgeAveragePosition(GroupEdgeID);
			// For self-loops, get polygon normal via newell's method
			FVector FaceNormal = FVector::ZeroVector;
			FVector Prev = Mesh->GetVertex(Vertices.Last());
			for (int32 Idx = 0; Idx < Vertices.Num(); ++Idx)
			{
				FVector Cur = Mesh->GetVertex(Vertices[Idx]);
				FaceNormal += FVector((Cur.Y - Prev.Y) * (Prev.Z + Cur.Z), (Cur.Z - Prev.Z) * (Prev.X + Cur.X), (Cur.X - Prev.X) * (Prev.Y + Cur.Y));
				Prev = Cur;
			}
			// If we found a valid plane normal, align the frame to it; otherwise leave the frame unchanged from default
			if (FaceNormal.Normalize())
			{
				StartFrame.AlignAxis(2, FaceNormal);
			}
		}
		return StartFrame;
	}
	if (NumCorners == 1)
	{
		StartFrame.Origin = Mesh->GetVertex(Corners[Selection.GetASelectedCornerID()].VertexID);
		return StartFrame;
	}

	FVector3d AccumulatedOrigin = FVector3d::Zero();
	FVector3d AccumulatedNormal = FVector3d::Zero();

	int AccumCount = 0;
	for (int32 CornerID : Selection.SelectedCornerIDs)
	{
		AccumulatedOrigin += Mesh->GetVertex(GetCornerVertexID(CornerID));
		AccumulatedNormal += FVector3d::UnitZ();
		AccumCount++;
	}

	for (int32 EdgeID : Selection.SelectedEdgeIDs)
	{
		const FGroupEdge& Edge = Edges[EdgeID];
		FVector3d StartPos = Mesh->GetVertex(Edge.Span.Vertices[0]);
		FVector3d EndPos = Mesh->GetVertex(Edge.Span.Vertices.Last());
		// special case self-loops to be consistent with the single-edge loop special case, above
		if (Edge.Span.Vertices[0] == Edge.Span.Vertices.Last())
		{
			AccumulatedOrigin += GetEdgeAveragePosition(EdgeID);
		}
		else
		{
			AccumulatedOrigin += 0.5 * (StartPos + EndPos);
		}
		AccumulatedNormal += FVector3d::UnitZ();
		AccumCount++;
	}

	for (int32 GroupID : Selection.SelectedGroupIDs)
	{
		if (FindGroupByID(GroupID) != nullptr)
		{
			FFrame3d GroupFrame = GetGroupFrame(GroupID);
			AccumulatedOrigin += GroupFrame.Origin;
			AccumulatedNormal += GroupFrame.Z();
			AccumCount++;
		}
	}

	FFrame3d AccumulatedFrame;
	if (AccumCount > 0)
	{
		AccumulatedOrigin /= (double)AccumCount;
		if (!AccumulatedNormal.Normalize())
		{
			AccumulatedNormal = FVector3d::UnitZ();
		}

		// We set our frame Z to be accumulated normal, and the other two axes are unconstrained, so
		// we want to set them to something that will make our frame generally more useful. If the normal
		// is aligned with world Z, then the entire frame might as well be aligned with world.
		double ZAlign = AccumulatedNormal.Dot(FVector3d::UnitZ());
		if (FMath::Abs(ZAlign) > 1 - KINDA_SMALL_NUMBER)
		{
			if (ZAlign < 0)
			{
				// Rotate frame 180 degrees around X
				AccumulatedFrame = FFrame3d(AccumulatedOrigin, FVector3d::UnitX(), -FVector3d::UnitY(), -FVector3d::UnitZ());
			}
			else
			{
				AccumulatedFrame = FFrame3d(AccumulatedOrigin, FQuaterniond::Identity());
			}
		}
		else
		{
			// Otherwise, let's place one of the other axes into the XY plane so that the frame is more
			// useful for translation. We somewhat arbitrarily choose Y for this. 
			FVector3d FrameY = Normalized(AccumulatedNormal.Cross(FVector3d::UnitZ())); // orthogonal to world Z and frame Z 
			FVector3d FrameX = FrameY.Cross(AccumulatedNormal); // safe to not normalize because already orthogonal
			AccumulatedFrame = FFrame3d(AccumulatedOrigin, FrameX, FrameY, AccumulatedNormal);
		}
	}

	return AccumulatedFrame;
}



FAxisAlignedBox3d FGroupTopology::GetSelectionBounds(
	const FGroupTopologySelection& Selection,
	TFunctionRef<FVector3d(const FVector3d&)> TransformFunc) const
{
	if (ensure(!Selection.IsEmpty()) == false)
	{
		return Mesh->GetBounds();
	}

	FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
	
	auto ContainVertex = [MeshPtr = Mesh, &Bounds, TransformFunc](int VertexID)
	{
		if (MeshPtr->IsVertex(VertexID))
		{
			Bounds.Contain( TransformFunc(MeshPtr->GetVertex(VertexID)) );
		}		
	};

	for (int32 CornerID : Selection.SelectedCornerIDs)
	{
		if (Corners.IsValidIndex(CornerID))
		{
			ContainVertex(GetCornerVertexID(CornerID));
		}
	}

	for (int32 EdgeID : Selection.SelectedEdgeIDs)
	{
		if (Edges.IsValidIndex(EdgeID))
		{
			const FGroupEdge& Edge = Edges[EdgeID];
			for (int32 vid : Edge.Span.Vertices)
			{
				ContainVertex(vid);
			}
		}
	}

	for (int32 GroupID : Selection.SelectedGroupIDs)
	{
		for (int32 TriangleID : GetGroupTriangles(GroupID))
		{
			if (Mesh->IsTriangle(TriangleID))
			{
				const FIndex3i TriVerts = Mesh->GetTriangle(TriangleID);
				ContainVertex(TriVerts.A);
				ContainVertex(TriVerts.B);
				ContainVertex(TriVerts.C);
			}
		}
	}

	return Bounds;
}

void FGroupTopology::GetSelectedTriangles(const FGroupTopologySelection& Selection, TArray<int32>& Triangles) const
{
	for (int32 GroupID : Selection.SelectedGroupIDs)
	{
		for (int32 TriangleID : GetGroupTriangles(GroupID))
		{
			Triangles.Add(TriangleID);
		}
	}
}

bool UE::Geometry::FGroupTopology::IsEdgeAngleSharp(const FDynamicMesh3* Mesh, int32 SharedVid, const FIndex2i& AttachedEids, double DotThreshold)
{
	if (!ensure(Mesh && Mesh->IsEdge(AttachedEids.A) && Mesh->IsEdge(AttachedEids.B)))
	{
		return false;
	}

	// Gets vector pointing from the Vid along the edge.
	auto GetEdgeUnitVector = [Mesh, SharedVid](int32 Eid, FVector3d& VectorOut)->bool
	{
		FIndex2i EdgeVids = Mesh->GetEdgeV(Eid);
		// Make sure that the Vid is at EdgeVids.A
		if (EdgeVids.B == SharedVid)
		{
			Swap(EdgeVids.A, EdgeVids.B);
		}
		VectorOut = Mesh->GetVertex(EdgeVids.B) - Mesh->GetVertex(EdgeVids.A);
		return VectorOut.Normalize(KINDA_SMALL_NUMBER);
	};

	FVector Edge1, Edge2;
	if (!GetEdgeUnitVector(AttachedEids.A, Edge1) || !GetEdgeUnitVector(AttachedEids.B, Edge2))
	{
		// If either edge was degenerate, we don't want to consider the angle "sharp"
		return false;
	}

	return Edge1.Dot(Edge2) >= DotThreshold;
}



void FGroupTopology::GetAllVertexGroups(int32 VertexID, TArray<int32>& GroupsOut) const
{
	for (int32 EdgeID : Mesh->VtxEdgesItr(VertexID))
	{
		FIndex2i EdgeTris = Mesh->GetEdgeT(EdgeID);
		GroupsOut.AddUnique(GetGroupID(EdgeTris.A));
		if (EdgeTris.B != FDynamicMesh3::InvalidID)
		{
			GroupsOut.AddUnique(GetGroupID(EdgeTris.B));
		}
	}
}






FTriangleGroupTopology::FTriangleGroupTopology(const FDynamicMesh3* Mesh, bool bAutoBuild) 
	: FGroupTopology(Mesh, false)
{
	// Note that we can't rely on the call to RebuildTopology in the base class constructor
	// because virtual function call resolution is not active in constructors (so it would
	// call the base class RebuildTopology)
	if (bAutoBuild)
	{
		RebuildTopology();
	}
}


bool FTriangleGroupTopology::RebuildTopology()
{
	Groups.Reset();
	Edges.Reset();
	Corners.Reset();

	int32 MaxGroupID = Mesh->MaxTriangleID();

	// initialize groups
	GroupIDToGroupIndexMap.Reset();
	GroupIDToGroupIndexMap.Init(-1, MaxGroupID);
	TArray<int> GroupFaceCounts;
	GroupFaceCounts.Init(1, MaxGroupID);
	for (int tid : Mesh->TriangleIndicesItr())
	{
		if (GroupIDToGroupIndexMap[tid] == -1)
		{
			FGroup NewGroup;
			NewGroup.GroupID = tid;
			NewGroup.Triangles.Add(tid);
			GroupIDToGroupIndexMap[tid] = Groups.Add(NewGroup);
		}
	}

	for (int vid : Mesh->VertexIndicesItr())
	{
		FCorner Corner = { vid };
		int NewCornerIndex = Corners.Num();
		VertexIDToCornerIDMap.Add(vid, NewCornerIndex);
		Corners.Add(Corner);
	}
	for (FCorner& Corner : Corners)
	{
		GetAllVertexGroups(Corner.VertexID, Corner.NeighbourGroupIDs);
	}

	TArray<int32> MeshEdgeToGroupEdge;
	MeshEdgeToGroupEdge.Init(INDEX_NONE, Mesh->MaxEdgeID());

	// construct boundary loops
	TArray<int> SpanVertices; SpanVertices.SetNum(2);
	for (FGroup& Group : Groups)
	{
		// find FGroupEdges and uses to populate Group.Boundaries
		Group.Boundaries.SetNum(1);
		FGroupBoundary& Boundary0 = Group.Boundaries[0];

		FIndex3i TriEdges = Mesh->GetTriEdges(Group.GroupID);
		for (int j = 0; j < 3; ++j)
		{
			int& GroupEdgeIndex = MeshEdgeToGroupEdge[TriEdges[j]];
			if (GroupEdgeIndex == INDEX_NONE)
			{
				FGroupEdge NewGroupEdge = { MakeEdgeGroupsPair(TriEdges[j]) };
				FIndex2i EdgeVerts = Mesh->GetEdgeV(TriEdges[j]);
				NewGroupEdge.Span = FEdgeSpan(Mesh);
				SpanVertices[0] = EdgeVerts.A; SpanVertices[1] = EdgeVerts.B;
				NewGroupEdge.Span.InitializeFromVertices(SpanVertices);
				NewGroupEdge.EndpointCorners = FIndex2i(GetCornerIDFromVertexID(SpanVertices[0]), GetCornerIDFromVertexID(SpanVertices[1]));
				check(NewGroupEdge.EndpointCorners.A != IndexConstants::InvalidID && NewGroupEdge.EndpointCorners.B != IndexConstants::InvalidID);
				GroupEdgeIndex = Edges.Add(NewGroupEdge);
			}
			Boundary0.GroupEdges.Add(GroupEdgeIndex);
		}

		// collect up .NeighbourGroupIDs and set .bIsOnBoundary
		// make all-neighbour-groups list at group level
		Boundary0.bIsOnBoundary = false;
		FIndex3i TriNbrTris = Mesh->GetTriNeighbourTris(Group.GroupID);
		for (int j = 0; j < 3; ++j)
		{
			if (TriNbrTris[j] != FDynamicMesh3::InvalidID)
			{
				Group.NeighbourGroupIDs.Add(TriNbrTris[j]);
				Boundary0.NeighbourGroupIDs.Add(TriNbrTris[j]);
			}
			else
			{
				Boundary0.bIsOnBoundary = true;
			}
		}
	}

	return true;
}
