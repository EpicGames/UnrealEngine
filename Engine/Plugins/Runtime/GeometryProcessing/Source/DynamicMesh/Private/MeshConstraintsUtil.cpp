// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshConstraintsUtil.h"
#include "Async/ParallelFor.h"
#include "GroupTopology.h"
#include "MeshBoundaryLoops.h"

using namespace UE::Geometry;

void FMeshConstraintsUtil::ConstrainAllSeams(FMeshConstraints& Constraints, const FDynamicMesh3& Mesh, bool bAllowSplits, bool bAllowSmoothing, bool bParallel)
{
	if (Mesh.HasAttributes() == false)
	{
		return;
	}
	const FDynamicMeshAttributeSet* Attributes = Mesh.Attributes();

	FEdgeConstraint EdgeConstraint = (bAllowSplits) ? FEdgeConstraint::SplitsOnly() : FEdgeConstraint::FullyConstrained();
	FVertexConstraint VtxConstraint = (bAllowSmoothing) ? FVertexConstraint::PermanentMovable() : FVertexConstraint::FullyConstrained();

	FCriticalSection ConstraintSetLock;

	int32 NumEdges = Mesh.MaxEdgeID();
	ParallelFor(NumEdges, [&](int EdgeID)
	{
		if (Mesh.IsEdge(EdgeID))
		{
			if (Attributes->IsSeamEdge(EdgeID))
			{
				FIndex2i EdgeVerts = Mesh.GetEdgeV(EdgeID);

				ConstraintSetLock.Lock();
				Constraints.SetOrUpdateEdgeConstraint(EdgeID, EdgeConstraint);
				Constraints.SetOrUpdateVertexConstraint(EdgeVerts.A, VtxConstraint);
				Constraints.SetOrUpdateVertexConstraint(EdgeVerts.B, VtxConstraint);
				ConstraintSetLock.Unlock();
			}
		}
	}, (bParallel==false) );
}

void
FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(FMeshConstraints& Constraints,
													 const FDynamicMesh3& Mesh,
													 EEdgeRefineFlags MeshBoundaryConstraint,
													 EEdgeRefineFlags GroupBoundaryConstraint,
													 EEdgeRefineFlags MaterialBoundaryConstraint,
													 bool bAllowSeamSplits, bool bAllowSeamSmoothing, bool bAllowSeamCollapse, 
													 bool bParallel)
{
	const FDynamicMeshAttributeSet* Attributes = Mesh.Attributes();

	// Seam edge can never flip, it is never fully unconstrained 
	EEdgeRefineFlags SeamEdgeConstraint = EEdgeRefineFlags::NoFlip;
	if (!bAllowSeamSplits)
	{
		SeamEdgeConstraint = EEdgeRefineFlags((int)SeamEdgeConstraint | (int)EEdgeRefineFlags::NoSplit);
	}
	if (!bAllowSeamCollapse)
	{
		SeamEdgeConstraint = EEdgeRefineFlags((int)SeamEdgeConstraint | (int)EEdgeRefineFlags::NoCollapse);
	}

	FCriticalSection ConstraintSetLock;

	int32 NumEdges = Mesh.MaxEdgeID();
	bool bHaveGroups = Mesh.HasTriangleGroups();


	ParallelFor(NumEdges, [&](int EdgeID)
	{
		FVertexConstraint VtxConstraintA = FVertexConstraint::Unconstrained();
		FVertexConstraint VtxConstraintB = FVertexConstraint::Unconstrained();

		FEdgeConstraint EdgeConstraint(EEdgeRefineFlags::NoConstraint);

		// compute the edge and vertex constraints.
		bool bHasUpdate = ConstrainEdgeBoundariesAndSeams(
		        EdgeID,
				Mesh,
				MeshBoundaryConstraint,
				GroupBoundaryConstraint,
				MaterialBoundaryConstraint,
				SeamEdgeConstraint,
				bAllowSeamSmoothing,
				EdgeConstraint, VtxConstraintA, VtxConstraintB);
		
		if (bHasUpdate)
		{
			// have updates - merge with existing constraints
			ConstraintSetLock.Lock();
			
			FIndex2i EdgeVerts = Mesh.GetEdgeV(EdgeID);
			
			Constraints.SetOrUpdateEdgeConstraint(EdgeID, EdgeConstraint);

			VtxConstraintA.CombineConstraint(Constraints.GetVertexConstraint(EdgeVerts.A));
			Constraints.SetOrUpdateVertexConstraint(EdgeVerts.A, VtxConstraintA);

			VtxConstraintB.CombineConstraint(Constraints.GetVertexConstraint(EdgeVerts.B));
			Constraints.SetOrUpdateVertexConstraint(EdgeVerts.B, VtxConstraintB);

			ConstraintSetLock.Unlock();
		}
	}, (bParallel == false) );
}

void FMeshConstraintsUtil::ConstrainSeamsInEdgeROI(FMeshConstraints& Constraints, const FDynamicMesh3& Mesh, const TArray<int>& EdgeROI, bool bAllowSplits, bool bAllowSmoothing, bool bParallel)
{
	if (Mesh.HasAttributes() == false)
	{
		return;
	}
	const FDynamicMeshAttributeSet* Attributes = Mesh.Attributes();

	FEdgeConstraint EdgeConstraint = (bAllowSplits) ? FEdgeConstraint::SplitsOnly() : FEdgeConstraint::FullyConstrained();
	FVertexConstraint VtxConstraint = (bAllowSmoothing) ? FVertexConstraint::PermanentMovable() : FVertexConstraint::FullyConstrained();

	FCriticalSection ConstraintSetLock;

	int32 NumEdges = EdgeROI.Num();
	ParallelFor(NumEdges, [&](int k)
	{
		int EdgeID = EdgeROI[k];
		FIndex2i EdgeVerts = Mesh.GetEdgeV(EdgeID);

		if (Attributes->IsSeamEdge(EdgeID))
		{
			ConstraintSetLock.Lock();
			Constraints.SetOrUpdateEdgeConstraint(EdgeID, EdgeConstraint);
			Constraints.SetOrUpdateVertexConstraint(EdgeVerts.A, VtxConstraint);
			Constraints.SetOrUpdateVertexConstraint(EdgeVerts.B, VtxConstraint);
			ConstraintSetLock.Unlock();
		}
		else
		{
			// Constrain edge end points if they belong to seams.
			// NOTE: It is possible that one (or both) of these vertices belongs to a seam edge that is not in EdgeROI. 
			// In such a case, we still want to constrain that vertex.
			for (int VertexID : {EdgeVerts[0], EdgeVerts[1]})
			{
				if (Attributes->IsSeamVertex(VertexID, true))
				{
					ConstraintSetLock.Lock();
					Constraints.SetOrUpdateVertexConstraint(VertexID, VtxConstraint);
					ConstraintSetLock.Unlock();
				}
			}
		}

	}, (bParallel==false) );
}

void FMeshConstraintsUtil::ConstrainROIBoundariesInEdgeROI(FMeshConstraints& Constraints,
	const FDynamicMesh3& Mesh,
	const TSet<int>& EdgeROI,
	const TSet<int>& TriangleROI,
	bool bAllowSplits,
	bool bAllowSmoothing)
{
	FEdgeConstraint EdgeConstraint = (bAllowSplits) ? FEdgeConstraint::SplitsOnly() : FEdgeConstraint::FullyConstrained();
	FVertexConstraint VtxConstraint = (bAllowSmoothing) ? FVertexConstraint::PermanentMovable() : FVertexConstraint::FullyConstrained();

	for (int EdgeID : EdgeROI)
	{
		FIndex2i EdgeTris = Mesh.GetEdgeT(EdgeID);
		bool bIsROIBoundary = (TriangleROI.Contains(EdgeTris.A) != TriangleROI.Contains(EdgeTris.B));
		if (bIsROIBoundary)
		{
			FIndex2i EdgeVerts = Mesh.GetEdgeV(EdgeID);
			Constraints.SetOrUpdateEdgeConstraint(EdgeID, EdgeConstraint);
			Constraints.SetOrUpdateVertexConstraint(EdgeVerts.A, VtxConstraint);
			Constraints.SetOrUpdateVertexConstraint(EdgeVerts.B, VtxConstraint);
		}
	}
}

bool FMeshConstraintsUtil::ConstrainEdgeBoundariesAndSeams(const int EdgeID,
	                                                       const FDynamicMesh3& Mesh,
	                                                       const EEdgeRefineFlags MeshBoundaryConstraintFlags,
	                                                       const EEdgeRefineFlags GroupBoundaryConstraintFlags,
	                                                       const EEdgeRefineFlags MaterialBoundaryConstraintFlags,
	                                                       const EEdgeRefineFlags SeamEdgeConstraintFlags,
	                                                       const bool bAllowSeamSmoothing,
	                                                       FEdgeConstraint& EdgeConstraint,
	                                                       FVertexConstraint& VertexConstraintA,
	                                                       FVertexConstraint& VertexConstraintB)
{
	const bool bAllowSeamCollapse = FEdgeConstraint::CanCollapse(SeamEdgeConstraintFlags);

	// initialize constraints
	VertexConstraintA = FVertexConstraint::Unconstrained();
	VertexConstraintB = FVertexConstraint::Unconstrained();
	EdgeConstraint = FEdgeConstraint::Unconstrained();

	const bool bIsEdge = Mesh.IsEdge(EdgeID);
	if (!bIsEdge)  return false;

	const bool bHaveGroups = Mesh.HasTriangleGroups();
	const FDynamicMeshAttributeSet* Attributes = Mesh.Attributes();

	const bool bIsMeshBoundary = Mesh.IsBoundaryEdge(EdgeID);
	const bool bIsGroupBoundary = bHaveGroups && Mesh.IsGroupBoundaryEdge(EdgeID);
	const bool bIsMaterialBoundary = Attributes && Attributes->IsMaterialBoundaryEdge(EdgeID);
	const bool bIsSeam = Attributes && Attributes->IsSeamEdge(EdgeID);

	FVertexConstraint CurVtxConstraint = FVertexConstraint::Unconstrained(); // note: this is needed since the default constructor is constrained.
	EEdgeRefineFlags EdgeFlags{};

	auto ApplyBoundaryConstraint = [&CurVtxConstraint, &EdgeFlags](EEdgeRefineFlags BoundaryConstraintFlags)
	{

		CurVtxConstraint.bCannotDelete = CurVtxConstraint.bCannotDelete ||
			(!FEdgeConstraint::CanCollapse(BoundaryConstraintFlags) &&
				!FEdgeConstraint::CanFlip(BoundaryConstraintFlags)
				);

		CurVtxConstraint.bCanMove = CurVtxConstraint.bCanMove &&
			(FEdgeConstraint::CanCollapse(BoundaryConstraintFlags) ||
				FEdgeConstraint::CanFlip(BoundaryConstraintFlags)
				);

		EdgeFlags = EEdgeRefineFlags((int)EdgeFlags | (int)BoundaryConstraintFlags);
	};


	if (bIsMeshBoundary)
	{
		ApplyBoundaryConstraint(MeshBoundaryConstraintFlags);
	}
	if (bIsGroupBoundary)
	{
		ApplyBoundaryConstraint(GroupBoundaryConstraintFlags);
	}
	if (bIsMaterialBoundary)
	{
		ApplyBoundaryConstraint(MaterialBoundaryConstraintFlags);
	}
	if (bIsSeam)	// TODO: mesh boundary edges are currently flagged as seams (based on the implementation of Attributes->IsSeamEdge), so are subject to seam constraints
	{
		CurVtxConstraint.bCannotDelete = CurVtxConstraint.bCannotDelete || !bAllowSeamCollapse;
		CurVtxConstraint.bCanMove = CurVtxConstraint.bCanMove && (bAllowSeamSmoothing || bAllowSeamCollapse);

		EdgeFlags = EEdgeRefineFlags((int)EdgeFlags | (int)(SeamEdgeConstraintFlags));

		// Additional logic to add the NoCollapse flag to any edge that is the start or end of a seam.
		if (bAllowSeamCollapse)
		{

			bool bHasSeamEnd = false;
			for (int32 i = 0; !bHasSeamEnd && (i < Attributes->NumUVLayers()); ++i)
			{
				const FDynamicMeshUVOverlay* UVLayer = Attributes->GetUVLayer(i);
				bHasSeamEnd = bHasSeamEnd || UVLayer->IsSeamEndEdge(EdgeID);
			}
			for (int32 i = 0; !bHasSeamEnd && (i < Attributes->NumNormalLayers()); ++i)
			{
				const FDynamicMeshNormalOverlay* NormalOverlay = Attributes->GetNormalLayer(i);
				bHasSeamEnd = bHasSeamEnd || NormalOverlay->IsSeamEndEdge(EdgeID);
			}

			if (bHasSeamEnd)
			{
				EdgeFlags = EEdgeRefineFlags((int)EdgeFlags | (int)EEdgeRefineFlags::NoCollapse);
			}
		}
	}
	if (bIsMeshBoundary || bIsGroupBoundary || bIsMaterialBoundary || bIsSeam)
	{
		EdgeConstraint = FEdgeConstraint(EdgeFlags);

		// only return true if we have a constraint
		if (!EdgeConstraint.IsUnconstrained() || !CurVtxConstraint.IsUnconstrained())
		{
			VertexConstraintA.CombineConstraint(CurVtxConstraint);
			VertexConstraintB.CombineConstraint(CurVtxConstraint);
			return  true;
		}
	}
	return false;
}


namespace
{
	// Utility class for finding mesh boundaries in the same way we find group and material boundaries
	class FMeshBoundaryTopology : public FGroupTopology
	{
	public:

		FMeshBoundaryTopology() {}

		FMeshBoundaryTopology(const FDynamicMesh3* Mesh, bool bAutoBuild) :
			FGroupTopology(Mesh, bAutoBuild)
		{}

		virtual int GetGroupID(int TriangleID) const override
		{
			return 1;
		}
	};
}


void FMeshConstraintsUtil::SetBoundaryConstraintsWithProjection(
	FMeshConstraints& Constraints,
	const EBoundaryType BoundaryToConstrain,
	const FDynamicMesh3& Mesh,
	double CornerAngleThreshold,
	float CurveConstraintToleranceDistance)
{
	TSet<int32> GroupCorners;

	TUniquePtr<FGroupTopology> Topology;

	switch (BoundaryToConstrain)
	{
	case EBoundaryType::Mesh:
		Topology = MakeUnique<FMeshBoundaryTopology>(&Mesh, true);
		break;
	case EBoundaryType::Group:
		// TODO: this will tag mesh boundaries as group boundaries, however Mesh.IsGroupBoundaryEdge() returns false at mesh boundary edges
		Topology = MakeUnique<FGroupTopology>(&Mesh, true);		
		break;
	case EBoundaryType::MaterialID:
		if (!(Mesh.HasAttributes() && Mesh.Attributes()->GetMaterialID() != nullptr))
		{
			return;
		}
		// TODO: this will tag mesh boundaries as attribute boundaries, however Mesh.Attributes->IsMaterialBoundaryEdge() returns false at mesh boundary edges
		Topology = MakeUnique<FGroupTopology>(&Mesh, Mesh.Attributes()->GetMaterialID(), true);
		break;
	}

	const int32 NumEdges = Topology->Edges.Num();
	const int32 OldNumCurves = Constraints.ProjectionData.ProjectionCurves.Num();
	const int32 NewNumCurves = OldNumCurves + NumEdges;

	Constraints.ProjectionData.ProjectionCurves.SetNum(NewNumCurves);


	float EdgeToleranceDistSq = CurveConstraintToleranceDistance > 0 ? CurveConstraintToleranceDistance * CurveConstraintToleranceDistance : -1.f;
	// helper to update edge tolerance distances
	auto MergeEdgeToleranceDistSq = [](float A, float B) -> float
	{
		if (A <= 0) // if A is invalid, return B
		{
			return B;
		}
		else if (B <= 0) // if B is invalid, return A
		{
			return A;
		}
		else // if both valid, return the smallest tolerance
		{
			return FMath::Min(A, B);
		}
	};

	for (int32 TopologyEdgeIndex = 0; TopologyEdgeIndex < NumEdges; ++TopologyEdgeIndex)
	{
		const FGroupTopology::FGroupEdge& GroupEdge = Topology->Edges[TopologyEdgeIndex];

		TSharedPtr<FMeshConstraintCurve> Curve = MakeShared<FMeshConstraintCurve>();
		Constraints.ProjectionData.ProjectionCurves[OldNumCurves + TopologyEdgeIndex] = Curve;

		const bool bIsLoop = (GroupEdge.EndpointCorners[0] == IndexConstants::InvalidID);

		// mark corner vertices for pinning
		if (!bIsLoop)
		{
			GroupCorners.Add(Topology->GetCornerVertexID(GroupEdge.EndpointCorners[0]));
			GroupCorners.Add(Topology->GetCornerVertexID(GroupEdge.EndpointCorners[1]));
		}

		// extract polyline
		GroupEdge.Span.GetPolyline(*Curve);

		const TArray<int32>& EdgeVerts = Topology->GetGroupEdgeVertices(TopologyEdgeIndex);
		const int32 NumEdgeVerts = EdgeVerts.Num();

		for (int32 EdgeVertIndex = 0; EdgeVertIndex < NumEdgeVerts - 1; ++EdgeVertIndex)
		{
			// Check for angle-based corners
			int32 PrevEdgeVert;
			const int32 NextEdgeVert = EdgeVertIndex + 1;

			if (EdgeVertIndex == 0)
			{
				if (!bIsLoop)
				{
					continue;
				}
				PrevEdgeVert = NumEdgeVerts - 2;		// Vertex at N-1 is the same vertex as at 0, for FGroupTopology loops
			}
			else
			{
				PrevEdgeVert = EdgeVertIndex - 1;
			}

			const int32 VertexID = EdgeVerts[EdgeVertIndex];

			const FVector3d PrevRel = Mesh.GetVertex(VertexID) - Mesh.GetVertex(EdgeVerts[PrevEdgeVert]);
			const FVector3d NextRel = Mesh.GetVertex(EdgeVerts[NextEdgeVert]) - Mesh.GetVertex(VertexID);

			if (PrevRel.Size() > UE_SMALL_NUMBER && NextRel.Size() > UE_SMALL_NUMBER)
			{
				const double CornerCosAngle = PrevRel.Dot(NextRel) / PrevRel.Size() / NextRel.Size();
				const double CornerAngleDeg = FMath::RadiansToDegrees(FMath::Acos(CornerCosAngle));

				if (FMath::Abs(CornerAngleDeg) > CornerAngleThreshold)
				{
					// Found a corner
					GroupCorners.Add(VertexID);
					continue;
				}
			}

			Constraints.SetOrCombineVertexConstraint(VertexID, FVertexConstraint(Curve.Get()));
		}

		const TArray<int32>& EdgeEdges = Topology->GetGroupEdgeEdges(TopologyEdgeIndex);
		for (int32 EdgeEdgeIndex = 0; EdgeEdgeIndex < EdgeEdges.Num(); ++EdgeEdgeIndex)
		{
			const int32 EdgeID = EdgeEdges[EdgeEdgeIndex];

			FEdgeConstraint FoundEdgeConstraint;
			if (Constraints.GetEdgeConstraint(EdgeID, FoundEdgeConstraint))
			{
				FoundEdgeConstraint.Target = Curve.Get();
				FoundEdgeConstraint.DistanceToleranceSq = MergeEdgeToleranceDistSq(EdgeToleranceDistSq, FoundEdgeConstraint.DistanceToleranceSq);
				Constraints.SetOrUpdateEdgeConstraint(EdgeID, FoundEdgeConstraint);
			}
			else
			{
				FEdgeConstraint GroupEdgeConstraint(EEdgeRefineFlags::NoConstraint, Curve.Get());
				GroupEdgeConstraint.DistanceToleranceSq = EdgeToleranceDistSq;
				Constraints.SetOrUpdateEdgeConstraint(EdgeID, GroupEdgeConstraint);
			}
		}
	}


	for (int32 CornerVertexID : GroupCorners)
	{
		Constraints.SetOrCombineVertexConstraint(CornerVertexID, FVertexConstraint::FullyConstrained());
	}
}