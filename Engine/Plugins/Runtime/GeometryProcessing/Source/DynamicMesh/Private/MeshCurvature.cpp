// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshCurvature.h"
#include "MeshWeights.h"
#include "DynamicMesh/MeshNormals.h"
#include "VectorUtil.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

template<typename GetPositionFuncType>
FVector3d TMeanCurvatureNormal(const FDynamicMesh3& mesh, int32 v_i, GetPositionFuncType GetPositionFunc, double CotClampRange = 1000 /* about 0.05 deg */)
{
	// based on equations in http://www.geometry.caltech.edu/pubs/DMSB_III.pdf

	FVector3d vSum = FVector3d::Zero();
	double wSum = 0;
	FVector3d Vi = GetPositionFunc(v_i);

	int32 v_j = FDynamicMesh3::InvalidID, opp_v1 = FDynamicMesh3::InvalidID, opp_v2 = FDynamicMesh3::InvalidID;
	int32 t1 = FDynamicMesh3::InvalidID, t2 = FDynamicMesh3::InvalidID;
	for (int32 eid : mesh.VtxEdgesItr(v_i))
	{
		opp_v2 = FDynamicMesh3::InvalidID;
		mesh.GetVtxNbrhood(eid, v_i, v_j, opp_v1, opp_v2, t1, t2);
		FVector3d Vj = GetPositionFunc(v_j);

		FVector3d Vo1 = GetPositionFunc(opp_v1);
		double cot_alpha_ij = VectorUtil::VectorCot((Vi - Vo1), (Vj - Vo1));
		double w_ij = FMathd::Clamp(cot_alpha_ij, -CotClampRange, CotClampRange);

		double cot_beta_ij = 0;
		if (opp_v2 != FDynamicMesh3::InvalidID)
		{
			FVector3d Vo2 = GetPositionFunc(opp_v2);
			cot_beta_ij = VectorUtil::VectorCot((Vi - Vo2), (Vj - Vo2));
		}
		w_ij += FMathd::Clamp(cot_beta_ij, -CotClampRange, CotClampRange);

		vSum += w_ij * (Vi - Vj);
	}
	if (vSum.SquaredLength() < FMathd::ZeroTolerance)
	{
		return FVector3d::Zero();
	}
	double Area = FMeshWeights::VoronoiArea(mesh, v_i, GetPositionFunc);
	if (Area < FMathd::ZeroTolerance)
	{
		return FVector3d::Zero();
	}
	return vSum / (2.0 * Area);
}


FVector3d UE::MeshCurvature::MeanCurvatureNormal(const FDynamicMesh3& Mesh, int32 VertexIndex)
{
	return TMeanCurvatureNormal(Mesh, VertexIndex, [&](int32 vid) { return Mesh.GetVertex(vid); });
}

FVector3d UE::MeshCurvature::MeanCurvatureNormal(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc)
{
	return TMeanCurvatureNormal(Mesh, VertexIndex, VertexPositionFunc);
}






template<typename GetPositionFuncType>
double TGaussianCurvature(const FDynamicMesh3& mesh, int32 v_i, GetPositionFuncType GetPositionFunc, double CotClampRange = 1000 /* about 0.05 deg */)
{
	// based on equation 9 in http://www.geometry.caltech.edu/pubs/DMSB_III.pdf
	double AngleSum = 0;
	double MixedAreaSum = 0;
	FVector3d Vi = GetPositionFunc(v_i);

	mesh.EnumerateVertexTriangles(v_i, [&](int32 tid)
	{
		FIndex3i t = mesh.GetTriangle(tid);
		int ti = (t[0] == v_i) ? 0 : ((t[1] == v_i) ? 1 : 2);
		FVector3d Vj = GetPositionFunc(t[(ti + 1) % 3]);
		FVector3d Vk = GetPositionFunc(t[(ti + 2) % 3]);

		FVector3d Vij = Vj - Vi;
		FVector3d Nij(Vij);
		double LenSqrVij = Normalize(Nij);
		LenSqrVij *= LenSqrVij;

		FVector3d Vik = Vk - Vi;
		FVector3d Nik(Vik);
		double LenSqrVik = Normalize(Nik);
		LenSqrVik *= LenSqrVik;

		double OpeningAngle = AngleR(Nij, Nik);
		AngleSum += OpeningAngle;

		if (VectorUtil::IsObtuse(Vi, Vj, Vk))
		{
			// if triangle is obtuse voronoi area is undefined and we just return portion of triangle area
			double Dot = Vij.Dot(Vik);
			double areaT = 0.5 * FMathd::Sqrt(LenSqrVij * LenSqrVik - Dot * Dot);
			MixedAreaSum += (OpeningAngle > FMathd::HalfPi) ?    // obtuse at v_i ?
				(areaT * 0.5) : (areaT * 0.25);
		}
		else
		{
			// voronoi area
			FVector3d Vkj = Vj - Vk;
			double cot_alpha_ij = VectorUtil::VectorCot(-Vik, Vkj);
			cot_alpha_ij = FMathd::Clamp(cot_alpha_ij, -CotClampRange, CotClampRange);
			double cot_beta_ik = VectorUtil::VectorCot(-Vij, -Vkj);
			cot_beta_ik = FMathd::Clamp(cot_beta_ik, -CotClampRange, CotClampRange);
			MixedAreaSum += LenSqrVij * cot_alpha_ij * 0.125;
			MixedAreaSum += LenSqrVik * cot_beta_ik * 0.125;
		}
	});

	if (MixedAreaSum < FMathf::ZeroTolerance)
	{
		return 0.0f;
	}
	else
	{
		return (FMathd::TwoPi - AngleSum) / MixedAreaSum;
	}
}



double UE::MeshCurvature::GaussianCurvature(const FDynamicMesh3& Mesh, int32 VertexIndex)
{
	return TGaussianCurvature(Mesh, VertexIndex, [&](int32 vid) { return Mesh.GetVertex(vid); });
}

double UE::MeshCurvature::GaussianCurvature(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc)
{
	return TGaussianCurvature(Mesh, VertexIndex, VertexPositionFunc);
}





void FMeshVertexCurvatureCache::BuildAll(const FDynamicMesh3& Mesh)
{
	int32 NumVertices = Mesh.MaxVertexID();
	Curvatures.SetNum(NumVertices);

	TArray<bool> IsValidFlag;
	IsValidFlag.Init(false, NumVertices);

	ParallelFor(NumVertices, [&](int32 vid)
	{
		if (Mesh.IsVertex(vid))
		{
			if (Mesh.IsBoundaryVertex(vid))
			{
				Curvatures[vid] = FVertexCurvature();
				return;
			}

			FVector3d VertexNormal = FMeshNormals::ComputeVertexNormal(Mesh, vid);

			//double Area = FMeshWeights::VoronoiArea(Mesh, vid);
			//double GaussCurvature = UE::MeshCurvature::GaussianCurvature(Mesh, vid) / Area;

			double GaussCurvature = UE::MeshCurvature::GaussianCurvature(Mesh, vid);

			FVector3d MeanCurvatureNormal = UE::MeshCurvature::MeanCurvatureNormal(Mesh, vid);
			double MeanCurvature = 0.5 * MeanCurvatureNormal.Length();
			double Dot = MeanCurvatureNormal.Dot(VertexNormal);
			MeanCurvature  *= FMathd::Sign(Dot);

			double DeltaCurvature = FMathd::Max(0, MeanCurvature * MeanCurvature - GaussCurvature);
			DeltaCurvature = FMathd::Sqrt(DeltaCurvature);
			double PrincipalCurvature1 = MeanCurvature + DeltaCurvature;
			double PrincipalCurvature2 = MeanCurvature - DeltaCurvature;

			Curvatures[vid] = { MeanCurvature, GaussCurvature, PrincipalCurvature1, PrincipalCurvature2 };
			IsValidFlag[vid] = true;
		}
	});


	TSampleSetStatisticBuilder<double> CurvatureStats(4);

	// count valid vertices
	int32 NumCount = 0;
	for (int32 k = 0; k < NumVertices; ++k)
	{
		if (IsValidFlag[k])
		{
			NumCount++;
		}
	}

	CurvatureStats.Begin_FixedCount(NumCount);

	for (int32 Pass = 0; Pass < 2; ++Pass)
	{
		if (Pass == 1)
		{
			CurvatureStats.StartSecondPass_FixedCount();
		}

		for (int32 k = 0; k < NumVertices; ++k)
		{
			if (IsValidFlag[k])
			{
				CurvatureStats.AccumulateValue_FixedCount(0, FMathd::Abs(Curvatures[k].Mean));
				CurvatureStats.AccumulateValue_FixedCount(1, FMathd::Abs(Curvatures[k].Gaussian));
				CurvatureStats.AccumulateValue_FixedCount(2, FMathd::Abs(Curvatures[k].MaxPrincipal));
				CurvatureStats.AccumulateValue_FixedCount(3, FMathd::Abs(Curvatures[k].MinPrincipal));
			}
		}

		if (Pass == 1)
		{
			CurvatureStats.CompleteSecondPass_FixedCount();
		}
	}

	MeanStats = CurvatureStats[0];
	GaussianStats = CurvatureStats[1];
	MaxPrincipalStats = CurvatureStats[2];
	MinPrincipalStats = CurvatureStats[3];

}