// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDerivatives.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Util/IndexUtil.h"

namespace UE
{
namespace Geometry
{
	FVector3d FMeshDerivatives::InteriorAngleGradient(const FDynamicMesh3& Mesh, int32 TriangleIndex, int32 VertexIndex, int32 WRTVertexIndex)
	{
		const FIndex3i Tri = Mesh.GetTriangle(TriangleIndex);
		const int TriVertIndex = IndexUtil::FindTriIndex(VertexIndex, Tri);
		check(TriVertIndex != IndexConstants::InvalidID);

		const int32 VertI = VertexIndex;
		const int32 VertJ = Tri[(TriVertIndex + 1) % 3];
		const int32 VertK = Tri[(TriVertIndex + 2) % 3];

		const FVector3d Normal = Mesh.GetTriNormal(TriangleIndex);

		if (WRTVertexIndex == VertI)
		{
			const FVector3d EdgeIJ = Mesh.GetVertex(VertJ) - Mesh.GetVertex(VertI);
			const FVector3d Grad_J = -EdgeIJ.Cross(Normal) / EdgeIJ.SquaredLength();
			const FVector3d EdgeIK = Mesh.GetVertex(VertK) - Mesh.GetVertex(VertI);
			const FVector3d Grad_K = EdgeIK.Cross(Normal) / EdgeIK.SquaredLength();
			return -(Grad_J + Grad_K);
		}
		else if (WRTVertexIndex == VertJ)
		{
			const FVector3d EdgeIJ = Mesh.GetVertex(VertJ) - Mesh.GetVertex(VertI);
			const FVector3d Grad_J = -EdgeIJ.Cross(Normal) / EdgeIJ.SquaredLength();
			return Grad_J;
		}
		else
		{
			check(WRTVertexIndex == VertK);
			const FVector3d EdgeIK = Mesh.GetVertex(VertK) - Mesh.GetVertex(VertI);
			const FVector3d Grad_K = EdgeIK.Cross(Normal) / EdgeIK.SquaredLength();
			return Grad_K;
		}
	}

	FMatrix3d FMeshDerivatives::TriangleNormalGradient(const FDynamicMesh3& Mesh, int32 TriangleIndex, int32 WRTVertexIndex)
	{
		FVector3d Normal;
		double Area;
		FVector3d Centroid;
		Mesh.GetTriInfo(TriangleIndex, Normal, Area, Centroid);

		const FIndex3i Tri = Mesh.GetTriangle(TriangleIndex);

		const int TriVertIndex = IndexUtil::FindTriIndex(WRTVertexIndex, Tri);
		check(TriVertIndex != IndexConstants::InvalidID);

		const FVector3d Edge = Mesh.GetTriVertex(TriangleIndex, (TriVertIndex + 2) % 3) - Mesh.GetTriVertex(TriangleIndex, (TriVertIndex + 1) % 3);
		return 1.0 / (2.0 * Area) * FMatrix3d(Normal.Cross(Edge), Normal);
	}


	FMatrix3d FMeshDerivatives::TriangleAreaScaledNormalGradient(const FDynamicMesh3& Mesh, int32 TriangleIndex, int32 WRTVertexIndex)
	{
		auto CrossProductMatrix = [](const FVector3d& Vec) ->FMatrix3d
		{
			return FMatrix3d{ 0, -Vec[2], Vec[1],
							  Vec[2], 0, -Vec[0],
							  -Vec[1], Vec[0], 0 };
		};

		FVector3d V0, V1, V2;
		Mesh.GetTriVertices(TriangleIndex, V0, V1, V2);

		const FIndex3i Tri = Mesh.GetTriangle(TriangleIndex);

		if (WRTVertexIndex == Tri[0])
		{
			const FMatrix3d CrossProductEdgeW = CrossProductMatrix(V1 - V2);
			return 0.5 * CrossProductEdgeW;
		}
		else if (WRTVertexIndex == Tri[1])
		{
			const FMatrix3d CrossProductEdgeV = CrossProductMatrix(V2 - V0);
			return 0.5 * CrossProductEdgeV;
		}
		else
		{
			check(WRTVertexIndex == Tri[2]);
			const FMatrix3d CrossProductEdgeU = CrossProductMatrix(V1 - V0);
			return -0.5 * CrossProductEdgeU;
		}
	}


} // end namespace UE::Geometry
} // end namespace UE
