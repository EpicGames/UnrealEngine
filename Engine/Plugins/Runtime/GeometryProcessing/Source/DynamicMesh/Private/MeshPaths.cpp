// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPaths.h"

using namespace UE::Geometry;

int FMeshPaths::GetNextEdge(int CurrentEdge, int FromVertex, int& CrossedVertex) const
{
	check(!UnvisitedEdges.Contains(CurrentEdge));

	FIndex2i EdgeV = Mesh->GetEdgeV(CurrentEdge);

	CrossedVertex = (EdgeV[0] == FromVertex) ? EdgeV[1] : EdgeV[0];

	int AdjacentUnvisitedEdge = IndexConstants::InvalidID;
	int NumAdjacentEdgesForThisVertex = 0;

	for (int EdgeID : Mesh->VtxEdgesItr(CrossedVertex))
	{
		if (EdgeID == CurrentEdge)
		{
			continue;
		}

		if (Edges.Contains(EdgeID))
		{
			++NumAdjacentEdgesForThisVertex;

			if (UnvisitedEdges.Contains(EdgeID))
			{
				AdjacentUnvisitedEdge = EdgeID;
			}
		}
	}

	if (NumAdjacentEdgesForThisVertex == 1)	 // Only consider simply connected edges
	{
		return AdjacentUnvisitedEdge;
	}

	return IndexConstants::InvalidID;
}



bool FMeshPaths::Compute()
{
	if (!Mesh)
	{
		return false;
	}

	UnvisitedEdges = Edges;

	while (!UnvisitedEdges.IsEmpty())
	{
		// get any unvisited edge, then find the span it's connected to
		const int InitialEdge = *UnvisitedEdges.CreateConstIterator();
		
		const FIndex2i InitialEdgeVertices = Mesh->GetEdgeV(InitialEdge);

		// Compute the span by traversing connected edges in both directions
		TArray<int> LeftSpan, RightSpan;

		for (int Direction = 0; Direction < 2; ++Direction)
		{
			int CurrentEdge = InitialEdge;
			int FromVertex = InitialEdgeVertices[Direction];
			int CrossedVertex = IndexConstants::InvalidID;

			while (CurrentEdge != IndexConstants::InvalidID)
			{
				if (Direction == 0)
				{
					LeftSpan.Add(CurrentEdge);
				}
				else if (CurrentEdge != InitialEdge)
				{
					RightSpan.Add(CurrentEdge);
				}
				
				UnvisitedEdges.Remove(CurrentEdge);
				CurrentEdge = GetNextEdge(CurrentEdge, FromVertex, CrossedVertex);
				FromVertex = CrossedVertex;
			}
		}

		Algo::Reverse(LeftSpan);
		LeftSpan.Append(RightSpan);

		FEdgeSpan NewSpan;
		NewSpan.InitializeFromEdges(Mesh, LeftSpan);
		checkSlow(NewSpan.CheckValidity());
		Spans.Emplace(NewSpan);
	}

	return true;
}

