// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshRefinerBase.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"

using namespace UE::Geometry;



/*
* Check if edge collapse will create a face-normal flip.
* Also checks if collapse would violate link condition, since we are iterating over one-ring anyway.
* This only checks one-ring of vid, so you have to call it twice, with vid and vother reversed, to check both one-rings
*/
bool FMeshRefinerBase::CheckIfCollapseCreatesFlipOrInvalid(int vid, int vother, const FVector3d& newv, int tc, int td) const
{
	FVector3d va = FVector3d::Zero(), vb = FVector3d::Zero(), vc = FVector3d::Zero();
	for (int tid : Mesh->VtxTrianglesItr(vid)) 
	{
		if (tid == tc || tid == td)
		{
			continue;
		}
		FIndex3i curt = Mesh->GetTriangle(tid);
		if (curt[0] == vother || curt[1] == vother || curt[2] == vother)
		{
			return true;		// invalid nbrhood for collapse
		}
		Mesh->GetTriVertices(tid, va, vb, vc);
		FVector3d ncur = (vb - va).Cross(vc - va);
		double sign = 0;
		if (curt[0] == vid) 
		{
			FVector3d nnew = (vb - newv).Cross(vc - newv);
			sign = ComputeEdgeFlipMetric(ncur, nnew);
		}
		else if (curt[1] == vid) 
		{
			FVector3d nnew = (newv - va).Cross(vc - va);
			sign = ComputeEdgeFlipMetric(ncur, nnew);
		}
		else if (curt[2] == vid) 
		{
			FVector3d nnew = (vb - va).Cross(newv - va);
			sign = ComputeEdgeFlipMetric(ncur, nnew);
		}
		else
		{
			check(false);   // this should never happen!
		}
		if (sign <= EdgeFlipTolerance)
		{
			return true;
		}
	}
	return false;
}


bool FMeshRefinerBase::CheckIfCollapseCreatesTinyTriangle(int VertexID, int OtherVertexID, const FVector3d& NewVertexPosition, int IncidentTriangleC, int IncidentTriangleD) const
{
	for (int TriangleID : Mesh->VtxTrianglesItr(VertexID))
	{
		if (TriangleID == IncidentTriangleC || TriangleID == IncidentTriangleD)
		{
			continue;
		}
		FIndex3i CurrentTriangle = Mesh->GetTriangle(TriangleID);
		if (CurrentTriangle[0] == OtherVertexID || CurrentTriangle[1] == OtherVertexID || CurrentTriangle[2] == OtherVertexID)
		{
			return true;		// invalid nbrhood for collapse
		}

		FVector3d TriangleVertices[3];
		Mesh->GetTriVertices(TriangleID, TriangleVertices[0], TriangleVertices[1], TriangleVertices[2]);

		// If the current triangle is already tiny, *do* allow the creation of a new tiny triangle (to allow remeshing to continue and hopefully improve things)
		FVector3d CurrentNormal = (TriangleVertices[1] - TriangleVertices[0]).Cross(TriangleVertices[2] - TriangleVertices[0]);
		if (CurrentNormal.SquaredLength() < TinyTriangleThreshold)
		{
			continue;
		}

		// Now find the size of the triangle that would result
		TriangleVertices[CurrentTriangle.IndexOf(VertexID)] = NewVertexPosition;
		FVector3d NewNormal = (TriangleVertices[1] - TriangleVertices[0]).Cross(TriangleVertices[2] - TriangleVertices[0]);
		if (NewNormal.SquaredLength() < TinyTriangleThreshold)
		{
			return true;
		}
	}

	return false;
}

/**
 * Check if edge flip might reverse normal direction.
 * Not entirely clear on how to best implement this test. Currently checking if any normal-pairs are reversed.
 */
bool FMeshRefinerBase::CheckIfFlipInvertsNormals(int a, int b, int c, int d, int t0) const
{
	FVector3d vC = Mesh->GetVertex(c), vD = Mesh->GetVertex(d);
	FIndex3i tri_v = Mesh->GetTriangle(t0);
	int oa = a, ob = b;
	IndexUtil::OrientTriEdge(oa, ob, tri_v);
	FVector3d vOA = Mesh->GetVertex(oa), vOB = Mesh->GetVertex(ob);
	FVector3d n0 = VectorUtil::NormalDirection(vOA, vOB, vC);
	FVector3d n1 = VectorUtil::NormalDirection(vOB, vOA, vD);
	FVector3d f0 = VectorUtil::NormalDirection(vC, vD, vOB);
	if (ComputeEdgeFlipMetric(n0, f0) <= EdgeFlipTolerance || ComputeEdgeFlipMetric(n1, f0) <= EdgeFlipTolerance)
	{
		return true;
	}
	FVector3d f1 = VectorUtil::NormalDirection(vD, vC, vOA);
	if (ComputeEdgeFlipMetric(n0, f1) <= EdgeFlipTolerance || ComputeEdgeFlipMetric(n1, f1) <= EdgeFlipTolerance)
	{
		return true;
	}

	// this only checks if output faces are pointing towards eachother, which seems 
	// to still result in normal-flips in some cases
	//if (f0.Dot(f1) < 0)
	//    return true;

	return false;
}


bool FMeshRefinerBase::CheckIfFlipCreatesTinyTriangle(int OriginalEdgeVertexA, int OriginalEdgeVertexB, int OppositeEdgeVertexC, int OppositeEdgeVertexD, int OriginalTriangleIndex) const
{
	FVector3d vC = Mesh->GetVertex(OppositeEdgeVertexC);
	FVector3d vD = Mesh->GetVertex(OppositeEdgeVertexD);
	FVector3d vA = Mesh->GetVertex(OriginalEdgeVertexA);
	FVector3d vB = Mesh->GetVertex(OriginalEdgeVertexB);

	// If the current triangles are already tiny, allow new triangles to be tiny as well
	double CurrNormalSquaredLen = (vA - vC).Cross(vB - vC).SquaredLength();
	if (CurrNormalSquaredLen < TinyTriangleThreshold)
	{
		return false;
	}

	CurrNormalSquaredLen = (vA - vD).Cross(vB - vD).SquaredLength();
	if (CurrNormalSquaredLen < TinyTriangleThreshold)
	{
		return false;
	}

	// Now check triangle area of potential new triangle
	double NewNormalSquaredLen = (vD - vC).Cross(vD - vB).SquaredLength();
	if (NewNormalSquaredLen < TinyTriangleThreshold)
	{
		return true;
	}

	NewNormalSquaredLen = (vD - vC).Cross(vD - vA).SquaredLength();
	if (NewNormalSquaredLen < TinyTriangleThreshold)
	{
		return true;
	}

	return false;
}



/**
 * Figure out if we can collapse edge eid=[a,b] under current constraint set.
 * First we resolve vertex constraints using CanCollapseVertex(). However this
 * does not catch some topological cases at the edge-constraint level, which
 * which we will only be able to detect once we know if we are losing a or b.
 * See comments on CanCollapseVertex() for what collapse_to is for.
 */
bool FMeshRefinerBase::CanCollapseEdge(int eid, int a, int b, int c, int d, int tc, int td, int& collapse_to) const
{
	collapse_to = -1;
	if (!Constraints)
	{
		return true;
	}

	// are the vertices themselves constrained in a way that prevents this collapse? 
	// nb: this modifies collapse_to if we have to keep a particular vert.
	bool bVtx = CanCollapseVertex(eid, a, b, collapse_to);
	if (bVtx == false)
	{
		return false;
	}

	// determine if edge constraints require keeping either vert.
	bool bMustRetainA = false;
	bool bMustRetainB = false;

	// if edge ac is constrained, we must keep it and thus vertex a, likewise for edge bc and vertex b.
	if (c != IndexConstants::InvalidID)
	{
		int32 eac = Mesh->FindEdgeFromTri(a, c, tc);
		int32 ebc = Mesh->FindEdgeFromTri(b, c, tc);

		bMustRetainA = bMustRetainA || (Constraints->GetEdgeConstraint(eac).IsUnconstrained() == false);
		bMustRetainB = bMustRetainB || (Constraints->GetEdgeConstraint(ebc).IsUnconstrained() == false);
		
	}

	// if edge ad is constrained, we must keep it and thus vertex a, likewise for edge bd and vertex b.
	if (d != IndexConstants::InvalidID)
	{
		int32 ead = Mesh->FindEdgeFromTri(a, d, td);
		int32 ebd = Mesh->FindEdgeFromTri(b, d, td);

		bMustRetainA = bMustRetainA || (Constraints->GetEdgeConstraint(ead).IsUnconstrained() == false);
		bMustRetainB = bMustRetainB || (Constraints->GetEdgeConstraint(ebd).IsUnconstrained() == false);
	}

	bool bCanCollapse = true;

	// adjacent edge constraints want us to keep both verts.. no can do.
	if (bMustRetainA && bMustRetainB)
	{
			bCanCollapse = false;
	}
	else
	{
		
		if (collapse_to == -1)
		{
			// the vertex constraints didn't require either vertex.  
			// if the adjacent edge constraints require one, record it 
			if (bMustRetainA && !bMustRetainB)
			{
				collapse_to = a;
			}
			else if (bMustRetainB && !bMustRetainA)
			{
				collapse_to = b;
			}
		}
		else if (collapse_to == a && bMustRetainB)
		{
			// vertex and adjacent edge constraints conflict
			bCanCollapse = false;
		}
		else if (collapse_to == b && bMustRetainA)
		{
			// vertex and adjacent edge constraints conflict
			bCanCollapse = false;
		}

	}

	return bCanCollapse;
}







/**
 * Resolve vertex constraints for collapsing edge eid=[a,b]. Generally we would
 * collapse a to b, and set the new position as 0.5*(v_a+v_b). However if a *or* b
 * are non-deletable, then we want to keep that vertex. This vertex (a or b) will be returned in collapse_to, 
 * which is -1 otherwise.
 * If a *and* b are non-deletable, then things are complicated (and documented below).
 */
bool FMeshRefinerBase::CanCollapseVertex(int eid, int a, int b, int& collapse_to) const
{
	collapse_to = -1;
	if (!Constraints)
	{
		return true;
	}

	FVertexConstraint ca = Constraints->GetVertexConstraint(a);
	FVertexConstraint cb = Constraints->GetVertexConstraint(b);
	
	bool bIsFixedA = (ca.bCanMove == false);
	bool bIsFixedB = (cb.bCanMove == false);

	if (bIsFixedA && bIsFixedB)
	{
		return false;
	}

	bool CanDeleteA = (ca.bCannotDelete == false);
	bool CanDeleteB = (cb.bCannotDelete == false);

	// no constraint at all
	if ( CanDeleteA && CanDeleteB && ca.Target == nullptr && cb.Target == nullptr )
	{
		return true;
	}

	// handle a or b non-deletable
	if ( ca.bCannotDelete && CanDeleteB )
	{
		// if b has a projection target, and it is different than a's target, we can't collapse
		if (cb.Target != nullptr && cb.Target != ca.Target)
		{
			return false;
		}
		collapse_to = a;
		return true;
	}

	if ( cb.bCannotDelete  && CanDeleteA )
	{
		if (ca.Target != nullptr && ca.Target != cb.Target)
		{
			return false;
		}
		collapse_to = b;
		return true;
	}

	// if both are non-deletable, and options allow, treat this edge as unconstrained (eg collapse to midpoint)
	// TODO: tried picking a or b here, but something weird happens, where
	//   eg cylinder cap will entirely erode away. Somehow edge lengths stay below threshold??
	if (AllowCollapseFixedVertsWithSameSetID
		&& ca.FixedSetID >= 0
		&& ca.FixedSetID == cb.FixedSetID) 
	{
		return true;
	}

	// handle a or b w/ target
	if (ca.Target != nullptr && cb.Target == nullptr) 
	{
		collapse_to = a;
		return true;
	}
	if (cb.Target != nullptr && ca.Target == nullptr) 
	{
		collapse_to = b;
		return true;
	}
	// if both vertices are on the same target, and the edge is on that target,
	// then we can collapse to either and use the midpoint (which will be projected
	// to the target). *However*, if the edge is not on the same target, then we 
	// cannot collapse because we would be changing the constraint topology!
	if (cb.Target != nullptr && ca.Target != nullptr && ca.Target == cb.Target) 
	{
		if (Constraints->GetEdgeConstraint(eid).Target == ca.Target)
		{
			return true;
		}
	}

	return false;
}






void FMeshRefinerBase::SetMeshChangeTracker(FDynamicMeshChangeTracker* Tracker)
{
	ActiveChangeTracker = Tracker;
}


void FMeshRefinerBase::SaveTriangleBeforeModify(int32 TriangleID)
{
	if (ActiveChangeTracker)
	{
		ActiveChangeTracker->SaveTriangle(TriangleID, true);
	}
}

void FMeshRefinerBase::SaveVertexTrianglesBeforeModify(int32 VertexID)
{
	if (ActiveChangeTracker)
	{
		Mesh->EnumerateVertexTriangles(VertexID, [&](int32 TriangleID)
		{
			ActiveChangeTracker->SaveTriangle(TriangleID, true);
		});
	}
}


void FMeshRefinerBase::SaveEdgeBeforeModify(int32 EdgeID)
{
	if (ActiveChangeTracker)
	{
		FIndex2i EdgeVerts = Mesh->GetEdgeV(EdgeID);
		SaveVertexTrianglesBeforeModify(EdgeVerts.A);
		SaveVertexTrianglesBeforeModify(EdgeVerts.B);
	}
}



void FMeshRefinerBase::RuntimeDebugCheck(int eid)
{
	if (DebugEdges.Contains(eid))
		ensure(false);
}

void FMeshRefinerBase::DoDebugChecks(bool bEndOfPass)
{
	if (DEBUG_CHECK_LEVEL == 0)
		return;

	DebugCheckVertexConstraints();

	if ((DEBUG_CHECK_LEVEL > 2) || (bEndOfPass && DEBUG_CHECK_LEVEL > 1))
	{
		Mesh->CheckValidity(FDynamicMesh3::FValidityOptions::Permissive());
		DebugCheckUVSeamConstraints();
	}
}


void FMeshRefinerBase::DebugCheckUVSeamConstraints()
{
	// verify UV constraints (temporary?)
	if (Mesh->HasAttributes() && Mesh->Attributes()->PrimaryUV() != nullptr && Constraints)
	{
		for (int eid : Mesh->EdgeIndicesItr())
		{
			if (Mesh->Attributes()->PrimaryUV()->IsSeamEdge(eid))
			{
				auto cons = Constraints->GetEdgeConstraint(eid);
				check(cons.IsUnconstrained() == false);
			}
		}
		for (int vid : Mesh->VertexIndicesItr())
		{
			if (Mesh->Attributes()->PrimaryUV()->IsSeamVertex(vid))
			{
				auto cons = Constraints->GetVertexConstraint(vid);
				check(cons.bCannotDelete == true);
			}
		}
	}
}


void FMeshRefinerBase::DebugCheckVertexConstraints()
{
	if (!Constraints)
	{
		return;
	}
	auto AllVtxConstraints = Constraints->GetVertexConstraints();
	for (const TPair<int, FVertexConstraint>& vc : AllVtxConstraints)
	{
		int vid = vc.Key;
		if (vc.Value.Target != nullptr)
		{
			FVector3d curpos = Mesh->GetVertex(vid);
			FVector3d projected = vc.Value.Target->Project(curpos, vid);
			check((curpos - projected).SquaredLength() < 0.0001f);
		}
	}
}
