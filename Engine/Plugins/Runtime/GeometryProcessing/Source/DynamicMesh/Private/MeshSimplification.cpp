// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "SegmentTypes.h"
#include "Util/IndexUtil.h"
#include "Async/ParallelFor.h"
#include "Templates/UnrealTypeTraits.h"
#include "MeshQueries.h"
#include "MeshSimplificationQuadrics.h"
#include "VectorUtil.h"

using namespace UE::Geometry;

template <typename QuadricErrorType>
inline bool CanAddVertexQuadric()
{
	return true;
}

template <>
inline bool CanAddVertexQuadric<FAttrBasedQuadricErrorV2d>()
{
	// to support wedges, we need more information of the topology when combining vertex quadrics into an edge quadric
	return false;
}


// Helper to get seam quadric scale correction from QuadricOptions
template <typename OptionsType>
inline double GetSeamQuadricScaleCorrection(const OptionsType& Options)
{
	return 1.0;
}

template <>
inline double GetSeamQuadricScaleCorrection<FAttrBasedQuadricErrorV2d::FOptions>(const FAttrBasedQuadricErrorV2d::FOptions& Options)
{
	// Edge is h
	// PlaneQuadric is ~h^2
	// TriangleQuadric is ~h^4
	return (Options.QuadricVariant == FAttrBasedQuadricErrorV2d::EQuadricVariant::PlaneQuadric) ?
		Options.ScaleCorrection :  
		Options.ScaleCorrection * Options.ScaleCorrection * Options.ScaleCorrection;
}

template <typename QuadricErrorType>
QuadricErrorType TMeshSimplification<QuadricErrorType>::ComputeFaceQuadric(const int tid, FVector3d& nface, FVector3d& c, double& Area) const
{
	// compute the new quadric for this tri.
	Mesh->GetTriInfo(tid, nface, Area, c);

	return FQuadricErrorType(nface, c);
}


// Face Quadric Error computation specialized for FAttrBasedQuadricErrord
template<>
FAttrBasedQuadricErrord TMeshSimplification<FAttrBasedQuadricErrord>::ComputeFaceQuadric(const int tid, FVector3d& nface, FVector3d& c, double& Area) const
{
	// compute the new quadric for this tri.
	Mesh->GetTriInfo(tid, nface, Area, c);

	FVector3f n0; FVector3f n1; FVector3f n2;

	if (NormalOverlay != nullptr)
	{
		if (NormalOverlay->IsSetTriangle(tid))
		{
			NormalOverlay->GetTriElements(tid, n0, n1, n2);
		}
		else
		{
			FVector3f FaceNormal = (FVector3f)Mesh->GetTriNormal(tid);
			n0 = n1 = n2 = FaceNormal;
		}
	}
	else
	{
		FIndex3i vids = Mesh->GetTriangle(tid);
		n0 = Mesh->GetVertexNormal(vids[0]);
		n1 = Mesh->GetVertexNormal(vids[1]);
		n2 = Mesh->GetVertexNormal(vids[2]);
	}


	FVector3d p0, p1, p2;
	Mesh->GetTriVertices(tid, p0, p1, p2);

	FVector3d n0d(n0.X, n0.Y, n0.Z);
	FVector3d n1d(n1.X, n1.Y, n1.Z);
	FVector3d n2d(n2.X, n2.Y, n2.Z);

	double attrweight = 16.;
	return FQuadricErrorType(p0, p1, p2, n0d, n1d, n2d, nface, c, attrweight);
}

template<>
FAttrBasedQuadricErrorV2d TMeshSimplification<FAttrBasedQuadricErrorV2d>::ComputeFaceQuadric(const int TID, FVector3d& Nface, FVector3d& C, double& Area) const
{
	// compute the new quadric for this tri.
	Mesh->GetTriInfo(TID, Nface, Area, C);

	FVector3d P0, P1, P2;
	Mesh->GetTriVertices(TID, P0, P1, P2);

	FAttrBasedQuadricErrorV2d Q(P0, P1, P2, QuadricOptions);

	FAttrBasedQuadricErrorV2d::FScopedAttributeDataBuilder ScopedAttributeDataBuilder(Q, P0, P1, P2, Nface, QuadricOptions.ScaleCorrection);

	FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();

	if (QuadricOptions.NormalAttributeWeight > 0.)
	{
		// collect normals
		FIndex3i Tri { FDynamicMesh3::InvalidID, FDynamicMesh3::InvalidID, FDynamicMesh3::InvalidID };

		FVector3f N0, N1, N2;
		if (NormalOverlay != nullptr)
		{
			Tri = NormalOverlay->GetTriangle(TID);
			if (NormalOverlay->IsSetTriangle(TID))
			{
				N0 = NormalOverlay->GetElement(Tri[0]);
				N1 = NormalOverlay->GetElement(Tri[1]);
				N2 = NormalOverlay->GetElement(Tri[2]);
			}
			else
			{
				FVector3f FaceNormal = (FVector3f)Mesh->GetTriNormal(TID);
				N0 = N1 = N2 = FaceNormal;
			}
		}
		else
		{
			Tri = Mesh->GetTriangle(TID);
			N0 = Mesh->GetVertexNormal(Tri[0]);
			N1 = Mesh->GetVertexNormal(Tri[1]);
			N2 = Mesh->GetVertexNormal(Tri[2]);
		}
		
		N0.Normalize();
		N1.Normalize();
		N2.Normalize();

		ScopedAttributeDataBuilder.AddWedgeAttribute( Tri, N0, N1, N2, QuadricOptions.NormalAttributeWeight );
	}
	
	FDynamicMeshNormalOverlay* TangentOverlay = Attributes ? Attributes->PrimaryTangents() : nullptr;
	if (TangentOverlay && QuadricOptions.TangentAttributeWeight > 0.)
	{
		if (TangentOverlay->IsSetTriangle(TID))
		{
			FIndex3i TangentTri = TangentOverlay->GetTriangle(TID);
		
			FVector3f t0 = TangentOverlay->GetElement(TangentTri[0]);
			FVector3f t1 = TangentOverlay->GetElement(TangentTri[1]);
			FVector3f t2 = TangentOverlay->GetElement(TangentTri[2]);

			ScopedAttributeDataBuilder.AddWedgeAttribute( TangentTri, t0, t1, t2, QuadricOptions.TangentAttributeWeight );
		}
		else
		{
			ScopedAttributeDataBuilder.AddMissingWedgeAttribute();
		}
	}

	FDynamicMeshNormalOverlay* BiTangentOverlay = Attributes ? Attributes->PrimaryBiTangents() : nullptr;
	if (BiTangentOverlay && QuadricOptions.TangentAttributeWeight > 0.)
	{
		if (BiTangentOverlay->IsSetTriangle(TID))
		{
			FIndex3i BiTangentTri = BiTangentOverlay->GetTriangle(TID);
			FVector3f b0 = BiTangentOverlay->GetElement(BiTangentTri[0]);
			FVector3f b1 = BiTangentOverlay->GetElement(BiTangentTri[1]);
			FVector3f b2 = BiTangentOverlay->GetElement(BiTangentTri[2]);

			ScopedAttributeDataBuilder.AddWedgeAttribute( BiTangentTri, b0, b1, b2, QuadricOptions.TangentAttributeWeight );
		}
		else
		{
			ScopedAttributeDataBuilder.AddMissingWedgeAttribute();
		}
	}

	FDynamicMeshColorOverlay* ColorOverlay = Attributes ? Attributes->PrimaryColors() : nullptr;
	if (ColorOverlay && QuadricOptions.ColorAttributeWeight > 0.)
	{
		if (ColorOverlay->IsSetTriangle(TID))
		{
			FIndex3i ColorTri = ColorOverlay->GetTriangle(TID);
			FVector4f c0 = ColorOverlay->GetElement(ColorTri[0]);
			FVector4f c1 = ColorOverlay->GetElement(ColorTri[1]);
			FVector4f c2 = ColorOverlay->GetElement(ColorTri[2]);
			
			ScopedAttributeDataBuilder.AddWedgeAttribute( ColorTri, FVector3f(c0.X, c0.Y, c0.Z), FVector3f(c1.X, c1.Y, c1.Z), FVector3f(c2.X, c2.Y, c2.Z), QuadricOptions.ColorAttributeWeight );
		}
		else
		{
			ScopedAttributeDataBuilder.AddMissingWedgeAttribute();
		}
	}

	const int NumWeightLayers = Attributes ? Attributes->NumWeightLayers() : 0;
	if (NumWeightLayers > 0 && QuadricOptions.WeightLayerWeight > 0.)
	{
		const FIndex3i VIDs = Mesh->GetTriangle(TID);
		const int NumWeightGroups = (NumWeightLayers + 2) / 3;
		for (int GroupIndex = 0; GroupIndex < NumWeightGroups; ++GroupIndex)
		{
			FVector3f WeightChannels[3] = { FVector3f(0.f), FVector3f(0.f), FVector3f(0.f) };
			for (int ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				const int WeightLayerIndex = GroupIndex * 3 + ChannelIndex;
				if (WeightLayerIndex < NumWeightLayers)
				{
					const FDynamicMeshWeightAttribute* WeightAttribute = Attributes->GetWeightLayer(WeightLayerIndex);
					WeightAttribute->GetValue<float*const>(VIDs[0], &WeightChannels[0][ChannelIndex]);
					WeightAttribute->GetValue<float*const>(VIDs[1], &WeightChannels[1][ChannelIndex]);
					WeightAttribute->GetValue<float*const>(VIDs[2], &WeightChannels[2][ChannelIndex]);
				}
			}
			ScopedAttributeDataBuilder.AddWedgeAttribute( VIDs, WeightChannels[0], WeightChannels[1], WeightChannels[2], QuadricOptions.WeightLayerWeight );
		}
	}

	const int NumUVOverlays = Attributes ? Attributes->NumUVLayers() : 0;
	for (int UVLayerIndex = 0; UVLayerIndex < NumUVOverlays; ++UVLayerIndex)
	{
		FDynamicMeshUVOverlay* UVOverlay = Attributes->GetUVLayer(UVLayerIndex);
		if (UVOverlay && QuadricOptions.TexCoordAttributeWeight > 0.)
		{
			if (UVOverlay->IsSetTriangle(TID))
			{
				// Collect UVs
				FIndex3i UVTri = UVOverlay->GetTriangle(TID);
				FVector2f uv0 = UVOverlay->GetElement(UVTri[0]);
				FVector2f uv1 = UVOverlay->GetElement(UVTri[1]);
				FVector2f uv2 = UVOverlay->GetElement(UVTri[2]);
				ScopedAttributeDataBuilder.AddWedgeAttribute( UVTri, FVector3f(uv0.X, uv0.Y, 0.f), FVector3f(uv1.X, uv1.Y, 0.f), FVector3f(uv2.X, uv2.Y, 0.f), QuadricOptions.TexCoordAttributeWeight  );
			}
			else
			{
				ScopedAttributeDataBuilder.AddMissingWedgeAttribute();
			}
		}
	}

	return Q;
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::InitializeTriQuadrics()
{
	const int NT = Mesh->MaxTriangleID();
	triQuadrics.SetNum(NT);
	triAreas.SetNum(NT);
	if (RegularizeUsesNormals())
	{
		triNormals.SetNum(NT);
	}

	// tested with ParallelFor - no measurable benefit
	//@todo parallel version
	//gParallel.BlockStartEnd(0, Mesh->MaxTriangleID - 1, (start_tid, end_tid) = > {
	FVector3d n(0), c;
	for (int tid : Mesh->TriangleIndicesItr())
	{
		triQuadrics[tid] = ComputeFaceQuadric(tid, triNormals.IsEmpty() ? n : triNormals[tid], c, triAreas[tid]);
	}

}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::InitializeSeamQuadrics()
{
	// early out if this feature isn't needed.
	if (!bAllowSeamCollapse)
	{
		return;
	}

	double EdgeWeight = this->SeamEdgeWeight;

	auto AddSeamQuadric = [EdgeWeight, this](int eid)
	{
		FDynamicMesh3::FEdge edge = Mesh->GetEdge(eid);
		FVector3d p0 = Mesh->GetVertex(edge.Vert[0]);
		FVector3d p1 = Mesh->GetVertex(edge.Vert[1]);

		// face normal 
		FVector3d nA = Mesh->GetTriNormal(edge.Tri.A);

		// this constrains the point to a plane aligned with the edge and normal to the face
		FSeamQuadricType& seamQuadric = seamQuadrics.Add(eid, CreateSeamQuadric<double>(p0, p1, nA));

		// add the other side - this constrains the point to the line where the two planes intersect.
		if (edge.Tri.B != FDynamicMesh3::InvalidID)
		{
			FVector3d nB = Mesh->GetTriNormal(edge.Tri.B);
			seamQuadric.Add(CreateSeamQuadric<double>(p0, p1, nB));
		}

		seamQuadric.Scale(EdgeWeight * GetSeamQuadricScaleCorrection(this->QuadricOptions));
	};

	if (Constraints) // The edge constraints an entry for each seam, boundary, group boundary and material boundary
	{
		const auto& EdgeConstraints = Constraints->GetEdgeConstraints();

		for (auto& ConstraintPair : EdgeConstraints)
		{
			int eid = ConstraintPair.Key;

			AddSeamQuadric(eid);
		}

	}
	else
	{
		const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();

		for (int eid : Mesh->EdgeIndicesItr())
		{
			bool bNeedsQuadric = Mesh->IsBoundaryEdge(eid);
			bNeedsQuadric = bNeedsQuadric || Mesh->IsGroupBoundaryEdge(eid);
			if (Attributes)
			{
				bNeedsQuadric = bNeedsQuadric || Attributes->IsMaterialBoundaryEdge(eid);
				bNeedsQuadric = bNeedsQuadric || Attributes->IsSeamEdge(eid);
			}

			if (bNeedsQuadric)
			{
				AddSeamQuadric(eid);
			}
		}
	}
}


template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::InitializeVertexQuadrics()
{

	int NV = Mesh->MaxVertexID();
	vertQuadrics.SetNum(NV);

	if (!bRetainQuadricMemory && CustomQuadricErrorScaleF)
	{
		AppliedVertexQuadricScales.Init(1., NV);
	}
	else
	{
		AppliedVertexQuadricScales.Empty();
	}

	// tested with ParallelFor - no measurable benefit 
	//gParallel.BlockStartEnd(0, Mesh->MaxVertexID - 1, (start_vid, end_vid) = > {
	for (int vid : Mesh->VertexIndicesItr())
	{
		FQuadricErrorType Q = FQuadricErrorType::Zero();
		
		double AreaSum = 0;
		FVector3d Normal(0);
		bool bNeedsNormals = RegularizeUsesNormals();

		for (int tid : Mesh->VtxTrianglesItr(vid))
		{
			AreaSum += triAreas[tid];
			if (bNeedsNormals)
			{
				Normal += triAreas[tid] * triNormals[tid];
			}
			Q.AccumulateFaceQuadric(IndexUtil::FindTriIndex(vid, Mesh->GetTriangle(tid)), triAreas[tid], 1., triQuadrics[tid]);
		}
		ApplyScalingAndRegularizationToVertexQuadric(Q, vid, AreaSum, Normal);
		
		vertQuadrics[vid] = Q;
	}
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::ApplyScalingAndRegularizationToVertexQuadric(FQuadricErrorType& Q, int32 VID, double AreaSum, FVector3d Normal)
{
	if (RegularizeWeight > 0.)
	{
		double UseRegularizeWeight = RegularizeWeight;

		FVector3d VertPos = Mesh->GetVertex(VID);
		if (RegularizeMethod == ERegularizeSimplificationMethod::NormalLine)
		{
			// Note: Very large weights line-regularizer weights introduce instability in the solve
			// so we clamp the map weight in this case
			constexpr double MaxLineRegularizerWeight = 100.;
			UseRegularizeWeight = FMath::Min(MaxLineRegularizerWeight, UseRegularizeWeight);
			Q.AddLineRegularizer(VertPos, Normal, UseRegularizeWeight * AreaSum);
		}
		else
		{
			Q.AddPositionRegularizer(VertPos, UseRegularizeWeight * AreaSum);
		}
	}
	if (CustomQuadricErrorScaleF)
	{
		double QEMScale = FMath::Max(UE_DOUBLE_SMALL_NUMBER, CustomQuadricErrorScaleF(*Mesh, VID));
		if (FMath::IsFinite(QEMScale))
		{
			if (!AppliedVertexQuadricScales.IsEmpty())
			{
				AppliedVertexQuadricScales[VID] = QEMScale;
			}
			Q.Scale(QEMScale);
		}
	}
}

template <typename QuadricErrorType>
QuadricErrorType TMeshSimplification<QuadricErrorType>::AssembleEdgeQuadric(const FDynamicMesh3::FEdge& edge) const
{
	//  form standard edge quadric as sum of the vertex quadrics for the edge endpoints
	QuadricErrorType EdgeQuadric = MergeVertexQuadricsToEdgeQuadric(edge);
	
	if (!EdgeQuadric.CanCollapse())
	{
		return EdgeQuadric;
	}

	if (!bRetainQuadricMemory || !CanAddVertexQuadric<QuadricErrorType>())
	{
		// the edge.Tri faces are double counted. Remove one.
		// if using scaled vertex quadrics, removed the smaller of the two
		double RemoveScale = 1.;
		if (!AppliedVertexQuadricScales.IsEmpty())
		{
			RemoveScale = FMath::Min(AppliedVertexQuadricScales[edge.Vert.A], AppliedVertexQuadricScales[edge.Vert.B]);
		}
		const FIndex2i& Tris = edge.Tri;
		if (Tris.A != FDynamicMesh3::InvalidID)
		{
			const int CornerIdx = IndexUtil::FindEdgeIndexInTri(edge.Vert.A, edge.Vert.B, Mesh->GetTriangle(Tris.A));
			EdgeQuadric.RemoveFaceQuadric(CornerIdx, triAreas[Tris.A], RemoveScale, triQuadrics[Tris.A]);
		}

		if (Tris.B != FDynamicMesh3::InvalidID)
		{
			const int CornerIdx = IndexUtil::FindEdgeIndexInTri(edge.Vert.A, edge.Vert.B, Mesh->GetTriangle(Tris.B));
			EdgeQuadric.RemoveFaceQuadric(CornerIdx, triAreas[Tris.B], RemoveScale, triQuadrics[Tris.B]);
		}
	}
	
	if (bAllowSeamCollapse)
	{ 
		// lambda that adds any adjacent seam quadrics to the edge quadric
		auto AddSeamQuadricsToEdge = [&, this](int vid)
		{
			for (int eid : Mesh->VtxEdgesItr(vid))
			{
				if (const FSeamQuadricType* seamQuadric =  seamQuadrics.Find(eid))
				{
					EdgeQuadric.AddSeamQuadric(*seamQuadric);
				} 
			}
		};
	
		// accumulate any adjacent seam quadrics onto this edge quadric.
		AddSeamQuadricsToEdge(edge.Vert.A);
		AddSeamQuadricsToEdge(edge.Vert.B);
	}

	return EdgeQuadric;
}


template <typename QuadricErrorType>
QuadricErrorType TMeshSimplification<QuadricErrorType>::MergeVertexQuadricsToEdgeQuadric(const FDynamicMesh3::FEdge& edge) const
{
	// form standard edge quadric as sum of the vertex quadrics for the edge endpoints
	QuadricErrorType EdgeQuadric(vertQuadrics[edge.Vert.A], vertQuadrics[edge.Vert.B]);
	
	return EdgeQuadric;
}

template <>
FAttrBasedQuadricErrorV2d TMeshSimplification<FAttrBasedQuadricErrorV2d>::MergeVertexQuadricsToEdgeQuadric(
	const FDynamicMesh3::FEdge& edge) const
{
	
	// the edge.Tri faces are double counted. Remove one.
	// if using scaled vertex quadrics, remove the smaller of the two
	double RemoveScale = 1.;
	if (!AppliedVertexQuadricScales.IsEmpty())
	{
		RemoveScale = FMath::Min(AppliedVertexQuadricScales[edge.Vert.A], AppliedVertexQuadricScales[edge.Vert.B]);
	}

	const FAttrBasedQuadricErrorV2d* TriangleQuadricA = nullptr;
	const FAttrBasedQuadricErrorV2d* TriangleQuadricB = nullptr;
	int LocalEdgeIndexA = IndexConstants::InvalidID;
	int LocalEdgeIndexB = IndexConstants::InvalidID;

	const FIndex2i Tris = edge.Tri;

	if (Tris.A != FDynamicMesh3::InvalidID)
	{
		LocalEdgeIndexA = IndexUtil::FindEdgeIndexInTri(edge.Vert.A, edge.Vert.B, Mesh->GetTriangle(Tris.A));
		check(LocalEdgeIndexA != IndexConstants::InvalidID);
		TriangleQuadricA = &triQuadrics[Tris.A];
	}
	if (Tris.B != FDynamicMesh3::InvalidID)
	{
		LocalEdgeIndexB = IndexUtil::FindEdgeIndexInTri(edge.Vert.A, edge.Vert.B, Mesh->GetTriangle(Tris.B));
		check(LocalEdgeIndexB != IndexConstants::InvalidID);
		TriangleQuadricB = &triQuadrics[Tris.B];
	}

	if (LocalEdgeIndexA != IndexConstants::InvalidID && LocalEdgeIndexB != IndexConstants::InvalidID)
	{
		const FIndex3i TriA = Mesh->GetTriangle(Tris.A);
		const FIndex3i TriB = Mesh->GetTriangle(Tris.B);

		if ( (TriA[LocalEdgeIndexA] != TriB[(LocalEdgeIndexB+1)%3]) || 
		     (TriB[LocalEdgeIndexB] != TriA[(LocalEdgeIndexA+1)%3]) )
		{
			// bowtie
			return FAttrBasedQuadricErrorV2d(); // not collapsable
		}
	}
	
	auto CheckOverlayCanCollapse = [&]<typename OverlayType>(const OverlayType& Overlay) -> bool
	{
		if (Overlay.IsSetTriangle(Tris.A) && Overlay.IsSetTriangle(Tris.B))
		{
			const FIndex3i& OverlayTriA = Overlay.GetTriangle(Tris.A);
			const FIndex3i& OverlayTriB = Overlay.GetTriangle(Tris.B);

			const bool MatchV0 = OverlayTriA[LocalEdgeIndexA] == OverlayTriB[(LocalEdgeIndexB+1)%3];
			const bool MatchV1 = OverlayTriB[LocalEdgeIndexB] == OverlayTriA[(LocalEdgeIndexA+1)%3];
		
			return MatchV0 == MatchV1; // either match EIDs at both vertices (no seam, or non of them matches, but not just one)

			// TODO check the link condition, to ensure there are no duplicate EIDs in the two sets of attribute one-rings.
		}
		return true;
	};

	if (Tris.B != IndexConstants::InvalidID)
	{
		// check for valid seams on each attribute

		if (NormalOverlay != nullptr && !CheckOverlayCanCollapse(*NormalOverlay))
		{
			return FAttrBasedQuadricErrorV2d(); // not collapsable
		}

		FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
		if (Attributes)
		{
			FDynamicMeshNormalOverlay* TangentOverlay = Attributes->PrimaryTangents();
			if (TangentOverlay && !CheckOverlayCanCollapse(*TangentOverlay))
			{
				return FAttrBasedQuadricErrorV2d();
			}

			FDynamicMeshNormalOverlay* BiTangentOverlay = Attributes->PrimaryBiTangents();
			if (BiTangentOverlay && !CheckOverlayCanCollapse(*BiTangentOverlay))
			{
				return FAttrBasedQuadricErrorV2d();
			}

			FDynamicMeshColorOverlay* ColorOverlay = Attributes->PrimaryColors();
			if (ColorOverlay && !CheckOverlayCanCollapse(*ColorOverlay))
			{
				return FAttrBasedQuadricErrorV2d();
			}

			const int NumUVOverlays = Attributes->NumUVLayers();
			for (int UVLayerIndex = 0; UVLayerIndex < NumUVOverlays; ++UVLayerIndex)
			{
				FDynamicMeshUVOverlay* UVOverlay = Attributes->GetUVLayer(UVLayerIndex);
				if (UVOverlay && !CheckOverlayCanCollapse(*UVOverlay))
				{
					return FAttrBasedQuadricErrorV2d();
				}
			}
		}
	}

	return FAttrBasedQuadricErrorV2d(vertQuadrics[edge.Vert.A], 
		                            vertQuadrics[edge.Vert.B],
									TriangleQuadricA,
									TriangleQuadricB,
									LocalEdgeIndexA,
									LocalEdgeIndexB,
									RemoveScale);
	
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::InitializeQueue()
{
	int NE = Mesh->EdgeCount();
	int MaxEID = Mesh->MaxEdgeID();

	EdgeQuadrics.SetNum(MaxEID);
	EdgeQueue.Initialize(MaxEID);
	TArray<FEdgeError> EdgeErrors;
	EdgeErrors.Init(FEdgeError{MAX_FLT, -1}, MaxEID);

	for (int eid : Mesh->EdgeIndicesItr())
	{
		FDynamicMesh3::FEdge edge = Mesh->GetEdge(eid);

		FQuadricErrorType Q = AssembleEdgeQuadric(edge);
		if (Q.CanCollapse()) 
		{
			FVector3d opt = OptimalPoint(eid, Q, edge.Vert.A, edge.Vert.B);
			EdgeErrors[eid] = { (float)Q.Evaluate(opt), eid };
			EdgeQuadrics[eid] = QEdge(eid, Q, opt);
		}
	}

	// sorted pq insert is faster, so sort edge errors array and index map
	EdgeErrors.Sort();

	// now do inserts
	int N = EdgeErrors.Num();
	for (int i = 0; i < N; ++i)
	{
		int eid = EdgeErrors[i].eid;
		if (Mesh->IsEdge(eid))
		{
			QEdge& edge = EdgeQuadrics[eid];
			float error = EdgeErrors[i].error;
			EdgeQueue.Insert(eid, error);
		}
	}

	/*
	// previous code that does unsorted insert. This is marginally slower, but
	// might get even slower on larger meshes? have only tried up to about 350k.
	// (still, this function is not the bottleneck...)
	int cur_eid = StartEdges();
	bool done = false;
	do {
		if (Mesh->IsEdge(cur_eid)) {
			QEdge edge = EdgeQuadrics[cur_eid];
			double err = errList[cur_eid];
			EdgeQueue.Enqueue(cur_eid, (float)err);
		}
		cur_eid = GetNextEdge(cur_eid, out done);
	} while (done == false);
	*/
}




template <typename QuadricErrorType>
FVector3d TMeshSimplification<QuadricErrorType>::OptimalPoint(int eid, const FQuadricErrorType& q, int ea, int eb)
{
	// Common tracking used when we pick the best of multiple points below
	FVector3d BestPt(0);
	double BestErr = FMathd::MaxReal;
	auto InitBestPt = [&BestPt, &BestErr, &q](const FVector3d& Pt)
	{
		BestErr = q.Evaluate(Pt);
		BestPt = Pt;
	};
	auto UpdateBestPt = [&BestPt, &BestErr, &q](const FVector3d& Pt)
	{
		double Err = q.Evaluate(Pt);
		if (Err < BestErr)
		{
			BestErr = Err;
			BestPt = Pt;
		}
	};

	// if we would like to preserve boundary, we need to know that here
	// so that we properly score these edges
	if (bHaveBoundary && bPreserveBoundaryShape)
	{
		if (Mesh->IsBoundaryEdge(eid))
		{
			const bool bModeAllowsVertMovement = (CollapseMode != ESimplificationCollapseModes::MinimalExistingVertexError);
			if (bModeAllowsVertMovement)
			{
				if (CollapseMode == ESimplificationCollapseModes::MinimalQuadricPositionError)
				{
					// We project the optimal point to the segment, but since this may not be minimal after projection,
					// we use this point only if it has lower error than the segment mid and endpoints.
					// Note: Could consider doing a line search here instead

					FVector3d A = Mesh->GetVertex(ea);
					FVector3d B = Mesh->GetVertex(eb);
					FSegment3d EdgeSeg(A, B);

					InitBestPt(A);
					UpdateBestPt(B);
					UpdateBestPt(EdgeSeg.Center);

					FVector3d UnconstrainedResult = FVector3d::Zero();
					if (q.OptimalPoint(UnconstrainedResult))
					{
						FVector3d ProjectedOpt = EdgeSeg.NearestPoint(UnconstrainedResult);
						UpdateBestPt(ProjectedOpt);
					}
					return BestPt;
				}
				else // ESimplificationCollapseModes::AverageVertexPosition
				{
					return (Mesh->GetVertex(ea) + Mesh->GetVertex(eb)) * 0.5;
				}
			} // else MinimalExistingVertexError case below will choose one of the vertex locations
		}
		else
		{
			if (IsBoundaryVertex(ea))
			{
				return Mesh->GetVertex(ea);
			}
			else if (IsBoundaryVertex(eb))
			{
				return Mesh->GetVertex(eb);
			}
		}
	}

	// [TODO] if we have constraints, we should apply them here, for same reason as bdry above...

	switch (CollapseMode)
	{
		case ESimplificationCollapseModes::AverageVertexPosition:
		{
			return GetProjectedPoint((Mesh->GetVertex(ea) + Mesh->GetVertex(eb)) * 0.5);
		}
		break;

		case ESimplificationCollapseModes::MinimalExistingVertexError:
		{
			InitBestPt(Mesh->GetVertex(ea));
			UpdateBestPt(Mesh->GetVertex(eb));
			return BestPt;
		}
		break;

		case ESimplificationCollapseModes::MinimalQuadricPositionError:
		{
			FVector3d result = FVector3d::Zero();
			if (q.OptimalPoint(result))
			{
				return GetProjectedPoint(result);
			}

			// degenerate matrix, evaluate quadric at edge end and midpoints
			// (could do line search here...)
			FVector3d VA = Mesh->GetVertex(ea);
			FVector3d VB = Mesh->GetVertex(eb);
			InitBestPt(VA);
			UpdateBestPt(VB);
			UpdateBestPt(GetProjectedPoint((VA + VB) * 0.5));
			return BestPt;
		}
		break;
	default:

		// should never happen
		checkSlow(0);
		return FVector3d::Zero();
	}
}




template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::UpdateNeighborhood(const FDynamicMesh3::FEdgeCollapseInfo& collapseInfo)
{	
	int kvid = collapseInfo.KeptVertex;
	int rvid = collapseInfo.RemovedVertex;

	FIndex2i removedTris = collapseInfo.RemovedTris;
	FIndex2i opposingVerts = collapseInfo.OpposingVerts;

	// --- Update the seam quadrics
	if (bAllowSeamCollapse)
	{
		FIndex2i removedEdges = collapseInfo.RemovedEdges;
		FIndex2i keptEdges = collapseInfo.KeptEdges;

		// update the map between edge id and seam quadrics 
		// if constraints exist, they define the edges with seam quadrics
		// otherwise require kept edges to have a seam quadric if either 
		// the kept or collapse edge had a seam quadric.
		if (Constraints)  // quadrics on the constrained edges
		{
			if (Constraints->HasEdgeConstraint(keptEdges.A))
			{
				seamQuadrics.Add(keptEdges.A);
			}
			else
			{
				seamQuadrics.Remove(keptEdges.A);
			}
			
			if (keptEdges.B != FDynamicMesh3::InvalidID)
			{ 
				if( Constraints->HasEdgeConstraint(keptEdges.B))
				{
					seamQuadrics.Add(keptEdges.B);
				}
				else
				{
					seamQuadrics.Remove(keptEdges.B);
				}
			}
		}
		else // propagate any existing seam quadric requirements.
		{			
			if (FSeamQuadricType* seamQuadric = seamQuadrics.Find(removedEdges.A))
			{
				seamQuadrics.Add(keptEdges.A);
			}
			if (removedEdges.B != FDynamicMesh3::InvalidID)
			{
				if (FSeamQuadricType* seamQuadric = seamQuadrics.Find(removedEdges.B))
				{
					seamQuadrics.Add(keptEdges.B);
				}
			}
		}

		// removed quadrics from deleted edges
		seamQuadrics.Remove(removedEdges.A);
		if (removedEdges.B != FDynamicMesh3::InvalidID)
		{
			seamQuadrics.Remove(removedEdges.B);
		}
		
		// update any seam quadrics adjacent to kvid to reflect changes in the seams
	
		double EdgeWeight = this->SeamEdgeWeight;

		for (int eid : Mesh->VtxEdgesItr(kvid))
		{
			FDynamicMesh3::FEdge ne = Mesh->GetEdge(eid);

			// need to recompute this seam quadric
			if (FSeamQuadricType* seamQuadric = seamQuadrics.Find(eid))
			{
				// rebuild the seam quadric

				FVector3d p0 = Mesh->GetVertex(ne.Vert[0]);
				FVector3d p1 = Mesh->GetVertex(ne.Vert[1]);

				// face normal 
				FVector3d nA = Mesh->GetTriNormal(ne.Tri.A);

				// this constrains the point to a plane aligned with the edge and normal to the face
				*seamQuadric = CreateSeamQuadric<double>(p0, p1, nA);
				// add the other side - this constrains the point to the line where the two planes intersect.
				if (ne.Tri.B != FDynamicMesh3::InvalidID)
				{
					FVector3d nB = Mesh->GetTriNormal(ne.Tri.B);
					seamQuadric->Add(CreateSeamQuadric<double>(p0, p1, nB));
				}

				seamQuadric->Scale(EdgeWeight * GetSeamQuadricScaleCorrection(this->QuadricOptions));
			}
		}
	}

	// --- Update the vertex quadrics
	if (bRetainQuadricMemory && CanAddVertexQuadric<QuadricErrorType>())
	{ 
		// Quadric "memory"  the retained vertex quadric is the sum of the two vert quadrics
		vertQuadrics[kvid] = QuadricErrorType(vertQuadrics[kvid], vertQuadrics[rvid]);
	}
	else
	{
		// compute the change in affected face quadrics, and then propagate 
		// that change to the face adjacent verts.
		FVector3d n, c;
		double NewTriArea;

		// Update the triangle areas and quadrics that will have changed
		for (int tid : Mesh->VtxTrianglesItr(kvid))
		{
			const double OldTriArea = triAreas[tid];
			const FQuadricErrorType OldTriQuadric = triQuadrics[tid];

			// compute the new quadric for this tri.
			FQuadricErrorType NewTriQuadric = ComputeFaceQuadric(tid, n, c, NewTriArea);

			// update the arrays that hold the current face area & quadric
			triAreas[tid] = NewTriArea;
			triQuadrics[tid] = NewTriQuadric;
			if (!triNormals.IsEmpty())
			{
				triNormals[tid] = n;
			}

			FIndex3i tri_vids = Mesh->GetTriangle(tid);

			// update the vert quadrics that are adjacent to vid.
			for (int32 i = 0; i < 3; ++i)
			{
				if (tri_vids[i] == kvid) continue;

				// correct the adjacent vertQuadrics
				double VertexScale = AppliedVertexQuadricScales.IsEmpty() ? 1. : AppliedVertexQuadricScales[tri_vids[i]];

				vertQuadrics[tri_vids[i]].RemoveFaceQuadric(i, OldTriArea, VertexScale, OldTriQuadric); // subtract old quadric
				vertQuadrics[tri_vids[i]].AccumulateFaceQuadric(i, NewTriArea, VertexScale, NewTriQuadric); // subtract old quadric
			}
		}

		// remove the influence of the dead tris from the two verts that were opposing the collapsed edge
		{
			for (int i = 0; i < 2; ++i)
			{
				if (removedTris[i] != FDynamicMesh3::InvalidID)
				{
					const double   oldArea = triAreas[removedTris[i]];
					const FQuadricErrorType& oldQuadric = triQuadrics[removedTris[i]];

					const int CornerIndex = IndexUtil::FindTriIndex(opposingVerts[i], Mesh->GetTriangle(removedTris[i]));

					// subtract the quadric from the opposing vert
					double VertexScale = AppliedVertexQuadricScales.IsEmpty() ? 1. : AppliedVertexQuadricScales[opposingVerts[i]];
					vertQuadrics[opposingVerts[i]].RemoveFaceQuadric(CornerIndex, oldArea, VertexScale, oldQuadric);

					// zero out the quadric & area for the removed tris.
					triQuadrics[removedTris[i]] = FQuadricErrorType::Zero();
					triAreas[removedTris[i]] = 0.;
				}
			}
		}

		// Rebuild the quadric for the vert that was retained during the collapse.
		// NB: in the version with memory this quadric took the value of the edge quadric that collapsed.
		{
			FQuadricErrorType vertQuadric = FQuadricErrorType::Zero();
			double AreaSum = 0;
			FVector3d Normal(0);
			bool bNeedsNormals = RegularizeUsesNormals();

			for (int tid : Mesh->VtxTrianglesItr(kvid))
			{
				AreaSum += triAreas[tid];
				
				if (bNeedsNormals)
				{
					Normal += triAreas[tid] * triNormals[tid];
				}
				vertQuadric.AccumulateFaceQuadric(IndexUtil::FindTriIndex(kvid, Mesh->GetTriangle(tid)), triAreas[tid], 1., triQuadrics[tid]);
			}
			ApplyScalingAndRegularizationToVertexQuadric(vertQuadric, kvid, AreaSum, Normal);
			vertQuadrics[kvid] = vertQuadric;
		}
	}

	// --- Update all edge quadrics in the nbrhood
	// NB: this has to follow updating all potential seam quadrics adjacent to kvid 
	// because an edge quadric gathers seam quadrics adjacent the ends 

	if (bRetainQuadricMemory && CanAddVertexQuadric<QuadricErrorType>())
	{ 
		for (int eid : Mesh->VtxEdgesItr(kvid))
		{
			FDynamicMesh3::FEdge ne = Mesh->GetEdge(eid);

			QuadricErrorType Q = AssembleEdgeQuadric(ne);

			if (Q.CanCollapse())
			{
				FVector3d opt = OptimalPoint(eid, Q, ne.Vert.A, ne.Vert.B);
				float err = (float)Q.Evaluate(opt);
				EdgeQuadrics[eid] = QEdge(eid, Q, opt);

				if (EdgeQueue.Contains(eid))
				{
					EdgeQueue.Update(eid, err);
				}
				else
				{
					EdgeQueue.Insert(eid, err);
				}
			}
			else
			{
				EdgeQueue.Update(eid, MAX_FLT);
			}
		}
	}
	else
	{
		TArray<int, TInlineAllocator<64>> EdgesToUpdate;
		for (int adjeid : Mesh->VtxEdgesItr(kvid))
		{
			EdgesToUpdate.Add(adjeid);

			const FIndex2i Verts = Mesh->GetEdgeV(adjeid);
			int adjvid = (Verts[0] == kvid) ? Verts[1] : Verts[0];
			if (adjvid != FDynamicMesh3::InvalidID)
			{
				for (int eid : Mesh->VtxEdgesItr(adjvid))
				{
					if (eid != adjeid)
					{
						EdgesToUpdate.AddUnique(eid);
					}
				}
			}
		}

		for (int eid : EdgesToUpdate)
		{
		
			const FDynamicMesh3::FEdge edgeData = Mesh->GetEdge(eid);
			FQuadricErrorType Q = AssembleEdgeQuadric(edgeData);

			FVector3d opt = OptimalPoint(eid, Q, edgeData.Vert[0], edgeData.Vert[1]);
			float err = (float)Q.Evaluate(opt);
			EdgeQuadrics[eid] = QEdge(eid, Q, opt);
			if (EdgeQueue.Contains(eid))
			{
				EdgeQueue.Update(eid, err);
			}
			else
			{
				EdgeQueue.Insert(eid, err);
			}
		}
	}
}





template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::Precompute(bool bMeshIsClosed)
{
	bHaveBoundary = false;
	IsBoundaryVtxCache.Reset();
	IsBoundaryVtxCache.SetNumZeroed(Mesh->MaxVertexID());
	if (bMeshIsClosed == false)
	{
		for (int eid : Mesh->BoundaryEdgeIndicesItr())
		{
			FIndex2i ev = Mesh->GetEdgeV(eid);
			IsBoundaryVtxCache[ev.A] = true;
			IsBoundaryVtxCache[ev.B] = true;
			bHaveBoundary = true;
		}
	}
}




template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::DoSimplify()
{
	if (Mesh->TriangleCount() == 0)    // badness if we don't catch this...
	{
		return;
	}

	if (Mesh->HasAttributes() && GetConstraints().IsSet() == false)
	{
		ensureMsgf(false, TEXT("Input Mesh has Attribute overlays but no Constraints are configured. Use FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams() to create a Constraint Set for Attribute seams."));
	}

	ProfileBeginPass();

	ProfileBeginSetup();
	Precompute();
	if (Cancelled())
	{
		return;
	}
	InitializeTriQuadrics();
	if (Cancelled())
	{
		return;
	}
	InitializeSeamQuadrics();
	if (Cancelled())
	{
		return;
	}
	InitializeVertexQuadrics();
	if (Cancelled())
	{
		return;
	}

	InitializeQueue();
	if (Cancelled())
	{
		return;
	}
	ProfileEndSetup();

	ProfileBeginOps();
	ProfileBeginCollapse();

	while (EdgeQueue.GetCount() > 0)
	{
		// termination criteria
		if (SimplifyMode == ETargetModes::VertexCount)
		{
			if (Mesh->VertexCount() <= TargetCount)
			{
				break;
			}
		}
		else if (SimplifyMode == ETargetModes::MaxError)
		{
			float qe = EdgeQueue.GetFirstNodePriority();
			if (FMath::Abs(qe) > MaxErrorAllowed && !ShouldIgnoreStoppingCriteria())
			{
				break;
			}
		}
		else
		{
			if (Mesh->TriangleCount() <= TargetCount)
			{
				break;
			}
		}
		
		COUNT_ITERATIONS++;	
		int eid = EdgeQueue.Dequeue(); 
		if (Mesh->IsEdge(eid) == false)
		{
			continue;
		}
		if (Cancelled())
		{
			return;
		}

		FDynamicMesh3::FEdgeCollapseInfo collapseInfo;

		// CollapseEdge checks whether the edge can be collapsed later, but some conditions need to be met
		// already during edge quadric construction time.
		// 
		if (!EdgeQuadrics[eid].q.CanCollapse())
		{
			continue;
		}

		ESimplificationResult result = CollapseEdge(eid, EdgeQuadrics[eid].collapse_pt, collapseInfo);

		if (result == ESimplificationResult::Ok_Collapsed)
		{
			// update the quadrics
			UpdateNeighborhood(collapseInfo);

		}
		else if (result == ESimplificationResult::Failed_IsolatedTriangle && Mesh->TriangleCount() > 2)
		{
			const FDynamicMesh3::FEdge Edge = Mesh->GetEdge(eid);
			RemoveIsolatedTriangle(Edge.Tri.A);
		}
	}
	ProfileEndCollapse();
	ProfileEndOps();

	if (Cancelled())
	{
		return;
	}
	// [TODO] - consider, skip this when CollapseMode == ESimplificationCollapseModes::MinimalExistingVertexError ? 
	Reproject();

	ProfileEndPass();
}


template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::SimplifyToTriangleCount(int nCount)
{
	SimplifyMode = ETargetModes::TriangleCount;
	TargetCount = FMath::Max(1, nCount);
	MinEdgeLength = FMathd::MaxReal;
	MaxErrorAllowed = FMathf::MaxReal;
	DoSimplify();
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::SimplifyToVertexCount(int nCount)
{
	SimplifyMode = ETargetModes::VertexCount;
	TargetCount = FMath::Max(3, nCount);
	MinEdgeLength = FMathd::MaxReal;
	MaxErrorAllowed = FMathf::MaxReal;
	DoSimplify();
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::SimplifyToEdgeLength(double minEdgeLen)
{
	SimplifyMode = ETargetModes::MinEdgeLength;
	TargetCount = 1;
	MinEdgeLength = minEdgeLen;
	MaxErrorAllowed = FMathf::MaxReal;
	DoSimplify();
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::SimplifyToMaxError(double MaxError)
{
	SimplifyMode = ETargetModes::MaxError;
	TargetCount = 1;
	MinEdgeLength = FMathd::MaxReal;
	MaxErrorAllowed = (float)MaxError;
	DoSimplify();
}



template<typename GetTriNormalFuncType>
static bool IsDevelopableVertex(const FDynamicMesh3& Mesh, int32 VertexID, double DotTolerance,
	GetTriNormalFuncType GetTriNormalFunc)
{
	FVector3d Normal1, Normal2;
	int32 Normal1Count = 0, Normal2Count = 0, OtherCount = 0;
	Mesh.EnumerateVertexTriangles(VertexID, [&](int32 tid)
	{
		FVector3d TriNormal = GetTriNormalFunc(tid);
		if (Normal1Count == 0)
		{
			Normal1 = TriNormal;
			Normal1Count++;
			return;
		}
		if (TriNormal.Dot(Normal1) > DotTolerance)
		{
			Normal1Count++;
			return;
		}
		if (Normal2Count == 0)
		{
			Normal2 = TriNormal;
			Normal2Count++;
			return;
		}
		if (TriNormal.Dot(Normal2) > DotTolerance)
		{
			Normal2Count++;
			return;
		}
		OtherCount++;
	});
	return OtherCount == 0;
}


bool IsCollapsableDevelopableEdge(const FDynamicMesh3& Mesh, 
	int32 CollapseEdgeID, int32 RemoveV, int32 KeepV, double DotTolerance,
	const TArray<FVector3d>& TriNormals, const TArray<bool> IsBoundaryVtxCache)
{
	FIndex2i CollapseEdgeT = Mesh.GetEdgeT(CollapseEdgeID);
	FVector3d Normal1 = TriNormals[CollapseEdgeT.A];

	if (CollapseEdgeT.B == IndexConstants::InvalidID)
	{
		// If we're collapsing a boundary edge, the only way to avoid changing the shape is for RemoveV 
		// to be flat and have exactly one other attached boundary edge that is colinear with this one. 
		// Start by finding the other boundary edge and making sure that there is only one.
		bool bFoundSecondBoundaryEdge = false;
		for (int32 Eid : Mesh.VtxEdgesItr(RemoveV))
		{
			if (Eid != CollapseEdgeID && Mesh.IsBoundaryEdge(Eid))
			{
				if (bFoundSecondBoundaryEdge)
				{
					// Found more than one other boundary edge, so not collapsable
					return false;
				}
				bFoundSecondBoundaryEdge = true;

				// Verify that this second boundary edge is colinear with ours.
				FVector3d KeepVert = Mesh.GetVertex(KeepV);
				FVector3d RemoveVert = Mesh.GetVertex(RemoveV);
				int32 OtherV = IndexUtil::FindEdgeOtherVertex(Mesh.GetEdgeV(Eid), RemoveV);
				FVector3d OtherVert = Mesh.GetVertex(OtherV);
				if (!(Normalized(RemoveVert - OtherVert).Dot(Normalized(KeepVert - RemoveVert)) > DotTolerance))
				{
					// Not colinear
					return false;
				}
			}
		}
		if (!bFoundSecondBoundaryEdge)
		{
			// Seems impossible for a vertex to have exactly one attached boundary edge
			return ensure(false);
		}

		// If we got to here, we found the other boundary edge, and we'll check for planarity further below.
	}
	else
	{
		// If this is not a boundary edge, then remove V must not be a boundary vertex, else we
		// would deform the boundary on collapse.
		if (IsBoundaryVtxCache[RemoveV])
		{
			return false;
		}
	}

	FVector3d Normal2 = (CollapseEdgeT.B == IndexConstants::InvalidID) ? FVector3d::ZeroVector : TriNormals[CollapseEdgeT.B];

	// planar case
	if (CollapseEdgeT.B == IndexConstants::InvalidID || Normal1.Dot(Normal2) > DotTolerance)
	{
		bool bIsFlat = true;
		Mesh.EnumerateVertexTriangles(RemoveV, [&](int32 tid)
		{
			if (TriNormals[tid].Dot(Normal1) < DotTolerance)
			{
				bIsFlat = false;
			}
		});
		return bIsFlat;
	}

	// if we are not planar, we need to find the 'other' developable edge at RemoveV.
	// This edge must be aligned w/ our collapse edge and have the same normals
	FVector3d A = Mesh.GetVertex(RemoveV), B = Mesh.GetVertex(KeepV);
	FVector3d EdgeDir(B - A); Normalize(EdgeDir);
	int32 FoldEdges = 0, FlatEdges = 0, OtherEdges = 0;
	for (int32 eid : Mesh.VtxEdgesItr(RemoveV))
	{
		if (eid != CollapseEdgeID)
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(eid);
			if (EdgeT.B == IndexConstants::InvalidID)
			{
				// We already handled the cases where RemoveV is a boundary vert, so this shouldn't happen.
				return ensure(false);
			}
			FVector3d Normal3 = TriNormals[EdgeT.A];
			FVector3d Normal4 = TriNormals[EdgeT.B];

			FIndex2i OtherEdgeV = Mesh.GetEdgeV(eid);
			int32 OtherV = IndexUtil::FindEdgeOtherVertex(OtherEdgeV, RemoveV);
			FVector3d C = Mesh.GetVertex(OtherV);
			if ( Normalized(A-C).Dot(EdgeDir) > DotTolerance)
			{
				if ((Normal3.Dot(Normal1) > DotTolerance && Normal4.Dot(Normal2) > DotTolerance) ||
					(Normal3.Dot(Normal2) > DotTolerance && Normal4.Dot(Normal1) > DotTolerance))
				{
					FoldEdges++;
				}
			}
			else if ( Normal3.Dot(Normal4) > DotTolerance)
			{
				FlatEdges++;
			}
			else
			{
				OtherEdges++;
			}
		}
	}
	return (FoldEdges == 1 && OtherEdges == 0);
}


template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::SimplifyToMinimalPlanar(
	double CoplanarAngleTolDeg,
	TFunctionRef<bool(int32 EdgeID)> EdgeFilterPredicate)
{
#define RETURN_IF_CANCELLED 	if (Cancelled()) { return; }

	if (Mesh->TriangleCount() == 0)    // badness if we don't catch this...
	{
		return;
	}

	// This function doesn't affect the shape of the boundary, so the value of bPreserveBoundaryShape
	// shouldn't matter. Yet in practice, having bPreserveBoundaryShape be true is problematic because
	// the related block in CollapseEdge() arbitrarily decides that just one of the verts of a boundary
	// edge can be collapsed to.
	// TODO: the above is probably a minor bug for other forms of simplification too, but fixing it requires
	// going through the details of other simplification methods. For SimplifyToMinimalPlanar, the simplest
	// solution is to just eliminate that factor as a concern, since it should be free to collapse along
	// colinear boundaries unless they are explicitly constrained.
	TGuardValue<bool> PreserveBoundaryShapeOverride(bPreserveBoundaryShape, false); // Sets to false, restores on exit

	// keep triangle normals
	TArray<FVector3d> TriNormals;
	TArray<bool> DevelopableVerts;

	ProfileBeginPass();

	ProfileBeginSetup();
	Precompute();
	RETURN_IF_CANCELLED;

	TriNormals.SetNum(Mesh->MaxTriangleID());
	ParallelFor(Mesh->MaxTriangleID(), [&](int32 tid)
	{
		if (Mesh->IsTriangle(tid))
		{
			TriNormals[tid] = Mesh->GetTriNormal(tid);
		}
	});
	RETURN_IF_CANCELLED;

	DevelopableVerts.SetNum(Mesh->MaxVertexID());
	double PlanarDotTol = FMathd::Cos( CoplanarAngleTolDeg * FMathd::DegToRad );
	ParallelFor(Mesh->MaxVertexID(), [&](int32 vid)
	{
		if (Mesh->IsVertex(vid))
		{
			DevelopableVerts[vid] = IsDevelopableVertex(*Mesh, vid, PlanarDotTol, [&](int32 tid) { return TriNormals[tid]; });
		}
	});
	RETURN_IF_CANCELLED;

	ProfileEndSetup();


	ProfileBeginOps();

	ProfileBeginCollapse();

	TArray<int32> CollapseEdges;
	int32 MaxRounds = 50;
	int32 num_last_pass = 0;
	for (int ri = 0; ri < MaxRounds; ++ri)
	{
		num_last_pass = 0;

		// collect up edges we have identified for collapse
		CollapseEdges.Reset();
		for (int32 eid : Mesh->EdgeIndicesItr())
		{
			if (EdgeFilterPredicate(eid) == false)
			{
				continue;
			}

			FIndex2i ev = Mesh->GetEdgeV(eid);
			if (DevelopableVerts[ev.A] || DevelopableVerts[ev.B])
			{
				CollapseEdges.Add(eid);
			}
		}


		FVector3d va = FVector3d::Zero(), vb = FVector3d::Zero();
		for ( int32 eid : CollapseEdges )
		{
			if (!Mesh->IsEdge(eid))
			{
				continue;
			}
			RETURN_IF_CANCELLED;
			COUNT_ITERATIONS++;

			FIndex2i ev = Mesh->GetEdgeV(eid);
			FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
			ESimplificationResult Result = ESimplificationResult::Failed_OpNotSuccessful;

			// Try collapsing to vert B.
			if (DevelopableVerts[ev.A]
				&& IsCollapsableDevelopableEdge(*Mesh, eid, ev.A, ev.B, PlanarDotTol, TriNormals, IsBoundaryVtxCache))
			{
				Result = CollapseEdge(eid, Mesh->GetVertex(ev.B), CollapseInfo, ev.B);
			}

			// If that didn't work, try collapsing to vert A
			if (Result != ESimplificationResult::Ok_Collapsed && DevelopableVerts[ev.B]
				&& IsCollapsableDevelopableEdge(*Mesh, eid, ev.B, ev.A, PlanarDotTol, TriNormals, IsBoundaryVtxCache))
			{
				Result = CollapseEdge(eid, Mesh->GetVertex(ev.A), CollapseInfo, ev.A);
			}

			if (Result == ESimplificationResult::Ok_Collapsed)
			{
				++num_last_pass;

				int vKeptID = CollapseInfo.KeptVertex;
				Mesh->EnumerateVertexTriangles(vKeptID, [&](int32 tid)
				{
					TriNormals[tid] = Mesh->GetTriNormal(tid);
				});
				for (int32 vid : Mesh->VtxVerticesItr(vKeptID))
				{
					DevelopableVerts[vid] = IsDevelopableVertex(*Mesh, vid, PlanarDotTol, [&](int32 tid) { return TriNormals[tid]; });
				}
				DevelopableVerts[vKeptID] = IsDevelopableVertex(*Mesh, vKeptID, PlanarDotTol, [&](int32 tid) { return TriNormals[tid]; });
			}
		}

		if (num_last_pass == 0)     // converged
		{
			break;
		}
	}
	ProfileEndCollapse();
	ProfileEndOps();

	RETURN_IF_CANCELLED;

	Reproject();

	ProfileEndPass();

#undef RETURN_IF_CANCELLED
}



template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::FastCollapsePass(double fMinEdgeLength, int nRounds, bool MeshIsClosedHint, uint32 MinTriangleCount)
{
	if ((uint32)Mesh->TriangleCount() <= MinTriangleCount)    // badness if we don't catch this...
	{
		return;
	}

	MinEdgeLength = fMinEdgeLength;
	double min_sqr = MinEdgeLength * MinEdgeLength;

	// we don't collapse on the boundary
	bHaveBoundary = false;

	ProfileBeginPass();

	ProfileBeginSetup();
	Precompute(MeshIsClosedHint);
	if (Cancelled())
	{
		return;
	}
	ProfileEndSetup();

	ProfileBeginOps();

	ProfileBeginCollapse();

	int N = Mesh->MaxEdgeID();
	int num_last_pass = 0;
	for (int ri = 0; ri < nRounds; ++ri)
	{
		num_last_pass = 0;

		FVector3d va = FVector3d::Zero(), vb = FVector3d::Zero();
		for (int eid = 0; eid < N; ++eid)
		{
			if ((!Mesh->IsEdge(eid)) || Mesh->IsBoundaryEdge(eid))
			{
				continue;
			}
			if ((uint32)Mesh->TriangleCount() <= MinTriangleCount)
			{
				break;
			}
			if (Cancelled())
			{
				return;
			}

			Mesh->GetEdgeV(eid, va, vb);
			double EdgeLenSq = DistanceSquared(va, vb);
			if (CustomEdgeLengthScaleF)
			{
				FIndex2i EdgeV = Mesh->GetEdgeV(eid);
				double Scale = CustomEdgeLengthScaleF(*Mesh, EdgeV.A, EdgeV.B);
				EdgeLenSq *= Scale * Scale;
			}
			if (EdgeLenSq > min_sqr)
			{
				continue;
			}

			COUNT_ITERATIONS++;

			FVector3d midpoint = (va + vb) * 0.5;
			FDynamicMesh3::FEdgeCollapseInfo collapseInfo;
			ESimplificationResult result = CollapseEdge(eid, midpoint, collapseInfo);
			if (result == ESimplificationResult::Ok_Collapsed)
			{
				++num_last_pass;
			}
		}

		if (num_last_pass == 0 || (uint32)Mesh->TriangleCount() <= MinTriangleCount)     // converged
		{
			break;
		}
	}
	ProfileEndCollapse();
	ProfileEndOps();

	if (Cancelled())
	{
		return;
	}

	Reproject();

	ProfileEndPass();
}






template <typename QuadricErrorType>
bool TMeshSimplification<QuadricErrorType>::CanCollapseEdge(int edgeID, int a, int b, int c, int d, int tc, int td, int& collapse_to, FCollapseInfo* OutCollapseInfo) const
{
	collapse_to = -1;
	if (!Constraints)
	{
		return true;
	}

	FVertexConstraint ca = Constraints->GetVertexConstraint(a);
	FVertexConstraint cb = Constraints->GetVertexConstraint(b);

	// Use the MeshRefinerBase::CanCollapseVertex method to test if the vertices are too constrained to collapse
	// and if there is a restriction on which vertices must be kept/removed
	bool bCanCollapse = CanCollapseVertex(edgeID, a, b, collapse_to);

	// Special handling for the case where we have two same-target constrained vertices connecting an unconstrained edge
	// Then we *can* safely collapse if the AC CB xor AD DB edges match that same target, and the shared C/D vertex is otherwise-unconstrained ...
	// as long as we track that the shared C/D vertex and matched edges should have their constraints removed post collapse (tracked via OutCollapseInfo)
	bool bIndirectConstraintConnection = false;
	FEdgeConstraint EdgeConstraint = Constraints->GetEdgeConstraint(edgeID);
	if (OutCollapseInfo && !bCanCollapse && ca.Target != nullptr && ca.Target == cb.Target && (ca.bCanMove || cb.bCanMove) && EdgeConstraint.IsUnconstrained())
	{
		int32 eac = Mesh->FindEdgeFromTri(a, c, tc);
		int32 ebc = Mesh->FindEdgeFromTri(b, c, tc);
		bool bMatchTargetsThroughACB =
			Constraints->GetEdgeConstraint(eac).Target == ca.Target &&
			Constraints->GetEdgeConstraint(ebc).Target == ca.Target;
		bool bMatchTargetsThroughADB = false;
		int32 ead = IndexConstants::InvalidID, ebd = IndexConstants::InvalidID;
		if (d != IndexConstants::InvalidID)
		{
			ead = Mesh->FindEdgeFromTri(a, d, td);
			ebd = Mesh->FindEdgeFromTri(b, d, td);
			bMatchTargetsThroughADB = 
				Constraints->GetEdgeConstraint(ead).Target == ca.Target &&
				Constraints->GetEdgeConstraint(ebd).Target == ca.Target;
		}
		if (bMatchTargetsThroughACB != bMatchTargetsThroughADB)
		{
			int32 PathVID, PathEA, PathEB;
			if (bMatchTargetsThroughACB)
			{
				PathVID = c;
				PathEA = eac;
				PathEB = ebc;
			}
			else
			{
				PathVID = d;
				PathEA = ead;
				PathEB = ebd;
			}
			FVertexConstraint CPath = Constraints->GetVertexConstraint(PathVID);
			if (!CPath.bCanMove || CPath.bCannotDelete || CPath.Target != ca.Target)
			{
				return false;
			}
			bool bOtherEdgesUnconstrained = true;
			Mesh->EnumerateVertexEdges(PathVID, [&](int32 EID)
			{
				if (EID == PathEA || EID == PathEB)
				{
					return;
				}
				bOtherEdgesUnconstrained = bOtherEdgesUnconstrained &&
					!Constraints->HasEdgeConstraint(EID);
			});
			if (bOtherEdgesUnconstrained)
			{
				bCanCollapse = true;
				bIndirectConstraintConnection = true;
				OutCollapseInfo->UnconstrainVIDs.Add(PathVID);
				OutCollapseInfo->UnconstrainEIDs.Add(PathEA);
				OutCollapseInfo->UnconstrainEIDs.Add(PathEB);
			}
		}
	}

	if (!bCanCollapse)
	{
		return false;
	}

	// This non-matching targets case is not caught by MeshRefinerBase::CanCollapseVertex;
	// for simplicity, we handle it after the indirect connections case is ruled out
	if (!bIndirectConstraintConnection && ca.Target != EdgeConstraint.Target && ca.Target != nullptr && cb.Target != nullptr)
	{
		return false;
	}

	// Helper to update collapse_to + bCanCollapse when considering additional constraints
	auto AddNewABConstraint = [&bCanCollapse, a, b, &collapse_to](bool bConstrainA, bool bConstrainB)
	{
		if (bConstrainA == bConstrainB)
		{
			bCanCollapse = !bConstrainA;
		}
		else // only constraining A or B
		{
			int32 ConstrainVID = bConstrainA ? a : b;
			if (collapse_to == IndexConstants::InvalidID)
			{
				collapse_to = ConstrainVID;
			}
			bCanCollapse = collapse_to == ConstrainVID;
		}
	};

	// if the vertex constraints did not prevent collapse, and we didn't follow the special case above,
	// then make sure the AC CB and AD DB edges don't prevent the collapse via their constraints
	// Note this follows the MeshRefinerBase::CanCollapseEdge logic, after the MeshRefinerBase::CanCollapseVertex call
	if (bCanCollapse && !bIndirectConstraintConnection)
	{
		// determine if edge constraints require keeping either vert.
		// if edge ac is constrained w.r.t. merging topology, we must keep it and thus vertex a, likewise for edge bc and vertex b.
		int32 eac = Mesh->FindEdgeFromTri(a, c, tc);
		int32 ebc = Mesh->FindEdgeFromTri(b, c, tc);
		FEdgeConstraint Ceac = Constraints->GetEdgeConstraint(eac);
		FEdgeConstraint Cebc = Constraints->GetEdgeConstraint(ebc);
		bool bMustRetainA = !Ceac.CanMergeTopology();
		bool bMustRetainB = !Cebc.CanMergeTopology();

		// if edge ad is constrained w.r.t. merging topology, we must keep it and thus vertex a, likewise for edge bd and vertex b.
		if (d != IndexConstants::InvalidID)
		{
			int32 ead = Mesh->FindEdgeFromTri(a, d, td);
			int32 ebd = Mesh->FindEdgeFromTri(b, d, td);

			FEdgeConstraint Cead = Constraints->GetEdgeConstraint(ead);
			FEdgeConstraint Cebd = Constraints->GetEdgeConstraint(ebd);
			bMustRetainA = bMustRetainA || !Cead.CanMergeTopology();
			bMustRetainB = bMustRetainB || !Cebd.CanMergeTopology();
		}

		AddNewABConstraint(bMustRetainA, bMustRetainB);
	}

	// make sure more general seam topology is preserved

	const bool bPreserveSeamTopology = (CollapseMode == ESimplificationCollapseModes::MinimalExistingVertexError);
	if (bCanCollapse && bAllowSeamCollapse && bPreserveSeamTopology)
	{
		if (!Constraints)
		{
			return bCanCollapse;
		}

		// NB: We have to be more restrictive with the MinimalQuadricPositionError mode
		// in order to preclude the possibility of a seam moving during collapse.

		// check if this edge is a seam
		if (Constraints->HasEdgeConstraint(edgeID))
		{
			// TODO [jimmy]: two issues here:
			// (1) conceptually it's in the wrong place; we should instead set no-move constraints on the seam intersections in the original constraint system
			// (2) it's incorrectly constraining both vertices of the seam edge, in cases where only one end is 'intersecting'
			
			// examine local topology
			bool bCanCollapseSeam = true;
			if (const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
			{

				for (int i = 0; bCanCollapseSeam && i < Attributes->NumUVLayers(); ++i)
				{
					auto* Overlay = Attributes->GetUVLayer(i);
					bool bIsNonIntersecting;
					if (Overlay->IsSeamEdge(edgeID, &bIsNonIntersecting))
					{
						bCanCollapseSeam = bCanCollapseSeam && bIsNonIntersecting;
					}
				}
				for (int i = 0; bCanCollapseSeam && i < Attributes->NumNormalLayers(); ++i)
				{
					auto* Overlay = Attributes->GetNormalLayer(i);
					bool bIsNonIntersecting;
					if (Overlay->IsSeamEdge(edgeID, &bIsNonIntersecting))
					{
						bCanCollapseSeam = bCanCollapseSeam && bIsNonIntersecting;
					}
				}
			}
			bCanCollapse = bCanCollapseSeam;
		}
		else
		{
			// this edge was not a seam, but need to check if one or both ends are part of other seams 
			// - this is done by checking for vertex constraint
			bool bVertexAOnSeam = Constraints->HasVertexConstraint(a);
			bool bVertexBOnSeam = Constraints->HasVertexConstraint(b);
			AddNewABConstraint(bVertexAOnSeam, bVertexBOnSeam);
		}
	}

	return bCanCollapse;
}



template <typename QuadricErrorType>
ESimplificationResult TMeshSimplification<QuadricErrorType>::CollapseEdge(int edgeID, FVector3d vNewPos, FDynamicMesh3::FEdgeCollapseInfo& collapseInfo, int32 RequireKeepVert)
{
	collapseInfo.KeptVertex = FDynamicMesh3::InvalidID;
	RuntimeDebugCheck(edgeID);

	FEdgeConstraint constraint =
		(!Constraints) ? FEdgeConstraint::Unconstrained() : Constraints->GetEdgeConstraint(edgeID);
	if (constraint.NoModifications())
	{
		return ESimplificationResult::Ignored_EdgeIsFullyConstrained;
	}
	if (constraint.CanCollapse() == false)
	{
		return ESimplificationResult::Ignored_EdgeIsFullyConstrained;
	}

	// look up verts and tris for this edge
	if (Mesh->IsEdge(edgeID) == false)
	{
		return ESimplificationResult::Failed_NotAnEdge;
	}
	const FDynamicMesh3::FEdge Edge = Mesh->GetEdge(edgeID);
	int a = Edge.Vert[0], b = Edge.Vert[1], t0 = Edge.Tri[0], t1 = Edge.Tri[1];
	bool bIsBoundaryEdge = (t1 == FDynamicMesh3::InvalidID);

	// look up 'other' verts c (from t0) and d (from t1, if it exists)
	FIndex3i T0tv = Mesh->GetTriangle(t0);
	int c = IndexUtil::FindTriOtherVtx(a, b, T0tv);
	FIndex3i T1tv = (bIsBoundaryEdge) ? FDynamicMesh3::InvalidTriangle : Mesh->GetTriangle(t1);
	int d = (bIsBoundaryEdge) ? FDynamicMesh3::InvalidID : IndexUtil::FindTriOtherVtx(a, b, T1tv);

	FVector3d vA = Mesh->GetVertex(a);
	FVector3d vB = Mesh->GetVertex(b);
	double edge_len_sqr = (vA - vB).SquaredLength();
	if (CustomEdgeLengthScaleF)
	{
		double Scale = CustomEdgeLengthScaleF(*Mesh, a, b);
		edge_len_sqr *= Scale * Scale;
	}
	if (edge_len_sqr > MinEdgeLength * MinEdgeLength && !ShouldIgnoreStoppingCriteria())
	{
		return ESimplificationResult::Ignored_EdgeTooLong;
	}

	ProfileBeginCollapse();
	ON_SCOPE_EXIT { ProfileEndCollapse(); };

	// check if we should collapse, and also find which vertex we should retain
	// in cases where we have constraints/etc
	int collapse_to = -1;
	FCollapseInfo CollapseInfo;
	bool bCanCollapse = CanCollapseEdge(edgeID, a, b, c, d, t0, t1, collapse_to, &CollapseInfo);

	if (bCanCollapse == false)
	{
		return ESimplificationResult::Ignored_Constrained;
	}

	const bool bMinimalVertexMode = (CollapseMode == ESimplificationCollapseModes::MinimalExistingVertexError);
	// if we have a boundary, we want to collapse to boundary
	if (bPreserveBoundaryShape && bHaveBoundary)
	{
		if (collapse_to != -1)
		{
			if ((IsBoundaryVertex(b) && collapse_to != b) ||
				(IsBoundaryVertex(a) && collapse_to != a))
			{
				return ESimplificationResult::Ignored_Constrained;
			}
		}

		if (!bMinimalVertexMode) // the minimal existing vertex error has already resolved this with more complicated logic
		{
			if (IsBoundaryVertex(b))
			{
				collapse_to = b;
			}
			else if (IsBoundaryVertex(a))
			{
				collapse_to = a;
			}
		}
	}

	if (RequireKeepVert == a || RequireKeepVert == b)
	{
		if (collapse_to != -1 && collapse_to != RequireKeepVert)
		{
			return ESimplificationResult::Ignored_Constrained;
		}
		collapse_to = RequireKeepVert;
	}

	// optimization: if edge cd exists, we cannot collapse or flip. look that up here?
	//  funcs will do it internally...
	//  (or maybe we can collapse if cd exists? edge-collapse doesn't check for it explicitly...)
	ESimplificationResult retVal = ESimplificationResult::Failed_OpNotSuccessful;

	int iKeep = b, iCollapse = a;
	bool bConstraintsSpecifyPosition = false;
	if (collapse_to != -1)
	{
		iKeep = collapse_to;
		iCollapse = (iKeep == a) ? b : a;

		// if constraints or collapse mode require a fixed position
		if (Constraints)
		{
			bConstraintsSpecifyPosition = bMinimalVertexMode || (!Constraints->GetVertexConstraint(collapse_to).bCanMove );
		}
	}
	double collapse_t = 0;
	if (!bConstraintsSpecifyPosition)
	{
		checkSlow(!bMinimalVertexMode || (vNewPos == vA || vNewPos == vB));

		FVector3d VCollapse = Mesh->GetVertex(iCollapse), VKeep = Mesh->GetVertex(iKeep);
		if (!bMinimalVertexMode)
		{
			vNewPos = GetProjectedCollapsePosition(iKeep, vNewPos);
			double SegLenSq_Unused;
			collapse_t = CurveUtil::ProjectToSegment(VKeep, VCollapse, SegLenSq_Unused, vNewPos);
		}
		else
		{
			collapse_t = FVector3d::DistSquared(vNewPos, VKeep) < FVector3d::DistSquared(vNewPos, VCollapse) ? 0. : 1.;
		}

		// If vertex is not explicitly constrained, and geometric error constraint is requested, we check it here.
		// If the check with the predicted vNewPos fails, try a second time with the linear-interpolated point.
		// (could optionally do a line search here, and/or be smarter about avoiding duplicate work, although
		//  it is small in the context of the larger algorithm)
		// Constraint is bypassed while triangle count is still above MaxResultTriangleCount.
		if (GeometricErrorConstraint != EGeometricErrorCriteria::None && !ShouldIgnoreStoppingCriteria())
		{
			if (CheckIfCollapseWithinGeometricTolerance(iKeep, iCollapse, vNewPos, t0, t1) == false)
			{
				if (bMinimalVertexMode)
				{
					return ESimplificationResult::Failed_GeometricDeviation;
				}
				// project new position back onto the edge
				vNewPos = (1.0-collapse_t)*Mesh->GetVertex(iKeep) + (collapse_t)*Mesh->GetVertex(iCollapse);
				if (CheckIfCollapseWithinGeometricTolerance(iKeep, iCollapse, vNewPos, t0, t1) == false)
				{
					return ESimplificationResult::Failed_GeometricDeviation;
				}
			}
		}
	}
	else
	{
		vNewPos = (collapse_to == a) ? vA : vB;

		// If geometric error constraint is requested, ensure that the fixed vNewPos satisfies the constraint.
		// This must be done, otherwise the free vertex being collapsed to a fixed vertex will be allowed
		// to violate the geometric constraint
		if (GeometricErrorConstraint != EGeometricErrorCriteria::None && !ShouldIgnoreStoppingCriteria())
		{
			if (CheckIfCollapseWithinGeometricTolerance(iKeep, iCollapse, vNewPos, t0, t1) == false)
			{
				return ESimplificationResult::Failed_GeometricDeviation;
			}
		}
	}

	// Check if edge constraint tolerance would prevent collapse
	// Note this must happen after the above block has already called GetProjectedCollapsePosition
	// to apply constraint projection to the vNewPos if needed
	if (CheckIfCollapseWithinEdgeConstraintsTolerance(iKeep, iCollapse, vNewPos, t0) == false)
	{
		return ESimplificationResult::Failed_ConstraintDeviation;
	}

	// check if this collapse will create a normal flip. Also checks
	// for invalid collapse nbrhood, since we are doing one-ring iter anyway.
	// [TODO] could we skip this one-ring check in CollapseEdge? pass in hints?
	if (CheckIfCollapseCreatesFlipOrInvalid(a, b, vNewPos, t0, t1) || CheckIfCollapseCreatesFlipOrInvalid(b, a, vNewPos, t0, t1))
	{
		return ESimplificationResult::Ignored_CreatesFlip;
	}

	if (bPreventTinyTriangles && (CheckIfCollapseCreatesTinyTriangle(a, b, vNewPos, t0, t1) || CheckIfCollapseCreatesTinyTriangle(b, a, vNewPos, t0, t1)))
	{
		return ESimplificationResult::Ignored_CreatesTinyTriangle;
	}

	// lots of cases where we cannot collapse, but we should just let
	// Mesh sort that out, right?
	COUNT_COLLAPSES++;

	EMeshResult result = Mesh->CollapseEdge(iKeep, iCollapse, collapse_t, collapseInfo);
	if (result == EMeshResult::Ok)
	{
		Mesh->SetVertex(iKeep, vNewPos);
		if (Constraints)
		{
			Constraints->ClearEdgeConstraint(edgeID);
			for (int32 ClearVID : CollapseInfo.UnconstrainVIDs)
			{
				Constraints->ClearVertexConstraint(ClearVID);
			}
			for (int32 ClearEID : CollapseInfo.UnconstrainEIDs)
			{
				Constraints->ClearEdgeConstraint(ClearEID);
			}

			auto ConstraintUpdator = [this](int cur_eid)->void
			{
				
				// Seam edge can never flip, it is never fully unconstrained 
				EEdgeRefineFlags SeamEdgeConstraint = EEdgeRefineFlags::NoFlip;
				if (!bAllowSeamCollapse)
				{
					SeamEdgeConstraint = EEdgeRefineFlags((int)SeamEdgeConstraint | (int)EEdgeRefineFlags::NoCollapse);
				}

				FEdgeConstraint UpdatedEdgeConstraint;
				FVertexConstraint UpdatedVertexConstraintA;
				FVertexConstraint UpdatedVertexConstraintB;


				bool bHaveUpdate = 
				FMeshConstraintsUtil::ConstrainEdgeBoundariesAndSeams(cur_eid,
					*Mesh,
					MeshBoundaryConstraint,
					GroupBoundaryConstraint,
					MaterialBoundaryConstraint,
					SeamEdgeConstraint,
					!bAllowSeamCollapse,
					UpdatedEdgeConstraint,
					UpdatedVertexConstraintA,
					UpdatedVertexConstraintB);

				if (bHaveUpdate)
				{
					FIndex2i EdgeVerts = Mesh->GetEdgeV(cur_eid);

					Constraints->SetOrUpdateEdgeConstraint(cur_eid, UpdatedEdgeConstraint);
					UpdatedVertexConstraintA.CombineConstraint(Constraints->GetVertexConstraint(EdgeVerts.A));
					Constraints->SetOrUpdateVertexConstraint(EdgeVerts.A, UpdatedVertexConstraintA);

					UpdatedVertexConstraintB.CombineConstraint(Constraints->GetVertexConstraint(EdgeVerts.B));
					Constraints->SetOrUpdateVertexConstraint(EdgeVerts.B, UpdatedVertexConstraintB);
				}
			};
			
			if (Constraints->HasEdgeConstraint(collapseInfo.RemovedEdges.A))
			{

				Constraints->ClearEdgeConstraint(collapseInfo.KeptEdges.A);
				Constraints->ClearEdgeConstraint(collapseInfo.RemovedEdges.A);

				ConstraintUpdator(collapseInfo.KeptEdges.A);
			}
			
			if (collapseInfo.RemovedEdges.B != FDynamicMesh3::InvalidID)
			{
				if (Constraints->HasEdgeConstraint(collapseInfo.RemovedEdges.B))
				{

					Constraints->ClearEdgeConstraint(collapseInfo.KeptEdges.B);
					Constraints->ClearEdgeConstraint(collapseInfo.RemovedEdges.B);

					ConstraintUpdator(collapseInfo.KeptEdges.B);
				}
			}
			Constraints->ClearVertexConstraint(iCollapse);
		}
		OnEdgeCollapse(edgeID, iKeep, iCollapse, collapseInfo);
		UpdateVertexAttributes(edgeID, iKeep, iCollapse);
		DoDebugChecks();

		retVal = ESimplificationResult::Ok_Collapsed;
	}
	else if (result == EMeshResult::Failed_CollapseTriangle)
	{
		retVal = ESimplificationResult::Failed_IsolatedTriangle;
	}

	return retVal;
}

template <typename QuadricErrorType>
bool TMeshSimplification<QuadricErrorType>::CheckIfCollapseWithinEdgeConstraintsTolerance(int VKeep, int VRemove, const FVector3d& NewPosition, int TC)
{
	if (!Constraints || !bLimitConstrainedSeamMovement)
	{
		return true;
	}

	// test edge midpoints, except the edge being collapsed
	int32 CollapseEdgeID = Mesh->FindEdgeFromTri(VKeep, VRemove, TC);
	auto EdgeMidpointsWithinTolerance = [this, CollapseEdgeID, NewPosition](int32 VID) -> bool
	{
		for (int32 EID : Mesh->VtxEdgesItr(VID))
		{
			if (EID != CollapseEdgeID)
			{
				FEdgeConstraint EdgeConstraint = Constraints->GetEdgeConstraint(EID);
				if (EdgeConstraint.Target && EdgeConstraint.DistanceToleranceSq > 0)
				{
					FIndex2i EdgeV = Mesh->GetEdgeV(EID);
					FVector3d OtherVertexPos = (EdgeV.A == VID) ? Mesh->GetVertex(EdgeV.B) : Mesh->GetVertex(EdgeV.A);
					FVector3d NewMidpoint = (OtherVertexPos + NewPosition) * 0.5;

					FVector3d ConstraintProj = EdgeConstraint.Target->Project(NewMidpoint);
					if (DistanceSquared(NewMidpoint, ConstraintProj) > EdgeConstraint.DistanceToleranceSq)
					{
						return false;
					}
				}
			}
		}
		return true;
	};

	return EdgeMidpointsWithinTolerance(VKeep) && EdgeMidpointsWithinTolerance(VRemove);
}

template <typename QuadricErrorType>
bool TMeshSimplification<QuadricErrorType>::CheckIfCollapseWithinGeometricTolerance(int vKeep, int vRemove, const FVector3d& NewPosition, int tc, int td)
{
	if (GeometricErrorConstraint == EGeometricErrorCriteria::PredictedPointToProjectionTarget)
	{
		// currently assuming projection target is what we want to measure geometric error against
		if (ProjectionTarget() != nullptr)
		{
			double ToleranceSqr = GeometricErrorTolerance * GeometricErrorTolerance;
			if (CustomGeometricErrorScaleF)
			{
				double Scale = CustomGeometricErrorScaleF(*Mesh, vKeep, vRemove);
				double ErrorScaleSq = Scale * Scale;
				// if error is scaled to zero, it will always pass tolerance
				if (ErrorScaleSq <= 0)
				{
					return true;
				}
				// We divide the tolerance; this is equivalent to scaling the errors below but a bit faster / simpler
				ToleranceSqr /= ErrorScaleSq;
			}

			// test new position to see if it is within geometric tolerance of projection surface
			FVector3d TargetPos = ProjectionTarget()->Project(NewPosition);
			double DistSqr = DistanceSquared(TargetPos, NewPosition);
			if (DistSqr > ToleranceSqr)
			{
				return false;
			}

			// test edge midpoints, except the edge being collapsed
			int32 CollapseEdgeID = Mesh->FindEdgeFromTri(vKeep, vRemove, tc);
			auto EdgeMidpointsWithinTolerance = [this, CollapseEdgeID, NewPosition, ToleranceSqr](int32 vid) {
				for ( int32 eid : Mesh->VtxEdgesItr(vid) )
				{
					if (eid != CollapseEdgeID)
					{
						FIndex2i EdgeV = Mesh->GetEdgeV(eid);
						FVector3d OtherVertexPos = (EdgeV.A == vid) ? Mesh->GetVertex(EdgeV.B) : Mesh->GetVertex(EdgeV.A);
						FVector3d NewMidpoint = (OtherVertexPos + NewPosition) * 0.5;
						FVector3d MidpointTargetPos = ProjectionTarget()->Project(NewMidpoint);
						if (DistanceSquared(NewMidpoint, MidpointTargetPos) > ToleranceSqr)
						{
							return false;
						}
					}
				}
				return true;
			};
			if (EdgeMidpointsWithinTolerance(vKeep) == false || EdgeMidpointsWithinTolerance(vRemove) == false)
			{
				return false;
			}


			// check tri centers, except the triangles being collapsed
			auto CentroidsWithinToleranceFunc = [this, vKeep, vRemove, tc, td, NewPosition, ToleranceSqr](int32 vid) {
				bool bInTolerance = true;
				Mesh->EnumerateVertexTriangles(vid, [&](int32 tid)
					{
						if (bInTolerance && tid != tc && tid != td)
						{
							FIndex3i Tri = Mesh->GetTriangle(tid);
							FVector3d NewCentroid = FVector3d::Zero();
							for (int32 j = 0; j < 3; ++j)
							{
								NewCentroid += (Tri[j] == vRemove || Tri[j] == vKeep) ? NewPosition : Mesh->GetVertex(Tri[j]);
							}
							NewCentroid *= (1.0 / 3.0);
							FVector3d CentroidTargetPos = ProjectionTarget()->Project(NewCentroid);
							if (DistanceSquared(NewCentroid, CentroidTargetPos) > ToleranceSqr)
							{
								bInTolerance = false;
							}
						}
					});
				return bInTolerance;
			};
			if (CentroidsWithinToleranceFunc(vKeep) == false || CentroidsWithinToleranceFunc(vRemove) == false)
			{
				return false;
			}
		}
	}
	return true;
}

template <typename QuadricErrorType>
bool TMeshSimplification<QuadricErrorType>::RemoveIsolatedTriangle(int tID)
{
	if (!Mesh->IsTriangle(tID)) return true;

	FIndex3i tv = Mesh->GetTriangle(tID);

	bool bIsIsolated = true;
	for (int i = 0; i < 3; ++i)
	{
		for (int nbtr : Mesh->VtxTrianglesItr(tv[i]))
		{
			bIsIsolated = bIsIsolated && (nbtr == tID);
		}
	}


	if (bIsIsolated)
	{
		const FIndex3i TriEdges = Mesh->GetTriEdges(tID);
		if (Mesh->RemoveTriangle(tID) == EMeshResult::Ok)
		{
			if (Constraints)
			{
				Constraints->ClearEdgeConstraint(TriEdges.A);
				Constraints->ClearEdgeConstraint(TriEdges.B);
				Constraints->ClearEdgeConstraint(TriEdges.C);

				Constraints->ClearVertexConstraint(tv.A);
				Constraints->ClearVertexConstraint(tv.B);
				Constraints->ClearVertexConstraint(tv.C);
			}
		}

		OnRemoveIsolatedTriangle(tID);
	}

	return bIsIsolated;

}



template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::OnEdgeCollapse(int edgeID, int va, int vb, const FDynamicMesh3::FEdgeCollapseInfo& collapseInfo)
{
	// this is for subclasses...
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::OnRemoveIsolatedTriangle(int tId)
{
	// this is for subclasses
}

template <>
void TMeshSimplification<FAttrBasedQuadricErrorV2d>::OnRemoveIsolatedTriangle(int tId)
{
}

// Project vertices onto projection target. 
// We can do projection in parallel if we have .net 
template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::FullProjectionPass()
{
	auto project = [&](int vID)
	{
		if (IsVertexPositionConstrained(vID))
		{
			return;
		}
		if (VertexControlF != nullptr && ((int)VertexControlF(vID) & (int)EVertexControl::NoProject) != 0)
		{
			return;
		}
		FVector3d curpos = Mesh->GetVertex(vID);
		FVector3d projected = ProjTarget->Project(curpos, vID);
		Mesh->SetVertex(vID, projected);
	};

	ApplyToProjectVertices(project);

	// TODO: optionally do projection in parallel?
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::ApplyToProjectVertices(const TFunction<void(int)>& apply_f)
{
	for (int vid : Mesh->VertexIndicesItr())
	{
		apply_f(vid);
	}
}

template <typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::ProjectVertex(int vID, IProjectionTarget* targetIn)
{
	FVector3d curpos = Mesh->GetVertex(vID);
	FVector3d projected = targetIn->Project(curpos, vID);
	Mesh->SetVertex(vID, projected);
}

// used by collapse-edge to get projected position for new vertex
template <typename QuadricErrorType>
FVector3d TMeshSimplification<QuadricErrorType>::GetProjectedCollapsePosition(int vid, const FVector3d& vNewPos)
{
	if (Constraints)
	{
		FVertexConstraint vc = Constraints->GetVertexConstraint(vid);
		if (vc.Target != nullptr)
		{
			return vc.Target->Project(vNewPos, vid);
		}
		if (vc.bCanMove == false)
		{
			return vNewPos;
		}
	}
	// no constraint applied, so if we have a target surface, project to that
	if (EnableInlineProjection() && ProjTarget != nullptr)
	{
		if (VertexControlF == nullptr || ((int)VertexControlF(vid) & (int)EVertexControl::NoProject) == 0)
		{
			return ProjTarget->Project(vNewPos, vid);
		}
	}
	return vNewPos;
}

template<typename QuadricErrorType>
void TMeshSimplification<QuadricErrorType>::UpdateVertexAttributes(int edgeID, int va, int vb)
{
}

// Custom behavior for FAttrBasedQuadric simplifier.
template<>
void TMeshSimplification<FAttrBasedQuadricErrord>::UpdateVertexAttributes(int edgeID, int va, int vb)
{
	// Update the normal
	FAttrBasedQuadricErrord& Quadric = EdgeQuadrics[edgeID].q;
	FVector3d collapse_pt = EdgeQuadrics[edgeID].collapse_pt;

	FVector3d UpdatedNormald;
	Quadric.ComputeAttributes(collapse_pt, UpdatedNormald);

	FVector3f UpdatedNormal((float)UpdatedNormald.X, (float)UpdatedNormald.Y, (float)UpdatedNormald.Z);
	Normalize(UpdatedNormal);

	if (NormalOverlay != nullptr)
	{
		// Get all the elements associated with this vertex (could be more than one to account for split vertex data)
		TArray<int> ElementIdArray;
		NormalOverlay->GetVertexElements(va, ElementIdArray);

		if (ElementIdArray.Num() > 1)
		{
			// keep whatever split normals are currently in the overlay.
			// @todo: normalize the split normals - since the values here result from a lerp
			return;
		}
	
		// at most one element
		for (int ElementId : ElementIdArray)
		{
			NormalOverlay->SetElement(ElementId, UpdatedNormal);
		}
	}
	else
	{
		Mesh->SetVertexNormal(va, UpdatedNormal);
	}
}

// Custom behavior for FAttrBasedQuadric simplifier.
template<>
void TMeshSimplification<FAttrBasedQuadricErrorV2d>::UpdateVertexAttributes(int edgeID, int va, int vb)
{
	// Update the normal
	const FAttrBasedQuadricErrorV2d& Q = EdgeQuadrics[edgeID].q;
	FVector3d collapse_pt = EdgeQuadrics[edgeID].collapse_pt;

	const FAttrBasedQuadricErrorV2d::FAttrArray WedgeAttributes = Q.ComputeAttributes(collapse_pt);
	const int AttributeCount = Q.GetAttributeCount();

	TArray<int> ElementIdArray;
	
	auto UpdateAttribute = [&ElementIdArray, &va, &Q, &WedgeAttributes, this]<typename OverlayT, typename TransformT>(
		const int WedgeBegin, const int WedgeCount, OverlayT* Overlay, TransformT Transform)
	{
		Overlay->GetVertexElements(va, ElementIdArray);

		// walk over wedges
		for (int j=0; j<WedgeCount; ++j)
		{
			if (!Q.IsWedgeActive(WedgeBegin + j))
			{
				continue;
			}

			const FIndex3i& AttrTri = Q.GetWedgeAttributeIndex(WedgeBegin + j);
			int EID = IndexConstants::InvalidID;

			if (ElementIdArray.Find(AttrTri.B) != INDEX_NONE)
			{
				EID = AttrTri.B;
			}
			else if (ElementIdArray.Find(AttrTri.A) != INDEX_NONE)
			{
				EID = AttrTri.A;
			}

			if (EID != IndexConstants::InvalidID) 
			{
			 	const auto Attribute = Transform(WedgeAttributes[WedgeBegin + j], EID);
				if (VectorUtil::IsFinite(Attribute))
				{
					Overlay->SetElement(EID, Attribute);
				}
			}
		}
	};

	int WedgeBegin = 0;

	int AttributeIndex = 0;

	if (QuadricOptions.NormalAttributeWeight > 0.)
	{
		if (!ensure(AttributeIndex < AttributeCount))
		{
			return;
		}

		const int N = Q.GetWedgeCount(AttributeIndex);

		if (NormalOverlay != nullptr)
		{
			UpdateAttribute(WedgeBegin, N, NormalOverlay, [&Q](const FVector3d& Normal, const int EID) -> FVector3f
			{
				FVector3f UpdatedNormal(Normal);
				return UpdatedNormal;
			} );
		}
		else
		{
			FVector3f UpdatedNormal(WedgeAttributes[WedgeBegin]);
			Normalize(UpdatedNormal);
			Mesh->SetVertexNormal(va, UpdatedNormal);
		}
		WedgeBegin += N;
		AttributeIndex++;
	}

	FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();

	FDynamicMeshNormalOverlay* TangentOverlay = Attributes ? Attributes->PrimaryTangents() : nullptr;
	if (TangentOverlay && QuadricOptions.TangentAttributeWeight > 0.)
	{
		if (!ensure(AttributeIndex < AttributeCount))
		{
			return;
		}

		// is it normal, that tangents are normal? 
		const int N = Q.GetWedgeCount(AttributeIndex);
		
		if (TangentOverlay)
		{
			UpdateAttribute(WedgeBegin, N, TangentOverlay, [](const FVector3d& Tangent, const int EID) -> FVector3f 
			{
				FVector3f UpdatedTangent(Tangent);
				Normalize(UpdatedTangent);
				return UpdatedTangent;
			} );
		}

		WedgeBegin += N;
		AttributeIndex++;
	}

	FDynamicMeshNormalOverlay* BiTangentOverlay = Attributes ? Attributes->PrimaryBiTangents() : nullptr;
	if (BiTangentOverlay && QuadricOptions.TangentAttributeWeight > 0.)
	{
		if (!ensure(AttributeIndex < AttributeCount))
		{
			return;
		}

		const int N = Q.GetWedgeCount(AttributeIndex);

		if (BiTangentOverlay)
		{
			UpdateAttribute(WedgeBegin, N, BiTangentOverlay, [](const FVector3d& BiTangent, const int EID) -> FVector3f 
			{
				FVector3f UpdatedBiTangent(BiTangent);
				Normalize(UpdatedBiTangent);
				return UpdatedBiTangent;
			} );
		}

		WedgeBegin += N;
		AttributeIndex++;
	}

	FDynamicMeshColorOverlay* ColorOverlay = Attributes ? Attributes->PrimaryColors() : nullptr;
	if (ColorOverlay && QuadricOptions.ColorAttributeWeight > 0.)
	{
		if (!ensure(AttributeIndex < AttributeCount))
		{
			return;
		}

		const int N = Q.GetWedgeCount(AttributeIndex);

		if (ColorOverlay)
		{
			UpdateAttribute(WedgeBegin, N, ColorOverlay, [&ColorOverlay](const FVector3d& Color, const int EID) -> FVector4f 
			{
				FVector4f UpdatedColor(ColorOverlay->GetElement(EID));
				UpdatedColor.X = static_cast<float>(Color.X);
				UpdatedColor.Y = static_cast<float>(Color.Y);
				UpdatedColor.Z = static_cast<float>(Color.Z);
				return UpdatedColor;
			} );
		}
		WedgeBegin += N;
		AttributeIndex++;
	}

	const int NumWeightLayers = Attributes ? Attributes->NumWeightLayers() : 0;
	if (NumWeightLayers > 0 && QuadricOptions.WeightLayerWeight > 0.)
	{
		const int NumWeightGroups = (NumWeightLayers + 2) / 3;
		for (int GroupIndex = 0; GroupIndex < NumWeightGroups; ++GroupIndex)
		{
			if (!ensure(AttributeIndex < AttributeCount))
			{
				return;
			}

			const int N = Q.GetWedgeCount(AttributeIndex);
			
			// weight layers are per-vertex, so no seams could create multiple wedges.
			ensure(N == 1);

			for (int j = 0; j < N; ++j)
			{
				if (!Q.IsWedgeActive(WedgeBegin + j))
				{
					continue;
				}

				FVector3f WeightChannels(WedgeAttributes[WedgeBegin + j]);

				for (int ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
				{
					const int WeightLayerIndex = GroupIndex * 3 + ChannelIndex;
					if (WeightLayerIndex < NumWeightLayers)
					{
						FDynamicMeshWeightAttribute* WeightAttribute = Attributes->GetWeightLayer(WeightLayerIndex);
						WeightAttribute->SetValue(va, &WeightChannels[ChannelIndex]);
					}
				}
			}

			WedgeBegin += N;
			AttributeIndex++;
		}
	}

	int NumUVOverlays = Attributes ? Attributes->NumUVLayers() : 0;
	for (int UVLayerIndex = 0; UVLayerIndex < NumUVOverlays; ++UVLayerIndex)
	{
		FDynamicMeshUVOverlay* UVOverlay = Attributes->GetUVLayer(UVLayerIndex);
		if (UVOverlay && QuadricOptions.TexCoordAttributeWeight > 0.)
		{
			if (!ensure(AttributeIndex < AttributeCount))
			{
				return;
			}
			const int N = Q.GetWedgeCount(AttributeIndex);

			UpdateAttribute(WedgeBegin, N, UVOverlay, [](const FVector3d& UV, const int EID) -> FVector2f 
			{
				return FVector2f(static_cast<float>(UV[0]), static_cast<float>(UV[1]));
			} );
			WedgeBegin += N;
			AttributeIndex++;
		}
	}
}

namespace UE
{
namespace Geometry
{

// These are explicit instantiations of the templates that are exported from the shared lib.
// Only these instantiations of the template can be used.
// This is necessary because we have placed most of the templated functions in this .cpp file, instead of the header.
#if PLATFORM_COMPILER_CLANG && !PLATFORM_MICROSOFT
template class DYNAMICMESH_API TMeshSimplification< FAttrBasedQuadricErrord >;
template class DYNAMICMESH_API TMeshSimplification< FAttrBasedQuadricErrorV2d >;
template class DYNAMICMESH_API TMeshSimplification< FVolPresQuadricErrord >;
template class DYNAMICMESH_API TMeshSimplification< FQuadricErrord >;
#else
template class TMeshSimplification< FAttrBasedQuadricErrord >;
template class TMeshSimplification< FAttrBasedQuadricErrorV2d >;
template class TMeshSimplification< FVolPresQuadricErrord >;
template class TMeshSimplification< FQuadricErrord >;
#endif

} // end namespace UE::Geometry
} // end namespace UE

