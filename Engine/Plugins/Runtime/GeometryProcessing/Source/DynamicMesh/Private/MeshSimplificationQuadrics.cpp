// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSimplificationQuadrics.h"

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

namespace UE
{

namespace Geometry
{

template <typename RealType>
void UpdateResidual(double& MaxRes, const RealType SM[6], const TVector<double>& X, const TVector<double>& B)
{
	const TVector<double> Ax = TQuadricError<RealType>::MultiplySymmetricMatrix(SM, X);
	const double ResSqr = (Ax - B) | (Ax - B);

	MaxRes = FMath::Max(ResSqr, MaxRes);
}

template <typename RealType>
struct SolverBuiltin
{
	SolverBuiltin(const RealType SM[6])
	{
		bIsValid = TQuadricError<RealType>::InvertSymmetricMatrix(SM, InvSM);
	}

	bool IsValid() const 
	{
		return bIsValid;
	}

	bool Solve(TVector<RealType>& X, const TVector<RealType>& Rhs) const
	{
		if (bIsValid)
		{
			X = TQuadricError<RealType>::MultiplySymmetricMatrix(InvSM, Rhs);
		}
		return bIsValid;
	}

	RealType InvSM[6];
	bool bIsValid { false };
};


template <typename RealType>
struct SolverEigen
{
	SolverEigen(const RealType SM[6])
	{
		M << SM[0], SM[1], SM[2],
		     SM[1], SM[3], SM[4],
		     SM[2], SM[4], SM[5];

		ldlt.compute(M);

		bIsValid = true;

		if (ldlt.info() != Eigen::Success)
		{
			bIsValid = false;
		}

		if(ldlt.vectorD().cwiseAbs().minCoeff() < 1e-12)
		{
			bIsValid = false;
		}
	}

	bool IsValid() const 
	{
		return bIsValid;
	}

	bool Solve(TVector<RealType>& X, const TVector<RealType>& Rhs) const
	{
		if (bIsValid)
		{
			Eigen::Vector3d x = ldlt.solve(Eigen::Vector3d(Rhs[0], Rhs[1], Rhs[2]));
			X.X = x(0);
			X.Y = x(1);
			X.Z = x(2);
		}
		return bIsValid;
	}

	Eigen::Matrix3d M;
	Eigen::LDLT<Eigen::Matrix3d> ldlt;
	bool bIsValid { false };
};

#ifdef UE_USE_INTERNAL_MATRIX_SOLVER
// requires dependencies to Source/Developer, so currently disabled. It would require making the solvers accessible
// to runtime plugins

template <typename RealType>
struct SolverLUP
{
	SolverLUP(const RealType SM[6])
	{
		M[0] = SM[0];
		M[1] = SM[1];
		M[2] = SM[2];

		M[3] = SM[1];
		M[4] = SM[3];
		M[5] = SM[4];

		M[6] = SM[2];
		M[7] = SM[4];
		M[8] = SM[5];

		FMemory::Memcpy( LU, M );
		bIsValid = LUPFactorize( LU, Pivot, 3, 1.e-12 );
	}

	bool IsValid() const 
	{
		return bIsValid;
	}

	bool Solve(TVector<RealType>& X, const TVector<RealType>& Rhs) const
	{
		RealType x[3];
		RealType b[3] = { Rhs[0], Rhs[1], Rhs[2] };

		if (!bIsValid)
		{
			return false;
		}

		if (LUPSolveIterate( M, LU, Pivot, 3, b, x ))
		{
			X[0] = x[0];
			X[1] = x[1];
			X[2] = x[2];
			return true;
		}

		return false;
	}

	RealType M[9];
	RealType LU[9];
	uint32 Pivot[3];
	bool bIsValid { false };
};

#endif

template <typename RealType>
template <typename SolverType>
bool TAttrBasedQuadricErrorV2<RealType>::OptimalPointImpl(
	UE::Math::TVector<RealType>& OutResult) const
{
	const TAttrBasedQuadricErrorV2<RealType>& Q = *this;

	RealType SM[6] = { Q.Axx, Q.Axy, Q.Axz, Q.Ayy, Q.Ayz, Q.Azz };
	TVector<RealType> Rhs(-Q.bx, -Q.by, -Q.bz);

	for (const auto& AttrWedgeQuadricData : Q.AttrWedgeQuadricDatas)
	{
		if (AttrWedgeQuadricData.TotalAreaTerm < UE_DOUBLE_SMALL_NUMBER)
		{
			continue;
		}

		const RealType InvAlpha = 1.0 / AttrWedgeQuadricData.TotalAreaTerm;
		const TVector<RealType> (&G)[3] = AttrWedgeQuadricData.G;
		const TVector<RealType>& D = AttrWedgeQuadricData.D;

		// C - alpha^{-1} B B^T
		// rhs = -b1 + alpha^{-1} B b2 
		for (int i=0; i<3; ++i)
		{
			SM[0] -= InvAlpha * G[i].X * G[i].X;
			SM[1] -= InvAlpha * G[i].X * G[i].Y;
			SM[2] -= InvAlpha * G[i].X * G[i].Z;

			SM[3] -= InvAlpha * G[i].Y * G[i].Y;
			SM[4] -= InvAlpha * G[i].Y * G[i].Z;

			SM[5] -= InvAlpha * G[i].Z * G[i].Z;

			Rhs.X += InvAlpha * G[i].X * D[i];
			Rhs.Y += InvAlpha * G[i].Y * D[i];
			Rhs.Z += InvAlpha * G[i].Z * D[i];
		}
	}

	SolverType Solver(SM);
	bool SolveValid = Solver.Solve(OutResult, Rhs);

	if (!SolveValid)
	{
		return false;
	}
	
	// Adjust with the volume constraint.
	// See: http://hhoppe.com/newqem.pdf or https://www.cc.gatech.edu/~turk/my_papers/memless_vis98.pdf

	// Compute the effect of the volumetric constraint.
	TVector<RealType> Ainv_g;
	SolveValid = Solver.Solve(Ainv_g, Q.GVol);

	if (SolveValid)
	{
		const RealType gt_Ainv_g = Q.GVol.Dot(Ainv_g);

		const RealType gt_unopt = Q.GVol.Dot(OutResult);
		const RealType lambda = (Q.DVol - gt_unopt) / gt_Ainv_g;

		// squared magnitude of the update
		const RealType magnitude_sqr = lambda * lambda * Ainv_g.SizeSquared();

		constexpr double VolThreshold = 0.5;

		if (magnitude_sqr < VolThreshold * Q.Area) 
		{
			// also, guard against outliers in which the new point is too far away relative to the local neighborhood size
			// in case of division by zero, lambda is NaN and this will evaluate to false

			OutResult += lambda * Ainv_g;
		}
		else
		{
			// solve for position succeeded, but volume constraint rejected. still accept the solve
			return true;
		}
	}
	
	return SolveValid;
}


template <typename RealType>
bool TAttrBasedQuadricErrorV2<RealType>::OptimalPoint(
	TVector<RealType>& OutResult, RealType MinThresh) const
{
	// return OptimalPointImpl<SolverBuiltin<RealType>>(OutResult);
	return OptimalPointImpl<SolverEigen<RealType>>(OutResult);

	// MatrixUtils.hh LU solver seems to be same accuracy as Eigen ldlt
	// return OptimalPointImpl<SolverLUP<RealType>>(OutResult);	
}

#if PLATFORM_COMPILER_CLANG && !PLATFORM_MICROSOFT
template class DYNAMICMESH_API TAttrBasedQuadricErrorV2<double>;
#else
template class TAttrBasedQuadricErrorV2<double>;
#endif

} // namespace Geometry
} // namespace UE
