// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshWeights.h"
#include "VectorUtil.h"

using namespace UE::Geometry;

FVector3d FMeshWeights::UniformCentroid(const FDynamicMesh3 & mesh, int32 VertexIndex)
{
	FVector3d Centroid;
	mesh.GetVtxOneRingCentroid(VertexIndex, Centroid);
	return Centroid;
}


FVector3d FMeshWeights::UniformCentroid(const FDynamicMesh3& mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc)
{
	FVector3d Centroid = FVector3d::Zero();
	int32 Count = 0;
	for (int32 nbrvid : mesh.VtxVerticesItr(VertexIndex))
	{
		Centroid += VertexPositionFunc(nbrvid);
		Count++;
	}
	return (Count == 0) ? VertexPositionFunc(VertexIndex) : (Centroid / (double)Count);
}


FVector3d FMeshWeights::FilteredUniformCentroid(const FDynamicMesh3& mesh, int32 VertexIndex, 
	TFunctionRef<FVector3d(int32)> VertexPositionFunc, TFunctionRef<bool(int32)> VertexFilterFunc)
{
	FVector3d Centroid = FVector3d::Zero();
	int32 Count = 0;
	for (int32 nbrvid : mesh.VtxVerticesItr(VertexIndex))
	{
		if (VertexFilterFunc(nbrvid))
		{
			Centroid += VertexPositionFunc(nbrvid);
			Count++;
		}
	}
	return (Count == 0) ? VertexPositionFunc(VertexIndex) : (Centroid / (double)Count);
}



template<typename GetPositionFuncType>
FVector3d TMeanValueCentroid(const FDynamicMesh3& mesh, int32 v_i, GetPositionFuncType GetPositionFunc, double WeightClamp)
{
	// based on equations in https://www.inf.usi.ch/hormann/papers/Floater.2006.AGC.pdf (formula 9)
	// refer to that paper for variable names/etc

	FVector3d vSum = FVector3d::Zero();
	double wSum = 0;
	FVector3d Vi = GetPositionFunc(v_i);

	int v_j = FDynamicMesh3::InvalidID, opp_v1 = FDynamicMesh3::InvalidID, opp_v2 = FDynamicMesh3::InvalidID;
	int t1 = FDynamicMesh3::InvalidID, t2 = FDynamicMesh3::InvalidID;
	for (int eid : mesh.VtxEdgesItr(v_i) ) 
	{
		opp_v2 = FDynamicMesh3::InvalidID;
		mesh.GetVtxNbrhood(eid, v_i, v_j, opp_v1, opp_v2, t1, t2);

		FVector3d Vj = GetPositionFunc(v_j);
		FVector3d vVj = (Vj - Vi);
		double len_vVj = Normalize(vVj);
		// TODO: is this the right thing to do? if vertices are coincident,
		//   weight of this vertex should be very high!
		if (len_vVj < FMathf::ZeroTolerance)
		{
			double w_ij = FMathd::Clamp(10000.0, 0, WeightClamp);
			vSum += w_ij * Vj;
			wSum += w_ij;
			continue;
		}
		FVector3d vVdelta = Normalized(GetPositionFunc(opp_v1) - Vi);
		double w_ij = VectorUtil::VectorTanHalfAngle(vVj, vVdelta);

		if (opp_v2 != FDynamicMesh3::InvalidID)
		{
			FVector3d vVgamma = Normalized(GetPositionFunc(opp_v2) - Vi);
			w_ij += VectorUtil::VectorTanHalfAngle(vVj, vVgamma);
		}

		w_ij /= len_vVj;
		w_ij = FMathd::Clamp(w_ij, 0, WeightClamp);

		vSum += w_ij * Vj;
		wSum += w_ij;
	}
	if (wSum < FMathf::ZeroTolerance)
	{
		return FMeshWeights::UniformCentroid(mesh, v_i, GetPositionFunc);
	}
	return vSum / wSum;
}



FVector3d FMeshWeights::MeanValueCentroid(const FDynamicMesh3& mesh, int32 v_i, double WeightClamp)
{
	return TMeanValueCentroid(mesh, v_i, [&](int32 vid) { return mesh.GetVertex(vid); }, WeightClamp);
}

FVector3d FMeshWeights::MeanValueCentroid(const FDynamicMesh3& mesh, int32 v_i, TFunctionRef<FVector3d(int32)> VertexPositionFunc, double WeightClamp)
{
	return TMeanValueCentroid(mesh, v_i, VertexPositionFunc, WeightClamp);
}


template<typename GetPositionFuncType>
FVector3d TCotanCentroid(const FDynamicMesh3& mesh, int32 v_i, GetPositionFuncType GetPositionFunc)
{
	// based on equations in http://www.geometry.caltech.edu/pubs/DMSB_III.pdf

	FVector3d vSum = FVector3d::Zero();
	double wSum = 0;
	FVector3d Vi = GetPositionFunc(v_i);

	int v_j = FDynamicMesh3::InvalidID, opp_v1 = FDynamicMesh3::InvalidID, opp_v2 = FDynamicMesh3::InvalidID;
	int t1 = FDynamicMesh3::InvalidID, t2 = FDynamicMesh3::InvalidID;
	bool bAborted = false;
	for (int eid : mesh.VtxEdgesItr(v_i)) 
	{
		opp_v2 = FDynamicMesh3::InvalidID;
		mesh.GetVtxNbrhood(eid, v_i, v_j, opp_v1, opp_v2, t1, t2);
		FVector3d Vj = GetPositionFunc(v_j);

		FVector3d Vo1 = GetPositionFunc(opp_v1);
		double cot_alpha_ij = VectorUtil::VectorCot( (Vi - Vo1), (Vj - Vo1) );
		if (cot_alpha_ij == 0) 
		{
			bAborted = true;
			break;
		}
		double w_ij = cot_alpha_ij;

		if (opp_v2 != FDynamicMesh3::InvalidID) 
		{
			FVector3d Vo2 = GetPositionFunc(opp_v2);
			double cot_beta_ij = VectorUtil::VectorCot( (Vi - Vo2), (Vj - Vo2) );
			if (cot_beta_ij == 0) {
				bAborted = true;
				break;
			}
			w_ij += cot_beta_ij;
		}

		vSum += w_ij * Vj;
		wSum += w_ij;
	}
	if (bAborted || fabs(wSum) < FMathd::ZeroTolerance)
	{
		return Vi;
	}
	return vSum / wSum;
}



FVector3d FMeshWeights::CotanCentroid(const FDynamicMesh3& mesh, int32 v_i)
{
	return TCotanCentroid(mesh, v_i, [&](int32 vid) { return mesh.GetVertex(vid); });
}

FVector3d FMeshWeights::CotanCentroid(const FDynamicMesh3& mesh, int32 v_i, TFunctionRef<FVector3d(int32)> VertexPositionFunc)
{
	return TCotanCentroid(mesh, v_i, VertexPositionFunc);
}








template<typename PositionFuncType>
FVector3d TCotanCentroidSafe(const FDynamicMesh3& mesh, int32 v_i, PositionFuncType PositionFunc, double DegenerateTol = 100.0, bool* bOutputIsUniform = nullptr)
{
	// based on equations in http://www.geometry.caltech.edu/pubs/DMSB_III.pdf

	FVector3d vSum = FVector3d::Zero();
	double wSum = 0;
	FVector3d Vi = PositionFunc(v_i);

	bool bDegenerated = false;

	int v_j = FDynamicMesh3::InvalidID, opp_v1 = FDynamicMesh3::InvalidID, opp_v2 = FDynamicMesh3::InvalidID;
	int t1 = FDynamicMesh3::InvalidID, t2 = FDynamicMesh3::InvalidID;
	for (int eid : mesh.VtxEdgesItr(v_i))
	{
		opp_v2 = FDynamicMesh3::InvalidID;
		mesh.GetVtxNbrhood(eid, v_i, v_j, opp_v1, opp_v2, t1, t2);
		FVector3d Vj = PositionFunc(v_j);

		FVector3d Vo1 = PositionFunc(opp_v1);
		double cot_alpha_ij = VectorUtil::VectorCot((Vi - Vo1), (Vj - Vo1));
		double w_ij = cot_alpha_ij;

		double cot_beta_ij = 0;
		if (opp_v2 != FDynamicMesh3::InvalidID)
		{
			FVector3d Vo2 = PositionFunc(opp_v2);
			cot_beta_ij = VectorUtil::VectorCot((Vi - Vo2), (Vj - Vo2));
		}
		w_ij += cot_beta_ij;

		if (FMathd::Abs(w_ij) > DegenerateTol)
		{
			bDegenerated = true;
			break;
		}

		vSum += w_ij * Vj;
		wSum += w_ij;
	}

	if (bOutputIsUniform != nullptr)
	{
		*bOutputIsUniform = bDegenerated;
	}

	if (bDegenerated || fabs(wSum) < FMathd::ZeroTolerance)
	{
		return FMeshWeights::UniformCentroid(mesh, v_i);
	}

	return vSum / wSum;
}



FVector3d FMeshWeights::CotanCentroidSafe(const FDynamicMesh3& mesh, int32 v_i, double DegenerateTol, bool* bFailedToUniform)
{
	return TCotanCentroidSafe(mesh, v_i, [&](int32 vid) { return mesh.GetVertex(vid); }, DegenerateTol, bFailedToUniform);
}

FVector3d FMeshWeights::CotanCentroidSafe(const FDynamicMesh3& mesh, int32 v_i, TFunctionRef<FVector3d(int32)> VertexPositionFunc, double DegenerateTol, bool* bFailedToUniform)
{
	return TCotanCentroidSafe(mesh, v_i, VertexPositionFunc, DegenerateTol, bFailedToUniform);
}




template<typename PositionFuncType>
void TCotanWeightsBlendSafe(const FDynamicMesh3& mesh, int32 v_i, PositionFuncType PositionFunc, TFunctionRef<void(int32, double)> BlendingFunc, double DegenerateTol = 100.0, bool* bOutputIsUniform = nullptr)
{
	// based on equations in http://www.geometry.caltech.edu/pubs/DMSB_III.pdf

	FVector3d Vi = PositionFunc(v_i);

	bool bDegenerated = false;
	
	TArray<int32, TInlineAllocator<32>> Neighbours;
	TArray<double, TInlineAllocator<32>> Weights;
	double wSum = 0;

	int v_j = FDynamicMesh3::InvalidID, opp_v1 = FDynamicMesh3::InvalidID, opp_v2 = FDynamicMesh3::InvalidID;
	int t1 = FDynamicMesh3::InvalidID, t2 = FDynamicMesh3::InvalidID;
	for (int eid : mesh.VtxEdgesItr(v_i))
	{
		opp_v2 = FDynamicMesh3::InvalidID;
		mesh.GetVtxNbrhood(eid, v_i, v_j, opp_v1, opp_v2, t1, t2);
		FVector3d Vj = PositionFunc(v_j);

		FVector3d Vo1 = PositionFunc(opp_v1);
		double cot_alpha_ij = VectorUtil::VectorCot((Vi - Vo1), (Vj - Vo1));
		double w_ij = cot_alpha_ij;

		double cot_beta_ij = 0;
		if (opp_v2 != FDynamicMesh3::InvalidID)
		{
			FVector3d Vo2 = PositionFunc(opp_v2);
			cot_beta_ij = VectorUtil::VectorCot((Vi - Vo2), (Vj - Vo2));
		}
		w_ij += cot_beta_ij;

		if (FMathd::Abs(w_ij) > DegenerateTol)
		{
			bDegenerated = true;
			break;
		}

		Neighbours.Add(v_j);
		Weights.Add(w_ij);
		wSum += w_ij;
	}

	if (bOutputIsUniform != nullptr)
	{
		*bOutputIsUniform = bDegenerated;
	}

	int32 N = Neighbours.Num();
	if (bDegenerated || fabs(wSum) < FMathd::ZeroTolerance)
	{
		double Weight = 1.0 / (double)N;
		for (int32 k = 0; k < N; ++k)
		{
			BlendingFunc(Neighbours[k], Weight);
		}
	}
	else
	{
		for (int32 k = 0; k < N; ++k )
		{
			BlendingFunc(Neighbours[k], Weights[k] / wSum);
		}
	}
}


void FMeshWeights::CotanWeightsBlendSafe(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<void(int32, double)> BlendingFunc, double DegenerateTol, bool* bFailedToUniform)
{
	TCotanWeightsBlendSafe(Mesh, VertexIndex, [&](int32 vid) { return Mesh.GetVertex(vid); }, BlendingFunc, DegenerateTol, bFailedToUniform);
}


template<typename GetPositionFuncType>
double TVoronoiArea(const FDynamicMesh3& mesh, int32 v_i, GetPositionFuncType GetPositionFunc, double CotClampRange = 1000 /* about 0.05 deg */ )
{
	// based on equations in http://www.geometry.caltech.edu/pubs/DMSB_III.pdf

	double areaSum = 0;
	FVector3d Vi = GetPositionFunc(v_i);

	for (int32 tid : mesh.VtxTrianglesItr(v_i))
	{
		FIndex3i t = mesh.GetTriangle(tid);
		int ti = (t[0] == v_i) ? 0 : ((t[1] == v_i) ? 1 : 2);
		FVector3d Vj = GetPositionFunc(t[(ti + 1) % 3]);
		FVector3d Vk = GetPositionFunc(t[(ti + 2) % 3]);

		if (VectorUtil::IsObtuse(Vi, Vj, Vk))
		{
			// if triangle is obtuse voronoi area is undefind and we just return portion of triangle area
			FVector3d Vij = Vj - Vi;
			FVector3d Vik = Vk - Vi;
			double Dot = Vij.Dot(Vik);
			double areaT = 0.5 * FMathd::Sqrt(Vij.SquaredLength() * Vik.SquaredLength() - Dot * Dot);
			Normalize(Vij); Normalize(Vik);
			areaSum += (AngleR(Vij, Vik) > FMathd::HalfPi) ?    // obtuse at v_i ?
				(areaT * 0.5) : (areaT * 0.25);
		}
		else
		{
			// voronoi area
			FVector3d Vji = Vi - Vj;
			FVector3d Vki = Vi - Vk;
			FVector3d Vkj = Vj - Vk;
			double cot_alpha_ij = VectorUtil::VectorCot(Vki, Vkj);
			cot_alpha_ij = FMathd::Clamp(cot_alpha_ij, -CotClampRange, CotClampRange);
			double cot_beta_ik = VectorUtil::VectorCot(Vji, -Vkj);
			cot_beta_ik = FMathd::Clamp(cot_beta_ik, -CotClampRange, CotClampRange);
			areaSum += Vji.SquaredLength() * cot_alpha_ij * 0.125;
			areaSum += Vki.SquaredLength() * cot_beta_ik * 0.125;
		}
	}
	return areaSum;
}




double FMeshWeights::VoronoiArea(const FDynamicMesh3& mesh, int32 v_i)
{
	return TVoronoiArea(mesh, v_i, [&](int32 vid) { return mesh.GetVertex(vid); });
}

double FMeshWeights::VoronoiArea(const FDynamicMesh3& mesh, int32 v_i, TFunctionRef<FVector3d(int32)> VertexPositionFunc)
{
	return TVoronoiArea(mesh, v_i, VertexPositionFunc);
}

