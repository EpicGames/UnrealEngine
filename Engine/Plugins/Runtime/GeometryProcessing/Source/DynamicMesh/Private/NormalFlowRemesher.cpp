// Copyright Epic Games, Inc. All Rights Reserved.

#include "NormalFlowRemesher.h"
#include "Async/ParallelFor.h"
#include "DynamicMesh/InfoTypes.h"
#include "Async/ParallelTransformReduce.h"

using namespace UE::Geometry;

void FNormalFlowRemesher::RemeshWithFaceProjection()
{
	if (Mesh->TriangleCount() == 0)
	{
		return;
	}

	ModifiedEdgesLastPass = 0;

	ResetQueue();

	// First we do fast splits to hit edge length target

	for (int k = 0; k < MaxFastSplitIterations; ++k)
	{
		if (Cancelled())
		{
			return;
		}

		int nSplits = FastSplitIteration();

		if ((double)nSplits / (double)Mesh->EdgeCount() < 0.01)
		{
			// Call it converged
			break;
		}
	}
	ResetQueue();

	// Now do queued remesh iterations. As we proceed we slowly step
	// down the smoothing factor, this helps us get triangles closer
	// to where they will ultimately want to go

	const double OriginalSmoothSpeed = SmoothSpeedT;
	
	int Iterations = 0;
	const double ProjectionDistanceThreshold = 0.1 * MinEdgeLength;

	bool bContinue = true;
	while (bContinue)
	{
		if (Cancelled())
		{
			break;
		}

		// currently disabling converge check
		RemeshIteration( []() {return false; } );

		if (Iterations > MaxRemeshIterations / 2)
		{
			SmoothSpeedT *= 0.9;
		}

		double MaxProjectionDistance = 0.0;
		constexpr bool bIsTuningIteration = false;
		if (bEnableParallelProjection)
		{
			TrackedFaceProjectionPass(MaxProjectionDistance, bIsTuningIteration);
		}
		else
		{
			TrackedFaceProjectionPass_Serial(MaxProjectionDistance, bIsTuningIteration);
		}
		

		// Stop if we've hit max iterations, hit triangle limit, or both:
		// - queue is empty and
		// - projection isn't moving anything
		bContinue =
			(Iterations++ < MaxRemeshIterations)
			&& ((ModifiedEdges->Num() > 0) || (MaxProjectionDistance > ProjectionDistanceThreshold))
			&& ((MaxTriangleCount == 0) || (Mesh->TriangleCount() < MaxTriangleCount));
	}

	SmoothSpeedT = OriginalSmoothSpeed;

	// Now just face projections and edge flips
	if (ProjTarget != nullptr)
	{
		for (int k = 0; k < NumExtraProjectionIterations; ++k)
		{
			if (Cancelled())
			{
				break;
			}

			double MaxProjectionDistance = 0.0;
			constexpr bool bIsTuningIteration = true;
			if (bEnableParallelProjection)
			{
				TrackedFaceProjectionPass(MaxProjectionDistance, bIsTuningIteration);
			}
			else
			{
				TrackedFaceProjectionPass_Serial(MaxProjectionDistance, bIsTuningIteration);
			}

			if (MaxProjectionDistance == 0.0)
			{
				break;
			}

			// See if we can flip edges to improve normal fit
			TrackedEdgeFlipPass();
		}
	}

}



void FNormalFlowRemesher::TrackedFaceProjectionPass(double& MaxDistanceMoved, bool bIsTuningIteration)
{
	ensure(ProjTarget != nullptr);

	IOrientedProjectionTarget* NormalProjTarget = static_cast<IOrientedProjectionTarget*>(ProjTarget);
	ensure(NormalProjTarget != nullptr);

	{
		// Make sure the temp buffers are an adequate size

		int NumTriangleVertices = 3 * Mesh->MaxTriangleID();
		if (ProjectionVertexBuffer.Num() < NumTriangleVertices)
		{
			ProjectionVertexBuffer.SetNumUninitialized(2 * NumTriangleVertices);
		}
		if (ProjectionWeightBuffer.Num() < NumTriangleVertices)
		{
			ProjectionWeightBuffer.SetNumUninitialized(2 * NumTriangleVertices);
		}

		int MaxVertexID = Mesh->MaxVertexID();
		if (TempWeightBuffer.Num() < MaxVertexID)
		{
			TempWeightBuffer.SetNumUninitialized(2 * MaxVertexID);
		}
		if (TempPosBuffer.Num() < MaxVertexID)
		{
			TempPosBuffer.SetNum(2 * MaxVertexID);
		}
		if (TempFlagBuffer.Num() < MaxVertexID)
		{
			TempFlagBuffer.SetNumUninitialized(2 * MaxVertexID);
		}

		int MaxEdgeIDs = Mesh->MaxEdgeID();
		if (EdgeShouldBeQueuedBuffer.Num() < MaxEdgeIDs)
		{
			EdgeShouldBeQueuedBuffer.SetNumUninitialized(2 * MaxEdgeIDs);
		}
	}

	const double MaxStepDistance = (bIsTuningIteration) ? (0.25 * MaxEdgeLength) : (0.5 * MaxEdgeLength);		// kind of arbitrary...
	const double SmoothAreaDistance = FillAreaDistanceMultiplier * MaxEdgeLength;

	const TFunction<FVector3d(const FDynamicMesh3&, int, double)> UseSmoothFunc = GetSmoothFunction();

	// For each triangle, rotate it such that it aligns with closest normal on the target surface.
	// Each vertex is then assigned a weighted combination of its corresponding triangle corner positions. The weighting
	// is chosen to favor triangles that don't rotate much (i.e. are already aligned with the target surface.)

	// First compute all rotated triangles and store their corner positions and weights
	ParallelFor(Mesh->MaxTriangleID(), [this, NormalProjTarget, bIsTuningIteration, SmoothAreaDistance, UseSmoothFunc, MaxStepDistance](int32 TriangleIndex)
	{
		if (!Mesh->IsTriangle(TriangleIndex))
		{
			return;
		}
		FVector3d TriangleNormal, Centroid;
		double Area;
		Mesh->GetTriInfo(TriangleIndex, TriangleNormal, Area, Centroid);

		FVector3d ProjectedNormal{ 1e30, 1e30, 1e30 };
		FVector3d ProjectedPosition = NormalProjTarget->Project(Centroid, ProjectedNormal);

		if (TriangleNormal.Length() < 0.9 || ProjectedNormal.Length() < 0.9)
		{
			return;		// skip this triangle
		}

		if (!ensure(ProjectedNormal[0] != 1e30))
		{
			return;
		}
		if (!ensure(ProjectedNormal.Length() > 1e-6))
		{
			return;
		}

		FIndex3i TriangleVertices = Mesh->GetTriangle(TriangleIndex);

		// if we are tuning and we are not within distance band from the target mesh, then this 
		// is likely an area that cannot project, ie a "fill" area. In those areas we just want to
		// smooth because the projection+remesh will have created very ugly geometry
		// Note: it seems like this ought to be done per-vertex, not triangle based. However this
		// so far has not produced as good of results - not entirely clear why, but it seems that
		// around the boundaries, the triangles that sill have a good projection extert some pull
		// back on the boundary vertices, which keeps them more stable, otherwise they shrink too
		// much and this causes artifacts around the border (perhaps because the normals are never converging).
		// It *might* work better to do this as a fully separate step, after everything else has 
		// converged, and do some kind of weighted-identification of these hole areas

		if (bIsTuningIteration && bSmoothInFillAreas && Distance(ProjectedPosition, Centroid) > SmoothAreaDistance && FillAreaSmoothMultiplier > 0)
		{
			for (int32 j = 0; j < 3; ++j)
			{
				bool bModified = false;
				FVector3d SmoothPos = ComputeSmoothedVertexPos(TriangleVertices[j], UseSmoothFunc, bModified);
				if (bModified)
				{
					double Weight = Area;
					SmoothPos = Lerp(Centroid, SmoothPos, FillAreaSmoothMultiplier * SmoothSpeedT);

					ProjectionVertexBuffer[3*TriangleIndex + j] = Weight * SmoothPos;
					ProjectionWeightBuffer[3*TriangleIndex + j] = Weight;
				}
			}
			return;
		}

		// apply damping to new position/normal
		ProjectedPosition = Lerp(Centroid, ProjectedPosition, SurfaceProjectionSpeed);
		ProjectedNormal = Normalized(Lerp(TriangleNormal, ProjectedNormal, NormalAlignmentSpeed));
		if (ProjectedNormal.Length() < 0.1)
		{
			ProjectedNormal = TriangleNormal;
		}

		// clamp movement of target position, to prevent moving too far in a single step
		FVector3d MoveDelta = (ProjectedPosition - Centroid);
		double MoveLength = Normalize(MoveDelta);

		MoveLength = FMathd::Min(MoveLength, MaxStepDistance);
		ProjectedPosition = Centroid + MoveLength * MoveDelta;

		FVector3d V0, V1, V2;
		Mesh->GetTriVertices(TriangleIndex, V0, V1, V2);

		FFrame3d TriF(Centroid, TriangleNormal);
		V0 = TriF.ToFramePoint(V0);
		V1 = TriF.ToFramePoint(V1);
		V2 = TriF.ToFramePoint(V2);

		TriF.AlignAxis(2, ProjectedNormal);
		TriF.Origin = ProjectedPosition;
		V0 = TriF.FromFramePoint(V0);
		V1 = TriF.FromFramePoint(V1);
		V2 = TriF.FromFramePoint(V2);

		double Dot = TriangleNormal.Dot(ProjectedNormal);
		Dot = FMath::Clamp(Dot, 0.0, 1.0);
		const double Weight = Area * (Dot * Dot * Dot);

		ProjectionVertexBuffer[3 * TriangleIndex] = Weight * V0;
		ProjectionWeightBuffer[3 * TriangleIndex] = Weight;
		ProjectionVertexBuffer[3 * TriangleIndex + 1] = Weight * V1;
		ProjectionWeightBuffer[3 * TriangleIndex + 1] = Weight;
		ProjectionVertexBuffer[3 * TriangleIndex + 2] = Weight * V2;
		ProjectionWeightBuffer[3 * TriangleIndex + 2] = Weight;
	});

	// Next compute all the weighted average of triangle corners for each vertex
	ParallelFor(Mesh->MaxVertexID(), [this, &MaxDistanceMoved, bIsTuningIteration, MaxStepDistance](int32 VertexIndex)
	{
		TempFlagBuffer[VertexIndex] = false;

		if (!Mesh->IsVertex(VertexIndex))
		{
			return;
		}

		TempPosBuffer[VertexIndex] = Mesh->GetVertex(VertexIndex);		// Don't modify the position of constrained vertex

		if (IsVertexPositionConstrained(VertexIndex))
		{
			return;
		}

		if (VertexControlF != nullptr && ((int)VertexControlF(VertexIndex) & (int)EVertexControl::NoProject) != 0)
		{
			return;
		}

		TempWeightBuffer[VertexIndex] = 0.0;
		TempPosBuffer[VertexIndex] = { 0.0, 0.0, 0.0 };

		FDynamicMesh3::FLocalIntArray IncidentTriangles;
		Mesh->GetVtxTriangles(VertexIndex, IncidentTriangles);

		for (int TriID : IncidentTriangles)
		{
			const FIndex3i Tri = Mesh->GetTriangle(TriID);

			// Find VertexIndex in Tri:
			for (int IndexInTri = 0; IndexInTri < 3; ++IndexInTri)
			{
				if (Tri[IndexInTri] == VertexIndex)
				{
					int BufferIndex = 3 * TriID + IndexInTri;
					TempWeightBuffer[VertexIndex] += ProjectionWeightBuffer[BufferIndex];
					TempPosBuffer[VertexIndex] += ProjectionVertexBuffer[BufferIndex];
				}
			}
		}

		if (FMath::IsNearlyZero(TempWeightBuffer[VertexIndex]))
		{
			return;
		}

		FVector3d CurrentPosition = Mesh->GetVertex(VertexIndex);
		FVector3d ProjectedPosition = TempPosBuffer[VertexIndex] / TempWeightBuffer[VertexIndex];

		// clamp movement of target position, to prevent moving too far in a single step
		FVector3d MoveDelta = (ProjectedPosition - CurrentPosition);
		double MoveLength = Normalize(MoveDelta);
		MoveLength = FMathd::Min(MoveLength, MaxStepDistance);
		ProjectedPosition = CurrentPosition + MoveLength * MoveDelta;

		if (VectorUtil::EpsilonEqual(CurrentPosition, ProjectedPosition, FMathd::ZeroTolerance))
		{
			return;
		}

		TempFlagBuffer[VertexIndex] = true;
		TempPosBuffer[VertexIndex] = ProjectedPosition;
	});


	// We queue any edges that moved far enough to fall under min/max edge length thresholds
	ParallelFor(Mesh->MaxEdgeID(), [this](int32 EdgeID)
	{
		EdgeShouldBeQueuedBuffer[EdgeID] = false;

		if (!Mesh->IsEdge(EdgeID))
		{
			return;
		}

		const FIndex2i& EdgeVertices = Mesh->GetEdgeV(EdgeID);
		const double NewEdgeLength = Distance(TempPosBuffer[EdgeVertices[0]], TempPosBuffer[EdgeVertices[1]]);
		if (NewEdgeLength < MinEdgeLength || NewEdgeLength > MaxEdgeLength)
		{
			EdgeShouldBeQueuedBuffer[EdgeID] = true;
		}
	});

	for (int EdgeID : Mesh->EdgeIndicesItr())
	{
		if (EdgeShouldBeQueuedBuffer[EdgeID])
		{
			QueueEdge(EdgeID);
		}
	}


	// Return the maximum distance moved by a vertex
	MaxDistanceMoved = ParallelTransformReduce(Mesh->MaxVertexID(), 0.0, [&](int64 InVertexIndex) -> double
	{
		check(InVertexIndex < Mesh->MaxVertexID());
		int VID = (int)InVertexIndex;
		if (TempFlagBuffer[VID] && Mesh->IsVertex(VID))
		{
			const FVector3d& CurrentPosition = Mesh->GetVertex(VID);
			return Distance(CurrentPosition, TempPosBuffer[VID]);
		}
		return 0.0;
	},
													   [](double A, double B) -> double
	{
		return FMath::Max(A, B);
	},
		32);

	// Update vertices
	constexpr bool bUpdateParallel = true;
	ApplyVertexBuffer(bUpdateParallel);
}



void FNormalFlowRemesher::TrackedFaceProjectionPass_Serial(double& MaxDistanceMoved, bool bIsTuningIteration)
{
	ensure(ProjTarget != nullptr);

	IOrientedProjectionTarget* NormalProjTarget = static_cast<IOrientedProjectionTarget*>(ProjTarget);
	ensure(NormalProjTarget != nullptr);

	InitializeVertexBufferForFacePass();

	double MaxStepDistance = (bIsTuningIteration) ? (0.25*MaxEdgeLength) : (0.5*MaxEdgeLength);		// kind of arbitrary...
	double SmoothAreaDistance = FillAreaDistanceMultiplier * MaxEdgeLength;

	TFunction<FVector3d(const FDynamicMesh3&, int, double)> UseSmoothFunc = GetSmoothFunction();

	// this function computes rotated position of triangle, such that it
	// aligns with face normal on target surface. We accumulate weighted-average
	// of vertex positions, which we will then use further down where possible.

	for (int TriangleIndex : Mesh->TriangleIndicesItr())
	{
		FVector3d TriangleNormal, Centroid;
		double Area;
		Mesh->GetTriInfo(TriangleIndex, TriangleNormal, Area, Centroid);
		FIndex3i TriangleVertices = Mesh->GetTriangle(TriangleIndex);
		
		FVector3d ProjectedNormal{ 1e30, 1e30, 1e30 };
		FVector3d ProjectedPosition = NormalProjTarget->Project(Centroid, ProjectedNormal);
		if (TriangleNormal.Length() < 0.9 || ProjectedNormal.Length() < 0.9 )
		{
			continue;		// skip this triangle
		}

		// if we are tuning and we are not within distance band from the target mesh, then this 
		// is likely an area that cannot project, ie a "fill" area. In those areas we just want to
		// smooth because the projection+remesh will have created very ugly geometry
		// Note: it seems like this ought to be done per-vertex, not triangle based. However this
		// so far has not produced as good of results - not entirely clear why, but it seems that
		// around the boundaries, the triangles that sill have a good projection extert some pull
		// back on the boundary vertices, which keeps them more stable, otherwise they shrink too
		// much and this causes artifacts around the border (perhaps because the normals are never converging).
		// It *might* work better to do this as a fully separate step, after everything else has 
		// converged, and do some kind of weighted-identification of these hole areas
		if (bIsTuningIteration && bSmoothInFillAreas && Distance(ProjectedPosition, Centroid) > SmoothAreaDistance && FillAreaSmoothMultiplier > 0)
		{
			for (int32 j = 0; j < 3; ++j)
			{
				bool bModified = false;
				FVector3d SmoothPos = ComputeSmoothedVertexPos(TriangleVertices[j], UseSmoothFunc, bModified);
				if (bModified)
				{
					double Weight = Area;
					SmoothPos = Lerp(Centroid, SmoothPos, FillAreaSmoothMultiplier*SmoothSpeedT);
					TempPosBuffer[TriangleVertices[j]] += Weight * SmoothPos;
					TempWeightBuffer[TriangleVertices[j]] += Weight;
				}
			}
			continue;
		}

		// apply damping to new position/normal
		ProjectedPosition = Lerp(Centroid, ProjectedPosition, SurfaceProjectionSpeed);
		ProjectedNormal = Normalized(Lerp(TriangleNormal, ProjectedNormal, NormalAlignmentSpeed));
		if (ProjectedNormal.Length() < 0.1)
		{
			ProjectedNormal = TriangleNormal;
		}

		// clamp movement of target position, to prevent moving too far in a single step
		FVector3d MoveDelta = (ProjectedPosition - Centroid);
		double MoveLength = Normalize(MoveDelta);

		MoveLength = FMathd::Min(MoveLength, MaxStepDistance);
		ProjectedPosition = Centroid + MoveLength * MoveDelta;

		FVector3d V0, V1, V2;
		Mesh->GetTriVertices(TriangleIndex, V0, V1, V2);

		FFrame3d TriF(Centroid, TriangleNormal);
		V0 = TriF.ToFramePoint(V0);
		V1 = TriF.ToFramePoint(V1);
		V2 = TriF.ToFramePoint(V2);

		TriF.AlignAxis(2, ProjectedNormal);
		TriF.Origin = ProjectedPosition;
		V0 = TriF.FromFramePoint(V0);
		V1 = TriF.FromFramePoint(V1);
		V2 = TriF.FromFramePoint(V2);

		double Dot = TriangleNormal.Dot(ProjectedNormal);
		Dot = FMath::Clamp(Dot, 0.0, 1.0);
		double Weight = Area * (Dot * Dot * Dot);

		TempPosBuffer[TriangleVertices.A] += Weight * V0;
		TempWeightBuffer[TriangleVertices.A] += Weight;
		TempPosBuffer[TriangleVertices.B] += Weight * V1;
		TempWeightBuffer[TriangleVertices.B] += Weight;
		TempPosBuffer[TriangleVertices.C] += Weight * V2;
		TempWeightBuffer[TriangleVertices.C] += Weight;
	}

	// ok now we filter out all the positions we can't change, as well as vertices that
	// did not actually move. We also queue any edges that moved far enough to fall
	// under min/max edge length thresholds

	MaxDistanceMoved = 0.0;

	for (int VertexID : Mesh->VertexIndicesItr())
	{
		TempFlagBuffer[VertexID] = false;

		if (FMath::IsNearlyZero(TempWeightBuffer[VertexID]))
		{
			continue;
		}

		if (IsVertexPositionConstrained(VertexID))
		{
			continue;
		}

		if (VertexControlF != nullptr && ((int)VertexControlF(VertexID) & (int)EVertexControl::NoProject) != 0)
		{
			continue;
		}

		FVector3d CurrentPosition = Mesh->GetVertex(VertexID);
		FVector3d ProjectedPosition = TempPosBuffer[VertexID] / TempWeightBuffer[VertexID];

		if (VectorUtil::EpsilonEqual(CurrentPosition, ProjectedPosition, FMathd::ZeroTolerance))
		{
			continue;
		}

		// clamp movement of target position, to prevent moving too far in a single step
		FVector3d MoveDelta = (ProjectedPosition - CurrentPosition);
		double MoveLength = Normalize(MoveDelta);
		MoveLength = FMathd::Min(MoveLength, MaxStepDistance);
		ProjectedPosition = CurrentPosition + MoveLength * MoveDelta;

		MaxDistanceMoved = FMath::Max(MaxDistanceMoved, Distance(CurrentPosition, ProjectedPosition));

		TempFlagBuffer[VertexID] = true;
		TempPosBuffer[VertexID] = ProjectedPosition;

		for (int EdgeID : Mesh->VtxEdgesItr(VertexID))
		{
			FIndex2i EdgeVertices = Mesh->GetEdgeV(EdgeID);
			int OtherVertexID = (EdgeVertices.A == VertexID) ? EdgeVertices.B : EdgeVertices.A;
			FVector3d OtherVertexPosition = Mesh->GetVertex(OtherVertexID);

			double NewEdgeLength = Distance(ProjectedPosition, OtherVertexPosition);
			if (NewEdgeLength < MinEdgeLength || NewEdgeLength > MaxEdgeLength)
			{
				QueueEdge(EdgeID);
			}
		}
	}

	// update vertices
	constexpr bool bUpdateParallel = true;
	ApplyVertexBuffer(bUpdateParallel);
}


namespace
{

	double ComputeNormalError(const FDynamicMesh3* Mesh, IOrientedProjectionTarget* NormalProjTarget, FVector3d TriangleNormal, FVector3d Centroid)
	{
		FVector3d ProjectedNormal{ 1e30, 1e30, 1e30 };
		FVector3d ProjectedPosition = NormalProjTarget->Project(Centroid, ProjectedNormal);

		double Err = 0.5 * (1.0 - TriangleNormal.Dot(ProjectedNormal));
		check(Err > -SMALL_NUMBER);
		check(Err < 1.0 + SMALL_NUMBER);

		return Err;
	}

	double ComputeNormalError(const FDynamicMesh3* Mesh, IOrientedProjectionTarget* NormalProjTarget, FIndex3i Triangle)
	{
		FVector3d v0 = Mesh->GetVertex(Triangle[0]);
		FVector3d v1 = Mesh->GetVertex(Triangle[1]);
		FVector3d v2 = Mesh->GetVertex(Triangle[2]);

		FVector3d Centroid = (v0 + v1 + v2) * (1.0 / 3.0);
		FVector3d Normal = VectorUtil::Normal(v0, v1, v2);

		return ComputeNormalError(Mesh, NormalProjTarget, Normal, Centroid);
	}
}


bool FNormalFlowRemesher::EdgeFlipWouldReduceNormalError(int EdgeID, double BadEdgeErrorThreshold, double ImprovementRatioThreshold) const
{
	IOrientedProjectionTarget* NormalProjTarget = static_cast<IOrientedProjectionTarget*>(ProjTarget);
	if (NormalProjTarget == nullptr)
	{
		return false;
	}

	FDynamicMesh3::FEdge Edge = Mesh->GetEdge(EdgeID);
	if (Edge.Tri[1] == FDynamicMesh3::InvalidID)
	{
		return false;
	}

	double CurrErr = 0.0;
	CurrErr += ComputeNormalError(Mesh, NormalProjTarget, Mesh->GetTriangle(Edge.Tri[0]));
	CurrErr += ComputeNormalError(Mesh, NormalProjTarget, Mesh->GetTriangle(Edge.Tri[1]));

	if (CurrErr > BadEdgeErrorThreshold)	// only consider edges having a certain error already
	{
		FIndex3i TriangleC = Mesh->GetTriangle(Edge.Tri[0]);
		FIndex3i TriangleD = Mesh->GetTriangle(Edge.Tri[1]);
		int VertexInTriangleC = IndexUtil::OrientTriEdgeAndFindOtherVtx(Edge.Vert[0], Edge.Vert[1], TriangleC);
		int VertexInTriangleD = IndexUtil::FindTriOtherVtx(Edge.Vert[0], Edge.Vert[1], TriangleD);

		int OtherEdge = Mesh->FindEdge(VertexInTriangleC, VertexInTriangleD);
		if (OtherEdge != FDynamicMesh3::InvalidID)
		{
			return false;
		}

		double OtherErr = 0.0;
		OtherErr += ComputeNormalError(Mesh, NormalProjTarget, FIndex3i{ VertexInTriangleC, VertexInTriangleD, Edge.Vert[1] });
		OtherErr += ComputeNormalError(Mesh, NormalProjTarget, FIndex3i{ VertexInTriangleD, VertexInTriangleC, Edge.Vert[0] });

		return (OtherErr < ImprovementRatioThreshold * CurrErr);	// return true if we improve error by enough
	}

	return false;
}


void FNormalFlowRemesher::TrackedEdgeFlipPass()
{
	check(ModifiedEdges);

	IOrientedProjectionTarget* NormalProjTarget = static_cast<IOrientedProjectionTarget*>(ProjTarget);
	check(NormalProjTarget != nullptr);

	if (bEnableParallelEdgeFlipPass)
	{
		if (EdgeShouldBeQueuedBuffer.Num() < Mesh->MaxEdgeID())
		{
			EdgeShouldBeQueuedBuffer.SetNum(2 * Mesh->MaxEdgeID());
		}
		EdgeShouldBeQueuedBuffer.Init(false, EdgeShouldBeQueuedBuffer.Num());

		int NumEdges = Mesh->MaxEdgeID();

		ParallelFor(NumEdges, [this](int EdgeID)
		{
			if (!Mesh->IsEdge(EdgeID))
			{
				return;
			}

			FEdgeConstraint Constraint =
				(!Constraints) ? FEdgeConstraint::Unconstrained() : Constraints->GetEdgeConstraint(EdgeID);

			if (!Constraint.CanFlip())
			{
				return;
			}

			if (EdgeFlipWouldReduceNormalError(EdgeID))
			{
				EdgeShouldBeQueuedBuffer[EdgeID] = true;
			}
		});

		for (int32 EdgeID = 0; EdgeID < NumEdges; ++EdgeID)
		{
			if (!EdgeShouldBeQueuedBuffer[EdgeID])
			{
				continue;
			}

			if (!Mesh->IsEdge(EdgeID))
			{
				continue;
			}

			if (!EdgeFlipWouldReduceNormalError(EdgeID))
			{
				continue;
			}

			DynamicMeshInfo::FEdgeFlipInfo FlipInfo;
			auto Result = Mesh->FlipEdge(EdgeID, FlipInfo);

			if (Result == EMeshResult::Ok)
			{
				FIndex2i EdgeVertices = Mesh->GetEdgeV(EdgeID);
				FIndex2i OpposingEdgeVertices = Mesh->GetEdgeOpposingV(EdgeID);

				QueueOneRing(EdgeVertices.A);
				QueueOneRing(EdgeVertices.B);
				QueueOneRing(OpposingEdgeVertices.A);
				QueueOneRing(OpposingEdgeVertices.B);
				OnEdgeFlip(EdgeID, FlipInfo);
			}
		}
	}
	else
	{
		for (auto EdgeID : Mesh->EdgeIndicesItr())
		{
			check(Mesh->IsEdge(EdgeID));

			FEdgeConstraint Constraint =
				(!Constraints) ? FEdgeConstraint::Unconstrained() : Constraints->GetEdgeConstraint(EdgeID);

			if (!Constraint.CanFlip())
			{
				continue;
			}

			if (EdgeFlipWouldReduceNormalError(EdgeID))
			{
				DynamicMeshInfo::FEdgeFlipInfo FlipInfo;
				auto Result = Mesh->FlipEdge(EdgeID, FlipInfo);

				if (Result == EMeshResult::Ok)
				{
					FIndex2i EdgeVertices = Mesh->GetEdgeV(EdgeID);
					FIndex2i OpposingEdgeVertices = Mesh->GetEdgeOpposingV(EdgeID);

					QueueOneRing(EdgeVertices.A);
					QueueOneRing(EdgeVertices.B);
					QueueOneRing(OpposingEdgeVertices.A);
					QueueOneRing(OpposingEdgeVertices.B);
					OnEdgeFlip(EdgeID, FlipInfo);
				}
			}
		}
	}

}

