// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/AverageOverlayToVertices.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "Async/ParallelFor.h"
#include "MeshQueries.h"
#include "Math/UnrealMathUtility.h"

using namespace UE::Geometry;

namespace FAverageOverlayToVerticesLocals
{
    /** 
     * Compute the weighted sum of elements values for all the mesh vertices.
     * 
     * @param Overlay overlay containing the elements to be averaged.
     * @param bWeightByArea if true, include weighting by the area of the triangle.
     * @param bWeightByAngle if true, include weighting by the interior angles of the triangle.
     * @param SummedElementValues Mesh->MaxVertexID()*ElementSize sized array of the weighted sum of the element values 
     *                            at a vertex.
     * @param HasElements Mesh->MaxVertexID() sized array where HasElements[VertexID] is true if VertexID has at 
     *                    least one element (i.e, at least one triangle sharing the vertex is set in the overlay).
     * @param VertexTotalWeights sum of all element weights at a vertex.
     */
    template<typename RealType, int ElementSize>
    void AverageFullMesh(const FDynamicMesh3* Mesh,
                         const TDynamicMeshOverlay<RealType, ElementSize>* Overlay,
                         bool bWeightByArea, 
                         bool bWeightByAngle,
                         TArray<RealType>& SummedElementValues,
                         TArray<bool>& HasElements,
                         TArray<double>& VertexTotalWeights)
    {
        SummedElementValues.Init((RealType)0.0, Mesh->MaxVertexID()*ElementSize);
        HasElements.Init(false, Mesh->MaxVertexID());
        VertexTotalWeights.Init(0.0, Mesh->MaxVertexID());

        // reuse buffer
        RealType ElementData[ElementSize];

        // Iterate over every triangle and accumulate the overlay element values for each triangle vertex
        for (const int32 TID : Mesh->TriangleIndicesItr())
        {
            if (Overlay->IsSetTriangle(TID))
            {
                const FIndex3i TriVert = Mesh->GetTriangle(TID);
                const FIndex3i TriElem = Overlay->GetTriangle(TID);
                
                // per-corner weights
                const FVector3d Weights = TMeshQueries<FDynamicMesh3>::GetVertexWeightsOnTriangle(*Mesh, TID, Mesh->GetTriArea(TID), bWeightByArea, bWeightByAngle);

                // iterate over every corner of the triangle
                for (int32 CornerIdx = 0; CornerIdx < 3; ++CornerIdx)
                {	
                    const int32 CornerVID = TriVert[CornerIdx];
                    Overlay->GetElement(TriElem[CornerIdx], ElementData);
                
                    const int32 StartIdx = CornerVID * ElementSize;      // starting offset into the output averaged values array
                    VertexTotalWeights[CornerVID] += Weights[CornerIdx]; // accumulate weights for the vertex to be used later for the normalization

                    // add the weighted contribution of the element for the vertex
                    for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
                    {
                        SummedElementValues[StartIdx + EIdx] += RealType(Weights[CornerIdx] * (double)ElementData[EIdx]);
                    }

                    HasElements[CornerVID] = true;
                }
            }
        }
    }

    /**
     * Compute the weighed sum of element values for the selection of the mesh vertices.
     * 
     * @param Overlay overlay containing the elements to be averaged.
     * @param SelectedVertices the array of vertex IDs representing the selection.
     * @param bWeightByArea if true, include weighting by the area of the triangle.
     * @param bWeightByAngle if true, include weighting by the interior angles of the triangle.
     * @param SummedElementValues Mesh->MaxVertexID()*ElementSize sized array of the weighted sum of the element values 
     *                            at a vertex.
     * @param HasElements Mesh->MaxVertexID() sized array where HasElements[VertexID] is true if VertexID has at 
     *                    least one element (i.e, at least one triangle sharing the vertex is set in the overlay).
     * @param VertexTotalWeights sum of all element weights at a vertex.
     */
    template<typename RealType, int ElementSize>
    void AverageSelection(const FDynamicMesh3* Mesh,
                          const TDynamicMeshOverlay<RealType, ElementSize>* Overlay,
                          const TArray<int32>& SelectedVertices,
                          bool bWeightByArea, 
                          bool bWeightByAngle,
                          TArray<RealType>& SummedElementValues,
                          TArray<bool>& HasElements,
                          TArray<double>& VertexTotalWeights)
    {
        // Map mesh vertex ID to selection idx
        TMap<int32, int32> VIDToSelected;
        int32 Count = 0;
        VIDToSelected.Reserve(SelectedVertices.Num());
        for (const int32 VID : SelectedVertices)
        {
            VIDToSelected.Add(VID, Count++);
        }

        // Get selected triangles
        TArray<int32> SelectedTriangles;
		SelectedTriangles = TMeshQueries<FDynamicMesh3>::GetVertexSelectedTriangles(*Mesh, SelectedVertices);
        
        SummedElementValues.Init((RealType)0.0, SelectedVertices.Num()*ElementSize);
        HasElements.Init(false, SelectedVertices.Num());
        VertexTotalWeights.Init(0.0, SelectedVertices.Num());

        // reuse buffer
        RealType ElementData[ElementSize];

        // Iterate over every triangle and accumulate the overlay element values for each triangle vertex
        for (const int32 TID : SelectedTriangles)
        {
            if (Mesh->IsTriangle(TID) && Overlay->IsSetTriangle(TID))
            {
                const FIndex3i TriVert = Mesh->GetTriangle(TID);
                const FIndex3i TriElem = Overlay->GetTriangle(TID);
                
                // per-corner weights
                const FVector3d Weights = TMeshQueries<FDynamicMesh3>::GetVertexWeightsOnTriangle(*Mesh, TID, Mesh->GetTriArea(TID), bWeightByArea, bWeightByAngle);

                // iterate over every corner of the triangle
                for (int32 CornerIdx = 0; CornerIdx < 3; ++CornerIdx)
                {	
                    const int32 CornerVID = TriVert[CornerIdx];
                    if (VIDToSelected.Contains(CornerVID)) // check if corner is one of the selected vertices
                    {
                        Overlay->GetElement(TriElem[CornerIdx], ElementData);
                    
                        const int32 SelectedIdx = VIDToSelected[CornerVID];    // remap vertex ID to the index into output averaged values array
                        const int32 StartIdx = SelectedIdx * ElementSize;      // starting offset into the output averaged values array
                        VertexTotalWeights[SelectedIdx] += Weights[CornerIdx]; // accumulate weights for the vertex to be used later for the normalization

                        // add the weighted contribution of the element for the vertex
                        for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
                        {
                            SummedElementValues[StartIdx + EIdx] += RealType(Weights[CornerIdx] * (double)ElementData[EIdx]);
                        }

                        HasElements[SelectedIdx] = true;
                    }
                }
            }
        }
    }
}

FAverageOverlayToVertices::FAverageOverlayToVertices(const FDynamicMesh3& InMesh)
:
Mesh(&InMesh)
{
}

template<typename RealType, int ElementSize>
bool FAverageOverlayToVertices::AverageOverlay(const TDynamicMeshOverlay<RealType, ElementSize>* Overlay,
                                               TArray<RealType>& VertexValues,
                                               TArray<bool>& HasElements)
{
	using namespace FAverageOverlayToVerticesLocals;

    if (Overlay == nullptr)
    {
        return false;
    }

    const bool bWeightByArea = TriangleWeight == ETriangleWeight::Area || TriangleWeight == ETriangleWeight::AngleArea;
    const bool bWeightByAngle = TriangleWeight == ETriangleWeight::Angle || TriangleWeight == ETriangleWeight::AngleArea;
    
    TArray<double> VertexTotalWeights;
    if (Selection.Num())
    {
		AverageSelection(Mesh, Overlay, Selection, bWeightByArea, bWeightByAngle, VertexValues, HasElements, VertexTotalWeights);
    }
    else 
    {
		AverageFullMesh(Mesh, Overlay, bWeightByArea, bWeightByAngle, VertexValues, HasElements, VertexTotalWeights);
    }

    //
    // Normalize the accumulated values
    //
    ParallelFor(HasElements.Num(), [&VertexValues, &HasElements, &VertexTotalWeights](int32 SelectedIdx)
    {
        if (HasElements[SelectedIdx])
        {
            double InvTotalWeight = 1.0;
            if (!FMath::IsNearlyZero(VertexTotalWeights[SelectedIdx]))
            {
                InvTotalWeight = 1.0/VertexTotalWeights[SelectedIdx];
            }

            const int32 StartIdx = SelectedIdx*ElementSize;
            for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
            {
                VertexValues[StartIdx + EIdx] = RealType(InvTotalWeight * (double)VertexValues[StartIdx + EIdx]);
            }
        }
    }, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

    
    return true;
}


// template instantiation
template DYNAMICMESH_API bool UE::Geometry::FAverageOverlayToVertices::AverageOverlay<float,4>(class  UE::Geometry::TDynamicMeshOverlay<float,4>  const *,class TArray<float, class TSizedDefaultAllocator<32> > &,class TArray<bool,class TSizedDefaultAllocator<32> > &);
template DYNAMICMESH_API bool UE::Geometry::FAverageOverlayToVertices::AverageOverlay<float,3>(class  UE::Geometry::TDynamicMeshOverlay<float,3>  const *,class TArray<float, class TSizedDefaultAllocator<32> > &,class TArray<bool,class TSizedDefaultAllocator<32> > &);
template DYNAMICMESH_API bool UE::Geometry::FAverageOverlayToVertices::AverageOverlay<float,2>(class  UE::Geometry::TDynamicMeshOverlay<float,2>  const *,class TArray<float, class TSizedDefaultAllocator<32> > &,class TArray<bool,class TSizedDefaultAllocator<32> > &);
template DYNAMICMESH_API bool UE::Geometry::FAverageOverlayToVertices::AverageOverlay<float,1>(class  UE::Geometry::TDynamicMeshOverlay<float,1>  const *,class TArray<float, class TSizedDefaultAllocator<32> > &,class TArray<bool,class TSizedDefaultAllocator<32> > &);
template DYNAMICMESH_API bool UE::Geometry::FAverageOverlayToVertices::AverageOverlay<double,3>(class UE::Geometry::TDynamicMeshOverlay<double,3> const *,class TArray<double,class TSizedDefaultAllocator<32> > &,class TArray<bool,class TSizedDefaultAllocator<32> > &);