// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/DetectExteriorVisibility.h"

#include <atomic>

#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMeshEditor.h"
#include "Sampling/SphericalFibonacci.h"

using namespace UE::Geometry;

namespace UE::Private::DetectExteriorVisibilityInternal
{

TArray<FVector3d> GenerateCardinalDirectionsHelper()
{
	TArray<FVector3d> CardinalDirections;
	// Small arbitrary offset vector, used to reduce chance of rays exactly fitting through small axis-aligned cracks
	const FVector3d TinyPerturb(UE_DOUBLE_KINDA_SMALL_NUMBER, .17 * UE_DOUBLE_KINDA_SMALL_NUMBER, .3 * UE_DOUBLE_KINDA_SMALL_NUMBER);
	for (int32 k = 0; k < 3; ++k)
	{
		FVector3d Direction = TinyPerturb;
		Direction[k] = 1.0;
		Direction.Normalize();
		CardinalDirections.Add(Direction);
		CardinalDirections.Add(-Direction);
	}
	return CardinalDirections;
}

TArray<FVector3d> GenerateTestDirectionsHelper(int32 NumVisibilityTestDirections, TConstArrayView<FVector3d> AdditionalDirections)
{
	// Small arbitrary offset vector, used to reduce chance of rays exactly fitting through small axis-aligned cracks
	const FVector3d TinyPerturb(UE_DOUBLE_KINDA_SMALL_NUMBER, .17 * UE_DOUBLE_KINDA_SMALL_NUMBER, .3 * UE_DOUBLE_KINDA_SMALL_NUMBER);
	//
	// Construct set of exterior visibility test directions, below we will check if sample 
	// points on the mesh triangles are visible from the exterior along these directions.
	// Order is modulo-shuffled in hopes that for visible tris we don't waste a bunch
	// of time on the 'far' side
	TSphericalFibonacci<double> SphereSampler(NumVisibilityTestDirections);
	TArray<FVector3d> VisibilityDirections;
	VisibilityDirections.Reserve(NumVisibilityTestDirections + AdditionalDirections.Num());
	FModuloIteration ModuloIter(NumVisibilityTestDirections);
	uint32 DirectionIndex = 0;
	while (ModuloIter.GetNextIndex(DirectionIndex))
	{
		VisibilityDirections.Add(Normalized(SphereSampler[DirectionIndex] + TinyPerturb));
	}
	VisibilityDirections.Append(AdditionalDirections);
	return VisibilityDirections;
}

// Templated helper to compute occlusion with custom AABB tree (can be specialized for different mesh types)
// Note that for the has-transparent case, we assume that Spatial *does not* include the transparent triangles
template<typename SpatialType, bool bHasTransparent>
void ComputePerTriangleOcclusionHelper(
	const FDynamicMesh3& TargetMesh, const SpatialType& Spatial, 
	TArray<bool>& OutTriOccluded, bool bPerPolyGroup, 
	const FExteriorVisibilitySampling& SamplingParameters, 
	TArrayView<const bool> SkipTris, 
	TFunctionRef<bool(int32)> IsTransparent,
	TConstArrayView<FVector3d> FirstTestDirections,
	TConstArrayView<FVector3d> FullTestDirections,
	bool bSampleNormalDirections)
{
	const bool bDoubleSided = SamplingParameters.bDoubleSided;
	const double SampleRadius = FMath::Max(UE_DOUBLE_KINDA_SMALL_NUMBER, SamplingParameters.SamplingDensity);
	const FVector3d TinyPerturb(UE_DOUBLE_KINDA_SMALL_NUMBER, .17 * UE_DOUBLE_KINDA_SMALL_NUMBER, .3 * UE_DOUBLE_KINDA_SMALL_NUMBER);
	int32 NumVisibilityTestDirections = FMath::Max(3, SamplingParameters.NumSearchDirections);

	FAxisAlignedBox3d Bounds = Spatial.GetBoundingBox();
	double Radius = Bounds.DiagonalLength();

	// geometric magic numbers used below that have been slightly tuned...
	double GlancingAngleDotTolerance = FMathd::Cos(85.0 * FMathd::DegToRad);
	constexpr double TriScalingAlpha = 0.999;
	constexpr double BaryCoordsThreshold = 0.001;

	auto FindHitTriangleTest = [&](FVector3d TargetPosition, FVector3d TargetNormal, FVector3d FarPosition, int32 TID, FVector3d& OutBaryCoords) -> int32
	{
		FVector3d RayDir(TargetPosition - FarPosition);
		double Distance = Normalize(RayDir);
		if (bDoubleSided == false && RayDir.Dot(TargetNormal) > -0.001)
		{
			return IndexConstants::InvalidID;
		}
		FRay3d Ray(FarPosition, RayDir, true);

		int32 HitTID;
		double UnusedNearestT;
		if constexpr (bHasTransparent)
		{
			if (IsTransparent(TID))
			{
				bool bHit = Spatial.FindNearestHitTriangle(Ray, UnusedNearestT, HitTID, OutBaryCoords, IMeshSpatial::FQueryOptions(Distance));
				if (!bHit)
				{
					// Note bary coords are only used to test if the sample is inside the triangle, so we don't accurate bary coords for this case
					// If you copy/adapt this lambda to some other use case, you will probably need to fix this!
					OutBaryCoords = FVector3d(1.0 / 3.0, 1.0 / 3.0, 1.0 / 3.0);
					return TID; // assume we reached the source (transparent) triangle, which isn't in the AABB tree
				}
				else
				{
					return HitTID;
				}
			} // not transparent, fall through to the standard test
		}

		bool bHit = Spatial.FindNearestHitTriangle(Ray, UnusedNearestT, HitTID, OutBaryCoords, IMeshSpatial::FQueryOptions(Distance + 1.0)); // 1.0 is random fudge factor here...
		return HitTID;
	};


	// TODO: it seems like a quite common failure case is triangles that are somewhat deeply
	// nested inside cavities/etc. A possible way to help here would be to essentially raytrace
	// orthographic images from top/bottom/etc. This could be done async and combined w/ the
	// visibilty determination below...


	// First pass. For each triangle, cast a ray at it's centroid from 
	// outside the model, along the X/Y/Z directions and tri normal.
	// If tri is hit we mark it as having 'known' status, allowing it
	// to be skipped in the more expensive pass below
	// Degenerate tris are treated as a special case: They will be treated as visible,
	// unless bPerPolyGroup==true and there are non-degenerate tris in the same group which are all not visible
	enum class EVisStatus : uint8
	{
		Unknown = 0,
		Visible = 1,
		DegenerateOrInvalid = 2
	};
	TArray<EVisStatus> TriStatusKnown;
	TriStatusKnown.SetNumZeroed(TargetMesh.MaxTriangleID());
	bool bHasSkipTris = SkipTris.Num() == TargetMesh.MaxTriangleID();
	int32 MaxUnknownTID = -1;
	for (int32 TID : TargetMesh.TriangleIndicesItr())
	{
		bool bKnown = bHasSkipTris && SkipTris[TID];
		if (bKnown)
		{
			TriStatusKnown[TID] = EVisStatus::Visible; // skip-tris are treated as visible
		}
		else
		{
			MaxUnknownTID = TID;
		}
	}
	ParallelFor(MaxUnknownTID + 1, [&](int32 TID)
	{
		if (!TargetMesh.IsTriangle(TID))
		{
			TriStatusKnown[TID] = EVisStatus::DegenerateOrInvalid;
			return;
		}
		FVector3d Normal, Centroid; double Area;
		TargetMesh.GetTriInfo(TID, Normal, Area, Centroid);
		if (Normal.SquaredLength() < 0.1 || Area <= FMathd::ZeroTolerance)
		{
			TriStatusKnown[TID] = EVisStatus::DegenerateOrInvalid;
			return;
		}

		FVector3d UnusedBaryCoords;
		for (FVector3d Direction : FirstTestDirections)
		{
			// if direction is orthogonal to the triangle, hit-test is unstable, but even
			// worse, on rectilinear shapes (eg imagine some stacked cubes or adjacent parts)
			// the ray can get "through" the cracks between adjacent connected triangles
			// and manage to hit the search triangle
			if (FMathd::Abs(Direction.Dot(Normal)) > GlancingAngleDotTolerance)
			{
				if (FindHitTriangleTest(Centroid, Normal, Centroid - Radius * Direction, TID, UnusedBaryCoords) == TID)
				{
					TriStatusKnown[TID] = EVisStatus::Visible;
					return;
				}
			}
		}
		
		// Slightly nudge the normal direction to reduce the chance of edge cases for meshes w/ features aligned to the normals
		const FVector3d NormalRayDir = Normal + TinyPerturb;
		if (bSampleNormalDirections && FindHitTriangleTest(Centroid, Normal, Centroid + (1.01 * Radius) * NormalRayDir, TID, UnusedBaryCoords) == TID)
		{
			TriStatusKnown[TID] = EVisStatus::Visible;
			return;
		}

		// triangle is not definitely visible or hidden
	});

	// Set up PolyGroup visibility if requested
	TArray<EVisStatus> GroupStatusKnown;
	TArray<std::atomic<bool>> ThreadSafeGroupVisible;
	if (bPerPolyGroup)
	{
		// Init initial group status from initial tri status
		GroupStatusKnown.Init(EVisStatus::DegenerateOrInvalid, TargetMesh.MaxGroupID());
		for (int32 TID : TargetMesh.TriangleIndicesItr())
		{
			if (TriStatusKnown[TID] == EVisStatus::Visible) // group has known-visible tri
			{
				int32 GID = TargetMesh.GetTriangleGroup(TID);
				if (GroupStatusKnown.IsValidIndex(GID))
				{
					GroupStatusKnown[GID] = EVisStatus::Visible;
				}
			}
			else if (TriStatusKnown[TID] == EVisStatus::Unknown) // group has non-degenerate tri
			{
				int32 GID = TargetMesh.GetTriangleGroup(TID);
				if (GroupStatusKnown.IsValidIndex(GID) && GroupStatusKnown[GID] == EVisStatus::DegenerateOrInvalid)
				{
					GroupStatusKnown[GID] = EVisStatus::Unknown;
				}
			}
		}
		// Copy to thread safe values
		ThreadSafeGroupVisible.SetNum(TargetMesh.MaxGroupID());
		for (int32 GroupIdx = 0; GroupIdx < GroupStatusKnown.Num(); ++GroupIdx)
		{
			ThreadSafeGroupVisible[GroupIdx] = GroupStatusKnown[GroupIdx] == EVisStatus::Visible;
		}
		// Copy group statuses back to tri statuses
		for (int32 TID : TargetMesh.TriangleIndicesItr())
		{
			int32 GID = TargetMesh.GetTriangleGroup(TID);
			if (GroupStatusKnown.IsValidIndex(GID) && GroupStatusKnown[GID] == EVisStatus::Visible)
			{
				TriStatusKnown[TID] = EVisStatus::Visible;
			}
		}
	}

	// Thread safe triangle visibility, atomics can be updated on any thread
	TArray<std::atomic<bool>> ThreadSafeTriVisible;
	ThreadSafeTriVisible.SetNum(TargetMesh.MaxTriangleID());
	// Collect the unknown triangles, to run parallel sampling on only those triangles
	TArray<int32> UnknownTIDs;
	for (int32 TID : TargetMesh.TriangleIndicesItr())
	{
		EVisStatus Status = TriStatusKnown[TID];
		ThreadSafeTriVisible[TID] = Status == EVisStatus::Visible;
		if (Status == EVisStatus::Unknown)
		{
			UnknownTIDs.Add(TID);
		}
	}

	// Test the requested number of directions + also the cardinal directions
	NumVisibilityTestDirections = FullTestDirections.Num();

	// For each triangle we will generate a set of sample points on the triangle surface,
	// and then check if that point is visible along any of the sample directions.
	// The number of sample points allocated to a triangle is based on it's area and the SampleRadius.
	// However for small triangles this may be < 1, so we will clamp to at least this many samples.
	// (this value should perhaps be relative to the mesh density, or exposed as a parameter...)
	const int32 MinTriSamplesPerSamplePoint = 8;

	// For each triangle, generate a set of sample points on the triangle surface,

	// This is the expensive part!
	ParallelFor(UnknownTIDs.Num(), [&](int32 UnknownIdx)
	{
		int32 TID = UnknownTIDs[UnknownIdx];
		
		int32 GID = 0;
		bool bHasValidGroup = bPerPolyGroup;
		if (bPerPolyGroup)
		{
			GID = TargetMesh.GetTriangleGroup(TID);
			bHasValidGroup = GroupStatusKnown.IsValidIndex(GID);
			if (bHasValidGroup && (GroupStatusKnown[GID] != EVisStatus::Unknown || ThreadSafeGroupVisible[GID]))
			{
				return;
			}
		}
		// if we already found out this triangle is visible or hidden, we can skip it
		if (TriStatusKnown[TID] != EVisStatus::Unknown || ThreadSafeTriVisible[TID])
		{
			return;
		}

		FVector3d A, B, C;
		TargetMesh.GetTriVertices(TID, A, B, C);
		FVector3d Centroid = (A + B + C) / 3.0;
		double TriArea;
		FVector3d TriNormal = VectorUtil::NormalArea(A, B, C, TriArea);		// TriStatusKnown should skip degen tris, do not need to check here

		// Slightly shrink the triangle, this helps to avoid spurious hits
		// TODO obviously should scale by an actual dimension and not just a relative %...
		FTriangle3d SampleTri(A, B, C);
		for (int32 k = 0; k < 3; ++k)
		{
			SampleTri.V[k] = Lerp(Centroid, SampleTri.V[k], TriScalingAlpha);
		}

		double DiscArea = (FMathd::Pi * SampleRadius * SampleRadius);
		int NumSamples = FMath::Max((int)(TriArea / DiscArea), MinTriSamplesPerSamplePoint);
		FVector3d V1 = SampleTri.V[1] - SampleTri.V[0];
		FVector3d V2 = SampleTri.V[2] - SampleTri.V[0];

		TArray<int32> HitTris, HitGroups;		// re-use this array in inner loop to avoid hitting atomics so often

		// always sample the (already inset) corners and mid-edges
		constexpr int32 NumFixedSamples = 6;
		FVector2d FixedSamples[NumFixedSamples]{ {0,0}, {0,1}, {1,0}, {.5,0}, {0,.5}, {.5, .5} };

		FRandomStream RandomStream(TID);
		for (int32 NumTested = 0; NumTested < NumSamples; ++NumTested)
		{
			FVector2d Sample;
			if (NumTested < NumFixedSamples)
			{
				Sample = FixedSamples[NumTested];
			}
			else // add random samples
			{
				Sample.X = RandomStream.GetFraction();
				Sample.Y = RandomStream.GetFraction();
				if (Sample.X + Sample.Y > 1) // if we would sample outside the triangle, reflect the sample
				{
					Sample.X = 1 - Sample.X;
					Sample.Y = 1 - Sample.Y;
				}
			}
			FVector3d Position = SampleTri.V[0] + Sample.X * V1 + Sample.Y * V2;

			// cast ray from all exterior sample locations for this triangle sample point
			HitTris.Reset();
			for (int32 k = 0; k < NumVisibilityTestDirections; ++k)
			{
				FVector3d Direction = FullTestDirections[k];
				if (FMathd::Abs(Direction.Dot(TriNormal)) < GlancingAngleDotTolerance)
				{
					continue;
				}

				FVector3d RayFrom = Position - 2.0 * Radius * FullTestDirections[k];
				FVector3d BaryCoords;
				int32 HitTriID = FindHitTriangleTest(Position, TriNormal, RayFrom, TID, BaryCoords);
				if (HitTriID != IndexConstants::InvalidID && TriStatusKnown[HitTriID] == EVisStatus::Unknown)
				{
					if (HitTriID == TID)
					{
						HitTris.Add(HitTriID);
						break;
					}
					// For tris other than the one that was sampled to create the ray, we want
					// to filter out on-edge triangle hits, as they are generally spurious and
					// will result in interior triangles remaining visible
					else if (BaryCoords.GetMin() > BaryCoordsThreshold &&
						BaryCoords.GetMax() < (1.0 - BaryCoordsThreshold))
					{
						HitTris.AddUnique(HitTriID);
					}
				}
			}

			// mark any hit tris
			for (int32 HitTriID : HitTris)
			{
				ThreadSafeTriVisible[HitTriID] = true;
			}
			if (bPerPolyGroup)
			{
				// Note: can tune the threshold at which we reset the hitgroups array, to trade off atomic accesses vs AddUnique search cost 
				constexpr int32 ResetGroupSize = 20;
				if (HitGroups.Num() >= ResetGroupSize)
				{
					HitGroups.Reset();
				}
				for (int32 HitTriID : HitTris)
				{
					int32 HitGID = TargetMesh.GetTriangleGroup(HitTriID);
					if (GroupStatusKnown.IsValidIndex(HitGID) && GroupStatusKnown[HitGID] == EVisStatus::Unknown && HitGroups.Num() == HitGroups.AddUnique(HitGID))
					{
						// Note: setting/checking the group visibility flag is optional and just allow threads to stop early
						// so it should be fine to do so w/ the relaxed memory ordering
						ThreadSafeGroupVisible[HitGID].store(true, std::memory_order_relaxed);
					}
				}
			}

			// if our triangle has become visible (in this thread or another) we can terminate now
			if (ThreadSafeTriVisible[TID] || (bHasValidGroup && ThreadSafeGroupVisible[GID].load(std::memory_order_relaxed)))
			{
				return;
			}
		}

		// Note: Consider periodically locking to update TriStatusKnown and GroupStatusKnown
	});

	// propagate tri visibility back through groups
	if (bPerPolyGroup)
	{
		for (int32 TID : TargetMesh.TriangleIndicesItr())
		{
			if (ThreadSafeTriVisible[TID])
			{
				int32 GID = TargetMesh.GetTriangleGroup(TID);
				if (ThreadSafeGroupVisible.IsValidIndex(GID))
				{
					ThreadSafeGroupVisible[GID] = true;
				}
			}
		}
		// Copy group statuses back to tri statuses
		for (int32 TID : TargetMesh.TriangleIndicesItr())
		{
			int32 GID = TargetMesh.GetTriangleGroup(TID);
			// Set the tris in the group to visible if the group is visible or invalid (where invalid corresponds to entirely degenerate / untestable)
			if (ThreadSafeGroupVisible.IsValidIndex(GID) && 
				(ThreadSafeGroupVisible[GID] || (SamplingParameters.bMarkDegenerateAsVisible && GroupStatusKnown[GID] == EVisStatus::DegenerateOrInvalid)) )
			{
				ThreadSafeTriVisible[TID] = true;
			}
		}
	}
	else if (SamplingParameters.bMarkDegenerateAsVisible)
	{
		for (int32 TID : TargetMesh.TriangleIndicesItr())
		{
			if (TriStatusKnown[TID] == EVisStatus::DegenerateOrInvalid)
			{
				ThreadSafeTriVisible[TID] = true;
			}
		}
	}

	// Transfer visibility results to output array
	OutTriOccluded.SetNum(ThreadSafeTriVisible.Num());
	for (int32 Idx = 0; Idx < OutTriOccluded.Num(); ++Idx)
	{
		OutTriOccluded[Idx] = !ThreadSafeTriVisible[Idx];
	}
}

template<typename SpatialType, bool bHasTransparent>
void ComputePerTriangleOcclusionHelper(
	const FDynamicMesh3& TargetMesh, const SpatialType& Spatial,
	TArray<bool>& OutTriOccluded, bool bPerPolyGroup,
	const FExteriorVisibilitySampling& SamplingParameters,
	TArrayView<const bool> SkipTris,
	TFunctionRef<bool(int32)> IsTransparent)
{
	TArray<FVector3d> CardinalDirections = GenerateCardinalDirectionsHelper();
	TArray<FVector3d> FullDirections = GenerateTestDirectionsHelper(SamplingParameters.NumSearchDirections, CardinalDirections);
	ComputePerTriangleOcclusionHelper<SpatialType, bHasTransparent>(
		TargetMesh, Spatial,
		OutTriOccluded, bPerPolyGroup,
		SamplingParameters,
		SkipTris,
		IsTransparent,
		CardinalDirections,
		FullDirections,
		true
	);
}

} // namespace UE::Private::DetectExteriorVisibilityInternal

void FExteriorVisibilitySampling::ComputePerTriangleOcclusion(const FDynamicMesh3& TargetMesh, TArray<bool>& OutTriOccluded, bool bPerPolyGroup, const FExteriorVisibilitySampling& SamplingParameters, TArrayView<const bool> SkipTris)
{
	using namespace UE::Private::DetectExteriorVisibilityInternal;
	TMeshAABBTree3<FDynamicMesh3> Spatial(&TargetMesh, true);
	return ComputePerTriangleOcclusionHelper< TMeshAABBTree3<FDynamicMesh3>, false >(TargetMesh, Spatial, OutTriOccluded, bPerPolyGroup, SamplingParameters, SkipTris, [](int32)->bool { checkNoEntry(); return false; });
}

// simple wrapper that skips transparent triangles, used for building an aabb tree without them
struct FSkipTransparentMeshWrapper
{
	const FDynamicMesh3* Mesh;
	TFunctionRef<bool(int32)> IsTriTransparent;
	const int32 TriCount;

	static int32 CountNonTransparent(const FDynamicMesh3* Mesh, TFunctionRef<bool(int32)> IsTriTransparent)
	{
		int32 Count = 0;
		for (int32 TID = 0; TID < Mesh->MaxTriangleID(); ++TID)
		{
			Count += int(Mesh->IsTriangle(TID) && !IsTriTransparent(TID));
		}
		return Count;
	}

	FSkipTransparentMeshWrapper(const FDynamicMesh3* Mesh, TFunctionRef<bool(int32)> IsTriTransparent) : Mesh(Mesh), IsTriTransparent(IsTriTransparent), TriCount(CountNonTransparent(Mesh, IsTriTransparent))
	{
	}

	FORCEINLINE bool IsTriangle(int32 Index) const
	{
		return Mesh->IsTriangle(Index) && !IsTriTransparent(Index);
	}

	FORCEINLINE bool IsVertex(int32 Index) const
	{
		return Mesh->IsVertex(Index);
	}

	FORCEINLINE int32 MaxTriangleID() const
	{
		return Mesh->MaxTriangleID();
	}

	FORCEINLINE int32 MaxVertexID() const
	{
		return Mesh->MaxVertexID();
	}

	// Triangle count is pre-computed to subtract the transparent triangles
	FORCEINLINE int32 TriangleCount() const
	{
		return TriCount;
	}

	FORCEINLINE int32 VertexCount() const
	{
		return Mesh->VertexCount();
	}

	FORCEINLINE uint64 GetChangeStamp() const
	{
		return Mesh->GetChangeStamp();
	}

	FORCEINLINE FIndex3i GetTriangle(int32 Index) const
	{
		return Mesh->GetTriangle(Index);
	}

	FORCEINLINE FVector3d GetVertex(int32 Index) const
	{
		return Mesh->GetVertex(Index);
	}

	FORCEINLINE void GetTriVertices(int32 TriIndex, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
	{
		return Mesh->GetTriVertices(TriIndex, V0, V1, V2);
	}
};

void FExteriorVisibilitySampling::ComputePerTriangleOcclusion(const FDynamicMesh3& TargetMesh, TFunctionRef<bool(int32)> IsTriTransparent, TArray<bool>& OutTriOccluded, bool bPerPolyGroup, const FExteriorVisibilitySampling& SamplingParameters, TArrayView<const bool> SkipTris)
{
	using namespace UE::Private::DetectExteriorVisibilityInternal;
	FSkipTransparentMeshWrapper SkipTransparent(&TargetMesh, IsTriTransparent);
	TMeshAABBTree3<FSkipTransparentMeshWrapper> Spatial(&SkipTransparent, true);
	return ComputePerTriangleOcclusionHelper< TMeshAABBTree3<FSkipTransparentMeshWrapper>, true >(TargetMesh, Spatial, OutTriOccluded, bPerPolyGroup, SamplingParameters, SkipTris, IsTriTransparent);
}

void FExteriorVisibilitySampling::ComputePerTriangleOcclusionFromDirections(const FDynamicMesh3& TargetMesh, TFunctionRef<bool(int32)> IsTriTransparent, TArray<bool>& OutTriOccluded, bool bPerPolyGroup,
	TConstArrayView<FVector3d> InitialViewTestDirections, TConstArrayView<FVector3d> FullViewTestDirections, bool bAlsoSampleTriangleNormalDirections,
	const FExteriorVisibilitySampling& SamplingParameters,
	TArrayView<const bool> SkipTris)
{
	using namespace UE::Private::DetectExteriorVisibilityInternal;
	FSkipTransparentMeshWrapper SkipTransparent(&TargetMesh, IsTriTransparent);
	TMeshAABBTree3<FSkipTransparentMeshWrapper> Spatial(&SkipTransparent, true);
	return ComputePerTriangleOcclusionHelper< TMeshAABBTree3<FSkipTransparentMeshWrapper>, true >(TargetMesh, Spatial, OutTriOccluded, bPerPolyGroup, SamplingParameters, SkipTris, IsTriTransparent,
		InitialViewTestDirections, FullViewTestDirections,bAlsoSampleTriangleNormalDirections);
}

TArray<FVector3d> FExteriorVisibilitySampling::GetRestrictedViewDirections(TConstArrayView<FVector3d> RestrictViewDirections, double RestrictViewAngle, int32 NumSphereSamples)
{
	using namespace UE::Private::DetectExteriorVisibilityInternal;

	TArray<FVector3d> SampleDirections = GenerateTestDirectionsHelper(NumSphereSamples, GenerateCardinalDirectionsHelper());
	TArray<FVector3d> FoundDirections(RestrictViewDirections);
	
	double CosThresh = FMath::Cos(FMath::DegreesToRadians(RestrictViewAngle));
	for (const FVector3d& SampleDir : SampleDirections)
	{
		bool bFoundInCone = false;
		for (const FVector3d& RestrictDir : RestrictViewDirections)
		{
			if (SampleDir.Dot(RestrictDir) >= CosThresh)
			{
				FoundDirections.Add(SampleDir);
				break;
			}
		}
	}

	return FoundDirections;
}


void FDetectPerDynamicMeshExteriorVisibility::ComputeHidden(TArray<bool>& OutIsOccluded, TArray<bool>* OutTransparentIsOccluded)
{
	OutIsOccluded.SetNumZeroed(Instances.Num());

	bool bTestTransparent = OutTransparentIsOccluded && TransparentInstances.Num();
	if (OutTransparentIsOccluded)
	{
		OutTransparentIsOccluded->SetNumZeroed(TransparentInstances.Num());
	}

	if (Instances.Num() == 0 && (!bTestTransparent || TransparentInstances.Num() == 0))
	{
		// no meshes to test
		return;
	}

	// Combine all Instances and OccludeInstances into a single AllMesh, to allow occlusion calculation via the single-mesh function above
	// Note: To support large meshes with many instances, consider adding an alternate path which uses a separate tree per unique instance.
	FDynamicMesh3 AllMesh(EMeshComponents::None);
	FDynamicMeshEditor CombineEditor(&AllMesh);
	FMeshIndexMappings Mappings;
	TArray<int32> InstanceEndTID;
	auto AddInstances = [this, &AllMesh, &CombineEditor, &Mappings, &InstanceEndTID](TArray<FDynamicMeshInstance>& ToAdd)
	{
		for (int32 InstIdx = 0; InstIdx < ToAdd.Num(); ++InstIdx)
		{
			CombineEditor.AppendMesh(ToAdd[InstIdx].SourceMesh, Mappings,
			[ToAdd, InstIdx](int32 VID, const FVector3d Pos) -> FVector3d
			{
				return ToAdd[InstIdx].Transforms.TransformPosition(Pos);
			});
			InstanceEndTID.Add(AllMesh.MaxTriangleID());
		}
	};
	AddInstances(Instances);
	int32 TransparentStartID = AllMesh.MaxTriangleID();
	if (bTestTransparent)
	{
		AddInstances(TransparentInstances);
	}
	int32 OccludeStartID = AllMesh.MaxTriangleID();
	AddInstances(OccludeInstances);

	// Should be safe to assume contiguous triangle IDs because we've constructed this mesh ourselves.
	// If this assumption breaks, will need to change the logic below which assigns PolyGroup IDs per instance
	if (!ensure(AllMesh.IsCompactT()))
	{
		return;
	}

	// Initialize output arrays and track the boundary between normal, transparent and occlude instances
	OutIsOccluded.Init(true, Instances.Num());
	int32 TestInstancesCount = OutIsOccluded.Num();
	int32 TransparentInstanceStart = TestInstancesCount;
	int32 OccludeInstancesStart = TestInstancesCount;
	if (bTestTransparent)
	{
		OutTransparentIsOccluded->Init(true, TransparentInstances.Num());
		TestInstancesCount += TransparentInstances.Num();
		OccludeInstancesStart = TestInstancesCount;
	}

	// Enable and set triangle groups *after* adding all meshes, so the groups initialize to zero and we are not affected by groups on the source meshes
	AllMesh.EnableTriangleGroups(0);
	for (int32 TID = InstanceEndTID[0], CurInstance = 1; TID < AllMesh.MaxTriangleID(); ++TID)
	{
		while (CurInstance < InstanceEndTID.Num() && TID == InstanceEndTID[CurInstance])
		{
			CurInstance++;
		}
		// Set triangle group by instance ID, but give all OccludeInstances the same ID
		AllMesh.SetTriangleGroup(TID, FMath::Min(CurInstance, TestInstancesCount));
	}

	// Mark any triangles from Occlude Instances as SkipTris
	TArray<bool> SkipTris;
	SkipTris.SetNumZeroed(AllMesh.MaxTriangleID());
	for (int32 TID = OccludeStartID; TID < AllMesh.MaxTriangleID(); ++TID)
	{
		SkipTris[TID] = true;
	}

	// Compute per-tri occlusion
	TArray<bool> TriOccluded;
	if (bTestTransparent)
	{
		FExteriorVisibilitySampling::ComputePerTriangleOcclusion(AllMesh, 
			[TransparentStartID, OccludeStartID](int32 TID) 
			{
				return TID >= TransparentStartID && TID < OccludeStartID;
			}, TriOccluded, true /*bPerPolyGroup*/, SamplingParameters, SkipTris);
	}
	else
	{
		FExteriorVisibilitySampling::ComputePerTriangleOcclusion(AllMesh, TriOccluded, true /*bPerPolyGroup*/, SamplingParameters, SkipTris);
	}

	// Transfer triangle occlusion back to source mesh occlusion via the triangle groups
	for (int32 TID : AllMesh.TriangleIndicesItr())
	{
		int32 GID = AllMesh.GetTriangleGroup(TID);
		if (GID < 0 || GID >= OccludeInstancesStart) // tri is from an OccludeInstances mesh
		{
			break; // OccludeInstances meshes are at the end; we can stop when we reach them
		}
		TArray<bool>* UseIsOccluded = &OutIsOccluded;
		if (OutTransparentIsOccluded && GID >= TransparentInstanceStart)
		{
			UseIsOccluded = OutTransparentIsOccluded;
		}
		int32 InstanceIdx = GID < TransparentInstanceStart ? GID : GID - TransparentInstanceStart;
		if (!(*UseIsOccluded)[InstanceIdx]) // already found visible
		{
			continue;
		}
		if (!TriOccluded[TID]) // found visible tri -> source is visible
		{
			(*UseIsOccluded)[InstanceIdx] = false;
		}
	}
}