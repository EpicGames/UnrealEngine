// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/ExtrudeBoundaryEdges.h"

#include "BoxTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "Math/NumericLimits.h"
#include "Util/ProgressCancel.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "VectorUtil.h"

namespace ExtrudeBoundaryEdgesLocals
{
	using namespace UE::Geometry;

	// Gives some vector orthogonal to NormalizedVectorIn
	FVector3d GetOrthogonalVector(const FVector3d& NormalizedVectorIn)
	{
		FVector3d Result = NormalizedVectorIn.Cross(FVector3d::UnitY());
		if (Result.IsZero())
		{
			// Must have been colinear with Y, so Z will work
			return FVector3d::UnitZ();
		}
		return Result;
	}

	// Given a non degenerate normal and edge, find direction in which we would want to extrude the edge to stay
	//  in the plane of the face and move edge orthogonally outward.
	FVector3d GetOutwardVector(const FVector3d& NormalVector, const FVector3d& EdgeVector)
	{
		// If we're looking at the triangle with edge vector pointing up and triangle to the left (and
		//  Z towards us), then X should point right. In unreal's left handed coordinate system that
		//  actually means this:
		return NormalVector.Cross(EdgeVector);
	}

	FExtrudeBoundaryEdges::FExtrudeFrame GetExtrudeFrameAtVertex(const FVector3d SourcePosition,
		const FVector3d& IncomingTriNormal, const FVector3d& IncomingEdgeVector,
		const FVector3d& OutgoingTriNormal, const FVector3d& OutgoingEdgeVector,
		double ScalingLimit)
	{
		// TODO: We try to do some handling of degenerate normals below, but it is generally not good
		//  enough, and we are very likely to get nonsense extrude frames when we have degenerate normals.
		//  One option is to propagate normal information from neighbors, or propagate frame information
		//  from adjacent edges where a good one could be picked. These options are a fair amount of
		//  work for something that may not have much benefit, because the user probably still won't have
		//  much control and at best we would be hiding the effects of flawed geometry... Still, might
		//  be something to come back to.

		if (OutgoingEdgeVector.IsZero() && IncomingEdgeVector.IsZero())
		{
			// Not much we can do here.
			return FExtrudeBoundaryEdges::FExtrudeFrame(FFrame3d(SourcePosition));
		}


		auto CreateFrameFromEdgeAndNormal = [&SourcePosition](const FVector3d& NormalVector, const FVector3d& EdgeVector)
		{
			FVector3d FrameZ = NormalVector;
			FVector3d FrameX = GetOutwardVector(NormalVector, EdgeVector);
			// With the way we calculate the outward vector, this will actually just always end up -EdgeVector,
			//  but we'll just do the proper thing.
			FVector3d FrameY = FrameZ.Cross(FrameX);

			return FExtrudeBoundaryEdges::FExtrudeFrame(
				FFrame3d(SourcePosition, FrameX, FrameY, FrameZ));
		};

		// Handle cases where we only have data on one neighbor.
		if (IncomingEdgeVector.IsZero())
		{
			ensure(!OutgoingEdgeVector.IsZero());
			return CreateFrameFromEdgeAndNormal(
				OutgoingTriNormal.IsZero() ? GetOrthogonalVector(OutgoingEdgeVector) : OutgoingTriNormal, 
				OutgoingEdgeVector);
		}
		if (OutgoingEdgeVector.IsZero())
		{
			ensure(!IncomingEdgeVector.IsZero());
			return CreateFrameFromEdgeAndNormal(
				IncomingTriNormal.IsZero() ? GetOrthogonalVector(IncomingTriNormal) : IncomingTriNormal, 
				IncomingEdgeVector);
		}

		// If we got to here, we had neighbors on both sides. We'd like to use the normals in setting up the extrude
		//  frame, but one or both of them could be zeroes if the triangles were degenerate. Not totally clear what
		//  the best solution is, but for now we either find an orthogonal direction if they are both degenerate, or
		//  else we pick a plane for the degenerate edge such that it lies in the same plane as the desired X for the
		//  other face.
		FVector3d IncomingNormalToUse = IncomingTriNormal;
		FVector3d OutgoingNormalToUse = OutgoingTriNormal;
		if (IncomingNormalToUse.IsZero())
		{
			if (OutgoingTriNormal.IsZero())
			{
				// Both faces degrenerate. X direction is orthogonal to both. Again, remember left handedness.
				FVector3d FrameX = IncomingEdgeVector.Cross(OutgoingEdgeVector);
				FrameX.Normalize();
				if (FrameX.IsZero())
				{
					FrameX = GetOrthogonalVector(IncomingEdgeVector);
				}

				IncomingNormalToUse = IncomingEdgeVector.Cross(FrameX);
				IncomingNormalToUse.Normalize();
				OutgoingNormalToUse = OutgoingEdgeVector.Cross(FrameX);
				OutgoingNormalToUse.Normalize();
			}
			else
			{
				FVector3d FrameX = GetOutwardVector(OutgoingTriNormal, OutgoingEdgeVector);

				IncomingNormalToUse = IncomingEdgeVector.Cross(FrameX);
				if (!IncomingNormalToUse.Normalize())
				{
					// If we were colinear with the X we picked, let's say that we're coplanar with the
					// other face.
					IncomingNormalToUse = OutgoingNormalToUse;
				}
			}
		}
		else if (OutgoingTriNormal.IsZero())
		{
			FVector3d FrameX = GetOutwardVector(IncomingTriNormal, IncomingEdgeVector);

			OutgoingNormalToUse = OutgoingEdgeVector.Cross(FrameX);
			if (!OutgoingNormalToUse.Normalize())
			{
				OutgoingNormalToUse = IncomingNormalToUse;
			}
		}

		// At this point we have non degenerate edge vectors and normals.
		// There are a few ways we can pick our frame. One option we've tried in the past is picking
		//  the X to be in the plane of both neighboring triangles, which works for following the "crease"
		//  of some edges nicely, but falls apart frequently if both edges lie in the plane of at least
		//  one of the triangles, because you frequently end up extruding along one of the edges. (E.g. imagine
		//  a classic pentagonal prism "house" shape and remove one of the roof faces, then try extruding that
		//  boundary. This method will nicely extend the adjacent faces in their planes. But if you delete only
		//  half of the roof face and try to extrude the boundary, you'll get very weird results as some vertices
		//  try to move along boundary edges instead, due to the flat portion of the roof constraining them.)
		// A significantly more reliable option is to make X an average of the two outward directions. This
		//  tends to be a lot more predictable and less likely to generate garbage geometry. This is what we do.

		FVector3d OutwardVector1 = GetOutwardVector(IncomingNormalToUse, IncomingEdgeVector);
		FVector3d OutwardVector2 = GetOutwardVector(OutgoingNormalToUse, OutgoingEdgeVector);
		FVector3d FrameX = OutwardVector1 + OutwardVector2;
		FrameX.Normalize();
		if (FrameX.IsZero())
		{
			// It seems like getting this situation would imply some very weird twisting topology
			//  where the next triangle manages to flip which side of the edge it is on, or Z fights
			//  the original triangle while moving backwards. There isn't a perfect thing to do here,
			//  so just use one of the edges and ignore the other.
			return CreateFrameFromEdgeAndNormal(
				IncomingTriNormal.IsZero() ? GetOrthogonalVector(IncomingTriNormal) : IncomingTriNormal,
				IncomingEdgeVector);
		}
		FVector3d FrameZ = IncomingNormalToUse + OutgoingNormalToUse;
		if (FrameZ.IsZero())
		{
			// This means that the normals flipped, usually by having the triangles fold back onto each
			//  other. We can pick a Z pointing in the plane of the edge by using the edge vectors here.
			FrameZ = IncomingEdgeVector - OutgoingEdgeVector;
		}
		// Not sure whether X and Z are necessarily orthogonal at this point with our approaches... Seems
		//  like it might be the case, but let's be safe and not assume it.
		FVector3d FrameY = FrameZ.Cross(FrameX);
		FrameY.Normalize();
		FrameZ = FrameX.Cross(FrameY);
		// X and Y normalized and orthogonal, so Z shouldn't need normalization.

		FExtrudeBoundaryEdges::FExtrudeFrame ExtrudeFrame(FFrame3d(
			SourcePosition, FrameX, FrameY, FrameZ));

		if (ScalingLimit > 1)
		{
			// This means that we're going to be adjusting our extrude directions in an attempt to keep extruded
			//  edges parallel. E.g. imagine extruding two edges at a 90 degree corner: the extrude directions
			//  at the flat sides are directly outward, whereas the outward direction at the corner is at an angle.
			//  If we extrude the same distance along each direction, the resulting corner is no longer 90 degrees
			//  because the vertex would need to move further along the diagonal.
			// The problem is made more complicated by the fact that we are not necessarily extruding in the plane
			//  of the edges, and that the extrude frames at each vertex might be arbitrarily rotated depending on
			//  the adjoining faces. So, having some in-frame adjustment that keeps edges parallel without knowledge 
			//  of adjacent frames is impossible.
			// Instead we'll try to do our best for some common use cases. In particular, imagine extruding some
			//  edges on the boundary of a tesselated open-top box partially outward. We will adjust them such
			//  that the projection onto the plane of the original boundary is parallel, and we will do it by
			//  scaling only in the appropriate direction in the plane of the boundary. This will handle "tube"
			//  extrusions and simple planar mesh extrusion, and for planar boundaries, the user can scale down
			//  the result along the orthogonal axis to get parallel output even if incident faces made for
			//  inconsistent frame rotations.

			// Direction to scale is average of the two incoming edge vectors
			FVector3d DirectionToScale = IncomingEdgeVector - OutgoingEdgeVector;
			DirectionToScale.Normalize();

			// If direction to scale is zero, that means we're along a flat section, where we don't need
			//  to adjust.
			if (!DirectionToScale.IsZero())
			{

				// We compare against an outward vector in the plane of the two vectors (rather than in the plane of
				//  adjoining triangles, which we used when finding the extrude frame)
				FVector3d NormalToEdges = IncomingEdgeVector.Cross(OutgoingEdgeVector); // Unnormalized
				FVector3d OutwardVectorInEdgePlane = IncomingEdgeVector.Cross(NormalToEdges);
				OutwardVectorInEdgePlane.Normalize();

				// Figure out how far to scale, with an upper limit
				double CosTheta = DirectionToScale.Dot(OutwardVectorInEdgePlane);
				if (CosTheta == 0)
				{
					// Presumably this comes about from the edges doubling back onto themselves, but we'll double
					//  check that we're not dealing with a straight edge to be safe.
					if (IncomingEdgeVector.Dot(OutgoingEdgeVector) < 0)
					{
						ExtrudeFrame.InFrameScaleDirection = ExtrudeFrame.Frame.ToFrameVector(DirectionToScale);
						ExtrudeFrame.Scaling = ScalingLimit;
					}
				}
				else
				{
					ExtrudeFrame.InFrameScaleDirection = ExtrudeFrame.Frame.ToFrameVector(DirectionToScale);
					ExtrudeFrame.Scaling = FMath::Min(1 / FMath::Abs(CosTheta), ScalingLimit);
				}
			}
		}

		return ExtrudeFrame;
	}

	int32 FindSharpestAngleMatchEdge(const FDynamicMesh3& Mesh, int32 EidToMatch, int32 Vid, 
		const FVector3d& VertexNormal, const FDynamicMesh3::FLocalIntArray& CandidateEids)
	{
		// The approach here is pretty much the same as in FMeshBoundaryLoops::FindLeftTurnEdge, except that
		//  we're actually looking for the right turn edge (better if you're concerned with following a hole/boundary
		//  instead of the smallest island), we only look at candidate eids, and we can look from outgoing edge
		//  instead.
		
		FIndex2i InputEdgeVids = Mesh.GetOrientedBoundaryEdgeV(EidToMatch);
		bool bInputIsIncomingEdge = InputEdgeVids.B == Vid;

		if (!ensure(bInputIsIncomingEdge || InputEdgeVids.A == Vid))
		{
			return IndexConstants::InvalidID;
		}

		// We're going to be measuring the angle from incoming to outgoing edge, and we want it to be the most
		//  clockwise. Normally that would mean most negative in the plane, but with left handed coordinates
		//  it is most positive...
		double BestAngle = TNumericLimits<double>::Lowest();
		int32 BestEid = IndexConstants::InvalidID;

		FVector3d InputEdgeVector = Mesh.GetVertex(InputEdgeVids.B) - Mesh.GetVertex(InputEdgeVids.A);
		
		for (int32 Eid : CandidateEids)
		{
			if (Eid == EidToMatch)
			{
				continue;
			}
			FIndex2i Vids = Mesh.GetOrientedBoundaryEdgeV(Eid);
			bool bIsIncomingEdge = Vids.B == Vid;
			if (!ensure(bIsIncomingEdge || Vids.A == Vid))
			{
				continue;
			}
			if (bInputIsIncomingEdge == bIsIncomingEdge)
			{
				continue;
			}

			FVector3d EdgeVector = Mesh.GetVertex(Vids.B) - Mesh.GetVertex(Vids.A);
			double Angle = TNumericLimits<double>::Min();
			if (bInputIsIncomingEdge)
			{
				Angle = VectorUtil::PlaneAngleSignedD(InputEdgeVector, EdgeVector, VertexNormal);
			}
			else
			{
				Angle = VectorUtil::PlaneAngleSignedD(EdgeVector, InputEdgeVector, VertexNormal);
			}
			
			if (Angle > BestAngle)
			{
				BestEid = Eid;
				BestAngle = Angle;
			}
		}

		return BestEid;
	}
}

namespace UE::Geometry
{
FExtrudeBoundaryEdges::FExtrudeBoundaryEdges(FDynamicMesh3* MeshIn)
	: Mesh(MeshIn)
{
}

bool FExtrudeBoundaryEdges::Apply(FProgressCancel* Progress)
{
	using namespace ExtrudeBoundaryEdgesLocals;

	// Complications in exturding edges:
	// 1. We can't just always split bowties in our input, because we might be using the operation to fix
	//  a bowtie. For instance imagine two circle holes tangent at a bowtie. We might be selecting just 
	//  one of the holes and extruding inward. If that bowtie were broken, we would get an undesirable
	//  split in the result, instead of a complete quad ring.
	// 2. We DO need to do something about bowties if we're extruding more than two incident boundary edges 
	//  (imagine extruding the boundaries of both holes in the example above). Keeping the vert alone in
	//  the destination would attempt to create a non-manifold edge on it.
	// 3. Degenerate edges are a problem when determining per-vertex extrude frames, because they don't
	//   have proper edge/normal vectors, and as such could end up going to different places than their
	//   neighbors.

	// We take the following approach for 1 and 2:
	// Every extruded edge gets a quad with verts a, b, c, d, where a -> b is one of the original edges,
	//	oriented in triangle direction, and c -> d is the "extruded edge". When creating c or d, we'll 
	//  enforce the constraint that each copy can be used as "c" in no more than one quad, and "d" in no
	//  more than one quad. In other words, we'll create a new copy if the source vid is the destination
	//  or the source for more than one edge in the original edges to extrude.
	// This means that we will actually end up splitting verts in the destination even if they were not
	//  a bowtie vert if the original vert had flipped neighbor triangles. That seems like a reasonable
	//  thing to do in a situation where the user really shouldn't have any expectation.
	//
	// For 3, we currently decide to not do anything, instead allowing for whatever garbage results
	//  might be created when a user tries to extrude degenerate edges (though the results won't be garbage
	//  when extruding in a single direction). In an ideal world, we would implement some grouping of
	//  vertices that are connected by degenerate edges, propage the vector information across them, and
	//  make sure that vertices in the group get the same extrude frame. While possible, that seems not
	//  worth the pain and potential pitfalls, especially if you consider pathological cases of some web
	//  of degenerate boundary edges rather than a single chain, where there are multiple extrude frames
	//  that each might want to use. The saner approach is to expect the user to collapse degenerate edges 
	//  in the input first.

	if (!ensure(Mesh && OffsetPositionFunc))
	{
		return false;
	}

	if (InputEids.IsEmpty())
	{
		return true;
	}

	if (Progress && Progress->Cancelled()) { return false; }

	TArray<FNewVertSourceData> NewVertData;
	TArray<int32> NewVertVids; // eventually 1:1 with NewVertData
	TMap<int32, FIndex2i> EidToIndicesIntoNewVerts;

	// Pair up the edges across vertices so that we know when to split verts
	bool bSuccess = FExtrudeBoundaryEdges::GetInputEdgePairings(
		*Mesh, InputEids, bAssignAnyBoundaryNeighborToUnmatched,
		NewVertData, EidToIndicesIntoNewVerts);
	if (!ensure(bSuccess)) { return false; }
	
	if (NewVertData.IsEmpty())
	{
		return true;
	}
	if (Progress && Progress->Cancelled()) { return false; }
	
	// Create all the new verts
	for (const FNewVertSourceData& VertSourceData : NewVertData)
	{
		int32 CreatedVid = Mesh->AppendVertex(*Mesh, VertSourceData.SourceVid);
		NewVertVids.Add(CreatedVid);
	}
	if (Progress && Progress->Cancelled()) { return false; }
	
	// Calculate extrude frames, if used
	TArray<FExtrudeFrame> NewVertExtrudeFrames; // if used, 1:1 with NewVertData
	if (bUsePerVertexExtrudeFrames)
	{
		for (const FNewVertSourceData& VertSourceData : NewVertData)
		{
			FExtrudeFrame& ExtrudeFrame = NewVertExtrudeFrames.Emplace_GetRef();
			GetExtrudeFrame(*Mesh, VertSourceData.SourceVid, 
				VertSourceData.SourceEidPair.A, VertSourceData.SourceEidPair.B, 
				ExtrudeFrame, ScalingAdjustmentLimit);
		}
		if (Progress && Progress->Cancelled()) { return false; }
	}

	// Perform the offset for all of the created Vids
	for (int32 i = 0; i < NewVertVids.Num(); ++i)
	{
		int32 NewVid = NewVertVids[i];
		int32 SourceVid = NewVertData[i].SourceVid;

		Mesh->SetVertex(NewVid, OffsetPositionFunc(
			Mesh->GetVertex(NewVid),
			bUsePerVertexExtrudeFrames ? NewVertExtrudeFrames[i] : FExtrudeFrame(),
			SourceVid));
	}
	if (Progress && Progress->Cancelled()) { return false; }

	// TODO: We could pretty easily add a version of stitching code to FDynamicMeshEditor that stitches
	//  arbitrary vert sequences rather than loops, and then maybe we could follow FOffsetMeshRegion and
	//  FInsetMeshRegion's examples to process the resulting strips. However it would require code that 
	//  assembles our input eids into sequences first. That's not too hard once we have the pairings above,
	//  but it isn't clear whether the extra effort will or will not be worthwhile. For now we do the easy
	//  thing and just prep all the quads and stitch them, then just box project UV's onto the whole thing.
	// The UV generation would be the main thing we'd want to improve, probably.
	
	int32 GroupID = 0;
	if (GroupsToSetPerEid.IsSet() && !ensure(GroupsToSetPerEid->Num() == InputEids.Num()))
	{
		GroupsToSetPerEid.Reset();
	}
	if (Mesh->HasTriangleGroups() && !GroupsToSetPerEid.IsSet())
	{
		GroupID = Mesh->AllocateTriangleGroup();
	}
	
	// Do the stitching
	for (int32 i = 0; i < InputEids.Num(); ++i)
	{
		int32 Eid = InputEids[i];
		GroupID = GroupsToSetPerEid.IsSet() ? (*GroupsToSetPerEid)[i] : GroupID;

		if (!Mesh->IsBoundaryEdge(Eid))
		{
			continue;
		}

		const FIndex2i* NewVertIndices = EidToIndicesIntoNewVerts.Find(Eid);
		if (!ensure(NewVertIndices 
			&& NewVertIndices->A >= 0 && NewVertIndices->A < NewVertVids.Num()
			&& NewVertIndices->B >= 0 && NewVertIndices->B < NewVertVids.Num()))
		{
			continue;
		}

		FIndex2i Vids = Mesh->GetOrientedBoundaryEdgeV(Eid);
		int32 VidA = Vids.A;
		int32 VidB = Vids.B;
		int32 VidC = NewVertVids[NewVertIndices->A];
		int32 VidD = NewVertVids[NewVertIndices->B];

		FIndex3i Tri1(VidB, VidA, VidD);
		NewTids.Add(Mesh->AppendTriangle(Tri1, GroupID));
		FIndex3i Tri2(VidA, VidC, VidD);
		NewTids.Add(Mesh->AppendTriangle(Tri2, GroupID));

		NewExtrudedEids.Add(Mesh->FindEdgeFromTri(VidC, VidD, NewTids.Last()));
	}

	// Give UVs to the new triangles
	if (Mesh->HasAttributes())
	{
		if (Progress && Progress->Cancelled()) { return false; }

		FAxisAlignedBox3d Box;
		for (int32 Tid : NewTids)
		{
			FVector3d Vert1, Vert2, Vert3;
			Mesh->GetTriVertices(Tid, Vert1, Vert2, Vert3);
			Box.Contain(Vert1);
			Box.Contain(Vert2);
			Box.Contain(Vert3);
		}

		FDynamicMeshUVEditor UVEd(Mesh, Mesh->Attributes()->PrimaryUV());
		FFrame3d BoxFrame(Box.Center());
		FVector3d BoxDimensions = Box.Extents() * 2;
		UVEd.SetTriangleUVsFromBoxProjection(NewTids, [](FVector3d Position) {return Position; }, BoxFrame, BoxDimensions);
	}
	
	return true;
}

// See comment in Apply()
/**
 * @param NewVertDataOut One entry for each vertex that should be created, with information on its relevant neighbor edges
 *   and the source vertex.
 * @param EidToIndicesIntoNewVertsOut Mapping from Eid to two indices into NewVertDataOut that indicate the two new vertices
 *   that would be stitched into the quad containing this edge.
 */
bool FExtrudeBoundaryEdges::GetInputEdgePairings(
	const FDynamicMesh3& Mesh, TArray<int32>& InputEids, bool bAssignAnyBoundaryNeighborToUnmatched,
	TArray<FNewVertSourceData>& NewVertDataOut, TMap<int32, FIndex2i>& EidToIndicesIntoNewVertsOut)
{
	using namespace ExtrudeBoundaryEdgesLocals;

	NewVertDataOut.Reset();
	EidToIndicesIntoNewVertsOut.Reset();

	// Helper that updates both NewVertDataOut and EidToIndicesIntoNewVertsOut
	auto AddEdgePairing = [&NewVertDataOut, &EidToIndicesIntoNewVertsOut](int32 SourceVid, int32 IncomingEid, int32 OutgoingEid)
	{
		int32 NewVertIndex = NewVertDataOut.Emplace();
		NewVertDataOut[NewVertIndex].SourceVid = SourceVid;
		NewVertDataOut[NewVertIndex].SourceEidPair = FIndex2i(IncomingEid, OutgoingEid);
		
		auto UpdateMapping = [NewVertIndex, &EidToIndicesIntoNewVertsOut](int32 Eid, bool bUpdatingOutgoingEdge)
		{
			if (Eid == IndexConstants::InvalidID)
			{
				return;
			}

			FIndex2i* ExistingEntry = EidToIndicesIntoNewVertsOut.Find(Eid);
			if (ExistingEntry)
			{
				// If we're updating the outgoing edge, then we are its first vertex
				(*ExistingEntry)[bUpdatingOutgoingEdge ? 0 : 1] = NewVertIndex;
			}
			else
			{
				FIndex2i Entry;
				Entry[bUpdatingOutgoingEdge ? 0 : 1] = NewVertIndex;
				EidToIndicesIntoNewVertsOut.Add(Eid, Entry);
			}
		};

		UpdateMapping(IncomingEid, false);
		UpdateMapping(OutgoingEid, true);
	};

	// We will be building up a list of associations of verts to two neighbor edges. If we run into
	//  a situation where there are more than two candidates, we will put the entry into AmbiguousVids
	//  and (later) remove it from this map.
	TMap<int32, FIndex2i> UnambiguousVidToEdgePairings;
	
	TArray<int32> AmbiguousVids;

	for (int32 Eid : InputEids)
	{
		if (!Mesh.IsBoundaryEdge(Eid))
		{
			continue;
		}
		
		FIndex2i Vids = Mesh.GetOrientedBoundaryEdgeV(Eid);
		for (int SubIdx = 0; SubIdx < 2; ++SubIdx)
		{
			int32 Vid = Vids[SubIdx];
			FIndex2i* EdgePairing = UnambiguousVidToEdgePairings.Find(Vid);
			if (!EdgePairing)
			{
				EdgePairing = &UnambiguousVidToEdgePairings.Add(Vid, FIndex2i::Invalid());
			}
			if ((*EdgePairing)[1-SubIdx] != IndexConstants::InvalidID)
			{
				// Found multiple incoming or outgoing edges. We'll handle this later.
				AmbiguousVids.Add(Vid);
			}
			else
			{
				(*EdgePairing)[1-SubIdx] = Eid;
			}
		}
	}

	// Crude (not angle-weighed) vertex normal calculation used for projection below, same as 
	//  FMeshBoundaryLoops::GetVertexNormal.
	auto CalculateVertexNormal = [&Mesh](int32 Vid)
	{
		FVector3d VertexNormal = FVector3d::Zero();
		for (int32 Tid : Mesh.VtxTrianglesItr(Vid))
		{
			VertexNormal += Mesh.GetTriNormal(Tid);
		}
		VertexNormal.Normalize();
		return VertexNormal;
	};

	TSet<int32> SelectedEids(InputEids);

	// Sort out the ambiguous verts first, and remove those entries from UnambiguousVidToEdgePairings
	for (int32 AmbiguousVid : AmbiguousVids)
	{
		UnambiguousVidToEdgePairings.Remove(AmbiguousVid);

		FDynamicMesh3::FLocalIntArray IncomingEids;
		FDynamicMesh3::FLocalIntArray OutgoingEids;
		for (int32 Eid : Mesh.VtxEdgesItr(AmbiguousVid))
		{
			if (!Mesh.IsBoundaryEdge(Eid) || !SelectedEids.Contains(Eid))
			{
				continue;
			}

			FIndex2i Vids = Mesh.GetOrientedBoundaryEdgeV(Eid);
			if (Vids.A == AmbiguousVid)
			{
				OutgoingEids.Add(Eid);
			}
			else
			{
				ensure(Vids.B == AmbiguousVid);
				IncomingEids.Add(Eid);
			}
		}

		FVector3d VertexNormal = CalculateVertexNormal(AmbiguousVid);

		for (int32 IncomingEid : IncomingEids)
		{
			int32 OutgoingEid = FindSharpestAngleMatchEdge(Mesh, IncomingEid, AmbiguousVid, VertexNormal, OutgoingEids);
			if (OutgoingEid != IndexConstants::InvalidID)
			{
				// One candidate used up
				OutgoingEids.Remove(OutgoingEid);
			}
			AddEdgePairing(AmbiguousVid, IncomingEid, OutgoingEid);
		}
		// If we still have some outgoing edges, we know they are unmatched now.
		for (int32 OutgoingEid : OutgoingEids)
		{
			AddEdgePairing(AmbiguousVid, IndexConstants::InvalidID, OutgoingEid);
		}
	}// done processing ambiguous verts

	for (const TPair<int32, FIndex2i>& KeyValue : UnambiguousVidToEdgePairings)
	{
		AddEdgePairing(KeyValue.Key, KeyValue.Value.A, KeyValue.Value.B);
	}

	if (bAssignAnyBoundaryNeighborToUnmatched)
	{
		for (FNewVertSourceData& VertData : NewVertDataOut)
		{
			if (!VertData.SourceEidPair.Contains(IndexConstants::InvalidID))
			{
				continue;
			}

			FVector3d VertexNormal = CalculateVertexNormal(VertData.SourceVid);
			FDynamicMesh3::FLocalIntArray NeighborhoodEids;
			Mesh.GetAllVtxBoundaryEdges(VertData.SourceVid, NeighborhoodEids);

			if (VertData.SourceEidPair.A == IndexConstants::InvalidID 
				&& ensure(VertData.SourceEidPair.B != IndexConstants::InvalidID))
			{
				VertData.SourceEidPair.A = FindSharpestAngleMatchEdge(Mesh, 
					VertData.SourceEidPair.B, VertData.SourceVid, VertexNormal, NeighborhoodEids);
			}
			else if (VertData.SourceEidPair.B == IndexConstants::InvalidID
				&& ensure(VertData.SourceEidPair.A != IndexConstants::InvalidID))
			{
				VertData.SourceEidPair.B = FindSharpestAngleMatchEdge(Mesh,
					VertData.SourceEidPair.A, VertData.SourceVid, VertexNormal, NeighborhoodEids);
			}
		}
	}

	return true;
}

bool FExtrudeBoundaryEdges::GetExtrudeFrame(const FDynamicMesh3& Mesh, int32 Vid,
	int32 IncomingEid, int32 OutgoingEid, FExtrudeFrame& ExtrudeFrameOut,
	double ScalingLimit)
{
	using namespace ExtrudeBoundaryEdgesLocals;

	auto GetEdgeVectorAndNormal = [&Mesh, Vid](int32 Eid, bool bIsIncoming, FVector3d& EdgeVectorOut, FVector3d& NormalVectorOut) -> bool
	{
		if (Eid == IndexConstants::InvalidID)
		{
			EdgeVectorOut = FVector3d::Zero();
			NormalVectorOut = FVector3d::Zero();
			return true;
		}

		if (!ensure(Mesh.IsBoundaryEdge(Eid)))
		{
			return false;
		}

		FIndex2i Vids = Mesh.GetOrientedBoundaryEdgeV(Eid);
		if (bIsIncoming && !ensure(Vids.B == Vid))
		{
			return false;
		}
		else if (!bIsIncoming && !ensure(Vids.A == Vid))
		{
			return false;
		}

		int32 Tid = Mesh.GetEdgeT(Eid).A;
		NormalVectorOut = Mesh.GetTriNormal(Tid);

		EdgeVectorOut = Mesh.GetVertex(Vids.B) - Mesh.GetVertex(Vids.A);
		EdgeVectorOut.Normalize();

		return true;
	};

	FVector3d EdgeVector1, EdgeVector2, Normal1, Normal2;

	if (!GetEdgeVectorAndNormal(IncomingEid, true, EdgeVector1, Normal1)
		|| !GetEdgeVectorAndNormal(OutgoingEid, false, EdgeVector2, Normal2))
	{
		return false;
	}

	ExtrudeFrameOut = GetExtrudeFrameAtVertex(
		Mesh.GetVertex(Vid),
		Normal1, EdgeVector1,
		Normal2, EdgeVector2, ScalingLimit);

	return true;
}

FVector3d FExtrudeBoundaryEdges::FExtrudeFrame::FromFramePoint(FVector3d FramePoint) const
{
	if (InFrameScaleDirection.IsSet() && Scaling != 1)
	{
		FVector3d Offset = FramePoint.Dot(InFrameScaleDirection.GetValue()) * (Scaling - 1) * InFrameScaleDirection.GetValue();
		return Frame.FromFramePoint(FramePoint + Offset);
	}
	return Frame.FromFramePoint(FramePoint);
}

FVector3d UE::Geometry::FExtrudeBoundaryEdges::FExtrudeFrame::ToFramePoint(FVector3d WorldPoint) const
{
	FVector3d FramePoint = Frame.ToFramePoint(WorldPoint);
	if (InFrameScaleDirection.IsSet() && Scaling != 1
		// Can't undo zero scaling, but that shouldn't come up as a value
		&& ensure(Scaling != 0))
	{
		double DotProduct = FramePoint.Dot(InFrameScaleDirection.GetValue());
		return FramePoint - (1 - (1 / Scaling)) * DotProduct * InFrameScaleDirection.GetValue();
	}
	return FramePoint;
}

}//end UE::Geometry