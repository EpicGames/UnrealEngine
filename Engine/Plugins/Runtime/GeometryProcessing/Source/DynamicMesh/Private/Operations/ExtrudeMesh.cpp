// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/ExtrudeMesh.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/MeshIndexUtil.h"

using namespace UE::Geometry;

FExtrudeMesh::FExtrudeMesh(FDynamicMesh3* mesh) : Mesh(mesh)
{
	ExtrudedPositionFunc = [this](const FVector3d& Position, const FVector3f& Normal, int VertexID) 
	{
		return Position + this->DefaultExtrudeDistance * (FVector3d)Normal;
	};
}


bool FExtrudeMesh::Apply()
{
	FMeshNormals Normals;
	bool bHaveVertexNormals = Mesh->HasVertexNormals();
	if (!bHaveVertexNormals)
	{
		Normals = FMeshNormals(Mesh);
		Normals.ComputeVertexNormals();
	}

	FMeshConnectedComponents MeshRegions(Mesh);
	MeshRegions.FindConnectedTriangles();

	bool bAllOK = true;
	Extrusions.SetNum(MeshRegions.Num());
	for (int k = 0; k < MeshRegions.Num(); ++k)
	{
		FExtrusionInfo& Extrusion = Extrusions[k];
		Extrusion.InitialTriangles = MoveTemp(MeshRegions.Components[k].Indices);
		if (ApplyExtrude(Extrusion, (bHaveVertexNormals) ? nullptr : &Normals) == false)
		{
			bAllOK = false;
		}
	}

	return bAllOK;
}





bool FExtrudeMesh::ApplyExtrude(FExtrusionInfo& Region, FMeshNormals* UseNormals)
{
	Region.InitialLoops.SetMesh(Mesh, Region.InitialTriangles);
	bool bOK = Region.InitialLoops.Compute();
	if (bOK == false)
	{
		return false;
	}
	int NumInitialLoops = Region.InitialLoops.GetLoopCount();
	if (bSkipClosedComponents && NumInitialLoops == 0)
	{
		return true;
	}

	UE::Geometry::TriangleToVertexIDs(Mesh, Region.InitialTriangles, Region.InitialVertices);

	// duplicate triangles of mesh

	FDynamicMeshEditor Editor(Mesh);

	FMeshIndexMappings IndexMap;
	FDynamicMeshEditResult DuplicateResult;
	Editor.DuplicateTriangles(Region.InitialTriangles, IndexMap, DuplicateResult);
	Region.OffsetTriangles = DuplicateResult.NewTriangles;
	Region.OffsetTriGroups = DuplicateResult.NewGroups;
	Region.InitialToOffsetMapV = IndexMap.GetVertexMap().GetForwardMap();

	// set vertices to new positions
	for (int vid : Region.InitialVertices)
	{
		if ( ! Region.InitialToOffsetMapV.Contains(vid) )
		{
			continue;
		}
		int newvid = Region.InitialToOffsetMapV[vid];
		if ( ! Mesh->IsVertex(newvid) )
		{
			continue;
		}

		FVector3d v = Mesh->GetVertex(vid);
		FVector3f n = (UseNormals != nullptr) ? (FVector3f)(*UseNormals)[vid] : Mesh->GetVertexNormal(vid);
		FVector3d newv = ExtrudedPositionFunc(v, n, vid);

		Mesh->SetVertex(newvid, newv);
	}

	// we need to reverse one side
	if (IsPositiveOffset)
	{
		Editor.ReverseTriangleOrientations(Region.InitialTriangles, true);
	}
	else
	{
		Editor.ReverseTriangleOrientations(Region.OffsetTriangles, true);
	}

	// stitch each loop
	Region.NewLoops.SetNum(NumInitialLoops);
	Region.StitchTriangles.SetNum(NumInitialLoops);
	Region.StitchPolygonIDs.SetNum(NumInitialLoops);
	int LoopIndex = 0;
	for (FEdgeLoop& BaseLoop : Region.InitialLoops.Loops)
	{
		int LoopCount = BaseLoop.GetVertexCount();

		TArray<int> OffsetLoop;
		OffsetLoop.SetNum(LoopCount);
		for (int k = 0; k < LoopCount; ++k)
		{
			OffsetLoop[k] = Region.InitialToOffsetMapV[BaseLoop.Vertices[k]];
		}

		FDynamicMeshEditResult StitchResult;
		bool bStitchSuccess;
		if (IsPositiveOffset)
		{
			bStitchSuccess = Editor.StitchVertexLoopsMinimal(OffsetLoop, BaseLoop.Vertices, StitchResult);
		}
		else
		{
			bStitchSuccess = Editor.StitchVertexLoopsMinimal(BaseLoop.Vertices, OffsetLoop, StitchResult);
		}

		if (bStitchSuccess)
		{
			StitchResult.GetAllTriangles(Region.StitchTriangles[LoopIndex]);
			Region.StitchPolygonIDs[LoopIndex] = StitchResult.NewGroups;

			// for each polygon we created in stitch, set UVs and normals
			if (Mesh->HasAttributes())
			{
				float AccumUVTranslation = 0;

				FFrame3d FirstProjectFrame;
				FVector3d FrameUp;

				int NumNewQuads = StitchResult.NewQuads.Num();
				for (int k = 0; k < NumNewQuads; k++)
				{
					FVector3f Normal = Editor.ComputeAndSetQuadNormal(StitchResult.NewQuads[k], true);

					// align axis 0 of projection frame to first edge, then for further edges,
					// rotate around 'up' axis to keep normal aligned and frame horizontal
					FFrame3d ProjectFrame;
					if (k == 0)
					{
						FVector3d FirstEdge = Mesh->GetVertex(BaseLoop.Vertices[1]) - Mesh->GetVertex(BaseLoop.Vertices[0]);
						Normalize(FirstEdge);
						FirstProjectFrame = FFrame3d(FVector3d::Zero(), (FVector3d)Normal);
						FirstProjectFrame.ConstrainedAlignAxis(0, FirstEdge, (FVector3d)Normal);
						FrameUp = FirstProjectFrame.GetAxis(1);
						ProjectFrame = FirstProjectFrame;
					}
					else
					{
						ProjectFrame = FirstProjectFrame;
						ProjectFrame.ConstrainedAlignAxis(2, (FVector3d)Normal, FrameUp);
					}

					if (k > 0)
					{
						AccumUVTranslation +=(float) Distance(Mesh->GetVertex(BaseLoop.Vertices[k]), Mesh->GetVertex(BaseLoop.Vertices[k - 1]));
					}

					// translate horizontally such that vertical spans are adjacent in UV space (so textures tile/wrap properly)
					float TranslateU = UVScaleFactor * AccumUVTranslation;
					Editor.SetQuadUVsFromProjection(StitchResult.NewQuads[k], ProjectFrame, UVScaleFactor, FVector2f(TranslateU, 0));
				}
			}

			Region.NewLoops[LoopIndex].InitializeFromVertices(Mesh, OffsetLoop);
		}
		LoopIndex++;
	}

	return true;
}