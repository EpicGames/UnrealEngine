// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/FFDLattice.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Util/ProgressCancel.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

namespace FFDLatticeHelpers
{
	double CubicBSplineKernel(double A)
	{
		// Using cubic kernel f(a) =
		//	(4 - 6a^2 + 3|a|^3) / 6		for 0 <= |a| < 1
		//	(2 - |a|)^3 / 6				for 1 <= |a| < 2
		//	0							otherwise
		//
		// So at a = {-2, -1, 0, 1, 2}, f(a) = {0, 1/6, 4/6, 1/6, 0}, and is piecewise cubic in between.

		double AbsA = FMath::Abs(A);
		if (AbsA < 1.0f)
		{
			double ASquared = AbsA * AbsA;
			return (4.0 - 6.0 * ASquared + 3.0 * ASquared * AbsA) / 6.0;
		}
		else if (AbsA < 2.0)
		{
			double TwoMinusAbsA = (2.0 - AbsA);
			return TwoMinusAbsA * TwoMinusAbsA * TwoMinusAbsA / 6.0;
		}

		return 0.0;
	}


	double CubicBSplineKernelDerivative(double A)
	{
		// f'(a) =
		//	a/2*(3|a| - 4)		            for 0 <= |a| < 1
		//	-(a*(2-|a|)^2) / (2*|a|) 		for 1 <= |a| < 2
		//	0								otherwise

		double AbsA = FMath::Abs(A);
		if (AbsA < 1.0)
		{
			return A / 2.0 * (3.0 * AbsA - 4.0);
		}
		else if (AbsA < 2.0)
		{
			double TwoMinusAbsA = (2.0 - AbsA);
			return -(A * TwoMinusAbsA * TwoMinusAbsA) / (2.0 * AbsA);
		}

		return 0.0;
	}

} // namespace FFDLatticeHelpers


FFFDLattice::FFFDLattice(const FVector3i& InDims, const FDynamicMesh3& Mesh, float Padding) :
	Transform(FTransformSRT3d::Identity()),
	Dimensions(InDims)
{
	check(InDims.X > 1 && InDims.Y > 1 && InDims.Z > 1);

	InitialBounds = Mesh.GetBounds();
	if (InitialBounds.IsEmpty())
	{
		InitialBounds = FAxisAlignedBox3d(FVector3d::Zero(), FVector3d::Zero());
	}

	InitLattice(Padding);

	ComputeInitialEmbedding(Mesh);
}

FFFDLattice::FFFDLattice(const FVector3i& InDims, const FDynamicMesh3* Mesh, float Padding, const FAxisAlignedBox3d& InInitialBounds, const FTransformSRT3d& LatticeTransform) :
	Transform(LatticeTransform),
	Dimensions(InDims),
	InitialBounds(InInitialBounds)
{
	check(InDims.X > 1 && InDims.Y > 1 && InDims.Z > 1);

	if (InitialBounds.IsEmpty())
	{
		InitialBounds = FAxisAlignedBox3d(FVector3d::Zero(), FVector3d::Zero());
	}

	InitLattice(Padding);

	if (Mesh)
	{
		ComputeInitialEmbedding(*Mesh);
	}
}

void FFFDLattice::InitLattice(float Padding)
{
	// Create initial bounding box with user-specified padding
	const FVector3d Extents = InitialBounds.Extents();
	const float ClampedPadding = FMath::Clamp(Padding, 0.0f, 5.0f);
	const double MaxDiagonal = MaxElement(InitialBounds.Diagonal());

	InitialBounds.Min -= FVector3d(0.5 * ClampedPadding * MaxDiagonal);
	InitialBounds.Max += FVector3d(0.5 * ClampedPadding * MaxDiagonal);

	const FVector3d Diag = InitialBounds.Diagonal();
	CellSize = Diag / FVector3d(Dimensions - 1);
}

void FFFDLattice::GenerateInitialLatticePositions(TArray<FVector3d>& OutLatticePositions) const
{
	int TotalNumLatticePoints = Dimensions.X * Dimensions.Y * Dimensions.Z;
	OutLatticePositions.SetNum(TotalNumLatticePoints);

	for (int i = 0; i < Dimensions.X; ++i)
	{
		double X = CellSize.X * i;
		for (int j = 0; j < Dimensions.Y; ++j)
		{
			double Y = CellSize.Y * j;
			for (int k = 0; k < Dimensions.Z; ++k)
			{
				int PointID = ControlPointIndexFromCoordinates(i, j, k);
				double Z = CellSize.Z * k;

				OutLatticePositions[PointID] = Transform.TransformPosition(InitialBounds.Min + FVector3d{ X,Y,Z });
			}
		}
	}
}


void FFFDLattice::GenerateLatticeEdges(TArray<FVector2i>& OutLatticeEdges) const
{
	OutLatticeEdges.Reset(3 * Dimensions.X * Dimensions.Y * Dimensions.Z);

	for (int i = 0; i < Dimensions.X; ++i)
	{
		for (int j = 0; j < Dimensions.Y; ++j)
		{
			for (int k = 0; k < Dimensions.Z; ++k)
			{
				int PointID = ControlPointIndexFromCoordinates(i, j, k);

				if (i + 1 < Dimensions.X)
				{
					int IPlusOne = ControlPointIndexFromCoordinates(i + 1, j, k);
					OutLatticeEdges.Add({ PointID, IPlusOne });
				}
				if (j + 1 < Dimensions.Y)
				{
					int JPlusOne = ControlPointIndexFromCoordinates(i, j + 1, k);
					OutLatticeEdges.Add({ PointID, JPlusOne });
				}
				if (k + 1 < Dimensions.Z)
				{
					int KPlusOne = ControlPointIndexFromCoordinates(i, j, k + 1);
					OutLatticeEdges.Add({ PointID, KPlusOne });
				}
			}
		}
	}
}


FVector3d FFFDLattice::ComputeTrilinearWeights(const FVector3d& QueryPoint, FVector3i& GridCoordinates) const
{
	const FVector3d GridPoint = (QueryPoint - InitialBounds.Min) / CellSize;
	GridCoordinates = FVector3i(GridPoint);
	FVector3d Weights = GridPoint - FVector3d(GridCoordinates);

	for (int32 Coord = 0; Coord < 3; ++Coord)
	{
		double OrigWeight = Weights[Coord];

		if (GridCoordinates[Coord] < 0)
		{
			GridCoordinates[Coord] = 0;
			Weights[Coord] = 0.0;
		}
		if (GridCoordinates[Coord] > Dimensions[Coord] - 2)
		{
			GridCoordinates[Coord] = Dimensions[Coord] - 2;
			Weights[Coord] = 1.0;
		}
	}

	return Weights;
}

void FFFDLattice::ComputeInitialEmbedding(const FDynamicMesh3& Mesh, FLatticeExecutionInfo ExecutionInfo)
{
	const int MaxVertexID = Mesh.MaxVertexID();
	VertexEmbeddings.SetNum(MaxVertexID);

	const EParallelForFlags ParallelForFlags = ExecutionInfo.bParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;

	// Expand the box a little to catch vertices right on the lattice boundary. Points which are a bit outside the box will have their embedding cells/weights clamped 
	// in ComputeTrilinearWeights. Note this is different than the user-supplied padding which changes the dimensions of the bounding box.
	FAxisAlignedBox3d ExpandedBox = InitialBounds;
	ExpandedBox.Expand(UE_KINDA_SMALL_NUMBER);

	ParallelFor(MaxVertexID,
				[this, &Mesh, &ExpandedBox, MaxVertexID](int VertexID)
	{
		FEmbedding& Embedding = VertexEmbeddings[VertexID];
		Embedding.LatticeCell = { -1, -1, -1 };

		if (!Mesh.IsVertex(VertexID))
		{
			return;
		}

		const FVector3d MeshVertexPosition = Mesh.GetVertex(VertexID);
		const FVector3d LatticeSpaceVertexPosition = Transform.InverseTransformPosition(MeshVertexPosition);

		if (!ExpandedBox.Contains(LatticeSpaceVertexPosition))
		{
			return;
		}

		Embedding.CellWeighting = ComputeTrilinearWeights(LatticeSpaceVertexPosition, Embedding.LatticeCell);
	},
	ParallelForFlags);
}

bool FFFDLattice::VertexHasValidEmbedding(int32 VertexIndex) const
{
	return (VertexIndex >= 0) 
		&& (VertexIndex < VertexEmbeddings.Num()) 
		&& (VertexEmbeddings[VertexIndex].LatticeCell[0] >= 0) 
		&& (VertexEmbeddings[VertexIndex].LatticeCell[1] >= 0) 
		&& (VertexEmbeddings[VertexIndex].LatticeCell[2] >= 0);
}


void FFFDLattice::GetDeformedMeshVertexPositions(const TArray<FVector3d>& LatticeControlPoints,
												 TArray<FVector3d>& OutVertexPositions,
												 ELatticeInterpolation Interpolation,
												 FLatticeExecutionInfo ExecutionInfo,
												 FProgressCancel* Progress) const
{
	int MaxVertexID = VertexEmbeddings.Num();
	OutVertexPositions.SetNumZeroed(MaxVertexID);

	EParallelForFlags ParallelForFlags = ExecutionInfo.bParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;
	ParallelForFlags |= EParallelForFlags::BackgroundPriority;

	bool bCancelled = false;

	check(VertexEmbeddings.Num() == OutVertexPositions.Num());

	auto InterpolationJob = [this, &OutVertexPositions, &LatticeControlPoints, &bCancelled, ExecutionInfo, MaxVertexID, Interpolation, Progress]
	(int VertexID)
	{
		// Every once in a while, check for cancellation
		if ((VertexID % ExecutionInfo.CancelCheckSize == 0) && Progress && Progress->Cancelled())
		{
			bCancelled = true;
		}

		if (bCancelled || VertexEmbeddings[VertexID].LatticeCell[0] < 0)
		{
			return;
		}

		if (Interpolation == ELatticeInterpolation::Cubic)
		{
			OutVertexPositions[VertexID] = InterpolatedPositionCubic(VertexEmbeddings[VertexID], LatticeControlPoints);
		}
		else
		{
			OutVertexPositions[VertexID] = InterpolatedPosition(VertexEmbeddings[VertexID], LatticeControlPoints);
		}
	};

	ParallelFor(MaxVertexID, InterpolationJob, ParallelForFlags);

}

void FFFDLattice::GetRotatedOverlayNormals(const TArray<FVector3d>& LatticeControlPoints,
										   const FDynamicMeshNormalOverlay* NormalOverlay,
										   TArray<FVector3f>& OutNormals,
										   ELatticeInterpolation Interpolation,
										   FLatticeExecutionInfo ExecutionInfo,
										   FProgressCancel* Progress) const
{
	int ElementCount = NormalOverlay->ElementCount();
	OutNormals.SetNumZeroed(ElementCount);

	EParallelForFlags ParallelForFlags = ExecutionInfo.bParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;
	ParallelForFlags |= EParallelForFlags::BackgroundPriority;

	bool bCancelled = false;
	auto InterpolationJob = [this, &OutNormals, &NormalOverlay, &LatticeControlPoints, &bCancelled, ExecutionInfo, Interpolation, Progress]
	(int OverlayElementID)
	{
		// Every once in a while, check for cancellation
		if ((OverlayElementID % ExecutionInfo.CancelCheckSize == 0) && Progress && Progress->Cancelled())
		{
			bCancelled = true;
		}

		int ParentVertexID = NormalOverlay->GetParentVertex(OverlayElementID);

		if (bCancelled || VertexEmbeddings[ParentVertexID].LatticeCell[0] < 0)
		{
			return;
		}

		FMatrix3d Jacobian = (Interpolation == ELatticeInterpolation::Linear) ?
			LinearInterpolationJacobian(VertexEmbeddings[ParentVertexID], LatticeControlPoints) :
			CubicInterpolationJacobian(VertexEmbeddings[ParentVertexID], LatticeControlPoints);

		// Typically we'd do transpose(inv(J)), however if a lattice cell inverts, the determinant is negative and the 
		// resulting normal will be flipped. So we instead multiply by det(J)*transpose(inv(J)) to get the sign right.
		FMatrix3d InvJacobian = Jacobian.DeterminantTimesInverseTranspose();
		OutNormals[OverlayElementID] = FMatrix3f(InvJacobian) * NormalOverlay->GetElement(OverlayElementID);
		Normalize(OutNormals[OverlayElementID]);
	};

	ParallelFor(ElementCount, InterpolationJob, ParallelForFlags);
}


void FFFDLattice::GetRotatedMeshVertexNormals(const TArray<FVector3d>& LatticeControlPoints,
											  const TArray<FVector3f>& OriginalNormals,
											  TArray<FVector3f>& OutNormals,
											  ELatticeInterpolation Interpolation,
											  FLatticeExecutionInfo ExecutionInfo,
											  FProgressCancel* Progress) const
{
	int MaxVertexID = OriginalNormals.Num();
	OutNormals.SetNumZeroed(MaxVertexID);

	EParallelForFlags ParallelForFlags = ExecutionInfo.bParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;
	ParallelForFlags |= EParallelForFlags::BackgroundPriority;

	bool bCancelled = false;

	check(VertexEmbeddings.Num() == OutNormals.Num());

	auto InterpolationJob = [this, &OutNormals, &OriginalNormals, &LatticeControlPoints, &bCancelled, ExecutionInfo, MaxVertexID, Interpolation, Progress]
	(int VertexID)
	{
		// Every once in a while, check for cancellation
		if ((VertexID % ExecutionInfo.CancelCheckSize == 0) && Progress && Progress->Cancelled())
		{
			bCancelled = true;
		}

		if (bCancelled || VertexEmbeddings[VertexID].LatticeCell[0] < 0)
		{
			return;
		}

		FMatrix3d Jacobian = (Interpolation == ELatticeInterpolation::Linear) ?
			LinearInterpolationJacobian(VertexEmbeddings[VertexID], LatticeControlPoints) :
			CubicInterpolationJacobian(VertexEmbeddings[VertexID], LatticeControlPoints);

		// Typically we'd do transpose(inv(J)), however if a lattice cell inverts, the determinant is negative and the 
		// resulting normal will be flipped. So we instead multiply by det(J)*transpose(inv(J)) to get the sign right.
		FMatrix3d InvJacobian = Jacobian.DeterminantTimesInverseTranspose();
		OutNormals[VertexID] = FMatrix3f( InvJacobian ) * OriginalNormals[VertexID];
		Normalize(OutNormals[VertexID]);
	};

	ParallelFor(MaxVertexID, InterpolationJob, ParallelForFlags);
}


void FFFDLattice::GetValuePair(int I, int J, int K, FVector3d& A, FVector3d& B, 
							   const TArray<FVector3d>& LatticeControlPoints) const
{
	int IndexA = ControlPointIndexFromCoordinates(I, J, K);
	check(IndexA < LatticeControlPoints.Num());
	check(IndexA >= 0);
	A = LatticeControlPoints[IndexA];

	int IndexB = ControlPointIndexFromCoordinates(I + 1, J, K);
	B = LatticeControlPoints[IndexB];
}


FVector3d FFFDLattice::InterpolatedPositionCubic(const FEmbedding& VertexEmbedding, 
												 const TArray<FVector3d>& LatticeControlPoints) const
{
	using FFDLatticeHelpers::CubicBSplineKernel;

	double T = VertexEmbedding.CellWeighting.X;
	double U = VertexEmbedding.CellWeighting.Y;
	double V = VertexEmbedding.CellWeighting.Z;

	FVector3d Sum{ 0.0f, 0.0f, 0.0f };

	// TODO: This can probably be replaced with some relatively simple linear algebra

	for (int DI = -1; DI <= 2; ++DI)
	{
		double WeightX = CubicBSplineKernel(T - DI);

		for (int DJ = -1; DJ <= 2; ++DJ)
		{
			double WeightY = CubicBSplineKernel(U - DJ);

			for (int DK = -1; DK <= 2; ++DK)
			{
				double WeightZ = CubicBSplineKernel(V - DK);
				double Weight = WeightX * WeightY * WeightZ;

				int i = VertexEmbedding.LatticeCell.X + DI;
				int j = VertexEmbedding.LatticeCell.Y + DJ;
				int k = VertexEmbedding.LatticeCell.Z + DK;

				FVector3d LatticePoint;
				if (i < 0 || i >= Dimensions.X || j < 0 || j >= Dimensions.Y || k < 0 || k >= Dimensions.Z)
				{
					// Get the extrapolated position for a "virtual" control point outside of the deformed lattice
					LatticePoint = ExtrapolatedLatticePosition({ i,j,k }, LatticeControlPoints);
				}
				else
				{
					int PointIndex = ControlPointIndexFromCoordinates(i, j, k);
					LatticePoint = LatticeControlPoints[PointIndex];
				}

				Sum += Weight * LatticePoint;
			}
		}
	}

	return Sum;
}

FVector3d FFFDLattice::InterpolatedPosition(const FEmbedding& VertexEmbedding, const TArray<FVector3d>& LatticeControlPoints) const
{
	// TODO: See if we can refactor TTriLinearGridInterpolant to make that usable in this class

	// Trilinear interpolation:
	//		V### is grid cell corner index
	//		AlphaN is [0,1] fraction of point in cell along N'th dimension  
	// return
	//		V000 * (1 - AlphaX) * (1 - AlphaY) * (1 - AlphaZ) +
	//		V001 * (1 - AlphaX) * (1 - AlphaY) * (AlphaZ) +
	//		V010 * (1 - AlphaX) * (AlphaY) * (1 - AlphaZ) +
	//		V011 * (1 - AlphaX) * (AlphaY) * (AlphaZ) +
	//		V100 * (AlphaX) * (1 - AlphaY) * (1 - AlphaZ) +
	//		V101 * (AlphaX) * (1 - AlphaY) * (AlphaZ) +
	//		V110 * (AlphaX) * (AlphaY) * (1 - AlphaZ) +
	//		V111 * (AlphaX) * (AlphaY) * (AlphaZ);

	int X0 = VertexEmbedding.LatticeCell.X;
	int Y0 = VertexEmbedding.LatticeCell.Y;
	int Y1 = Y0 + 1;
	int Z0 = VertexEmbedding.LatticeCell.Z;
	int Z1 = Z0 + 1;

	double AlphaX = VertexEmbedding.CellWeighting.X;
	double AlphaY = VertexEmbedding.CellWeighting.Y;
	double AlphaZ = VertexEmbedding.CellWeighting.Z;
	double OneMinusAlphaX = 1.0 - AlphaX;

	FVector3d Sum{ 0,0,0 };

	FVector3d FV000, FV100;
	GetValuePair(X0, Y0, Z0, FV000, FV100, LatticeControlPoints);
	double YZ = (1 - AlphaY) * (1 - AlphaZ);
	Sum = (OneMinusAlphaX * FV000 + AlphaX * FV100) * YZ;

	FVector3d FV001, FV101;
	GetValuePair(X0, Y0, Z1, FV001, FV101, LatticeControlPoints);
	YZ = (1 - AlphaY) * (AlphaZ);
	Sum += (OneMinusAlphaX * FV001 + AlphaX * FV101) * YZ;

	FVector3d FV010, FV110;
	GetValuePair(X0, Y1, Z0, FV010, FV110, LatticeControlPoints);
	YZ = (AlphaY) * (1 - AlphaZ);
	Sum += (OneMinusAlphaX * FV010 + AlphaX * FV110) * YZ;

	FVector3d FV011, FV111;
	GetValuePair(X0, Y1, Z1, FV011, FV111, LatticeControlPoints);
	YZ = (AlphaY) * (AlphaZ);
	Sum += (OneMinusAlphaX * FV011 + AlphaX * FV111) * YZ;

	return Sum;
}

FMatrix3d FFFDLattice::LinearInterpolationJacobian(const FEmbedding& VertexEmbedding, const TArray<FVector3d>& LatticeControlPoints) const
{
	int X0 = VertexEmbedding.LatticeCell.X;
	int Y0 = VertexEmbedding.LatticeCell.Y;
	int Y1 = Y0 + 1;
	int Z0 = VertexEmbedding.LatticeCell.Z;
	int Z1 = Z0 + 1;

	FVector3d FV000, FV100;
	GetValuePair(X0, Y0, Z0, FV000, FV100, LatticeControlPoints);
	FVector3d FV001, FV101;
	GetValuePair(X0, Y0, Z1, FV001, FV101, LatticeControlPoints);
	FVector3d FV010, FV110;
	GetValuePair(X0, Y1, Z0, FV010, FV110, LatticeControlPoints);
	FVector3d FV011, FV111;
	GetValuePair(X0, Y1, Z1, FV011, FV111, LatticeControlPoints);

	double AlphaX = VertexEmbedding.CellWeighting.X;
	double AlphaY = VertexEmbedding.CellWeighting.Y;
	double AlphaZ = VertexEmbedding.CellWeighting.Z;
	double OneMinusAlphaX = 1.0 - AlphaX;

	// Partial wrt x
	FVector3d PartialX = (FV100 - FV000) * (1 - AlphaY) * (1 - AlphaZ) +
		(FV101 - FV001) * (1 - AlphaY) * (AlphaZ)+
		(FV110 - FV010) * (AlphaY) * (1 - AlphaZ) +
		(FV111 - FV011) * (AlphaY) * (AlphaZ);

	// common terms for partialy and partialz
	FVector3d T0 = OneMinusAlphaX * FV000 + AlphaX * FV100;
	FVector3d T1 = OneMinusAlphaX * FV001 + AlphaX * FV101;
	FVector3d T2 = OneMinusAlphaX * FV010 + AlphaX * FV110;
	FVector3d T3 = OneMinusAlphaX * FV011 + AlphaX * FV111;

	// Partial wrt y
	FVector3d PartialY = T0 * (AlphaZ - 1) - T1 * AlphaZ + T2 * (1 - AlphaZ) + T3 * AlphaZ;

	// Partial wrt z
	FVector3d PartialZ = T0 * (AlphaY - 1) + T1 * (1 - AlphaY) - T2 * AlphaY + T3 * AlphaY;

	FMatrix3d Mat(PartialX, PartialY, PartialZ, false);

	return Mat;
}

FMatrix3d FFFDLattice::CubicInterpolationJacobian(const FEmbedding& VertexEmbedding, const TArray<FVector3d>& LatticeControlPoints) const
{
	using FFDLatticeHelpers::CubicBSplineKernel;
	using FFDLatticeHelpers::CubicBSplineKernelDerivative;

	// TODO: This was written for clarity and correctness. Could definitely be faster.

	double T = VertexEmbedding.CellWeighting.X;
	double U = VertexEmbedding.CellWeighting.Y;
	double V = VertexEmbedding.CellWeighting.Z;

	FMatrix3d Sum = FMatrix3d::Zero();

	for (int DI = -1; DI <= 2; ++DI)
	{
		double WeightX = CubicBSplineKernel(T - DI);
		double DWeightXDX = CubicBSplineKernelDerivative(T - DI);

		for (int DJ = -1; DJ <= 2; ++DJ)
		{
			double WeightY = CubicBSplineKernel(U - DJ);
			double DWeightYDY = CubicBSplineKernelDerivative(U - DJ);

			for (int DK = -1; DK <= 2; ++DK)
			{
				double WeightZ = CubicBSplineKernel(V - DK);
				double DWeightZDZ = CubicBSplineKernelDerivative(V - DK);

				double DWeightDX = DWeightXDX * WeightY * WeightZ;
				double DWeightDY = WeightX * DWeightYDY * WeightZ;
				double DWeightDZ = WeightX * WeightY * DWeightZDZ;

				int i = VertexEmbedding.LatticeCell.X + DI;
				int j = VertexEmbedding.LatticeCell.Y + DJ;
				int k = VertexEmbedding.LatticeCell.Z + DK;

				FVector3d LatticePoint;
				if (i < 0 || i >= Dimensions.X || j < 0 || j >= Dimensions.Y || k < 0 || k >= Dimensions.Z)
				{
					// Get the extrapolated position for a "virtual" control point outside of the deformed lattice
					LatticePoint = ExtrapolatedLatticePosition({ i,j,k }, LatticeControlPoints);
				}
				else
				{
					int PointIndex = ControlPointIndexFromCoordinates(i, j, k);
					LatticePoint = LatticeControlPoints[PointIndex];
				}

				Sum += FMatrix3d(DWeightDX * LatticePoint, DWeightDY * LatticePoint, DWeightDZ * LatticePoint, false);
			}
		}
	}

	return Sum;
}

FVector3d FFFDLattice::ClosestLatticePosition(const FVector3i& VirtualControlPointIndex, const TArray<FVector3d>& LatticeControlPoints) const
{
	// Clamp to valid lattice index
	FVector3i NearestControlPointIndex = Max(Min(VirtualControlPointIndex, Dimensions - 1), FVector3i(0, 0, 0));

	return LatticeControlPoints[ControlPointIndexFromCoordinates(NearestControlPointIndex)];
}

FVector3d FFFDLattice::ExtrapolatedLatticePosition(const FVector3i& VirtualControlPointIndex, const TArray<FVector3d>& LatticeControlPoints) const
{
	// Use the location of the nearest control point and the location of a control point in the opposite direction of 
	// the extrapolation to get the extrapolated location.

	FVector3i NearestControlPointIndex = Max(Min(VirtualControlPointIndex, Dimensions - 1), FVector3i(0, 0, 0));
	FVector3i Delta = VirtualControlPointIndex - NearestControlPointIndex;
	check(Delta != FVector3i::Zero());

	FVector3i TraceBackControlPointIndex = NearestControlPointIndex - Delta;
	check(TraceBackControlPointIndex.X >= 0 && TraceBackControlPointIndex.X < Dimensions.X);
	check(TraceBackControlPointIndex.Y >= 0 && TraceBackControlPointIndex.Y < Dimensions.Y);
	check(TraceBackControlPointIndex.Z >= 0 && TraceBackControlPointIndex.Z < Dimensions.Z);

	const FVector3d& A = LatticeControlPoints[ControlPointIndexFromCoordinates(TraceBackControlPointIndex)];
	const FVector3d& B = LatticeControlPoints[ControlPointIndexFromCoordinates(NearestControlPointIndex)];

	FVector3d Position = B + (B - A);
	return Position;
}
