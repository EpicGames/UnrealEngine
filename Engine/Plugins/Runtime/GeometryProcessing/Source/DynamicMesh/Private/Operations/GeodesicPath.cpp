// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/GeodesicPath.h"
#include "Operations/MeshGeodesicSurfaceTracer.h" 


using namespace UE::Geometry;


// -- FEdgePath methods ---//

int32 FEdgePath::GetHeadSegmentID() const
{
	NodeType* Node = PathLinkList.GetHead();
	if (!Node)
	{
		return InvalidID;
	}

	int32 SID = Node->GetValue().SID;
	return SID;
}

int32 FEdgePath::GetTailSegmentID() const
{
	NodeType* Node = PathLinkList.GetTail();
	if (!Node)
	{
		return InvalidID;
	}

	int32 SID = Node->GetValue().SID;
	return SID;
}

FEdgePath::FDirectedSegment FEdgePath::GetSegment(int32 SID) const
{
	FDirectedSegment Result = { InvalidID, InvalidID };
	if (!IsSegment(SID))
	{
		return Result;
	}
	const FSegmentAndSID& SegmentAndSID = SIDtoNode[SID]->GetValue();
	Result.EID = SegmentAndSID.EID;
	Result.HeadIndex = SegmentAndSID.HeadIndex;
	return Result;
}

int32 FEdgePath::GetPrevSegmentID(int32 SID) const
{
	if (!IsSegment(SID))
	{
		return InvalidID;
	}
	const NodeType* Node = SIDtoNode[SID];
	const NodeType* PrevNode = Node->GetPrevNode();
	if (PrevNode)
	{
		return PrevNode->GetValue().SID;
	}
	else
	{
		return InvalidID;
	}
}

int32 FEdgePath::GetNextSegmentID(int32 SID) const
{
	if (!IsSegment(SID))
	{
		return InvalidID;
	}
	const NodeType* Node = SIDtoNode[SID];
	const NodeType* NextNode = Node->GetNextNode();
	if (NextNode)
	{
		return NextNode->GetValue().SID;
	}
	else
	{
		return InvalidID;
	}
}

int32 FEdgePath::AppendSegment(const FDirectedSegment& Segment)
{
	
	// record the mesh edge as a new segment
	const int32 SID = DirectedSegmentRefCounts.Allocate();
	FSegmentAndSID SegmentAndSID = {{Segment.EID, Segment.HeadIndex}, SID};
	PathLinkList.AddTail(SegmentAndSID);

	NodeType* Node = PathLinkList.GetTail();
	SIDtoNode.InsertAt(Node, SID);

	return SID;
}

int32 FEdgePath::InsertSegmentBefore(const FEdgePath::FDirectedSegment& Segment, int32 SegmentIDToInsertBefore)
{
	NodeType* NodeToInsertBefore = (SegmentIDToInsertBefore > -1) ? SIDtoNode[SegmentIDToInsertBefore] : nullptr;

	int32 SID = DirectedSegmentRefCounts.Allocate();
	FSegmentAndSID SegmentAndSID = {{Segment.EID, Segment.HeadIndex}, SID};
	PathLinkList.InsertNode(SegmentAndSID, NodeToInsertBefore);

	NodeType* Node = (NodeToInsertBefore) ? NodeToInsertBefore->GetPrevNode() : PathLinkList.GetTail();
	SIDtoNode.InsertAt(Node, SID);

	return SID;
}

void FEdgePath::RemoveSegment(int32 SID)
{
	if (!IsSegment(SID))
	{
		return;
	}

	NodeType* Node = SIDtoNode[SID];
	PathLinkList.RemoveNode(Node); // does delete
	SIDtoNode[SID] = nullptr;

	DirectedSegmentRefCounts.Decrement(SID);
}


/**
* Visits a subset of the triangle neighborhood of vertex CVID, 
* i.e. faces in CW or CCW-order about vertex CVID starting with edge StartEID and ending at either a boundary or edge EndEID,
* returns false if a boundary is encountered.
*
* At each face a Visitor functor is called.
*
* @param EdgeFlipMesh - IntrinsicEdgeFlip mesh
* @param StartEID     - edge incident on CVID
* @param EndEID       - edge incident on CVID
* @param CVID         - vertex at center when visiting triangle neighborhood
* @param Visitor      - functor called on each tri between StartEID and EndEID
* @param bClockwise   - specifies the direction of travel around VID as clockwise (CW) or counter clockwise (CCW).
*
* the functor Visitor(int32 TriID, int32 IndexOfCenterVID, FIndex3i& TriEIDs)
*                           TriID             - the triangle visited
*                           IndexOfCenterVID  - the corner index of the specified VID in the triangle
*                           TriEIDs           - edge IDs for the triangle.
*
*/
template <typename EdgeFlipMeshType, typename FunctorType>
bool VisitWedgeTriangles(const EdgeFlipMeshType& EdgeFlipMesh, const int32 StartEID, const int32 EndEID, const int32 CVID,
                         FunctorType& Visitor, bool bClockwise = false)
{
	constexpr static int InvalidID = IndexConstants::InvalidID;

	int32 CurEID = StartEID;

	FIndex2i CurEdgeT   = EdgeFlipMesh.GetEdgeT(CurEID);
	int32 CurTID        = CurEdgeT[0];
	FIndex3i CurTriEIDs = EdgeFlipMesh.GetTriEdges(CurTID);
	int32 IndexOf       = CurTriEIDs.IndexOf(CurEID);

	// make sure we are starting with the triangle on the correct side of the start edge for clockwise / counter clockwise traversal
	if (( bClockwise && (EdgeFlipMesh.GetTriangle(CurTID)[IndexOf] == CVID)) ||
		(!bClockwise && (EdgeFlipMesh.GetTriangle(CurTID)[IndexOf] != CVID)))
	{
		CurTID     = CurEdgeT[1];
		if (CurTID == InvalidID)
		{
			return false;
		}
		CurTriEIDs = EdgeFlipMesh.GetTriEdges(CurTID);
		IndexOf    = CurTriEIDs.IndexOf(CurEID);
	}
	const int32 NextEdgeOffset = (bClockwise) ? 1 : 2;

	do
	{
		Visitor(CurTID, IndexOf, CurTriEIDs);

		// advance to next edge
		CurEID   = CurTriEIDs[(NextEdgeOffset + IndexOf) % 3];
		CurEdgeT = EdgeFlipMesh.GetEdgeT(CurEID);
		CurTID = (CurEdgeT[0] == CurTID) ? CurEdgeT[1] : CurEdgeT[0];

		if (CurTID == InvalidID)
		{
			return (CurEID == EndEID);
		}

		CurTriEIDs = EdgeFlipMesh.GetTriEdges(CurTID);
		IndexOf = CurTriEIDs.IndexOf(CurEID);

	} while (CurEID != EndEID);

	return true;
}

// -- Deformable Edge Path --//


FDeformableEdgePath::FDeformableEdgePath(const FDynamicMesh3& SurfaceMeshIn, const TArray<FEdgePath::FDirectedSegment>& PathAsDirectedSegments)
	: EdgeFlipMesh(SurfaceMeshIn)
	, PathLength(0.)
	, NumFlips(0)
{

	for (const FEdgePath::FDirectedSegment& DirectedSegment : PathAsDirectedSegments)
	{
		const int32 EID = DirectedSegment.EID;
		if (!EdgeFlipMesh.IsEdge(EID))
		{
			continue;
		}
	
		
		// compare with last segment eliminate double-back
		const int32 TailSID = EdgePath.GetTailSegmentID();
		const FEdgePath::FDirectedSegment TailSegment = EdgePath.GetSegment(TailSID);
		
		if (TailSegment.EID == DirectedSegment.EID && TailSegment.HeadIndex != DirectedSegment.HeadIndex)
		{
			EdgePath.RemoveSegment(TailSID);
		}
		else
		{ 
			int32 SID = EdgePath.AppendSegment(DirectedSegment);
			TArray<int32>& Segments = EIDToSIDsMap.FindOrAdd(EID);
			Segments.Add(SID);

			UpdateJointAndQueue(SID);
		}
	}

	// no segments in this path.
	if (EdgePath.NumSegments() == 0)
	{
		return;
	}

	// compute the initial length of the path
	const int32 MaxSID = EdgePath.MaxSegmentID();
	for (int32 SID = EdgePath.GetHeadSegmentID(); SID < MaxSID; ++SID)
	{
		if (!EdgePath.IsSegment(SID))
		{
			continue;
		}
		const int32 EID = EdgePath.GetSegment(SID).EID;
		const double EdgeLength = EdgeFlipMesh.GetEdgeLength(EID);
		PathLength += EdgeLength;
	}
}

void FDeformableEdgePath::Minimize(FDeformableEdgePath::FEdgePathDeformationInfo& DeformedPathInfo, const int32 MaxNumIterations)
{

	DeformedPathInfo = FEdgePathDeformationInfo();
	DeformedPathInfo.OriginalLength = PathLength;

	constexpr double AngleEpsilon = 1.e-3;

	int32 NumIterations = 0;
	while (JointAngleQueue.Num() > 0 && MaxNumIterations > NumIterations)
	{
		// get ID of Outgoing segment in the joint, and the associated smaller internal angle.
		const uint32 QueueSegmentID = JointAngleQueue.Top();
		const double InternalAngle = JointAngleQueue.GetKey(QueueSegmentID);

		JointAngleQueue.Pop();

		if (InternalAngle > TMathUtilConstants<double>::Pi - AngleEpsilon)
		{
			// the smaller internal angle was greater than Pi, this joint can't be further straightened 
			continue;
		}

		const int32 SID = int32(QueueSegmentID);
		FPathJoint& PathJoint = PathJoints[SID];

		if (!IsJointFlexible(PathJoint))
		{
			continue;
		}

		DeformJoint(PathJoint);
		NumIterations++;
	}

	DeformedPathInfo.NumIterations = NumIterations;
	DeformedPathInfo.NumEdgeFlips = NumFlips;
	DeformedPathInfo.FinalLength = PathLength;
}


void FDeformableEdgePath::DeformJoint(FDeformableEdgePath::FPathJoint& PathJoint) 
{
	const int32 JointVID = PathJoint.VID;

	const int32 OutgoingSID = PathJoint.OutgoingSID;
	const int32 IncomingSID = EdgePath.GetPrevSegmentID(OutgoingSID);

	if (IncomingSID < 0 || JointVID == InvalidID) // a joint is composed of incoming and outgoing edges meeting at VID 
	{
		return;
	}

	const int32 IncomingEID = EdgePath.GetSegment(IncomingSID).EID;
	const int32 OutgoingEID = EdgePath.GetSegment(OutgoingSID).EID;

	TArray<FEdgePath::FDirectedSegment> DeformedPath;

	switch (PathJoint.InteriorSide)
	{
		case ESide::Right:
		{
			const bool bClockwise = false;
			DeformJoint(IncomingEID, OutgoingEID, JointVID, DeformedPath, bClockwise);
		}
		break;

		case ESide::Left:
		{
			const bool bClockwise = true;
			DeformJoint(IncomingEID, OutgoingEID, JointVID, DeformedPath, bClockwise);
		}
		break;

		default:
			check(0);
	}

	ReplaceJointWithPath(PathJoint, DeformedPath);
}

bool FDeformableEdgePath::DeformJoint( const int32 IncomingEID, const int32 OutgoingEID, int32 JointVID, 
                                       TArray<FEdgePath::FDirectedSegment>& DeformedPath, const bool bClockwise )
{
	bool Result = OuterArcFlipEdges(IncomingEID, OutgoingEID, JointVID, bClockwise);

	// collect deformed path
	if (Result)
	{
		const int32 HeadOffset = (bClockwise) ? 1 : 2;
		auto PathAccumulator = [this, HeadOffset, &DeformedPath, bClockwise, JointVID](int32 TriID, int32 IndexOfEID, FIndex3i& TriEIDs)
		{
			const int32 IndexOfCenterVID = (bClockwise) ? (IndexOfEID + 1) % 3 : IndexOfEID;
			const int32 OpposingEID = TriEIDs[(IndexOfCenterVID + 1) % 3];
			const FIndex3i TriVIDs = EdgeFlipMesh.GetTriangle(TriID);
			checkSlow(JointVID == TriVIDs[IndexOfCenterVID]);
			const int32 HeadVID = TriVIDs[(IndexOfCenterVID + HeadOffset) % 3];
			const FIndex2i OpposingEdgeV = EdgeFlipMesh.GetEdgeV(OpposingEID);

			FEdgePath::FDirectedSegment DirectedSegment;
			DirectedSegment.EID = OpposingEID;
			DirectedSegment.HeadIndex = (OpposingEdgeV.A == HeadVID) ? 0 : 1;
			checkSlow(OpposingEdgeV[DirectedSegment.HeadIndex] == HeadVID);

			DeformedPath.Add(DirectedSegment);

		};

		Result = VisitWedgeTriangles(EdgeFlipMesh, IncomingEID, OutgoingEID, JointVID, PathAccumulator, bClockwise);
		checkSlow(Result); // since OuterArcFlipEdges was true, this should be too.
	}

	return Result;
}

bool FDeformableEdgePath::IsJointFlexible(const FDeformableEdgePath::FPathJoint& Joint) const
{

	const int32 SID = Joint.OutgoingSID;
	const int32 PrevSID = EdgePath.GetPrevSegmentID(SID);

	if ( SID < 0 || PrevSID < 0)
	{
		return false;
	}

	// Get the Interior Side and JointVID 
	const int32 JointVID     = Joint.VID;
	const ESide InteriorSide = Joint.InteriorSide;


	bool bIsBlocked = false;
	int32 StartEID = EdgePath.GetSegment(SID).EID;
	int32 EndEID   = EdgePath.GetSegment(PrevSID).EID;

	const bool bClockwise = (InteriorSide == ESide::Right);

	auto OccupiedEdgeTestor = [&bIsBlocked, StartEID, EndEID, this](int32 TriID, int32 IndexOfCenterVID, FIndex3i& TriEIDs)
	{

		for (int32 i = 0; i < 3 && !bIsBlocked; ++i)
		{
			const int32 CurEID = TriEIDs[i];
			if (CurEID == StartEID || CurEID == EndEID)
			{
				continue;
			}
			const TArray<int32>* SegmentArray = EIDToSIDsMap.Find(CurEID);
			bIsBlocked = (SegmentArray) || bIsBlocked;
		}
	};

	// check StartEID and EndEID.  These should have one entry each.
	// To prevent the introduction of a path crossing, this  prevents the local joint from being deformed
	// in the case that other segments of the path have doubled back on either of these edges.
	// [todo] when multiple path segments share the same edge, ascertain their relative interleaving (e.g. left to right)
	//        so we can identify situations when the local joint can be deformed without producing a crossing.
	{
		if (const TArray<int32>* SegmentArray = EIDToSIDsMap.Find(StartEID))
		{
			bIsBlocked = bIsBlocked ||(SegmentArray->Num() > 1);
		}
		if (const TArray<int32>* SegmentArray = EIDToSIDsMap.Find(EndEID))
		{
			bIsBlocked = bIsBlocked || (SegmentArray->Num() > 1);
		}
	}

	const bool bHasBoundary = !VisitWedgeTriangles(EdgeFlipMesh, StartEID, EndEID, JointVID,
												   OccupiedEdgeTestor, bClockwise);

	return (!bHasBoundary && !bIsBlocked);
}


void FDeformableEdgePath::UpdateJointAndQueue(int32 SID)
{
	
	int32 PrevSID = EdgePath.GetPrevSegmentID(SID);

	// a joint is composed of cur segment and prev segment
	if (PrevSID != InvalidID)
	{
		const int32 JointVID = SegmentHeadVID(PrevSID);
		checkSlow(JointVID == SegmentTailVID(SID)); 

		const int32 IncomingEID = EdgePath.GetSegment(PrevSID).EID;
		const int32 OutgoingEID = EdgePath.GetSegment(SID).EID;
		

		double LAngle = 0., RAngle = 0.;
		ComputeWedgeAngles(IncomingEID, OutgoingEID, JointVID, LAngle, RAngle);
		const ESide InteriorSide = (LAngle < RAngle) ? ESide::Left : ESide::Right;

		// encode the joint
		FPathJoint PathJoint = { SID, InteriorSide, JointVID };
		PathJoints.InsertAt(PathJoint, SID);


		uint32 QueueSegmentID = uint32(SID);
		double SortKey = FMath::Min(LAngle, RAngle);

		if ( JointAngleQueue.IsPresent(QueueSegmentID) )
		{
			JointAngleQueue.Update(SortKey, QueueSegmentID);
		}
		else
		{
			JointAngleQueue.Add(SortKey, QueueSegmentID);
		}
	}
}

void FDeformableEdgePath::RemoveSegment(int32 SID)
{
	if (SID < 0)
	{
		return;
	}

	const int32 EID = EdgePath.GetSegment(SID).EID;
	JointAngleQueue.Remove(SID);
	EdgePath.RemoveSegment(SID);

	if (TArray<int32>* SegmentArray = EIDToSIDsMap.Find(EID))
	{
		SegmentArray->RemoveSwap(SID);
		if (SegmentArray->Num() == 0)
		{
			EIDToSIDsMap.Remove(EID);
		}
	}

	// subtract length of segment from total path length
	PathLength -= EdgeFlipMesh.GetEdgeLength(EID);

	checkSlow(!EdgePath.IsSegment(SID));
}

void FDeformableEdgePath::ReplaceJointWithPath(const FDeformableEdgePath::FPathJoint PathJoint, const TArray<FEdgePath::FDirectedSegment>& PathAsDirectedEdges)
{
	int32 OutgoingSID = PathJoint.OutgoingSID;
	if (OutgoingSID < 0)
	{
		return;
	}
	// The Joint is comprised of AtoB and BtoC
	int32 IncomingSID = EdgePath.GetPrevSegmentID(OutgoingSID);  

	// segment for first edge element after Joint.
	int32 NextSegmentSID = EdgePath.GetNextSegmentID(OutgoingSID); // Note this can be InvalidID, and that is ok

	// remove the joint
	RemoveSegment(IncomingSID);
	RemoveSegment(OutgoingSID);

	// add path before NextSegment
	if (NextSegmentSID != InvalidID)
	{ 
		for (const FEdgePath::FDirectedSegment& DirectedSegment : PathAsDirectedEdges)
		{
			int32 NewSID = EdgePath.InsertSegmentBefore(DirectedSegment, NextSegmentSID);
			PathLength += EdgeFlipMesh.GetEdgeLength(DirectedSegment.EID);
			UpdateJointAndQueue(NewSID);
		}

		// update the joint that now connects NextEdgeNode to the last NewNode.
		UpdateJointAndQueue(NextSegmentSID);
	}
	else   // add to tail
	{
		for (const FEdgePath::FDirectedSegment& DirectedSegment : PathAsDirectedEdges)
		{
			int32 NewSID = EdgePath.AppendSegment(DirectedSegment);
			PathLength += EdgeFlipMesh.GetEdgeLength(DirectedSegment.EID);
			UpdateJointAndQueue(NewSID);
		}
	}

}

bool FDeformableEdgePath::OuterArcFlipEdges(int32 StartEID, int32 EndEID, int32 CVID, bool bClockwise)
{
	int32 CurEID = StartEID;

	FIndex2i CurEdgeT   = EdgeFlipMesh.GetEdgeT(CurEID);
	int32 CurTID        = CurEdgeT[0];
	FIndex3i CurTriEIDs = EdgeFlipMesh.GetTriEdges(CurTID);
	int32 IndexOf       = CurTriEIDs.IndexOf(CurEID);

	// make sure we are starting with the triangle on the correct side of the start edge for clockwise / counter clockwise traversal
	if (( bClockwise && (EdgeFlipMesh.GetTriangle(CurTID)[IndexOf] == CVID)) ||
		(!bClockwise && (EdgeFlipMesh.GetTriangle(CurTID)[IndexOf] != CVID)))
	{
		CurTID     = CurEdgeT[1];
		if (CurTID == InvalidID)
		{
			return false;
		}
		CurTriEIDs = EdgeFlipMesh.GetTriEdges(CurTID);
		IndexOf    = CurTriEIDs.IndexOf(CurEID);
	}
	const int32 NextEdgeOffset = (bClockwise) ? 1 : 2;

	int32 NextEID = CurTriEIDs[(IndexOf + NextEdgeOffset) % 3];

	while (NextEID != EndEID)
	{
		FIndex2i PreFlipT = EdgeFlipMesh.GetEdgeT(NextEID);
		// The ID of the triangle adjacent to CurEID after flipping NextEID
		// relies on the logic in FIntrinsicEdgeFlipMesh::FlipEdgeTopology().. if that changes this will break
		int32 TmpTID = CurTID;
		if (bClockwise)
		{
			TmpTID = (CurTID == PreFlipT[1]) ? PreFlipT[0] : PreFlipT[1];
		}

		IntrinsicMeshType::FEdgeFlipInfo EdgeFlipInfo;
		UE::Geometry::EMeshResult Result = EdgeFlipMesh.FlipEdge(NextEID, EdgeFlipInfo);

		if (Result == UE::Geometry::EMeshResult::Failed_IsBoundaryEdge)
		{
			return false;
		}

		if (Result == UE::Geometry::EMeshResult::Ok)
		{
			this->NumFlips++;

			CurTID     = TmpTID;
			CurTriEIDs = EdgeFlipMesh.GetTriEdges(CurTID);
			IndexOf    = CurTriEIDs.IndexOf(CurEID);
			checkSlow(IndexOf != -1);  // would only fail if the TmpID logic above is wrong.
			NextEID    = CurTriEIDs[(IndexOf + NextEdgeOffset) % 3];
		}
		else
		{
			// didn't flip edge.  advance to next triangle on other side of that edge
			CurEID     = NextEID;
			CurTID     = (PreFlipT[0] == CurTID) ? PreFlipT[1] : PreFlipT[0];
			CurTriEIDs = EdgeFlipMesh.GetTriEdges(CurTID);
			IndexOf    = CurTriEIDs.IndexOf(CurEID);
			NextEID    = CurTriEIDs[(IndexOf + NextEdgeOffset) % 3];
		}
	}

	return true;
}

void FDeformableEdgePath::ComputeWedgeAngles(int32 IncomingEID, 
                                             int32 OutgoingEID, 
											 int32 CenterVID,
	                                         double& LeftSideAngle, double& RightSideAngle) const
{

	const bool bClockwise = false;

	double WedgeAngle = 0.;
	auto WedgeAngleAccumulator = [this, &WedgeAngle](int32 TID, int32 IndexOf, const FIndex3i& TriEIDs)
	{
		WedgeAngle += EdgeFlipMesh.GetTriInternalAngleR(TID, IndexOf);
	};


	bool bLeftContainsBoundary = !VisitWedgeTriangles(EdgeFlipMesh, OutgoingEID, IncomingEID, CenterVID, WedgeAngleAccumulator, bClockwise);
	double LeftWedgeAngle = WedgeAngle;
	
	WedgeAngle = 0.;
	bool bRightContainsBoundary = !VisitWedgeTriangles(EdgeFlipMesh, IncomingEID, OutgoingEID, CenterVID, WedgeAngleAccumulator, bClockwise);
	double RightWedgeAngle = WedgeAngle;

	LeftSideAngle  = (bLeftContainsBoundary)  ? TMathUtilConstants<double>::MaxReal : LeftWedgeAngle;
	RightSideAngle = (bRightContainsBoundary) ? TMathUtilConstants<double>::MaxReal : RightWedgeAngle;
}

TArray<FDeformableEdgePath::FSurfacePoint> FDeformableEdgePath::AsSurfacePoints(double CoalesceThreshold) const
{
	TArray<FSurfacePoint> PathSurfacePoints;

	const int32 NumIntrinsicPathSegments = EdgePath.NumSegments();
	
	// empty path case
	if (NumIntrinsicPathSegments == 0)
	{
		return PathSurfacePoints;
	}

	IntrinsicMeshType::FEdgeCorrespondence EdgeCorrespondence = EdgeFlipMesh.ComputeEdgeCorrespondence();

	int32 SID = EdgePath.GetHeadSegmentID();
	while (SID != InvalidID)
	{ 
	
		const int32 NextSID = EdgePath.GetNextSegmentID(SID);
		const bool bIsLastSegment = (NextSID == InvalidID);
		
		const int32 EID = EdgePath.GetSegment(SID).EID;
		const bool bReverseEdge = (EdgePath.GetSegment(SID).HeadIndex == 0);

		// get intrinsic edge as a sequence of surface points, note each segment starts and ends at a surface mesh vertex but may cross several surface mesh edges.
		TArray<FSurfacePoint> SegmentSurfacePoints = EdgeCorrespondence.TraceEdge(EID, CoalesceThreshold, bReverseEdge);

		PathSurfacePoints.Append(MoveTemp(SegmentSurfacePoints));
		if (!bIsLastSegment)
		{
			// delete last element since it will be the same as the first element in the next segment
			PathSurfacePoints.Pop(); 
		}
		
		SID = NextSID;
	}
	
	return MoveTemp(PathSurfacePoints);
}

double UE::Geometry::SumPathLength(const FDeformableEdgePath& DeformableEdgePath)
{
	double TotalPathLength = 0;
	const FEdgePath& EdgePath = DeformableEdgePath.GetEdgePath();
	const FDeformableEdgePath::IntrinsicMeshType& EdgeFlipMesh = DeformableEdgePath.GetIntrinsicMesh();

	const int32 NumIntrinsicPathSegments = EdgePath.NumSegments();

	// empty path case
	if (NumIntrinsicPathSegments == 0)
	{
		return TotalPathLength;
	}

	int32 SID = EdgePath.GetHeadSegmentID();
	while (SID != FDeformableEdgePath::InvalidID)
	{
		const int32 EID = EdgePath.GetSegment(SID).EID;
		const double SegmentLength = EdgeFlipMesh.GetEdgeLength(EID);
		TotalPathLength += SegmentLength;
			
		SID = EdgePath.GetNextSegmentID(SID);
	}

	return TotalPathLength;
}