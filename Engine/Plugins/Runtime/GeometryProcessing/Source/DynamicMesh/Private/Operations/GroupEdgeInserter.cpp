// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/GroupEdgeInserter.h"

#include "Algo/ForEach.h"
#include "CompGeom/PolygonTriangulation.h"
#include "ConstrainedDelaunay2.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMeshEditor.h"
#include "FrameTypes.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "MeshRegionBoundaryLoops.h"
#include "Operations/GroupEdgeInserter.h"
#include "Operations/MeshPlaneCut.h"
#include "Operations/EmbedSurfacePath.h"
#include "Operations/SimpleHoleFiller.h"
#include "Operations/LocalPlanarSimplify.h"
#include "Selections/MeshConnectedComponents.h"
#include "Util/ProgressCancel.h"
#include "Util/IndexUtil.h"

using namespace UE::Geometry;

// Forward declarations of local helper functions. Normally these would be marked as static or
// in an anonymous namespace, but apparently this could still result in clashes due to unity builds.
namespace GroupEdgeInserterLocals {

bool GetEdgeLoopOpposingEdgeAndCorner(const FGroupTopology& Topology, int32 GroupID, int32 GroupEdgeIDIn,
	int32 CornerIDIn, int32& GroupEdgeIDOut, int32& CornerIDOut, int32& BoundaryIndexOut, 
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut);
bool InsertEdgeLoopEdgesInDirection(const FGroupEdgeInserter::FEdgeLoopInsertionParams& Params,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& StartEndpoints,
	int32 NextGroupID, int32 NextEdgeID, int32 NextCornerID, int32 NextBoundaryID,
	TSet<int32>& AlteredGroups, int32& NumInserted, FGroupEdgeInserter::FOptionalOutputParams& OptionalOut,
	FProgressCancel* Progress);
bool InsertNewVertexEndpoints(
	const FGroupEdgeInserter::FEdgeLoopInsertionParams& Params,
	int32 GroupEdgeID, int32 StartCornerID,
	TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& EndPointsOut, 
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut);
void ConvertProportionsToArcLengths(
	const FGroupTopology& Topology, int32 GroupEdgeID,
	const TArray<double>& ProportionsIn,
	TArray<double>& ArcLengthsOut, TArray<double>* PerVertexLengthsOut);
bool ConnectEndpoints(
	const FGroupEdgeInserter::FEdgeLoopInsertionParams& Params, int32 GroupID,
	const FGroupTopology::FGroupBoundary& GroupBoundary,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& StartPoints,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& EndPoints,
	int32& NumGroupsCreated, FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, 
	FProgressCancel* Progress);
bool ConnectMultipleUsingRetriangulation(
	FDynamicMesh3& Mesh, const FGroupTopology& Topology, int32 GroupID,
	const FGroupTopology::FGroupBoundary& GroupBoundary,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& StartPoints,
	const TArray <FGroupEdgeInserter::FGroupEdgeSplitPoint>& EndPoints,
	int32& NumGroupsCreated, FGroupEdgeInserter::FOptionalOutputParams& OptionalOut,
	FProgressCancel* Progress);
bool DeleteGroupTrianglesAndGetLoop(FDynamicMesh3& Mesh, const FGroupTopology& Topology, int32 GroupID,
	const FGroupTopology::FGroupBoundary& GroupBoundary, TArray<int32>& BoundaryVerticesOut,
	TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>>& BoundaryVidUVMapsOut, 
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut,
	FProgressCancel* Progress);
void AppendInclusiveRangeWrapAround(const TArray<int32>& InputArray, TArray<int32>& OutputArray,
	int32 StartIndex, int32 InclusiveEndIndex);
bool RetriangulateLoop(FDynamicMesh3& Mesh, const TArray<int32>& LoopVertices,
	int32 NewGroupID, TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>>& VidUVMaps);

bool ConnectMultipleUsingPlaneCut(FDynamicMesh3& Mesh,
	const FGroupTopology& Topology, int32 GroupID,
	const FGroupTopology::FGroupBoundary& GroupBoundary,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& StartPoints,
	const TArray <FGroupEdgeInserter::FGroupEdgeSplitPoint>& EndPoints,
	double VertexTolerance, bool bSimplifyAlongPath, int32& NumGroupsCreated,
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, FProgressCancel* Progress);
bool EmbedPlaneCutPath(FDynamicMesh3& Mesh, const FGroupTopology& Topology, int32 GroupID,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& StartPoint,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& EndPoint,
	double VertexTolerance, bool bSimplifyAlongPath, TSet<int32>& PathEidsOut, 
	TSet<int32>* ChangedTrisOut, FProgressCancel* Progress);
bool CreateNewGroups(FDynamicMesh3& Mesh, TSet<int32>& PathEids, int32 OriginalGroupID, int32& NumGroupsCreated, 
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, FProgressCancel* Progress);
bool GetPlaneCutPath(const FDynamicMesh3& Mesh, int32 GroupID,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& StartPoint, const FGroupEdgeInserter::FGroupEdgeSplitPoint& EndPoint,
	TArray<TPair<FMeshSurfacePoint, int>>& OutputPath, double VertexCutTolerance, 
	const TSet<int32>& DisallowedVids, FProgressCancel* Progress);

bool InsertSingleWithRetriangulation(FDynamicMesh3& Mesh, FGroupTopology& Topology,
	int32 GroupID, int32 BoundaryIndex,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& StartPoint,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& EndPoint,
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, FProgressCancel* Progress);
}

/** Inserts an edge loop into a mesh, where an edge loop is a sequence of (group) edges across quads. */
bool FGroupEdgeInserter::InsertEdgeLoops(const FEdgeLoopInsertionParams& Params, FOptionalOutputParams OptionalOut, FProgressCancel* Progress)
{
	using namespace GroupEdgeInserterLocals;

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Validate the inputs
	check(Params.Mesh);
	check(Params.Topology);
	check(Params.SortedInputLengths);
	check(Params.GroupEdgeID != FDynamicMesh3::InvalidID);
	check(Params.StartCornerID != FDynamicMesh3::InvalidID);

	const FGroupTopology::FGroupEdge& GroupEdge = Params.Topology->Edges[Params.GroupEdgeID];

	// We check whether we have a valid path forward or backward first, because we don't want
	// to do any edge splits if we have neither.
	int32 ForwardGroupID = GroupEdge.Groups.A;
	int32 ForwardEdgeID, ForwardCornerID, ForwardBoundaryIndex;
	bool bHaveForwardEdge = GetEdgeLoopOpposingEdgeAndCorner(*Params.Topology, ForwardGroupID,
		Params.GroupEdgeID, Params.StartCornerID, ForwardEdgeID, ForwardCornerID, ForwardBoundaryIndex, OptionalOut);

	int32 BackwardGroupID = GroupEdge.Groups.B;
	int32 BackwardEdgeID, BackwardCornerID, BackwardBoundaryIndex;
	bool bHaveBackwardEdge = GetEdgeLoopOpposingEdgeAndCorner(*Params.Topology, BackwardGroupID,
		Params.GroupEdgeID, Params.StartCornerID, BackwardEdgeID, BackwardCornerID, BackwardBoundaryIndex, OptionalOut);

	if (!bHaveForwardEdge && !bHaveBackwardEdge)
	{
		// Neither of the neighbors is quad-like, so we can't insert an edge loop at this edge.
		return false;
	}

	// Depending on the topology, it is possible for our loop to attempt to cross itself from the side. We
	// could support this case, because the "loop" will still eventually end. However, this isn't a particularly 
	// useful feature, so for now, we'll keep things a cleaner by ending the loop if we arrive at a group that
	// we've already altered. This also allows us to avoid updating the topology as we go along.
	TSet<int32> AlteredGroups;

	// We will need to keep the first endpoints around in case we use them to close the loop.
	TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint> StartEndpoints;

	// Although we have code that can insert new edges at edge endpoints, it is cleaner to do splits for all the loops
	// down an edge ahead of time to make vertex endpoints, in part because if we don't, a split can change the eid
	// of the next endpoint.
	bool bSuccess = InsertNewVertexEndpoints(Params, Params.GroupEdgeID, Params.StartCornerID, StartEndpoints, OptionalOut);

	if (!bSuccess || StartEndpoints.Num() == 0 || (Progress && Progress->Cancelled()))
	{
		return false;
	}

	// Insert edges in both directions. In case of a loop, the second call won't do anything because
	// AlteredGroups will be updated.
	int32 TotalNumInserted = 0;
	if (bHaveForwardEdge)
	{
		bSuccess = InsertEdgeLoopEdgesInDirection(Params, StartEndpoints, ForwardGroupID, ForwardEdgeID,
			ForwardCornerID, ForwardBoundaryIndex, AlteredGroups, TotalNumInserted, OptionalOut, Progress);
	}
	if (bSuccess && bHaveBackwardEdge)
	{
		// TODO: The behavior here is not ideal in that if we fail to insert edges across a group we fail
		// the entire insertion. For instance, in a curved staircase, where the bottom is a C-shape but otherwise
		// a quad, we may fail to perform a plane cut, but we should still insert edges on the steps, where it
		// doesn't fail. Unfortunately, being tolerant of insertion failures requires us to be able to revert
		// the failed group to its original state on failure. We need to add support for that.

		int32 NumInserted = 0;
		bSuccess = InsertEdgeLoopEdgesInDirection(Params, StartEndpoints, BackwardGroupID, BackwardEdgeID,
			BackwardCornerID, BackwardBoundaryIndex, AlteredGroups, NumInserted, OptionalOut, Progress) && bSuccess;
		TotalNumInserted += NumInserted;
	}

	if (TotalNumInserted == 0 || (Progress && Progress->Cancelled()))
	{
		return false;
	}

	return Params.Topology->RebuildTopology() && bSuccess;
}

namespace GroupEdgeInserterLocals {

/**
 * Given a group edge and the (adjoining) quad-like group across which we want to continue an edge loop,
 * finds the id of the opposite (i.e., destination) group edge. Additionally, gives the corner ID  attached
 * to the provided one. Safe to call with an FDynamicMesh3::InvalidID parameters (will return false)
 *
 * If OptionalOut.ProblemGroupEdgeIDsOut is not null, will add the boundaries of a non-quad component to
 * that set if the component turns out not to be valid due to not being quad-like.
 * 
 * @returns true if a satisfactory edge was found.
 */
bool GetEdgeLoopOpposingEdgeAndCorner(const FGroupTopology& Topology, int32 GroupID, int32 GroupEdgeIDIn,
	int32 CornerIDIn, int32& GroupEdgeIDOut, int32& CornerIDOut, int32& BoundaryIndexOut, 
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut)
{
	GroupEdgeIDOut = FDynamicMesh3::InvalidID;
	CornerIDOut = FDynamicMesh3::InvalidID;
	BoundaryIndexOut = FDynamicMesh3::InvalidID;
	if (GroupEdgeIDIn == FDynamicMesh3::InvalidID || GroupID == FDynamicMesh3::InvalidID)
	{
		return false;
	}

	const FGroupTopology::FGroup* Group = Topology.FindGroupByID(GroupID);
	check(Group);

	for (int32 i = 0; i < Group->Boundaries.Num(); ++i)
	{
		const FGroupTopology::FGroupBoundary& Boundary = Group->Boundaries[i];
		int32 GroupEdgeIndex = Boundary.GroupEdges.IndexOfByKey(GroupEdgeIDIn);
		if (GroupEdgeIndex != INDEX_NONE)
		{
			if (Boundary.GroupEdges.Num() != 4)
			{
				if (OptionalOut.ProblemGroupEdgeIDsOut)
				{
					OptionalOut.ProblemGroupEdgeIDsOut->Append(Boundary.GroupEdges);
				}
				return false;
			}

			GroupEdgeIDOut = Boundary.GroupEdges[(GroupEdgeIndex + 2) % 4];
			BoundaryIndexOut = i;

			// Get the corner attached to the one we were given
			if (CornerIDIn != FDynamicMesh3::InvalidID)
			{
				FGroupTopology::FGroupEdge SideEdge1 = Topology.Edges[Boundary.GroupEdges[(GroupEdgeIndex + 1) % 4]];
				FGroupTopology::FGroupEdge SideEdge2 = Topology.Edges[Boundary.GroupEdges[(GroupEdgeIndex + 3) % 4]];
				if (SideEdge1.EndpointCorners.A == CornerIDIn)
				{
					CornerIDOut = SideEdge1.EndpointCorners.B;
				}
				else if (SideEdge1.EndpointCorners.B == CornerIDIn)
				{
					CornerIDOut = SideEdge1.EndpointCorners.A;
				}
				else if (SideEdge2.EndpointCorners.A == CornerIDIn)
				{
					CornerIDOut = SideEdge2.EndpointCorners.B;
				}
				else if (SideEdge2.EndpointCorners.B == CornerIDIn)
				{
					CornerIDOut = SideEdge2.EndpointCorners.A;
				}
			}

			return true;
		}
	}
	return false;
}

/**
 * Helper function, continues the loop in one direction from a start edge.
 * @returns false if there is an error.
 */
bool InsertEdgeLoopEdgesInDirection(const FGroupEdgeInserter::FEdgeLoopInsertionParams& Params,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& StartEndpoints,
	int32 NextGroupID, int32 NextEdgeID, int32 NextCornerID, int32 NextBoundaryIndex,
	TSet<int32>& AlteredGroups, int32& NumInserted, FGroupEdgeInserter::FOptionalOutputParams& OptionalOut,
	FProgressCancel* Progress)
{
	NumInserted = 0;
	if (AlteredGroups.Contains(NextGroupID) || StartEndpoints.Num() == 0)
	{
		return true;
	}

	// We keep endpoints in two arrays and swap the current one as we move along
	TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint> EndpointStorage1 = StartEndpoints;
	TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint> EndpointStorage2;
	TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>* CurrentEndpoints = &EndpointStorage1;
	TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>* NextEndpoints = &EndpointStorage2;

	bool bHaveNextGroup = true;
	bool bSuccess = true;
	while (bHaveNextGroup && !AlteredGroups.Contains(NextGroupID))
	{
		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		const FGroupTopology::FGroup* CurrentGroup = Params.Topology->FindGroupByID(NextGroupID);
		check(CurrentGroup);
		const FGroupTopology::FGroupBoundary& Boundary = CurrentGroup->Boundaries[NextBoundaryIndex];

		// See if we looped around to the start
		if (NextEdgeID == Params.GroupEdgeID)
		{
			int32 NumGroupsCreated;
			bSuccess = ConnectEndpoints(Params, NextGroupID, Boundary, *CurrentEndpoints, StartEndpoints, 
				NumGroupsCreated, OptionalOut, Progress);
			AlteredGroups.Add(NextGroupID);
			NumInserted += (NumGroupsCreated > 1 ? 1 : 0);
			break;
		}

		// Otherwise, create next endpoints
		bSuccess = InsertNewVertexEndpoints(Params, NextEdgeID, NextCornerID, *NextEndpoints, OptionalOut);
		if (!bSuccess || (Progress && Progress->Cancelled()))
		{
			return false;
		}
		if (NextEndpoints->Num() == 0)
		{
			// Next edge wasn't long enough for the input lengths we wanted. Stop here.
			// TODO: Actually we now clamp to max length. Should we?
			return true;
		}

		// Connect up the endpoints
		int32 NumGroupsCreated;
		bSuccess = ConnectEndpoints(Params, NextGroupID, Boundary, *CurrentEndpoints, *NextEndpoints, 
			NumGroupsCreated, OptionalOut, Progress);
		AlteredGroups.Add(NextGroupID);
		NumInserted += (NumGroupsCreated > 1 ? 1 : 0);

		if (!bSuccess || (Progress && Progress->Cancelled()))
		{
			return false;
		}

		// Get the next group edge target
		if (Params.Topology->IsBoundaryEdge(NextEdgeID))
		{
			break;
		}
		NextGroupID = Params.Topology->Edges[NextEdgeID].OtherGroupID(NextGroupID);
		bHaveNextGroup = GetEdgeLoopOpposingEdgeAndCorner(*Params.Topology, NextGroupID, NextEdgeID, NextCornerID,
			NextEdgeID, NextCornerID, NextBoundaryIndex, OptionalOut); // outputs

		Swap(CurrentEndpoints, NextEndpoints);
	}
	return bSuccess;
}

/**
 * Inserts vertices along an existing group edge that will be used as endpoints
 * for new group edges.

 * Note that due to tolerance, multiple inputs can map to the same vertex. We
 * want to keep this functionality because in the case of proportion inputs, the
 * snapping will partly depend on the narrowness of an edge, and we still want to
 * allow connection to adjacent edges.
 *
 * Clears EndPointsOut before use.
 */
bool InsertNewVertexEndpoints(
	const FGroupEdgeInserter::FEdgeLoopInsertionParams& Params,
	int32 GroupEdgeID, int32 StartCornerID,
	TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& EndPointsOut,
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut)
{
	EndPointsOut.Reset();
	if (Params.SortedInputLengths->Num() == 0)
	{
		return false;
	}

	const FGroupTopology::FGroupEdge& GroupEdge = Params.Topology->Edges[GroupEdgeID];

	// Prep the list of vertex ids and the corresponding cumulative lengths. It is easier
	// to make our own copies because we may need to iterate backwards relative to the
	// order in the topology.
	bool bGoBackward = (GroupEdge.Span.Vertices.Last() == Params.Topology->GetCornerVertexID(StartCornerID));
	TArray<int32> SpanVids;
	if (!bGoBackward)
	{
		SpanVids = GroupEdge.Span.Vertices;
	}
	else
	{
		for (int i = GroupEdge.Span.Vertices.Num() - 1; i >= 0; --i)
		{
			SpanVids.Add(GroupEdge.Span.Vertices[i]);
		}
	}

	TArray<double> PerVertexLengths;
	TArray<double> ArcLengths;
	if (Params.bInputsAreProportions)
	{
		ConvertProportionsToArcLengths(*Params.Topology, GroupEdgeID,
			*Params.SortedInputLengths, ArcLengths, &PerVertexLengths);
	}
	else
	{
		ArcLengths = *Params.SortedInputLengths;
		Params.Topology->GetEdgeArcLength(GroupEdgeID, &PerVertexLengths); // Get per vertex lengths
	}
	
	double TotalLength = PerVertexLengths.Last();
	if (bGoBackward)
	{
		// Reverse order and update lengths to be TotalLength-length. Could do in one pass but then
		// don't forget to modify the middle.
		Algo::Reverse(PerVertexLengths);
		Algo::ForEach(PerVertexLengths, [TotalLength](double& Length) {
			Length = TotalLength - Length;
			});
	}

	// We're going to walk forward selecting existing vertices or adding new ones as we go along.
	// CurrentVid and CurrentArcLength may take on values that are not in SpanVids or PerVertexLengths
	// as we insert new vertices. NextIndex, however is always an index into those two structures
	// of the next vertex in front of the current one.
	int32 CurrentVid = SpanVids[0];
	double CurrentArcLength = 0;
	int32 NextIndex = 1;

	for (double TargetLength : ArcLengths)
	{
		// If the next target is beyond the last vertex, clamp it to the last vertex
		if (TargetLength > TotalLength + Params.VertexTolerance)
		{
			TargetLength = TotalLength;
		}

		// Advance until the next vertex would overshoot the target length.
		while (NextIndex < PerVertexLengths.Num() && PerVertexLengths[NextIndex] <= TargetLength)
		{
			CurrentVid = SpanVids[NextIndex];
			CurrentArcLength = PerVertexLengths[NextIndex];
			++NextIndex;
		}

		// The point is now either at the current vertex, or on the edge going forward (if there is one)
		FGroupEdgeInserter::FGroupEdgeSplitPoint SplitPoint;

		// Used if we find ourselves needing to snap to one of the endpoints of the edge
		auto SetSplitPointToVertex = [&SplitPoint, &SpanVids, &Params, NextIndex](int32 Vid)
		{
			SplitPoint.ElementID = Vid;
			SplitPoint.bIsVertex = true;
			
			// Get the tangent vector. We haven't been keeping track of any previous verts we inserted,
			// but inserted verts must be on an edge and must have the forward edge as their tangent.
			bool bVertexIsOriginal = (Vid == SpanVids[NextIndex - 1]);
			if (!bVertexIsOriginal)
			{
				SplitPoint.Tangent = Normalized(Params.Mesh->GetVertex(SpanVids[NextIndex]) - Params.Mesh->GetVertex(Vid));
			}
			else
			{
				FVector3d VertexPosition = Params.Mesh->GetVertex(Vid);
				SplitPoint.Tangent = FVector3d::Zero();
				if (NextIndex > 1)
				{
					SplitPoint.Tangent += Normalized(VertexPosition - Params.Mesh->GetVertex(SpanVids[NextIndex - 2]));
				}
				if (NextIndex < SpanVids.Num())
				{
					SplitPoint.Tangent += Normalized(Params.Mesh->GetVertex(SpanVids[NextIndex]) - VertexPosition);
				}
				Normalize(SplitPoint.Tangent);
			}
		};

		// See if the target is at the current vertex
		if (TargetLength - CurrentArcLength <= Params.VertexTolerance)
		{
			SetSplitPointToVertex(CurrentVid);
		}
		else if (NextIndex < PerVertexLengths.Num()
			&& PerVertexLengths[NextIndex] - CurrentArcLength <= Params.VertexTolerance)
		{
			SetSplitPointToVertex(SpanVids[NextIndex]);
		}
		else
		{
			// Target must be on the edge that goes to the next vertex
			int32 CurrentEid = Params.Mesh->FindEdge(CurrentVid, SpanVids[NextIndex]);

			if (!ensure(CurrentEid >= 0))
			{
				// We shouldn't end up here, but if we do, it could be because something failed and
				// stopped loop progress in one direction despite having performed edge splits forward,
				// and now we have arrived at the split edge from the other direction, not realizing
				// that SpanVids has changed
				return false;
			}

			double SplitT = (TargetLength - CurrentArcLength) / (PerVertexLengths[NextIndex] - CurrentArcLength);

			// See if the edge is stored backwards relative to the direction we're traveling
			if (Params.Mesh->GetEdge(CurrentEid).Vert.B != SpanVids[NextIndex])
			{
				SplitT = 1 - SplitT;
			}

			// Perform the split
			FDynamicMesh3::FEdgeSplitInfo EdgeSplitInfo;
			Params.Mesh->SplitEdge(CurrentEid, EdgeSplitInfo, SplitT);

			// Record the changed triangles
			if (OptionalOut.ChangedTidsOut)
			{
				OptionalOut.ChangedTidsOut->Add(EdgeSplitInfo.OriginalTriangles.A);
				if (EdgeSplitInfo.OriginalTriangles.B != FDynamicMesh3::InvalidID)
				{
					OptionalOut.ChangedTidsOut->Add(EdgeSplitInfo.OriginalTriangles.B);
				}
			}

			// Update our position given that we're at a new vertex.
			CurrentVid = EdgeSplitInfo.NewVertex;
			CurrentArcLength = TargetLength;

			// Assemble the actual output
			SplitPoint.ElementID = CurrentVid;
			SplitPoint.bIsVertex = true; // we made the vertex that is this target
			SplitPoint.Tangent = Normalized(Params.Mesh->GetVertex(SpanVids[NextIndex]) - Params.Mesh->GetVertex(CurrentVid));
		}

		EndPointsOut.Add(SplitPoint);
	}//end going through targets

	return true;
}

void ConvertProportionsToArcLengths(
	const FGroupTopology& Topology, int32 GroupEdgeID,
	const TArray<double>& ProportionsIn,
	TArray<double>& ArcLengthsOut, TArray<double>* PerVertexLengthsOut)
{
	ArcLengthsOut.Reset(ProportionsIn.Num());
	double TotalLength = Topology.GetEdgeArcLength(GroupEdgeID, PerVertexLengthsOut);
	for (double Proportion : ProportionsIn)
	{
		ArcLengthsOut.Add(Proportion * TotalLength);
	}
}

/**
 * Helper function, assumes that StartPoints and EndPoints are all vertices and connects them.
 * See ConnectMultipleUsingRetriangulation for details.
 *
 * @param GroupID Group across which to make the connections.
 * @param GroupBoundary Boundary of the group across which the connections are being made.
 */
bool ConnectEndpoints(
	const FGroupEdgeInserter::FEdgeLoopInsertionParams& Params, int32 GroupID,
	const FGroupTopology::FGroupBoundary& GroupBoundary,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& StartPoints,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& EndPoints,
	int32& NumGroupsCreated, FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, 
	FProgressCancel* Progress)
{
	if (Params.Mode == FGroupEdgeInserter::EInsertionMode::Retriangulate)
	{
		return ConnectMultipleUsingRetriangulation(*Params.Mesh, *Params.Topology, GroupID, 
			GroupBoundary, StartPoints, EndPoints, NumGroupsCreated, OptionalOut, Progress);
	}
	else if (Params.Mode == FGroupEdgeInserter::EInsertionMode::PlaneCut)
	{
		return ConnectMultipleUsingPlaneCut(*Params.Mesh, *Params.Topology, GroupID,
			GroupBoundary, StartPoints, EndPoints, Params.VertexTolerance, Params.bSimplifyAlongPath,
			NumGroupsCreated, OptionalOut, Progress);
	}
	else
	{
		checkf(false, TEXT("GroupEdgeInserter:ConnectEndpoints: Unimplemented insertion method."));
	}
	return false;
}

/**
 * Connects multiple endpoints across the same group, making the assumption that StartPoints and
 * EndPoints are composed only of vertex endpoints, are 1:1 in their arrays and are ordered
 * sequentially away from the first startpoint and first endpoint, which have no endpoints between
 * them. So the arrangement is something like this (may be mirrored):
	 *---*
	 |   |
	 s0-e0
	 |   |
	 s1-e1
	 |   |
	 *---*
 * The assumption is used to order the generated loops.
 * The function exists because it is more efficient than generating each edge one at a
 * time in a multi-loop case.
 * @returns false if there's an error.
 */
bool ConnectMultipleUsingRetriangulation(
	FDynamicMesh3& Mesh,
	const FGroupTopology& Topology,
	int32 GroupID,
	const FGroupTopology::FGroupBoundary& GroupBoundary,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& StartPoints,
	const TArray <FGroupEdgeInserter::FGroupEdgeSplitPoint>& EndPoints,
	int32& NumGroupsCreated, FGroupEdgeInserter::FOptionalOutputParams& OptionalOut,
	FProgressCancel* Progress)
{
	NumGroupsCreated = 0;
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	int32 NumNewEdges = FMath::Min(StartPoints.Num(), EndPoints.Num());
	if (NumNewEdges == 0)
	{
		return true; // Nothing to do.
	}

	TArray<int32> BoundaryVertices;
	TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>> VidUVMaps;
	bool bSuccess = DeleteGroupTrianglesAndGetLoop(Mesh, Topology, GroupID, GroupBoundary, 
		BoundaryVertices, VidUVMaps, OptionalOut, Progress);

	if (!bSuccess || (Progress && Progress->Cancelled()))
	{
		return false;
	}

	// Convert endpoint arrays to arrays of indices into the boundary vertex array.
	TArray<int32> StartIndices;
	TArray<int32> EndIndices;
	for (int32 i = 0; i < NumNewEdges; ++i)
	{
		check(StartPoints[i].bIsVertex && EndPoints[i].bIsVertex);
		
		int32 StartIndex = BoundaryVertices.Find(StartPoints[i].ElementID);
		int32 EndIndex = BoundaryVertices.Find(EndPoints[i].ElementID);
		check(StartIndex != INDEX_NONE && EndIndex != INDEX_NONE);

		StartIndices.Add(StartIndex);
		EndIndices.Add(EndIndex);
	}

	// We don't know which way the vertices are ordered relative to the counterclockwise ordering
	// of the original group. If we were to go ccw from the first start vertex, we would expect
	// to reach the second start vertex before the first end vertex. If we reach the end vertex
	// first, then the diagram in the function header is flipped.
	bool bReverseSubloopDirection = NumNewEdges > 1 &&
		(StartIndices[1] - StartIndices[0] + BoundaryVertices.Num()) % BoundaryVertices.Num() // ccw distance from start
				> (EndIndices[0] - StartIndices[0] + BoundaryVertices.Num()) % BoundaryVertices.Num();// ccw distance from start

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Due to snapping and such, we may end up with degenerate loops that don't need triangulation. 
	// This could be the first loop, so we need to assign the existing group ID to the first loop 
	// that is not degenerate.
	bool bUsedOriginalGroup = false;

	// Do the first loop
	TArray<int32> LoopVids;
	if (!bReverseSubloopDirection)
	{
		AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, EndIndices[0], StartIndices[0]);
	}
	else
	{
		AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, StartIndices[0], EndIndices[0]);
	}
	if (LoopVids.Num() > 2)
	{
		bSuccess = RetriangulateLoop(Mesh, LoopVids, GroupID, VidUVMaps);
		bUsedOriginalGroup = true;
		NumGroupsCreated += (bSuccess ? 1 : 0);
	}

	// Do the middle loops
	for (int32 i = 1; i < NumNewEdges; ++i)
	{
		if (!bSuccess || (Progress && Progress->Cancelled()))
		{
			return false;
		}

		// Check for a degenerate loop
		if (StartIndices[i - 1] == StartIndices[i] && EndIndices[i - 1] == EndIndices[i])
		{
			continue;
		}

		LoopVids.Reset();
		if (!bReverseSubloopDirection)
		{
			AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, StartIndices[i - 1], StartIndices[i]); // previous to current
			AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, EndIndices[i], EndIndices[i - 1]); // current to previous
		}
		else
		{
			AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, StartIndices[i], StartIndices[i - 1]); // current to previous
			AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, EndIndices[i - 1], EndIndices[i]); // previous to current
		}

		int32 GroupIDToUse = bUsedOriginalGroup ? Mesh.AllocateTriangleGroup() : GroupID;
		bSuccess = RetriangulateLoop(Mesh, LoopVids, GroupIDToUse, VidUVMaps);
		bUsedOriginalGroup = true;
		NumGroupsCreated += (bSuccess ? 1 : 0);
	}

	if (!bSuccess || (Progress && Progress->Cancelled()))
	{
		return false;
	}

	// Do the last loop
	LoopVids.Reset();
	if (!bReverseSubloopDirection)
	{
		AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, StartIndices.Last(), EndIndices.Last());
	}
	else
	{
		AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, EndIndices.Last(), StartIndices.Last());
	}
	if (LoopVids.Num() > 2)
	{
		int32 GroupIDToUse = bUsedOriginalGroup ? Mesh.AllocateTriangleGroup() : GroupID;
		bSuccess = RetriangulateLoop(Mesh, LoopVids, GroupIDToUse, VidUVMaps);
		bUsedOriginalGroup = true;
		NumGroupsCreated += (bSuccess ? 1 : 0);
	}

	if (OptionalOut.NewEidsOut)
	{
		for (int32 i = 0; i < NumNewEdges; ++i)
		{
			OptionalOut.NewEidsOut->Add(Mesh.FindEdge(StartPoints[i].ElementID, EndPoints[i].ElementID));
		}
	}

	return bSuccess;
}

/**
 * Helper function, deletes triangles in a group connected component and outputs the corresponding boundary.
 * Does not delete the vertices.
 */
bool DeleteGroupTrianglesAndGetLoop(FDynamicMesh3& Mesh, const FGroupTopology& Topology, int32 GroupID,
	const FGroupTopology::FGroupBoundary& GroupBoundary, TArray<int32>& BoundaryVerticesOut,
	TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>>& BoundaryVidUVMapsOut, 
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Since groups may not be contiguous, we have to do a connected component search
	// rather than deleting all triangles marked with that group, so get the seeds for the search.
	int32 FirstEid = Topology.Edges[GroupBoundary.GroupEdges[0]].Span.Edges[0];
	FIndex2i PotentialSeedTriangles = Mesh.GetEdge(FirstEid).Tri;
	TArray<int32> SeedTriangles;
	if (Mesh.GetTriangleGroup(PotentialSeedTriangles.A) == GroupID)
	{
		SeedTriangles.Add(PotentialSeedTriangles.A);
	}
	else
	{
		check(PotentialSeedTriangles.B != FDynamicMesh3::InvalidID && Mesh.GetTriangleGroup(PotentialSeedTriangles.B) == GroupID);
		SeedTriangles.Add(PotentialSeedTriangles.B);
	}

	FMeshConnectedComponents ConnectedComponents(&Mesh);
	ConnectedComponents.FindTrianglesConnectedToSeeds(SeedTriangles, [&](int32 t0, int32 t1) {
		return Mesh.GetTriangleGroup(t0) == Mesh.GetTriangleGroup(t1);
		});

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	FMeshConnectedComponents::FComponent& Component = ConnectedComponents.GetComponent(0);

	// Get the boundary loop
	FMeshRegionBoundaryLoops RegionLoops(&Mesh, Component.Indices, true);
	if (RegionLoops.bFailed || RegionLoops.Loops.Num() != 1)
	{
		// We don't support components with multiple boundaries (like a single cylinder side) because
		// group edge insertion only works in very limited circumstances here (for instance, connecting
		// multiple boundaries generally can't be done with a single group edge, since the group will
		// remain connected), and retriangulation would be a huge pain.
		return false;
	}
	RegionLoops.Loops[0].Reverse();
	BoundaryVerticesOut = RegionLoops.Loops[0].Vertices;


	if (Mesh.HasAttributes())
	{
		const FDynamicMeshAttributeSet* Attributes = Mesh.Attributes();
		for (int i = 0; i < Attributes->NumUVLayers(); ++i)
		{
			BoundaryVidUVMapsOut.Emplace();
			RegionLoops.GetLoopOverlayMap(RegionLoops.Loops[0], 
				*Mesh.Attributes()->GetUVLayer(i), BoundaryVidUVMapsOut.Last());
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	if (OptionalOut.ChangedTidsOut)
	{
		OptionalOut.ChangedTidsOut->Append(Component.Indices);
	}

	// When deleting, we don't we don't want to remove isolated verts on the boundary,
	// but we do want to remove isolated verts on the interior of the component. We 
	// could finish the retriangulation and look for isolated verts afterwards, but
	// that requires us to keep track of the old verts until we're done triangulating.
	// Instead, we'll just go ahead and delete any old verts not on the boundary.

	// Get all verts in the component, and the verts on the boundary
	TArray<int32> ComponentVids;
	UE::Geometry::TriangleToVertexIDs(&Mesh, Component.Indices, ComponentVids);
	TSet<int32> BoundaryVidSet(RegionLoops.Loops[0].Vertices);

	// Delete the triangles
	FDynamicMeshEditor Editor(&Mesh);
	Editor.RemoveTriangles(Component.Indices, false); // don't remove isolated verts
	
	// Remove verts that weren't on the boundary
	Algo::ForEachIf(ComponentVids, 
		[&BoundaryVidSet](int32 Vid) 
	{ 
		return !BoundaryVidSet.Contains(Vid); 
	},
		[&Mesh](int32 Vid) 
	{ 
		checkSlow(!Mesh.IsReferencedVertex(Vid));
		constexpr bool bPreserveManifold = false;
		Mesh.RemoveVertex(Vid, bPreserveManifold);
	}
	);

	if (Mesh.HasAttributes())
	{
		const FDynamicMeshAttributeSet* Attributes = Mesh.Attributes();
		for (int i = 0; i < Attributes->NumUVLayers(); ++i)
		{
			RegionLoops.UpdateLoopOverlayMapValidity(BoundaryVidUVMapsOut[i],
				*Mesh.Attributes()->GetUVLayer(i));
		}
	}

	return true;
}

/**
 * Appends entries from an input array from a start index to end index (inclusive), wrapping around
 * at the end.
 */
void AppendInclusiveRangeWrapAround(const TArray<int32>& InputArray, TArray<int32>& OutputArray,
	int32 StartIndex, int32 InclusiveEndIndex)
{
	check(InclusiveEndIndex >= 0 && InclusiveEndIndex < InputArray.Num()
		&& StartIndex >= 0 && StartIndex < InputArray.Num());

	int32 CurrentIndex = StartIndex;
	while (CurrentIndex != InclusiveEndIndex)
	{
		OutputArray.Add(InputArray[CurrentIndex]);
		CurrentIndex = (CurrentIndex + 1) % InputArray.Num();
	}
	OutputArray.Add(InputArray[InclusiveEndIndex]);
}

bool RetriangulateLoop(FDynamicMesh3& Mesh,
	const TArray<int32>& LoopVertices, int32 NewGroupID,
	TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>>& VidUVMaps)
{
	TArray<int32> LoopEdges;
	FEdgeLoop::VertexLoopToEdgeLoop(&Mesh, LoopVertices, LoopEdges);
	FEdgeLoop Loop(&Mesh, LoopVertices, LoopEdges);
	FSimpleHoleFiller HoleFiller(&Mesh, Loop, FSimpleHoleFiller::EFillType::PolygonEarClipping);
	if (!HoleFiller.Fill(NewGroupID))
	{
		// If the hole filler failed, it is probably because two non-adjacent vertices in the loop
		// already have an edge between them ("on the other side" of this group), so inserting
		// two more triangles between the verts is not possible while keeping things manifold. 
		// As an example, imagine a cube with one face deleted but then an obtuse triangle added
		// to one of the boundary edges to make one cube side an irregular pentagon.

		// Granted, perhaps the user shouldn't be trying to be doing simple retriangulation in a 
		// situation like this, but let's do something reasonable anyway. We'll retriangulate with
		// an added center vert, since this will only add to the edges that we know are boundaries.

		// Delete any triangles we already added
		FDynamicMeshEditor Editor(&Mesh);
		Editor.RemoveTriangles(HoleFiller.NewTriangles, false); // don't remove isolated verts
		
		// Change the hole fill type
		HoleFiller.FillType = FSimpleHoleFiller::EFillType::TriangleFan;
		if (!HoleFiller.Fill(NewGroupID))
		{
			// Not sure how this could happen... Did we not delete the interior triangles somehow
			// or give the wrong loop?
			return ensure(false);
		}
	}

	if (Mesh.HasAttributes())
	{
		if (!HoleFiller.UpdateAttributes(VidUVMaps))
		{
			return false;
		}
	}

	return true;
}

/**
 * See ConnectMultipleUsingRetriangulation for details.
 */
bool ConnectMultipleUsingPlaneCut(FDynamicMesh3& Mesh, 
	const FGroupTopology& Topology, int32 GroupID,
	const FGroupTopology::FGroupBoundary& GroupBoundary,
	const TArray<FGroupEdgeInserter::FGroupEdgeSplitPoint>& StartPoints,
	const TArray <FGroupEdgeInserter::FGroupEdgeSplitPoint>& EndPoints,
	double VertexTolerance, bool bSimplifyAlongPath, int32& NumGroupsCreated,
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, 
	FProgressCancel* Progress)
{
	NumGroupsCreated = 0;
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	int32 NumEdgesToInsert = FMath::Min(StartPoints.Num(), EndPoints.Num());
	TSet<int32> PathsEids;
	for (int32 i = 0; i < NumEdgesToInsert; ++i)
	{
		bool bSuccess = EmbedPlaneCutPath(Mesh, Topology, GroupID, StartPoints[i], EndPoints[i],
			VertexTolerance, bSimplifyAlongPath, PathsEids, OptionalOut.ChangedTidsOut, Progress);

		if (!bSuccess || (Progress && Progress->Cancelled()))
		{
			return false;
		}
	}

	if (OptionalOut.NewEidsOut)
	{
		*OptionalOut.NewEidsOut = OptionalOut.NewEidsOut->Union(PathsEids);
	}

	bool bSuccess = CreateNewGroups(Mesh, PathsEids, GroupID, NumGroupsCreated, OptionalOut, Progress);
	return bSuccess;
}

/**
 * Places a plane path connecting the endpoints into the mesh, but does not give the triangles new groups yet.
 * However, outputs the path edge ID's so that can be done later.
 */
bool EmbedPlaneCutPath(FDynamicMesh3& Mesh, const FGroupTopology& Topology,
	int32 GroupID, const FGroupEdgeInserter::FGroupEdgeSplitPoint& StartPoint,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& EndPoint,
	double VertexTolerance, bool bSimplifyAlongPath, TSet<int32>& PathEidsOut, 
	TSet<int32>* ChangedTrisOut, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// We don't allow snapping to any of the boundary vertices via plane distance because this can lead
	// to a situation where we snap via plane cut but not via absolute distance (depending on plane
	// orientation relative boundary), which results in us arriving at the boundary at a nearby vertex
	// and then walking along the boundary to the destination. Aside from looking/behaving weirdly, 
	// this is very bad for edge loops, where doing so can join the edge at a different point from the
	// one at which it continues on the other side, which means that quad-like topology gets
	// inadvertantly broken without the user realizing why.
	TSet<int32> DisallowedVids;
	const FGroupTopology::FGroup* Group = Topology.FindGroupByID(GroupID);
	if (ensure(Group))
	{
		for (const FGroupTopology::FGroupBoundary& Boundary : Group->Boundaries)
		{
			for (int32 GroupEdgeID : Boundary.GroupEdges)
			{
				if (ensure(GroupEdgeID < Topology.Edges.Num()))
				{
					DisallowedVids.Append(Topology.Edges[GroupEdgeID].Span.Vertices);
				}
			}
		}
	}
	

	// Find the path we're going to take.
	TArray<TPair<FMeshSurfacePoint, int>> CutPath;
	bool bSuccess = GetPlaneCutPath(Mesh, GroupID, StartPoint, EndPoint,
		CutPath, VertexTolerance, DisallowedVids, Progress);
	if (!bSuccess || (Progress && Progress->Cancelled()))
	{
		return false;
	}
	check(CutPath.Num() >= 2);

	// Save triangles that will change
	if (ChangedTrisOut)
	{
		for (int32 i = 0; i < CutPath.Num(); ++i)
		{
			FMeshSurfacePoint& Point = CutPath[i].Key;
			if (Point.PointType == ESurfacePointType::Edge)
			{
				FIndex2i EdgeTris = Mesh.GetEdgeT(Point.ElementID);
				ChangedTrisOut->Add(EdgeTris.A);
				if (EdgeTris.B != FDynamicMesh3::InvalidID)
				{
					ChangedTrisOut->Add(EdgeTris.B);
				}
			}
		}
	}

	// Embed the path.
	FMeshSurfacePath PathEmbedder(&Mesh);
	PathEmbedder.Path = CutPath;
	TArray<int32> PathVertices;
	bSuccess = PathEmbedder.EmbedSimplePath(false, PathVertices, false);
	if (!bSuccess || (Progress && Progress->Cancelled()))
	{
		return false;
	}
	check(PathVertices.Num() >= 2);

	for (int32 i = 1; i < PathVertices.Num(); ++i)
	{
		int32 Eid = Mesh.FindEdge(PathVertices[i - 1], PathVertices[i]);
		if (ensure(Eid >= 0))
		{
			PathEidsOut.Add(Eid);
		}
	}
	
	if (bSimplifyAlongPath)
	{
		FLocalPlanarSimplify Simplify;
		// Note: Vertex normals are not reliable here, so preserving them will just prevent simplifications that are fine ...
		Simplify.bPreserveVertexNormals = false;
		// Note: Any triangles that this simplify can remove should already be in ChangedTrisOut, so we shouldn't need to update that array here
		Simplify.SimplifyAlongEdges(Mesh, PathEidsOut);
	}

	return true;
}

/**
 * Uses the given path edge ID's to split a group into new groups.
 */
bool CreateNewGroups(FDynamicMesh3& Mesh, TSet<int32>& PathEids, int32 OriginalGroup, int32& NumGroupsCreated, 
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, FProgressCancel* Progress)
{
	NumGroupsCreated = 0;

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Create the new groups.
	TSet<int32> SeedTriangleSet;
	for (int32 Eid : PathEids)
	{
		FIndex2i Tris = Mesh.GetEdgeT(Eid);
		if (Mesh.GetTriangleGroup(Tris.A) == OriginalGroup)
		{
			SeedTriangleSet.Add(Tris.A);
		}
		if (Tris.B != FDynamicMesh3::InvalidID && Mesh.GetTriangleGroup(Tris.B) == OriginalGroup)
		{
			SeedTriangleSet.Add(Tris.B);
		}
	}

	FMeshConnectedComponents ConnectedComponents(&Mesh);
	ConnectedComponents.FindTrianglesConnectedToSeeds(SeedTriangleSet.Array(), [&](int32 t0, int32 t1) {

		// Triangles are connected only if they have the same group and are not across one of the
		// newly inserted group edges.
		int32 Group0 = Mesh.GetTriangleGroup(t0);
		int32 Group1 = Mesh.GetTriangleGroup(t1);
		if (Group0 == Group1)
		{
			int32 SharedEdge = Mesh.FindEdgeFromTriPair(t0, t1);
			return !PathEids.Contains(SharedEdge);
		}
		return false;
		});

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Assign a new group id for each component. The first component keeps the old group ID.
	for (int32 i = 1; i < ConnectedComponents.Num(); ++i)
	{
		FMeshConnectedComponents::FComponent& Component = ConnectedComponents.GetComponent(i);

		if (OptionalOut.ChangedTidsOut)
		{
			OptionalOut.ChangedTidsOut->Append(Component.Indices);
		}

		int32 NewGroupID = Mesh.AllocateTriangleGroup();
		for (int Tid : Component.Indices)
		{
			Mesh.SetTriangleGroup(Tid, NewGroupID);
		}
	}

	NumGroupsCreated = ConnectedComponents.Num();

	return true;
}
}//end namespace GroupEdgeInserterLocals

/** Inserts a group edge into a given group. */
bool FGroupEdgeInserter::InsertGroupEdge(FGroupEdgeInsertionParams& Params, FGroupEdgeInserter::FOptionalOutputParams OptionalOut, FProgressCancel* Progress)
{
	using namespace GroupEdgeInserterLocals;

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Validate the inputs
	check(Params.Mesh);
	check(Params.Topology);
	check(Params.GroupID != FDynamicMesh3::InvalidID);
	check(Params.StartPoint.ElementID != FDynamicMesh3::InvalidID);
	check(Params.EndPoint.ElementID != FDynamicMesh3::InvalidID);

	if (Params.StartPoint.bIsVertex == Params.EndPoint.bIsVertex
		&& Params.StartPoint.ElementID == Params.EndPoint.ElementID)
	{
		// Points are on same vertex or edge.
		return false;
	}

	if (Params.Mode == EInsertionMode::PlaneCut)
	{
		TSet<int32> TempNewEids;
		TSet<int32>* NewEids = OptionalOut.NewEidsOut ? OptionalOut.NewEidsOut : &TempNewEids;

		bool bSuccess = EmbedPlaneCutPath(*Params.Mesh, *Params.Topology, Params.GroupID, Params.StartPoint,
			Params.EndPoint, Params.VertexTolerance, Params.bSimplifyAlongPath, *NewEids, OptionalOut.ChangedTidsOut, Progress);
		if (!bSuccess || (Progress && Progress->Cancelled()))
		{
			return false;
		}
		
		int32 NumGroupsCreated;
		bSuccess = CreateNewGroups(*Params.Mesh, *NewEids, Params.GroupID, NumGroupsCreated, OptionalOut, Progress);
		if (!bSuccess || (Progress && Progress->Cancelled()))
		{
			return false;
		}
	}
	else if (Params.Mode == EInsertionMode::Retriangulate)
	{
		bool bSuccess = InsertSingleWithRetriangulation(*Params.Mesh, *Params.Topology, 
			Params.GroupID, Params.GroupBoundaryIndex, Params.StartPoint, Params.EndPoint, 
			OptionalOut, Progress);
		if (!bSuccess || (Progress && Progress->Cancelled()))
		{
			return false;
		}
	}
	else
	{
		checkf(false, TEXT("GroupEdgeInserter:InsertGroupEdge: Unimplemented insertion method."));
	}

	Params.Topology->RebuildTopology();

	return true;
}

namespace GroupEdgeInserterLocals {

/** Helper function. Not used when inserting multiple edges at once into a group to avoid continuously retriangulating and deleting. */
bool InsertSingleWithRetriangulation(FDynamicMesh3& Mesh, FGroupTopology& Topology, 
	int32 GroupID, int32 BoundaryIndex,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& StartPoint,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& EndPoint, 
	FGroupEdgeInserter::FOptionalOutputParams& OptionalOut, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	if (StartPoint.bIsVertex == EndPoint.bIsVertex
		&& StartPoint.ElementID == EndPoint.ElementID)
	{
		// Points are on same vertex or edge.
		return false;
	}

	// Function we use to split the endpoints if needed
	auto AddVertOnEdge = [](FDynamicMesh3& Mesh, int32 Eid, double EdgeTValue, TSet<int32>* ChangedTidsOut) {
		FDynamicMesh3::FEdgeSplitInfo SplitInfo;
		Mesh.SplitEdge(Eid, SplitInfo, EdgeTValue);

		// Record the changed triangles
		if (ChangedTidsOut)
		{
			ChangedTidsOut->Add(SplitInfo.OriginalTriangles.A);
			if (SplitInfo.OriginalTriangles.B != FDynamicMesh3::InvalidID)
			{
				ChangedTidsOut->Add(SplitInfo.OriginalTriangles.B);
			}
		}
		return SplitInfo.NewVertex;
	};

	int32 StartVid = StartPoint.ElementID;
	if (!StartPoint.bIsVertex)
	{
		StartVid = AddVertOnEdge(Mesh, StartPoint.ElementID, StartPoint.EdgeTValue, OptionalOut.ChangedTidsOut);
	}

	int32 EndVid = EndPoint.ElementID;
	if (!EndPoint.bIsVertex)
	{
		EndVid = AddVertOnEdge(Mesh, EndPoint.ElementID, EndPoint.EdgeTValue, OptionalOut.ChangedTidsOut);
	}

	const FGroupTopology::FGroup* Group = Topology.FindGroupByID(GroupID);
	check(Group && BoundaryIndex >= 0 && BoundaryIndex < Group->Boundaries.Num());
	TArray<int32> BoundaryVertices;
	TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>> VidUVMaps;
	bool bSuccess = DeleteGroupTrianglesAndGetLoop(Mesh, Topology, GroupID, Group->Boundaries[BoundaryIndex], 
		BoundaryVertices, VidUVMaps, OptionalOut, Progress);
	if (!bSuccess || (Progress && Progress->Cancelled()))
	{
		return false;
	}

	int32 IndexA = BoundaryVertices.IndexOfByKey(StartVid);
	int32 IndexB = BoundaryVertices.IndexOfByKey(EndVid);

	TArray<int32> LoopVids;
	AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, IndexA, IndexB);
	if (LoopVids.Num() < 3)
	{
		// If one our endpoints turn out to be adjacent, there's nothing to insert.
		// TODO: we could do a tiny bit more work to detect this earlier.
		return false;
	}
	bSuccess = RetriangulateLoop(Mesh, LoopVids, GroupID, VidUVMaps);

	if (!bSuccess || (Progress && Progress->Cancelled()))
	{
		return false;
	}

	LoopVids.Reset();
	AppendInclusiveRangeWrapAround(BoundaryVertices, LoopVids, IndexB, IndexA);
	if (LoopVids.Num() < 3)
	{
		return false;
	}
	bSuccess = RetriangulateLoop(Mesh, LoopVids, Mesh.AllocateTriangleGroup(), VidUVMaps);

	if (OptionalOut.NewEidsOut)
	{
		OptionalOut.NewEidsOut->Add(Mesh.FindEdge(StartVid, EndVid));
	}

	return bSuccess;
}


/**
 * Creates a path of FMeshSurfacePoint instances across a group that can be embedded into the mesh,
 * based on a plane cut from start to end. Does not actually embed that path yet.
 *
 * Assumes that the start and end points are on the boundary of the group, and doesn't try to deal
 * with some complicated edge cases that could arise in nonplanar groups.
 *
 * Instead of having this function, we should modify EmbedSurfacePath.cpp::WalkMeshPlanar to allow the start and
 * end points to be edges/vertices and to have a filter function that we can use to filter out triangles that are
 * not in the group we want.
 *
 * @returns false if path could not be found.
 */
bool GetPlaneCutPath(const FDynamicMesh3& Mesh, int32 GroupID,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& StartPoint, const FGroupEdgeInserter::FGroupEdgeSplitPoint& EndPoint,
	TArray<TPair<FMeshSurfacePoint, int>>& OutputPath, double VertexCutTolerance, 
	const TSet<int32>& DisallowedVids, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Used to make sure we don't end up walking in a loop or backwards. This could happen with a high vertex cut
	// tolerance and pathological topology via vertices. It shouldn't be possible via edges for the cases where
	// we use this function (between points of a contiguous group boundary), but we'll be safe.
	TSet<int32> CrossedVids;
	TSet<int32> CrossedEids;


	// Start by determining the plane we will use. 
	FVector3d StartPosition = StartPoint.bIsVertex ? Mesh.GetVertex(StartPoint.ElementID)
		: Mesh.GetEdgePoint(StartPoint.ElementID, StartPoint.EdgeTValue);
	FVector3d EndPosition = EndPoint.bIsVertex ? Mesh.GetVertex(EndPoint.ElementID)
		: Mesh.GetEdgePoint(EndPoint.ElementID, EndPoint.EdgeTValue);

	FVector3d InPlaneVector = Normalized(EndPosition - StartPosition);

	// Get components of the two tangents that are orthogonal to the vector between the points.
	FVector3d NormalA = Normalized(StartPoint.Tangent - StartPoint.Tangent.Dot(InPlaneVector) * InPlaneVector, 
		static_cast<double>(KINDA_SMALL_NUMBER));
	FVector3d NormalB = Normalized(EndPoint.Tangent - EndPoint.Tangent.Dot(InPlaneVector) * InPlaneVector,
		static_cast<double>(KINDA_SMALL_NUMBER));

	if (NormalA.IsZero() || NormalB.IsZero())
	{
		// One or both of the tangents were pointing directly toward the destination, and in such
		// cases the cutting plane we fit is likely to be nonsense (along the edge of a cube,
		// for instance, we may end up trying to use a plane roughly coplanar with the face we're
		// trying to "cut").
		return false;
	}

	// Make the vectors be in the same half space so that their average represents the closer average of the
	// corresponding lines.
	if (NormalA.Dot(NormalB) < 0)
	{
		NormalB = -NormalB;
	}

	// Do the averaging
	FVector CutPlaneNormal = (FVector)Normalized(NormalA + NormalB);
	if (CutPlaneNormal.IsZero())
	{
		// This shouldn't happen because we already checked for both of them being zero, and we made them be in
		// the same half space.
		return ensure(false);
	}

	FVector CutPlaneOrigin = (FVector)StartPosition;

	// These store distances of the current edge from the plane, so they don't have to be recomputed when
	// finding the next point.
	float CurrentEdgeVertPlaneDistances[2];

	// Prep the first point.
	OutputPath.Empty();
	if (StartPoint.bIsVertex)
	{
		OutputPath.Emplace(FMeshSurfacePoint(StartPoint.ElementID), FDynamicMesh3::InvalidID);
	}
	else
	{
		// Note that we do not clamp the endpoints in this function because clamping based
		// on plane distance can result in different behavior depending on which way the plane is oriented, which
		// could end up clamping to different endpoints as we connect multiple paths through the same start/end
		// point (such as when following a loop)
		FIndex2i EdgeVids = Mesh.GetEdgeV(StartPoint.ElementID);
		CurrentEdgeVertPlaneDistances[0] = (float)FVector::PointPlaneDist((FVector)Mesh.GetVertex(EdgeVids.A), CutPlaneOrigin, CutPlaneNormal);
		CurrentEdgeVertPlaneDistances[1] = (float)FVector::PointPlaneDist((FVector)Mesh.GetVertex(EdgeVids.B), CutPlaneOrigin, CutPlaneNormal);

			OutputPath.Emplace(FMeshSurfacePoint::MakeEdgePoint(StartPoint.ElementID, StartPoint.EdgeTValue), FDynamicMesh3::InvalidID);
		}
	check(OutputPath.Num() == 1);

	// Set up a few more variables we'll need as we walk from start to end
	bool bCurrentPointIsVertex = (OutputPath[0].Key.PointType == ESurfacePointType::Vertex);
	int32 CurrentElementID = OutputPath[0].Key.ElementID;

	int32 PointCount = 1; // Used as a sanity check

	// Tracks which triangle (if not an edge) we traversed to get to the next point, so we can avoid
	// backtracking in the next step.
	int32 TraversedTid = FDynamicMesh3::InvalidID;

	// Do the walk. The exit condition is that the last point of our output path has the same ID and type
	// as our endpoint.
	while (!(CurrentElementID == EndPoint.ElementID && bCurrentPointIsVertex == EndPoint.bIsVertex))
	{
		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		// Prevent ourselves from going in a loop
		if (bCurrentPointIsVertex)
		{
			if (CrossedVids.Contains(CurrentElementID))
			{
				return false;
			}
			CrossedVids.Add(CurrentElementID);
		}
		else
		{
			if (CrossedEids.Contains(CurrentElementID))
			{
				return false;
			}
			CrossedEids.Add(CurrentElementID);
		}

		check(PointCount < Mesh.EdgeCount()); // sanity check to avoid infinite loop

		if (bCurrentPointIsVertex)
		{
			FMeshSurfacePoint NextPoint(FDynamicMesh3::InvalidID);
			FVector3d CurrentPosition = OutputPath.Last().Key.Pos(&Mesh);

			// Look through the surrounding triangles that have the group we want and find one that intersects the plane
			int32 CandidateTraversedTid = FDynamicMesh3::InvalidID;
			for (int32 Tid : Mesh.VtxTrianglesItr(CurrentElementID))
			{
				if (Tid == TraversedTid || Mesh.GetTriangleGroup(Tid) != GroupID)
				{
					continue;
				}

				// See if one of the triangle edges has the endpoint, in which case we can go straight there.
				if (!EndPoint.bIsVertex)
				{
					const FIndex3i& TriangleEids = Mesh.GetTriEdges(Tid);
					for (int32 i = 0; i < 3; ++i)
					{
						if (EndPoint.ElementID == TriangleEids[i])
						{
							// Got to the end
							OutputPath.Emplace(FMeshSurfacePoint::MakeEdgePoint(EndPoint.ElementID, EndPoint.EdgeTValue), FDynamicMesh3::InvalidID);
							return true;
						}
					}
				}

				// Check to see if one of the triangle vertices is the end
				const FIndex3i& TriangleVids = Mesh.GetTriangle(Tid);
				int32 VertA = (TriangleVids.A == CurrentElementID) ? TriangleVids.C : TriangleVids.A;
				int32 VertB = (TriangleVids.B == CurrentElementID) ? TriangleVids.C : TriangleVids.B;
				if (EndPoint.bIsVertex && (EndPoint.ElementID == VertA || EndPoint.ElementID == VertB))
				{
					OutputPath.Emplace(FMeshSurfacePoint(EndPoint.ElementID), FDynamicMesh3::InvalidID);
					return true;
				}

				// See if one of the other vertices is on the plane (and is therefore the next destination)
				float PlaneDistanceA = (float)FVector::PointPlaneDist((FVector)Mesh.GetVertex(VertA), CutPlaneOrigin, CutPlaneNormal);
				float PlaneDistanceB = (float)FVector::PointPlaneDist((FVector)Mesh.GetVertex(VertB), CutPlaneOrigin, CutPlaneNormal);
				bool bVertAIsOnPlane = abs(PlaneDistanceA) <= VertexCutTolerance;
				bool bVertBIsOnPlane = abs(PlaneDistanceB) <= VertexCutTolerance;

				auto UpdateNextPoint = [&InPlaneVector, &Mesh, &CurrentPosition, &NextPoint](const FMeshSurfacePoint& CandidateSurfacePoint)
				{
					if (NextPoint.ElementID == CandidateSurfacePoint.ElementID && NextPoint.PointType == CandidateSurfacePoint.PointType)
					{
						// We're looking at the same point from an adjacent triangle
						return false;
					}

					// Update the next point if the candidate moves more directly toward the destination. 
					// TOOD: This hack is necessary to deal with some common ambiguous cases (such as starting from a 
					// vertex in the concave region of a nonconvex planar surface), but the proper solution is to alter 
					// and use EmbedSurfacePath.cpp::WalkMeshPlanar
					if (NextPoint.ElementID == FDynamicMesh3::InvalidID
						|| (InPlaneVector.Dot(NextPoint.Pos(&Mesh) - CurrentPosition) < 
							InPlaneVector.Dot(CandidateSurfacePoint.Pos(&Mesh) - CurrentPosition)))
					{
						NextPoint = CandidateSurfacePoint;
						return true;
					}
					return false;
				};

				bool bEdgeVertIsPreferred = false;
				if (bVertAIsOnPlane && !DisallowedVids.Contains(VertA))
				{
					bEdgeVertIsPreferred = true;
					UpdateNextPoint(FMeshSurfacePoint(VertA));
				}
				if (bVertBIsOnPlane && !DisallowedVids.Contains(VertB))
				{
					bEdgeVertIsPreferred = true;
					UpdateNextPoint(FMeshSurfacePoint(VertB));
				}
				if (!bEdgeVertIsPreferred && PlaneDistanceA * PlaneDistanceB < 0)
				{
					// The triangle's opposite edge crosses the plane, and the edge verts are not
					// valid snap targets
					int32 Eid = Mesh.FindEdgeFromTri(VertA, VertB, Tid);

					double EdgeTValue = PlaneDistanceA / (PlaneDistanceA - PlaneDistanceB);

					if (VertA != Mesh.GetEdgeV(Eid).A)
					{
						EdgeTValue = 1 - EdgeTValue;
					}

					if (UpdateNextPoint(FMeshSurfacePoint::MakeEdgePoint(Eid, EdgeTValue)))
					{
						CurrentEdgeVertPlaneDistances[0] = PlaneDistanceA;
						CurrentEdgeVertPlaneDistances[1] = PlaneDistanceB;
						if (VertA != Mesh.GetEdgeV(Eid).A)
						{
							Swap(CurrentEdgeVertPlaneDistances[0], CurrentEdgeVertPlaneDistances[1]);
						}
						CandidateTraversedTid = Tid;
					}
				}
			}//end going through triangles

			// Make sure we found the next point.
			if (NextPoint.ElementID == FDynamicMesh3::InvalidID)
			{
				return false;
			}
			else
			{
				OutputPath.Emplace(NextPoint, FDynamicMesh3::InvalidID);

				TraversedTid = (NextPoint.PointType == ESurfacePointType::Edge) ? CandidateTraversedTid
					: FDynamicMesh3::InvalidID;
			}
		}//end if at vert
		else
		{
			const FDynamicMesh3::FEdge& Edge = Mesh.GetEdge(CurrentElementID);

			// We're starting from an edge. Get the triangle that we're dealing with.
			int32 NextTid;
			if (Edge.Tri.A == TraversedTid)
			{
				NextTid = Edge.Tri.B;
			}
			else if (Edge.Tri.B == TraversedTid)
			{
				NextTid = Edge.Tri.A;
			}
			else
			{
				NextTid = Mesh.GetTriangleGroup(Edge.Tri.A) == GroupID ? Edge.Tri.A : Edge.Tri.B;
			}

			if (NextTid == FDynamicMesh3::InvalidID || Mesh.GetTriangleGroup(NextTid) != GroupID)
			{
				// We hit a dead end before getting to the end.
				return false;
			}
			TraversedTid = NextTid;

			// See if the opposite vert is our endpoint.
			int32 OppositeVert = IndexUtil::FindTriOtherVtx(Edge.Vert.A, Edge.Vert.B, Mesh.GetTriangle(NextTid));
			if (EndPoint.bIsVertex && EndPoint.ElementID == OppositeVert)
			{
				OutputPath.Emplace(FMeshSurfacePoint(EndPoint.ElementID), FDynamicMesh3::InvalidID);
				return true;
			}

			// See if one of the triangle edges has the endpoint, in which case we can go straight there.
			if (!EndPoint.bIsVertex)
			{
				const FIndex3i& TriangleEids = Mesh.GetTriEdges(NextTid);
				for (int32 i = 0; i < 3; ++i)
				{
					if (EndPoint.ElementID == TriangleEids[i])
					{
						// Got to the end
						OutputPath.Emplace(FMeshSurfacePoint::MakeEdgePoint(EndPoint.ElementID, EndPoint.EdgeTValue), FDynamicMesh3::InvalidID);
						return true;
					}
				}
			}

			// We'll keep going. Get the placement of the opposite vert relative to the plane.
			float OppositeVertPlaneDistance = (float)FVector::PointPlaneDist((FVector)Mesh.GetVertex(OppositeVert), CutPlaneOrigin, CutPlaneNormal);
			if (abs(OppositeVertPlaneDistance) <= VertexCutTolerance && !DisallowedVids.Contains(OppositeVert))
			{
				// We are cutting through a vertex
				OutputPath.Emplace(FMeshSurfacePoint(OppositeVert), FDynamicMesh3::InvalidID);
			}
			else
			{
				// We are cutting through an edge. Figure out which one
				int32 SecondVertOfNextEdge;
				float SecondPlaneDistance;
				if (CurrentEdgeVertPlaneDistances[0] * OppositeVertPlaneDistance < 0)
				{
					SecondVertOfNextEdge = Edge.Vert.A;
					SecondPlaneDistance = CurrentEdgeVertPlaneDistances[0];
				}
				else if (CurrentEdgeVertPlaneDistances[1] * OppositeVertPlaneDistance < 0)
				{
					SecondVertOfNextEdge = Edge.Vert.B;
					SecondPlaneDistance = CurrentEdgeVertPlaneDistances[1];
				}
				else
				{
					// For neither of the edge vertices to be on the opposite side of the plane from
					// the opposite vertex, the edge must lie in the plane. This can only happen if
					// we started on an edge and then fit a bad cutting plane (otherwise we would have
					// snapped to a vertex instead). This should usually not happen since we check the
					// tangents, but we use a different type of check/tolerance there, and it could 
					// still happen if endpoints weren't snapped for some reason and the edge is super short.
					return false;
				}

				// Add the edge point to output
				int32 Eid = Mesh.FindEdge(OppositeVert, SecondVertOfNextEdge);

				// The division here is safe because SecondPlaneDistance is guaranteed to have an opposite sign to
				// OppositeVertPlaneDistance by the if-else statements above.
				double EdgeTValue = OppositeVertPlaneDistance / (OppositeVertPlaneDistance - SecondPlaneDistance);

				CurrentEdgeVertPlaneDistances[0] = OppositeVertPlaneDistance;
				CurrentEdgeVertPlaneDistances[1] = SecondPlaneDistance;

				if (OppositeVert != Mesh.GetEdgeV(Eid).A)
				{
					EdgeTValue = 1 - EdgeTValue;
					Swap(CurrentEdgeVertPlaneDistances[0], CurrentEdgeVertPlaneDistances[1]);
				}

				OutputPath.Emplace(FMeshSurfacePoint::MakeEdgePoint(Eid, EdgeTValue), FDynamicMesh3::InvalidID);
			}//end cutting through edge
		}//end if last point was edge

		++PointCount;
		check(PointCount == OutputPath.Num()) // Another sanity check, to make sure we're always advancing

		CurrentElementID = OutputPath.Last().Key.ElementID;
		bCurrentPointIsVertex = (OutputPath.Last().Key.PointType == ESurfacePointType::Vertex);
	}//end until we get to end

	return true;
}
}//end namespace GroupEdgeInserterLocals
