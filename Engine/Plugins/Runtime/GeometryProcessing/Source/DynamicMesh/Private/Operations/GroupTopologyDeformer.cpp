// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/GroupTopologyDeformer.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GroupTopology.h"
#include "SegmentTypes.h"
#include "Async/ParallelFor.h"
#include "Containers/BitArray.h"
#include "DynamicMesh/MeshNormals.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"

using namespace UE::Geometry;

void FGroupTopologyDeformer::Initialize(const FDynamicMesh3* MeshIn, const FGroupTopology* TopologyIn)
{
	Mesh = MeshIn;
	Topology = TopologyIn;
}



void FGroupTopologyDeformer::Reset()
{
	HandleVertices.Reset();
	HandleBoundaryVertices.Reset();
	FixedBoundaryVertices.Reset();
	ROIEdgeVertices.Reset();
	ROIFaces.Reset();
	InitialPositions.Reset();
	ROIEdges.Reset();

	EdgeEncodings.Reset();
	FaceEncodings.Reset();
}


void FGroupTopologyDeformer::SetActiveHandleFaces(const TArray<int>& FaceGroupIDs)
{
	Reset();

	// is this right? some HandleBoundaryVertices may also be interior if we do it per-face...
	for (int GroupID : FaceGroupIDs)
	{
		Topology->CollectGroupVertices(GroupID, HandleVertices);
		Topology->CollectGroupBoundaryVertices(GroupID, HandleBoundaryVertices);
	}
	ModifiedVertices = HandleVertices;

	// find neighbour group set
	TArray<int> HandleGroups = FaceGroupIDs;
	TArray<int> AllGroupNbrGroups;
	for (int GroupID : FaceGroupIDs)
	{
		const TArray<int>& CurGroupNbrGroups = Topology->GetGroupNbrGroups(GroupID);
		for (int NbrGroupID : CurGroupNbrGroups)
		{
			if (HandleGroups.Contains(NbrGroupID) == false)
			{
				AllGroupNbrGroups.AddUnique(NbrGroupID);
			}
		}
	}
	CalculateROI(HandleGroups, AllGroupNbrGroups);

	SaveInitialPositions();

	ComputeEncoding();
}



void FGroupTopologyDeformer::SetActiveHandleEdges(const TArray<int>& TopologyEdgeIDs)
{
	Reset();

	for (int EdgeID : TopologyEdgeIDs)
	{
		const TArray<int>& EdgeVerts = Topology->GetGroupEdgeVertices(EdgeID);
		for (int VertID : EdgeVerts)
		{
			HandleVertices.Add(VertID);
		}
	}
	HandleBoundaryVertices = HandleVertices;
	ModifiedVertices = HandleVertices;

	TArray<int> HandleGroups;
	TArray<int> NbrGroups;
	Topology->FindEdgeNbrGroups(TopologyEdgeIDs, NbrGroups);
	CalculateROI(HandleGroups, NbrGroups);

	SaveInitialPositions();

	ComputeEncoding();
}


void FGroupTopologyDeformer::SetActiveHandleCorners(const TArray<int>& CornerIDs)
{
	Reset();

	for (int CornerID : CornerIDs)
	{
		int VertID = Topology->GetCornerVertexID(CornerID);
		if (VertID >= 0)
		{
			HandleVertices.Add(VertID);
		}
	}
	HandleBoundaryVertices = HandleVertices;
	ModifiedVertices = HandleVertices;

	TArray<int> HandleGroups;
	TArray<int> NbrGroups;
	Topology->FindCornerNbrGroups(CornerIDs, NbrGroups);
	CalculateROI(HandleGroups, NbrGroups);

	SaveInitialPositions();

	ComputeEncoding();
}


void FGroupTopologyDeformer::SetActiveHandleFromSelection(const FGeometrySelection& Selection)
{
	if (!ensure(Selection.TopologyType == EGeometryTopologyType::Polygroup))
	{
		return;
	}

	TArray<int> Faces;
	TArray<int> Edges;
	TArray<int> Verts;
	if (Selection.ElementType == EGeometryElementType::Face)
	{
		for (uint64 Element : Selection.Selection)
		{
			int32 GroupID = FGeoSelectionID(Element).TopologyID;
			if (Topology->FindGroupByID(GroupID) != nullptr)
			{
				Faces.Add(GroupID);
			}
		}
	}
	else if (Selection.ElementType == EGeometryElementType::Edge)
	{
		for (uint64 Element : Selection.Selection)
		{
			int32 GroupEdgeID = FGeoSelectionID(Element).TopologyID;
			if (GroupEdgeID >= 0 && GroupEdgeID < Topology->Edges.Num())
			{
				Edges.Add(GroupEdgeID);
			}
		}
	}
	else if (Selection.ElementType == EGeometryElementType::Vertex)
	{
		for (uint64 Element : Selection.Selection)
		{
			int32 CornerID = FGeoSelectionID(Element).TopologyID;
			if (CornerID >= 0 && CornerID < Topology->Corners.Num())
			{
				Verts.Add(CornerID);
			}
		}
	}
	
	for (const TPair<uint64, double>& WeightedVert : Selection.SoftSelectedVertices)
	{
		int32 CornerID = FGeoSelectionID(WeightedVert.Key).TopologyID;
		if (CornerID >= 0 && CornerID < Topology->Corners.Num())
		{
			Verts.Add(CornerID);
		}
	}

	SetActiveHandle(Faces, Edges, Verts);

}


void FGroupTopologyDeformer::SetActiveHandle(
	const TArray<int>& FaceGroupIDs,
	const TArray<int>& TopologyEdgeIDs,
	const TArray<int>& TopologyCornerIDs
	)
{
	Reset();

	// Face
	
	// is this right? some HandleBoundaryVertices may also be interior if we do it per-face...
	for (int GroupID : FaceGroupIDs)
	{
		Topology->CollectGroupVertices(GroupID, HandleVertices);
		Topology->CollectGroupBoundaryVertices(GroupID, HandleBoundaryVertices);
	}

	// find neighbour group set
	TArray<int> HandleGroups = FaceGroupIDs;
	TArray<int> NbrGroups;
	for (int GroupID : FaceGroupIDs)
	{
		const TArray<int>& CurGroupNbrGroups = Topology->GetGroupNbrGroups(GroupID);
		for (int NbrGroupID : CurGroupNbrGroups)
		{
			if (HandleGroups.Contains(NbrGroupID) == false)
			{
				NbrGroups.AddUnique(NbrGroupID);
			}
		}
	}

	// Edge
	for (int EdgeID : TopologyEdgeIDs)
	{
		const TArray<int>& EdgeVerts = Topology->GetGroupEdgeVertices(EdgeID);
		for (int VertID : EdgeVerts)
		{
			HandleVertices.Add(VertID);
			HandleBoundaryVertices.Add(VertID);
		}
	}

	TArray<int> EdgeNbrGroups;
	Topology->FindEdgeNbrGroups(TopologyEdgeIDs, EdgeNbrGroups);
	for (int NbrGroupID : EdgeNbrGroups)
	{
		if (HandleGroups.Contains(NbrGroupID) == false)
		{
			NbrGroups.AddUnique(NbrGroupID);
		}
	}	

	// Corner
	for (int CornerID : TopologyCornerIDs)
	{
		int VertID = Topology->GetCornerVertexID(CornerID);
		if (VertID >= 0)
		{
			HandleVertices.Add(VertID);
			HandleBoundaryVertices.Add(VertID);
		}
	}


	TArray<int> CornerNbrGroups;
	Topology->FindCornerNbrGroups(TopologyCornerIDs, CornerNbrGroups);
	for (int NbrGroupID : CornerNbrGroups)
	{
		if (HandleGroups.Contains(NbrGroupID) == false)
		{
			NbrGroups.AddUnique(NbrGroupID);
		}
	}	
	
	ModifiedVertices = HandleVertices;
	
	CalculateROI(HandleGroups, NbrGroups);

	SaveInitialPositions();

	ComputeEncoding();
}


void FGroupTopologyDeformer::SaveInitialPositions()
{
	for (int VertID : ModifiedVertices)
	{
		InitialPositions.AddVertex(Mesh, VertID);
	}
}



void FGroupTopologyDeformer::CalculateROI(const TArray<int>& HandleGroups, const TArray<int>& ROIGroups)
{
	// sort ROI edges into various sets
	//   HandleBoundaryVertices: all vertices on border of handle
	//   FixedBoundaryVertices: all vertices on fixed border (ie part of edges not connected to handle)
	//   ROIEdges: list of edge data structures for edges we will deform
	ROIEdges.Reserve(ROIGroups.Num() * 5);  // guesstimate
	Topology->ForGroupSetEdges(ROIGroups, [this, &HandleGroups](const FGroupTopology::FGroupEdge& Edge, int EdgeIndex)
	{
		if (HandleGroups.Contains(Edge.Groups.A) || HandleGroups.Contains(Edge.Groups.B))
		{
			return;		// this is a Handle boundary edge
		}

		bool bIsConnectedToHandle = Edge.IsConnectedToVertices(HandleBoundaryVertices);
		if (bIsConnectedToHandle)
		{
			for (int VertID : Edge.Span.Vertices)
			{
				ROIEdgeVertices.Add(VertID);
			}
			FROIEdge& ROIEdge = ROIEdges[ROIEdges.Add(FROIEdge())];
			ROIEdge.EdgeIndex = EdgeIndex;
			ROIEdge.Span = Edge.Span;
			if (HandleBoundaryVertices.Contains(ROIEdge.Span.Vertices[0]) == false)
			{
				ROIEdge.Span.Reverse();
			}
		}
		else
		{
			for (int VertID : Edge.Span.Vertices)
			{
				FixedBoundaryVertices.Add(VertID);
			}
		}
	});


	// create ROI faces
	ROIFaces.SetNum(ROIGroups.Num());
	int FaceIdx = 0;
	for (int NbrGroupID : ROIGroups)
	{
		FaceVertsTemp.Reset();
		FaceBoundaryVertsTemp.Reset();
		Topology->CollectGroupVertices(NbrGroupID, FaceVertsTemp);
		Topology->CollectGroupBoundaryVertices(NbrGroupID, FaceBoundaryVertsTemp);

		FROIFace& Face = ROIFaces[FaceIdx++];
		for (int vid : FaceVertsTemp)
		{
			TArray<int>& AddTo = (FaceBoundaryVertsTemp.Contains(vid)) ? Face.BoundaryVerts : Face.InteriorVerts;
			AddTo.Add(vid);
			ModifiedVertices.Add(vid);
		}
	}

	if (Mesh->HasAttributes() && Mesh->Attributes()->PrimaryNormals() != nullptr)
	{
		const FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();
		for (int32 vid : ModifiedVertices)
		{
			for (int32 tid : Mesh->VtxTrianglesItr(vid))
			{
				if (Normals->IsSetTriangle(tid))
				{
					FIndex3i NormalTri = Normals->GetTriangle(tid);
					ModifiedOverlayNormals.Add(NormalTri.A);
					ModifiedOverlayNormals.Add(NormalTri.B);
					ModifiedOverlayNormals.Add(NormalTri.C);
				}
			}
		}
	}

	// ModifiedVertices currently holds all the ROI group vertices, but the fixed vertices
	// don't actually change in position. However we do this after collecting the modified
	// overlay normals, since those do change.
	ModifiedVertices = ModifiedVertices.Difference(FixedBoundaryVertices);
}






void FGroupTopologyDeformer::ComputeEncoding()
{
	// encode ROI edges
	int NumROIEdges = ROIEdges.Num();
	EdgeEncodings.SetNum(NumROIEdges);
	for (int ei = 0; ei < NumROIEdges; ei++)
	{
		FEdgeSpan& Span = ROIEdges[ei].Span;
		int NumVerts = Span.Vertices.Num();

		FEdgeEncoding& Encoding = EdgeEncodings[ei];
		Encoding.Vertices.SetNum(NumVerts);

		FVector3d StartV = Mesh->GetVertex(Span.Vertices[0]);
		FVector3d EndV = Mesh->GetVertex(Span.Vertices[NumVerts - 1]);
		FSegment3d Seg(StartV, EndV);

		for (int k = 1; k < NumVerts - 1; ++k)
		{
			FVector3d Pos = Mesh->GetVertex(Span.Vertices[k]);
			FEdgeVertexEncoding& Enc = Encoding.Vertices[k];
			Enc.T = Seg.ProjectUnitRange(Pos);
			//Enc.T *= Enc.T;
			//Enc.T = FMathd::Sqrt(Enc.T);
			Enc.Delta = Pos - Seg.PointBetween(Enc.T);
		}
	}

	// encode ROI faces
	int NumROIFaces = ROIFaces.Num();
	FaceEncodings.SetNum(NumROIFaces);
	for (int fi = 0; fi < NumROIFaces; ++fi)
	{
		FROIFace& Face = ROIFaces[fi];
		int NumBoundaryV = Face.BoundaryVerts.Num();
		int NumInteriorV = Face.InteriorVerts.Num();
		FFaceEncoding& Encoding = FaceEncodings[fi];
		Encoding.Vertices.SetNum(NumInteriorV);
		for (int vi = 0; vi < NumInteriorV; ++vi)
		{
			FVector3d Pos = Mesh->GetVertex(Face.InteriorVerts[vi]);
			FFaceVertexEncoding& VtxEncoding = Encoding.Vertices[vi];
			VtxEncoding.Weights.SetNum(NumBoundaryV);
			VtxEncoding.Deltas.SetNum(NumBoundaryV);
			double WeightSum = 0;
			for (int k = 0; k < NumBoundaryV; ++k)
			{
				FVector3d BorderPos = Mesh->GetVertex(Face.BoundaryVerts[k]);
				FVector3d DeltaVec = Pos - BorderPos;
				VtxEncoding.Deltas[k] = DeltaVec;

				double DeltaSquaredLength = DeltaVec.SquaredLength();
				if (DeltaSquaredLength == 0)
				{
					// This weight should be 1, and everything else should be zero.
					for (int32 i = 0; i < VtxEncoding.Weights.Num(); ++i)
					{
						VtxEncoding.Weights[i] = 0;
					}
					VtxEncoding.Weights[k] = 1;
					WeightSum = 1;
					break;
				}

				double Weight = 1.0 / DeltaVec.SquaredLength();
				VtxEncoding.Weights[k] = Weight;
				WeightSum += Weight;
			}
			FVector3d Reconstruct(0, 0, 0);
			for (int k = 0; k < NumBoundaryV; ++k)
			{
				VtxEncoding.Weights[k] /= WeightSum;
#if DO_GUARD_SLOW
				Reconstruct += VtxEncoding.Weights[k] * (Mesh->GetVertex(Face.BoundaryVerts[k]) + VtxEncoding.Deltas[k]);
#endif
			}
			checkSlow(Distance(Reconstruct, Pos) < 0.0001);   // sanity check
		}
	}
}



void FGroupTopologyDeformer::ClearSolution(FDynamicMesh3* TargetMesh)
{
	InitialPositions.SetPositions(TargetMesh);
}


void FGroupTopologyDeformer::UpdateSolution(FDynamicMesh3* TargetMesh, const TFunction<FVector3d(FDynamicMesh3* Mesh, int)>& HandleVertexDeformFunc)
{
	InitialPositions.SetPositions(TargetMesh);

	for (int VertIdx : HandleVertices)
	{
		FVector3d DeformPos = HandleVertexDeformFunc(TargetMesh, VertIdx);
		TargetMesh->SetVertex(VertIdx, DeformPos);
	}

	// reconstruct edges
	int NumEdges = ROIEdges.Num();
	for (int ei = 0; ei < NumEdges; ++ei)
	{
		const FEdgeSpan& Span = ROIEdges[ei].Span;
		const FEdgeEncoding& Encoding = EdgeEncodings[ei];
		int NumVerts = Span.Vertices.Num();
		FVector3d A = TargetMesh->GetVertex(Span.Vertices[0]);
		FVector3d B = TargetMesh->GetVertex(Span.Vertices[NumVerts - 1]);
		FSegment3d Seg(A, B);
		for (int k = 1; k < NumVerts - 1; ++k)
		{
			if (HandleVertices.Contains(Span.Vertices[k]))
			{
				// If we're explicitly moving this vertex, we should keep the explicitly set location.
				continue;
			}
			FVector3d NewPos = Seg.PointBetween(Encoding.Vertices[k].T);
			NewPos += Encoding.Vertices[k].Delta;
			TargetMesh->SetVertex(Span.Vertices[k], NewPos);
		}
	}

	// reconstruct faces
	int NumFaces = ROIFaces.Num();
	ParallelFor(NumFaces, [this, &TargetMesh](int FaceIdx) {
		const FROIFace& Face = ROIFaces[FaceIdx];
		const FFaceEncoding& Encoding = FaceEncodings[FaceIdx];
		int NumBorder = Face.BoundaryVerts.Num();
		int NumVerts = Face.InteriorVerts.Num();
		for (int vi = 0; vi < NumVerts; ++vi)
		{
			const FFaceVertexEncoding& VtxEncoding = Encoding.Vertices[vi];
			FVector3d Sum(0, 0, 0);
			for (int k = 0; k < NumBorder; ++k)
			{
				Sum += VtxEncoding.Weights[k] *
					(TargetMesh->GetVertex(Face.BoundaryVerts[k]) + VtxEncoding.Deltas[k]);
			}
			TargetMesh->SetVertex(Face.InteriorVerts[vi], Sum);
		}
	});

	FMeshNormals::QuickRecomputeOverlayNormals(*TargetMesh);
}



