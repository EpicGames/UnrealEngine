// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/InsetMeshRegion.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Selections/MeshVertexSelection.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Selections/MeshConnectedComponents.h"
#include "Distance/DistLine3Line3.h"
#include "DynamicSubmesh3.h"
#include "Solvers/ConstrainedMeshDeformer.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Operations/PolyEditingEdgeUtil.h"

using namespace UE::Geometry;

FInsetMeshRegion::FInsetMeshRegion(FDynamicMesh3* mesh) : Mesh(mesh)
{
}

FInsetMeshRegion::~FInsetMeshRegion()
{
}

bool FInsetMeshRegion::Apply()
{
	FMeshNormals Normals;
	bool bHaveVertexNormals = Mesh->HasVertexNormals();
	if (!bHaveVertexNormals)
	{
		Normals = FMeshNormals(Mesh);
		Normals.ComputeVertexNormals();
	}

	FMeshConnectedComponents RegionComponents(Mesh);
	RegionComponents.FindConnectedTriangles(Triangles);

	bool bAllOK = true;
	InsetRegions.SetNum(RegionComponents.Num());
	for (int32 k = 0; k < RegionComponents.Num(); ++k)
	{
		FInsetInfo& Region = InsetRegions[k];
		Region.InitialTriangles = MoveTemp(RegionComponents.Components[k].Indices);
		if (ApplyInset(Region, (bHaveVertexNormals) ? nullptr : &Normals) == false)
		{
			bAllOK = false;
		}
		else
		{
			AllModifiedTriangles.Append(Region.InitialTriangles);
			for (TArray<int32>& RegionTris : Region.StitchTriangles)
			{
				AllModifiedTriangles.Append(RegionTris);
			}
		}
	}

	return bAllOK;


}


bool FInsetMeshRegion::ApplyInset(FInsetInfo& Region, FMeshNormals* UseNormals)
{
	if (ChangeTracker)
	{
		ChangeTracker->SaveTriangles(Region.InitialTriangles, true);
	}

	FDynamicMeshEditor Editor(Mesh);

	TArray<FDynamicMeshEditor::FLoopPairSet> LoopPairs;
	bool bOK = Editor.DisconnectTriangles(Region.InitialTriangles, LoopPairs, true);
	if (bOK == false)
	{
		return false;
	}

	// Before we start insetting, prep things for stitching. We might end up breaking bowties in the disconnected
	// region, which will mess up the Vids on the inner loop. So convert the inner loop to a sequence of identifiers
	// that won't change across vertex splits (tri and subindex pairs). Note that we don't need to do the same to the
	// outer loop because we are not splitting bowties there, and in fact the outer loop may end up with detached verts
	// (without a triangle), so we would have needed a slightly messier system to account for that there.
	typedef TPair<int32, TPair<int8, int8>> TriVertPair;
	TArray<TArray<TriVertPair>> InsetStitchSides;
	InsetStitchSides.SetNum(LoopPairs.Num());
	for (int32 i = 0; i < LoopPairs.Num(); ++i)
	{
		FDynamicMeshEditor::ConvertLoopToTriVidPairSequence(*Mesh, LoopPairs[i].InnerVertices, LoopPairs[i].InnerEdges, InsetStitchSides[i]);
	}

	// Now that we've stored information that we need for stitching, split bowties in the disconnected region.
	FDynamicMeshEditResult SplitBowtiesResult;
	Editor.SplitBowtiesAtTriangles(Region.InitialTriangles, SplitBowtiesResult);

	// make copy of separated submesh for deformation
	// (could we defer this copy until we know we need it?)
	// This should happen after splitting bowties.
	FDynamicSubmesh3 SubmeshCalc(Mesh, Region.InitialTriangles, (int)EMeshComponents::None, false);
	FDynamicMesh3& Submesh = SubmeshCalc.GetSubmesh();
	bool bHaveInteriorVerts = false;
	for (int32 vid : Submesh.VertexIndicesItr())
	{
		if (Submesh.IsBoundaryVertex(vid) == false)
		{
			bHaveInteriorVerts = true;
			break;
		}
	}

	// If we did split bowties, some of the inner loops are now broken/merged (think of a small circle tangent inside
	// a large circle- now you have a thick C shape instead). We'll need to generate new loops there for doing the 
	// inset.
	bool bSomeLoopsBroken = SplitBowtiesResult.NewVertices.Num() > 0;

	Region.InsetLoops.Reset(LoopPairs.Num());
	auto InsetLoop = [this, &Region](const TArray<int32>& LoopVids, const TArray<int32>& LoopEids)
	{
		int32 NumEdges = LoopEids.Num();

		TArray<FLine3d> InsetLines;
		UE::Geometry::ComputeInsetLineSegmentsFromEdges(*Mesh, LoopEids, InsetDistance, InsetLines);

		TArray<FVector3d> NewPositions;
		UE::Geometry::SolveInsetVertexPositionsFromInsetLines(*Mesh, InsetLines, LoopVids, NewPositions, true);

		if (NewPositions.Num() == LoopVids.Num())
		{
			for (int32 k = 0; k < LoopVids.Num(); ++k)
			{
				Mesh->SetVertex(LoopVids[k], NewPositions[k]);
			}
		}

		Region.InsetLoops.Emplace();
		Region.InsetLoops.Last().InitializeFromVertices(Mesh, LoopVids);
	};

	if (bSomeLoopsBroken)
	{
		// Recalc the loops
		FMeshRegionBoundaryLoops UpdatedInsetLoops(Mesh, Region.InitialTriangles, false);
		bOK = UpdatedInsetLoops.Compute();
		if (!bOK)
		{
			return false;
		}

		for (const FEdgeLoop& Loop : UpdatedInsetLoops.Loops)
		{
			InsetLoop(Loop.Vertices, Loop.Edges);
		}
	}
	else
	{
		// Can use the original loops we got while disconnecting
		for (const FDynamicMeshEditor::FLoopPairSet& LoopPair : LoopPairs)
		{
			InsetLoop(LoopPair.InnerVertices, LoopPair.InnerEdges);
		}
	}

	// Stitch the bands we saved.
	int32 NumInitialLoops = LoopPairs.Num();
	Region.BaseLoops.SetNum(NumInitialLoops);
	Region.StitchTriangles.SetNum(NumInitialLoops);
	Region.StitchPolygonIDs.SetNum(NumInitialLoops);
	TArray<TArray<FIndex2i>> QuadStrips;
	QuadStrips.Reserve(NumInitialLoops);
	for (int32 LoopIndex = 0; LoopIndex < NumInitialLoops; ++LoopIndex)
	{
		const FDynamicMeshEditor::FLoopPairSet& LoopPair = LoopPairs[LoopIndex];
		const TArray<int32>& BaseLoopV = LoopPair.OuterVertices;

		int32 NumLoopV = BaseLoopV.Num();

		// allocate a new group ID for each pair of input group IDs, and build up list of new group IDs along loop
		TArray<int32> NewGroupIDs;
		TArray<int32> EdgeGroups;
		TMap<TPair<int32, int32>, int32> NewGroupsMap;
		for (int32 k = 0; k < NumLoopV; ++k)
		{
			int32 InsetGroupID = Mesh->GetTriangleGroup(InsetStitchSides[LoopIndex][k].Key);

			// base edge may not exist if we inset entire region. In that case just use single GroupID
			int32 BaseEdgeID = Mesh->FindEdge(BaseLoopV[k], BaseLoopV[(k + 1) % NumLoopV]);
			int32 BaseGroupID = (BaseEdgeID >= 0) ? Mesh->GetTriangleGroup(Mesh->GetEdgeT(BaseEdgeID).A) : InsetGroupID;

			TPair<int32, int32> GroupPair(FMath::Min(BaseGroupID, InsetGroupID), FMath::Max(BaseGroupID, InsetGroupID));
			if (NewGroupsMap.Contains(GroupPair) == false)
			{
				int32 NewGroupID = Mesh->AllocateTriangleGroup();
				NewGroupIDs.Add(NewGroupID);
				NewGroupsMap.Add(GroupPair, NewGroupID);
			}
			EdgeGroups.Add(NewGroupsMap[GroupPair]);
		}

		// Stitch the loops. In case of broken loops, we'll still end up stitching the proper things, just
		// in bands that fill in the proper regions together.
		FDynamicMeshEditResult StitchResult;
		Editor.StitchVertexLoopToTriVidPairSequence(InsetStitchSides[LoopIndex], LoopPair.OuterVertices, StitchResult);

		// set the groups of the new quads along the stitch
		int32 NumNewQuads = StitchResult.NewQuads.Num();
		for (int32 k = 0; k < NumNewQuads; k++)
		{
			Mesh->SetTriangleGroup(StitchResult.NewQuads[k].A, EdgeGroups[k]);
			Mesh->SetTriangleGroup(StitchResult.NewQuads[k].B, EdgeGroups[k]);
		}

		// save the stitch triangles set and associated group IDs
		StitchResult.GetAllTriangles(Region.StitchTriangles[LoopIndex]);
		Region.StitchPolygonIDs[LoopIndex] = NewGroupIDs;

		QuadStrips.Add(MoveTemp(StitchResult.NewQuads));

		Region.BaseLoops[LoopIndex].InitializeFromVertices(Mesh, BaseLoopV);
	}

	// if we have interior vertices or just want to try to resolve foldovers we
	// do a Laplacian solve using the inset positions determined geometrically
	// as weighted soft constraints.
	if ( (bHaveInteriorVerts || Softness > 0.0) && bSolveRegionInteriors )
	{
		bool bReprojectInset = bReproject;
		bool bReprojectInterior = bReproject;
		bool bSolveBoundary = (Softness > 0.0);

		// Build AABBTree for initial surface so that we can reproject onto it.
		// (conceivably this could be cached during interactive operations, also not
		//  necessary if we are not projecting!)
		FDynamicMesh3 ProjectSurface(Submesh);
		FDynamicMeshAABBTree3 Projection(&ProjectSurface, bReproject);

		// if we are reprojecting, do inset border immediately so that the measurements below
		// use the projected values
		if (bReprojectInset)
		{
			for (const FEdgeLoop& Loop : Region.InsetLoops)
			{
				for (int32 BaseVID : Loop.Vertices)
				{
					int32 SubmeshVID = SubmeshCalc.MapVertexToSubmesh(BaseVID);
					Mesh->SetVertex(BaseVID, Projection.FindNearestPoint(Mesh->GetVertex(BaseVID)));
				}
			}
		}

		// compute area of inserted quad-strip border
		double TotalBorderQuadArea = 0;
		int32 NumStrips = QuadStrips.Num();
		for (int32 StripIndex = 0; StripIndex < NumStrips; ++StripIndex)
		{
			int32 NumQuads = QuadStrips[StripIndex].Num();
			for (int32 k = 0; k < NumQuads; k++)
			{
				TotalBorderQuadArea += Mesh->GetTriArea(QuadStrips[StripIndex][k].A);
				TotalBorderQuadArea += Mesh->GetTriArea(QuadStrips[StripIndex][k].B);
			}
		}

		// Figure how much area changed by subtracting area of quad-strip from original area.
		// (quad-strip area seems implausibly high at larger distances, ie becomes larger than initial area. Possibly due to sawtooth-shaped profile
		//  of non-planar quads - measure each quad in planar projection?)
		FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(Submesh);
		double InitialArea = VolArea.Y;
		double TargetArea = FMathd::Max(0, InitialArea - TotalBorderQuadArea);
		double AreaRatio = TargetArea / InitialArea;
		double LinearAreaScale = FMathd::Max(0.1, FMathd::Sqrt(AreaRatio));

		// compute deformation
		TUniquePtr<UE::Solvers::IConstrainedLaplacianMeshSolver> Solver = UE::MeshDeformation::ConstructSoftMeshDeformer(Submesh);

		// configure area correction based on scaling parameter
		double AreaCorrectT = FMathd::Clamp(AreaCorrection, 0.0, 1.0);
		LinearAreaScale = (1 - AreaCorrectT) * 1.0 + (AreaCorrectT)*LinearAreaScale;
		Solver->UpdateLaplacianScale(LinearAreaScale);
		
		// Want to convert [0,1] softness parameter to a per-boundary-vertex Weight. 
		// Trying to use Vertex Count and Scaling factor to normalize for scale
		// (really should scale mesh down to consistent size, but this is messy due to mapping back to Mesh)
		// Laplacian scale above also impacts this...and perhaps we should only be counting boundary vertices??
		double UnitScalingMeasure = FMathd::Max(0.01, FMathd::Sqrt(VolArea.Y / 6.0));
		double NonlinearT = FMathd::Pow(Softness, 2.0);
		double ScaledPower = (NonlinearT / 50.0) * (double)Submesh.VertexCount() * UnitScalingMeasure;
		double Weight = (ScaledPower < FMathf::ZeroTolerance) ? 100.0 : (1.0 / ScaledPower);

		// add constraints on all the boundary vertices
		for (const FEdgeLoop& Loop : Region.InsetLoops)
		{
			for (int32 BaseVID : Loop.Vertices)
			{
				int32 SubmeshVID = SubmeshCalc.MapVertexToSubmesh(BaseVID);
				FVector3d CurPosition = Mesh->GetVertex(BaseVID);
				Solver->AddConstraint(SubmeshVID, Weight, CurPosition, bSolveBoundary == false);
			}
		}

		// solve for deformed (and possibly reprojected) positions and update mesh
		TArray<FVector3d> DeformedPositions;
		if (Solver->Deform(DeformedPositions))
		{
			for (int32 SubmeshVID : Submesh.VertexIndicesItr())
			{
				if (bSolveBoundary || Solver->IsConstrained(SubmeshVID) == false)
				{
					int32 BaseVID = SubmeshCalc.MapVertexToBaseMesh(SubmeshVID);

					FVector3d SolvePosition = DeformedPositions[SubmeshVID];
					if (bReprojectInterior)
					{
						SolvePosition = Projection.FindNearestPoint(SolvePosition);
					}

					Mesh->SetVertex(BaseVID, SolvePosition);
				}
			}
		}
	}


	// calculate UVs/etc
	if (Mesh->HasAttributes())
	{
		int32 NumStrips = QuadStrips.Num();
		for ( int32 StripIndex = 0; StripIndex < NumStrips; ++StripIndex)
		{
			FDynamicMeshEditor::FLoopPairSet& OriginalLoopPair = LoopPairs[StripIndex];
			TArray<int32>& BaseLoopV = OriginalLoopPair.OuterVertices;

			// for each polygon we created in stitch, set UVs and normals
			// TODO copied from FExtrudeMesh, doesn't really make sense in this context...
			float AccumUVTranslation = 0;
			FFrame3d FirstProjectFrame;
			FVector3d FrameUp;

			int32 NumQuads = QuadStrips[StripIndex].Num();
			for (int32 k = 0; k < NumQuads; k++)
			{
				FVector3f Normal = Editor.ComputeAndSetQuadNormal( QuadStrips[StripIndex][k], true);

				// align axis 0 of projection frame to first edge, then for further edges,
				// rotate around 'up' axis to keep normal aligned and frame horizontal
				FFrame3d ProjectFrame;
				if (k == 0)
				{
					FVector3d FirstEdge = Mesh->GetVertex(BaseLoopV[1]) - Mesh->GetVertex(BaseLoopV[0]);
					Normalize(FirstEdge);
					FirstProjectFrame = FFrame3d(FVector3d::Zero(), (FVector3d)Normal);
					FirstProjectFrame.ConstrainedAlignAxis(0, FirstEdge, (FVector3d)Normal);
					FrameUp = FirstProjectFrame.GetAxis(1);
					ProjectFrame = FirstProjectFrame;
				}
				else
				{
					ProjectFrame = FirstProjectFrame;
					ProjectFrame.ConstrainedAlignAxis(2, (FVector3d)Normal, FrameUp);
				}

				if (k > 0)
				{
					AccumUVTranslation += (float)Distance(Mesh->GetVertex(BaseLoopV[k]), Mesh->GetVertex(BaseLoopV[k - 1]));
				}

				// translate horizontally such that vertical spans are adjacent in UV space (so textures tile/wrap properly)
				float TranslateU = UVScaleFactor * AccumUVTranslation;
				Editor.SetQuadUVsFromProjection(QuadStrips[StripIndex][k], ProjectFrame, UVScaleFactor, FVector2f(TranslateU, 0));
			}
		}
	}

	return true;
}


