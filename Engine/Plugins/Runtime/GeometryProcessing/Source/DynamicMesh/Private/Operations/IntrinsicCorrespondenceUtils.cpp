// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/IntrinsicCorrespondenceUtils.h"
#include "Operations/MeshGeodesicSurfaceTracer.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

/**------------------------------------------------------------------------------
* FSurfacePoint
*-------------------------------------------------------------------------------*/

IntrinsicCorrespondenceUtils::FSurfacePoint::FSurfacePoint()
{
	PositionType = EPositionType::Vertex;
	Position.VertexPosition.VID = IndexConstants::InvalidID;
}

IntrinsicCorrespondenceUtils::FSurfacePoint::FSurfacePoint(int32 VID)
{
	PositionType = EPositionType::Vertex;
	Position.VertexPosition.VID = VID;
}

IntrinsicCorrespondenceUtils::FSurfacePoint::FSurfacePoint(int32 EdgeID, double Alpha)
{
	PositionType = EPositionType::Edge;
	Position.EdgePosition.EdgeID = EdgeID;
	Position.EdgePosition.Alpha  = Alpha;
}

IntrinsicCorrespondenceUtils::FSurfacePoint::FSurfacePoint(int32 TriID, const FVector3d& BaryCentrics)
{
	PositionType = EPositionType::Triangle;
	Position.TriPosition.TriID = TriID;
	Position.TriPosition.BarycentricCoords = BaryCentrics;
}

FVector3d IntrinsicCorrespondenceUtils::AsR3Position(const IntrinsicCorrespondenceUtils::FSurfacePoint& SurfacePoint, const FDynamicMesh3& Mesh, bool& bIsValid)
{
	const IntrinsicCorrespondenceUtils::FSurfacePoint::FSurfacePositionUnion& Position = SurfacePoint.Position;
	FVector3d Result(0., 0., 0.);
	switch (SurfacePoint.PositionType)
	{
		case FSurfacePoint::EPositionType::Vertex:
		{
			const int32 SurfaceVID = Position.VertexPosition.VID;
			bIsValid               = Mesh.IsVertex(SurfaceVID);
			if (bIsValid)
			{
				Result = Mesh.GetVertex(SurfaceVID);
			}
		}
		break;
		case FSurfacePoint::EPositionType::Edge:
		{
			const int32 SurfaceEID = Position.EdgePosition.EdgeID;
			const double Alpha     = Position.EdgePosition.Alpha;
			bIsValid               = Mesh.IsEdge(SurfaceEID);
			if (bIsValid)
			{
				const FIndex2i SurfaceEdgeV = Mesh.GetEdgeV(SurfaceEID);
				Result = Alpha * (Mesh.GetVertex(SurfaceEdgeV.A)) + (1. - Alpha) * (Mesh.GetVertex(SurfaceEdgeV.B));
			}
		}
		break;
		case FSurfacePoint::EPositionType::Triangle:
		{
			const int32 SurfaceTID = Position.TriPosition.TriID;
			bIsValid = Mesh.IsTriangle(SurfaceTID);

			if (bIsValid)
			{
				Result = Mesh.GetTriBaryPoint( SurfaceTID,
											   Position.TriPosition.BarycentricCoords[0],
											   Position.TriPosition.BarycentricCoords[1],
											   Position.TriPosition.BarycentricCoords[2]);
			}

		}
		break;
		default:
		{
			bIsValid = false;
			// shouldn't be able to reach this point
			check(0);
		}
	}
	return Result;
}

/**------------------------------------------------------------------------------
* FMeshConnection
*-------------------------------------------------------------------------------*/
IntrinsicCorrespondenceUtils::FMeshConnection::FMeshConnection(const FDynamicMesh3& SurfMesh)
{
	Reset(SurfMesh);
}

void IntrinsicCorrespondenceUtils::FMeshConnection::Reset(TUniquePtr<FDynamicMesh3>& SurfMesh)
{
	Reset(*SurfMesh);

	OwnedSurfaceMesh = MoveTemp(SurfMesh);
	SurfaceMesh = OwnedSurfaceMesh.Get();
}

void IntrinsicCorrespondenceUtils::FMeshConnection::Reset(const FDynamicMesh3& SurfMesh)
{
	OwnedSurfaceMesh.Reset();

	SurfaceMesh = &SurfMesh;

	VIDToReferenceEID.Reset();
	TIDToReferenceEID.Reset();

	// pick local reference directions for each vertex and triangle.
	const int32 MaxVertexID   = SurfMesh.MaxVertexID();
	const int32 MaxTriangleID = SurfMesh.MaxTriangleID();

	{
		VIDToReferenceEID.AddUninitialized(MaxVertexID);
		TIDToReferenceEID.AddUninitialized(MaxTriangleID);

		for (int32 VID = 0; VID < MaxVertexID; ++VID)
		{
			if (SurfMesh.IsVertex(VID))
			{
				int32 RefEID = -1;
				if (SurfMesh.IsBoundaryVertex(VID) && !SurfMesh.IsBowtieVertex(VID))
				{
					// vertex is on the mesh boundary, 
					// choose the boundary edge such that traveling CCW (about the vertex) moves into the mesh
					for (int32 EID : SurfMesh.VtxEdgesItr(VID))
					{
						if (SurfMesh.IsBoundaryEdge(EID))
						{
							const int32 TID     = SurfMesh.GetEdgeT(EID).A;
							const int32 IndexOf = SurfMesh.GetTriEdges(TID).IndexOf(EID);
							if (SurfMesh.GetTriangle(TID)[IndexOf] == VID)
							{
								RefEID = EID;
								break;
							}
						}
					}
				}
				else
				{
					PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
					for (int32 EID : SurfMesh.VtxEdgesItr(VID))
					{
						RefEID = EID;
						break;
					}
					PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS
				}
				VIDToReferenceEID[VID] = RefEID;
			}
		}
		for (int32 TID = 0; TID < MaxTriangleID; ++TID)
		{
			if (SurfMesh.IsTriangle(TID))
			{
				TIDToReferenceEID[TID] = SurfMesh.GetTriEdge(TID, 0);
			}
		}
	}
}


/**------------------------------------------------------------------------------
* FNormalCoordinates
*-------------------------------------------------------------------------------*/

IntrinsicCorrespondenceUtils::FNormalCoordinates::FNormalCoordinates(const FDynamicMesh3& SurfMesh)
	: MyBase()
{
	Reset(SurfMesh);
}

void IntrinsicCorrespondenceUtils::FNormalCoordinates::Reset(TUniquePtr<FDynamicMesh3>& SurfMesh)
{
	MyBase::Reset(SurfMesh);

	RebuildNormalCoordinates(*this->SurfaceMesh);
}

void IntrinsicCorrespondenceUtils::FNormalCoordinates::Reset(const FDynamicMesh3& SurfMesh)
{
	MyBase::Reset(SurfMesh);

	RebuildNormalCoordinates(SurfMesh);
}

void IntrinsicCorrespondenceUtils::FNormalCoordinates::RebuildNormalCoordinates(const FDynamicMesh3& SurfMesh)
{
	NormalCoord.Clear();
	RoundaboutOrder.Clear();

	RefVertDegree.Reset();
	EdgeOrder.Reset();

	const int32 MaxEID = SurfMesh.MaxEdgeID();
	const int32 MaxVID = SurfMesh.MaxVertexID();
	const int32 MaxTID = SurfMesh.MaxTriangleID();

	// on construction the intrinsic mesh will have the same triangles as the surface mesh,
	// with the intrinsic edges identical to the surface edges. 
	// the value -1 indicates the intrinsic mesh edge is a surface mesh edge segment.
	NormalCoord.Resize(MaxEID, -1);
	

	// populate Ref Vert Degree. 
	RefVertDegree.SetNumZeroed(MaxVID);
	for (int32 VID : SurfMesh.VertexIndicesItr())
	{
		int32 Degree = 0;
		for (int32 EID : SurfMesh.VtxEdgesItr(VID))
		{
			Degree++;
		}
		RefVertDegree[VID] = Degree;
	}

	// init the roundabout data as invalid
	RoundaboutOrder.Resize(MaxTID, FIndex3i::Invalid());
	

	// correctly populate the roundabout data.  Note bow-tie vertices should prohibited at a higher level
	for (int32 CVID : SurfMesh.VertexIndicesItr())
	{

		int32 Order = 0;
		auto SetRoundaboutOrder = [&Order, this](int32 TID, int32 EID, int32 IdxOf)->bool
									{
										RoundaboutOrder[TID][IdxOf] = Order;
										Order++;
										constexpr bool bShouldBreak = false;
										return bShouldBreak;
									};
		VisitVertexAdjacentElements(SurfMesh, CVID, VIDToReferenceEID[CVID], SetRoundaboutOrder);
	}

	// Edge Order is a copy of the roundabout before edge flips.  Edge order describes the surface mesh, and shouldn't be updated after construction
	EdgeOrder.SetNumUninitialized(MaxTID);
	for (int32 i = 0; i < MaxTID; ++i)
	{
		EdgeOrder[i] = RoundaboutOrder[i];
	}
}

bool  IntrinsicCorrespondenceUtils::FNormalCoordinates::OnFlipEdge(const int32 T0ID, const FIndex3i& BeforeT0EIDs, const int32 T0OppVID,
																   const int32 T1ID, const FIndex3i& BeforeT1EIDs, const int32 T1OppVID,
																   const int32 FlippedEID)
{
	// see Gillespi et al, eqn 15

	// prior to flip of edge ij to become edge lk:
	// TO triangle is a cyclic permutation of (i,j,k)  and likewise T1 is of (j,i,l)
	// with this notation T0OppVID is the vertex at 'k' corner, and T1OppVID is the vertex at 'l' corner

	// index of the edge to be flipped
	const int32 T0IndexOf = BeforeT0EIDs.IndexOf(FlippedEID); // exits i in t0
	const int32 T1IndexOf = BeforeT1EIDs.IndexOf(FlippedEID); // exits j in t1

	checkSlow(FlippedEID < NormalCoord.Num());
	if (T0IndexOf == FDynamicMesh3::InvalidID || T1IndexOf == FDynamicMesh3::InvalidID)
	{
		checkSlow(0);
		return false;
	}


	// the corners of the two triangles before flip (alternately the edges)
	const int32 T0_i = T0IndexOf;                // t0:eij
	const int32 T0_j = (T0IndexOf + 1) % 3;      // t0:ejk
	const int32 T0_k = (T0IndexOf + 2) % 3;      // t0:eki

	const int32 T1_j = T1IndexOf;                // t1:eji
	const int32 T1_i = (T1IndexOf + 1) % 3;      // t1:eil
	const int32 T1_l = (T1IndexOf + 2) % 3;      // t1:elj

	// ---- update the NormalCoord for the flipped edge
	{
		// corner crossings
		const int32 Cji_l = NumCornerCrossingRefEdges(BeforeT1EIDs, T1_l);
		const int32 Cij_k = NumCornerCrossingRefEdges(BeforeT0EIDs, T0_k);

		const int32 Cil_j = NumCornerCrossingRefEdges(BeforeT1EIDs, T1_j);
		const int32 Cki_j = NumCornerCrossingRefEdges(BeforeT0EIDs, T0_j);

		const int32 Clj_i = NumCornerCrossingRefEdges(BeforeT1EIDs, T1_i);
		const int32 Cjk_i = NumCornerCrossingRefEdges(BeforeT0EIDs, T0_i);

		// corner emanating
		const int32 Eji_l = NumCornerEmanatingRefEdges(BeforeT1EIDs, T1_l);
		const int32 Eij_k = NumCornerEmanatingRefEdges(BeforeT0EIDs, T0_k);

		const int32 Elj_i = NumCornerEmanatingRefEdges(BeforeT1EIDs, T1_i);
		const int32 Ejk_i = NumCornerEmanatingRefEdges(BeforeT0EIDs, T0_i);

		const int32 Eil_j = NumCornerEmanatingRefEdges(BeforeT1EIDs, T1_j);
		const int32 Eki_j = NumCornerEmanatingRefEdges(BeforeT0EIDs, T0_j);

		// if the edge being flipped aligns with a surface edge, then it will cross that surface edge after the  flip
		const int32 Kronecker = IsSurfaceEdgeSegment(FlippedEID) ? 1 : 0;

		// total number of surfaces edges this edge crosses after the flip
		const int32 N_kl = Cji_l + Cij_k + (TMathUtil<int32>::Abs(Cil_j - Cki_j) + TMathUtil<int32>::Abs(Clj_i - Cjk_i) - Eji_l - Eij_k) / 2
						 + Elj_i + Ejk_i + Eil_j + Eki_j + Kronecker;

		NormalCoord[FlippedEID] = N_kl;
	}

	// ---- update the roundabouts for the two directions of the flipped edge. Note this must follow the normal coord update as it relies on those results
	{
		// after edge-flip the T0 and T1 vertices and edges are re-ordered so the flipped edge is the first edge. The new order is given by
		//             T0 edges =  ekl, elj, ejk
		//             T1 edges =  elk, eki, eil
		const int32 Edge_lj = BeforeT1EIDs[T1_l];
		const int32 Edge_jk = BeforeT0EIDs[T0_j];
		const int32 Edge_ki = BeforeT0EIDs[T0_k];
		const int32 Edge_il = BeforeT1EIDs[T1_i];

		const FIndex3i AfterT0EIDs(FlippedEID, Edge_lj, Edge_jk);
		const FIndex3i AfterT1EIDs(FlippedEID, Edge_ki, Edge_il);

		//  Roundabouts before flip ( respecting old edge order)
		const FIndex3i T0Rab = RoundaboutOrder[T0ID];
		const FIndex3i T1Rab = RoundaboutOrder[T1ID];

		const int32 VID_k = T0OppVID;
		const int32 VID_l = T1OppVID;

		// the roundabouts for the outer half-edges (the edges that don't flip)
		const int32 R_ki = T0Rab[T0_k];
		const int32 R_jk = T0Rab[T0_j];
		const int32 R_il = T1Rab[T1_i];
		const int32 R_lj = T1Rab[T1_l];

		// --compute the roundabouts for the two half-edges associated with the edge that does flip. (i.e  R_kl, R_lk)
		// note roundabouts are only computed for the original vertex set (i.e. those also in the surface mesh)
		const int32 R_kl = [&] 
							{
								if (!SurfaceMesh->IsVertex(VID_k))
								{
									return -1;
								}
								const int32 Kronecker_ki = IsSurfaceEdgeSegment(Edge_ki) ? 1 : 0;
								const int32 Eil_k = NumCornerEmanatingRefEdges(AfterT1EIDs, 1/*k corner*/); // AfterT1 = (l, k, i)
								return  (R_ki + Eil_k + Kronecker_ki) % RefVertDegree[VID_k];
							}();

		const int32 R_lk = [&]
							{
								if (!SurfaceMesh->IsVertex(VID_l))
								{
									return -1;
								}
								const int32 Kronecker_lj = IsSurfaceEdgeSegment(Edge_lj) ? 1 : 0;
								const int32 Ejk_l = NumCornerEmanatingRefEdges(AfterT0EIDs, 1/*l corner*/); // AfterT0 = (k, l, j)
								return (R_lj + Ejk_l + Kronecker_lj) % RefVertDegree[VID_l];
							}();
		
		// update the roundabouts with new edge order and values for the flipped edge
		RoundaboutOrder[T0ID] = FIndex3i(R_kl, R_lj, R_jk);
		RoundaboutOrder[T1ID] = FIndex3i(R_lk, R_ki, R_il);
	}

	return true;
}


int32 IntrinsicCorrespondenceUtils::FNormalCoordinates::GetNthEdgeID(const int32 VID, const int32 N) const
{

	if (!SurfaceMesh->IsVertex(VID))
	{
		return -1;
	}
	const int32 Degree = RefVertDegree[VID];
	if (N - 1 > Degree)
	{
		return -1;
	}

	const int32 RefEID = VIDToReferenceEID[VID];
	int32 Count        = 0;
	int32 ResultEID    = -1;
	auto EdgeFinder = [N, &Count, &ResultEID](int32 TID, int32 EID, int32 IdxOf)->bool
						{
							bool bShouldBreak = false;
							if (N == Count)
							{
								ResultEID = EID;
								bShouldBreak = true;
							}
							Count++;
							return bShouldBreak;
						};
	VisitVertexAdjacentElements(*SurfaceMesh, VID, RefEID, EdgeFinder);

	return ResultEID;
}



int32 IntrinsicCorrespondenceUtils::FNormalCoordinates::GetEdgeOrder(const int32 VID, const int32 AdjEID) const
{
	if (!SurfaceMesh->IsVertex(VID) || !SurfaceMesh->IsEdge(AdjEID))
	{
		return -1;
	}

	FIndex2i EdgeV = SurfaceMesh->GetEdgeV(AdjEID);
	if (EdgeV.A != VID && EdgeV.B != VID)
	{
		return -1;
	}

	FIndex2i EdgeT   = SurfaceMesh->GetEdgeT(AdjEID);
	int32 TID        = EdgeT.A;
	FIndex3i TriEIDs = SurfaceMesh->GetTriEdges(TID);
	int32 IndexOf    = TriEIDs.IndexOf(AdjEID);

	if (SurfaceMesh->GetTriangle(TID)[IndexOf] != VID)
	{
		TID = EdgeT.B;
		if (TID != -1)
		{
			TriEIDs = SurfaceMesh->GetTriEdges(TID);
			IndexOf = TriEIDs.IndexOf(AdjEID);
			checkSlow(SurfaceMesh->GetTriangle(TID)[IndexOf] == VID);
			return EdgeOrder[TID][IndexOf];
		}
		else
		{
			return RefVertDegree[VID] - 1; // we don't have an out-going edge from the vertex on this boundary, but we know it is the last edge.
		}
	}

	return EdgeOrder[TID][IndexOf];
}

/**------------------------------------------------------------------------------
* FSignpost
*-------------------------------------------------------------------------------*/

IntrinsicCorrespondenceUtils::FSignpost::FSignpost(const FDynamicMesh3& SrcMesh)
	:MyBase(SrcMesh)
{
	// During construction the mesh is a deep copy of the SrcMesh ( i.e. has the same IDs)

	const int32 MaxVertexID = SrcMesh.MaxVertexID();
	const int32 MaxTriangleID = SrcMesh.MaxTriangleID();


	// add surface position.
	{
		IntrinsicVertexPositions.SetNum(MaxVertexID);

		// initialize: identify with vertex in Extrinsic Mesh
		for (int32 VID = 0; VID < MaxVertexID; ++VID)
		{
			if (SrcMesh.IsVertex(VID))
			{
				FSurfacePoint VertexSurfacePoint(VID);
				IntrinsicVertexPositions[VID] = VertexSurfacePoint;
			}
		}
	}

	// add edge directions and geometric info
	{
		const int32 MaxExtEdgeID = SrcMesh.MaxEdgeID();
		IntrinsicEdgeAngles.SetNum(MaxTriangleID);

		GeometricVertexInfo.SetNum(MaxVertexID);
		IntrinsicVertexPositions.SetNum(MaxVertexID);

		ParallelFor(MaxVertexID, [this, &SrcMesh](int32 VID)
		{
			if (!SrcMesh.IsVertex(VID))
			{
				return;
			}

			constexpr int32 PlusTwoModThree[3] = { 2, 0, 1 }; // PlusTwoModThree[i] = (i + 2)%3

			// initialize by computing the angles
			FDynamicMesh3::FLocalIntArray Triangles;
			FDynamicMesh3::FLocalIntArray ContiguousGroupLengths;
			FDynamicMesh3::FLocalBoolArray GroupIsLoop;

			{

				FGeometricInfo GeometricInfo;
				// note: since the structure and IDs are a deep copy of the source mesh, we can use it to find the contiguous triangles
				const EMeshResult Result = SrcMesh.GetVtxContiguousTriangles(VID, Triangles, ContiguousGroupLengths, GroupIsLoop);
				if (Result == EMeshResult::Ok && GroupIsLoop.Num() == 1)
				{
					GeometricInfo.bIsInterior = (GroupIsLoop[0] == true);
					// Contiguous tris could be cw or ccw?  Need them to  be ccw
					if (Triangles.Num() > 1)
					{
						const int32 TID0     = Triangles[0];
						FIndex3i TriVIDs     = SrcMesh.GetTriangle(TID0);
						FIndex3i TriEIDs     = SrcMesh.GetTriEdges(TID0);
						int32 VertSubIdx     = TriVIDs.IndexOf(VID);
						const int32 EID      = TriEIDs[VertSubIdx];
						const int32 NextEID  = TriEIDs[PlusTwoModThree[VertSubIdx]];
						const int32 TID1     = Triangles[1];
						TriVIDs              = SrcMesh.GetTriangle(TID1);
						TriEIDs              = SrcMesh.GetTriEdges(TID1);
						VertSubIdx           = TriVIDs.IndexOf(VID);

						if (TriEIDs[VertSubIdx] != NextEID)
						{
							// need to reverse order to have CCW
							Algo::Reverse(Triangles);
						}
					}

					const int32 NumEdges = ContiguousGroupLengths[0];

					double AngleOffset       = 0;
					double WalkedAngle       = 0;
					const int32 ReferenceEID = VIDToReferenceEID[VID];
					checkSlow(ReferenceEID != FDynamicMesh3::InvalidID)

					TMap<int32, int32> VisitedTriSubID;
					for (int32 TID : Triangles)
					{
						const FIndex3i TriVIDs = SrcMesh.GetTriangle(TID);
						const FIndex3i TriEIDs = SrcMesh.GetTriEdges(TID);
						const int32 VertSubIdx = TriVIDs.IndexOf(VID);
						IntrinsicEdgeAngles[TID][VertSubIdx] = WalkedAngle;

						VisitedTriSubID.Add(TID, VertSubIdx);
						const int32 EID = TriEIDs[VertSubIdx];
						if (EID == ReferenceEID)
						{
							AngleOffset = WalkedAngle;
						}

						WalkedAngle += SrcMesh.GetTriInternalAngleR(TID, VertSubIdx);
					}
					// need to include last edge if the one ring of triangles isn't a closed loop, it could be the reference edge.
					if (GroupIsLoop[0] == false)
					{
						const int32 TID        = Triangles.Last();
						const FIndex3i TriVIDs = SrcMesh.GetTriangle(TID);
						const FIndex3i TriEIDs = SrcMesh.GetTriEdges(TID);
						const int32 VertSubIdx = TriVIDs.IndexOf(VID);
						const int32 EID        = TriEIDs[PlusTwoModThree[VertSubIdx]];
						if (EID == ReferenceEID)
						{
							AngleOffset = WalkedAngle;
						}
					}

					// orient with the reference edge at zero (or pi) angle  and rescale to radians and use 0,2pi to define polar angle.
					const double ToRadians = TMathUtilConstants<double>::TwoPi / WalkedAngle;
					GeometricInfo.ToRadians = ToRadians;
					double RefAngle = (SrcMesh.GetEdgeV(ReferenceEID).B == VID) ? TMathUtilConstants<double>::Pi : 0.;
					for (TPair<int32, int32>& TriSubIDPair : VisitedTriSubID)
					{
						int32 TID     = TriSubIDPair.Key;
						int32 SubID   = TriSubIDPair.Value;
						double& Angle = IntrinsicEdgeAngles[TID][SubID];
						Angle         = ToRadians * (Angle - AngleOffset) + RefAngle;
						Angle         = AsZeroToTwoPi(Angle);
					}
				}
				else // bow-tie or boundary
				{
					GeometricInfo.bIsInterior = false;
				}

				GeometricVertexInfo[VID] = GeometricInfo;
			}
		}, false /* bForceSingleThread*/);
	}
}

void IntrinsicCorrespondenceUtils::FSignpost::OnFlipEdge(const int32 EID, const FIndex2i Tris, const FIndex2i OpposingVerts, const FIndex2i PreFlipIndexOf, double NewAngleAtOpp0, const double NewAngleAtOpp1)
{
	constexpr  int32 AddOneModThree[3] = { 1, 2, 0 };
	constexpr  int32 AddTwoModThree[3] = { 2, 0, 1 };
	const FVector3d PreFlipDirections[2] = { IntrinsicEdgeAngles[Tris[0]], IntrinsicEdgeAngles[Tris[1]] };

	// polar directions for the four edges that don't change.
	const double bcDir = PreFlipDirections[0][AddOneModThree[PreFlipIndexOf[0]]];
	const double caDir = PreFlipDirections[0][AddTwoModThree[PreFlipIndexOf[0]]];

	const double adDir = PreFlipDirections[1][AddOneModThree[PreFlipIndexOf[1]]];
	const double dbDir = PreFlipDirections[1][AddTwoModThree[PreFlipIndexOf[1]]];



	// compute the polar edge directions for the new edge (ie. dcDir and cdDir)
	// dcDir 
	const double newAngleAtD = NewAngleAtOpp1;
	const double ToPolarAtD  = GeometricVertexInfo[OpposingVerts[1]].ToRadians;
	const double dcDir       = AsZeroToTwoPi(dbDir + ToPolarAtD * newAngleAtD);

	// cdDir 
	const double newAngleAtC = NewAngleAtOpp0;
	const double ToPolarAtC  = GeometricVertexInfo[OpposingVerts[0]].ToRadians;
	const double cdDir       = AsZeroToTwoPi(caDir + ToPolarAtC * newAngleAtC);

	IntrinsicEdgeAngles[Tris[0]] = FVector3d(cdDir, dbDir, bcDir); // edges leaving c, d, b
	IntrinsicEdgeAngles[Tris[1]] = FVector3d(dcDir, caDir, adDir); // edges leaving d, c, a
}

