// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/IntrinsicTriangulationMesh.h"
#include "Operations/MeshGeodesicSurfaceTracer.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Util/IndexUtil.h"
#include "Algo/Reverse.h"
#include "Containers/BitArray.h"
#include "Containers/Queue.h"
#include "MathUtil.h"
#include "Async/ParallelFor.h"
#include "VectorTypes.h"
#include "MatrixTypes.h"

using namespace UE::Geometry;


/**
 *  Helper functions that use triangle edge lengths instead of vertex positions to do some euclidean triangle-based calculations.
 *  Many of these are based on the "law of cosines" 
 */
namespace
{
	/**
	* Given a flat triangle with edges L0, L1, L2 in CCW order, compute the angle between
	* the L0 and L2 edge in radians [0, Pi] range.
	* 
	* used by Sharp, Soliman and Crane [2019, ACM Transactions on Graphics] section 3.2
	*
	* but can be understood as trivial application of "the law of cosines"
	*/
	double InteriorAngle(const double L0, const double L1, const double L2)
	{
		// V1 = V2 - V0.   L1^2 = ||V1||^2 = ||V2||^2 + ||V0||^2 - 2 Dot(V2, V0)  =  L2^2 + L0^2 - 2 L2 L0 CosTheta
		// CosTheta = (L0^2 + L1^2 - L2^2) / (2 L0 L1)
		double CosTheta = (L0 * L0 + L2 * L2 - L1 * L1) / (2. * L0 * L2);

		// account for roundoff, theoretically this would already be in the [-1,1] range
		CosTheta = FMath::Clamp(CosTheta, -1., 1.);

		return FMath::Acos(CosTheta);
	}

	/**
	* For a triangle, use the law of cos to compute the third edge length
	*  - Given interior angle in radians AngleR, and adjacent edge lengths L0 and L1, this computes the opposing edge length. 
	*/
	double LawOfCosLength(const double AngleR, const double L0, const double L1)
	{
		double L2sqr = L0 * L0 + L1 * L1 - 2.*L0*L1*FMath::Cos(AngleR);
		// theoretically this is always positive, but protect round-off.   Alternately, could compute (L0-L1)^2 + 4 L0 L1 Sin^2 ( AngleR/2)  
		L2sqr = FMath::Max(L2sqr, 0.);
		return FMath::Sqrt(L2sqr);
	}

	/**
	* Compute the unsigned distance between two points, described as barycentric coordinates
	* relative to a triangle with side lengths L0, L1, L2 is CCW order.
	* @params L0, L1, L2 - triangle side lengths in CCW order
	* @param BCPoint0  - Barycentric coordinates for the first point
	* @param BCPoint1  - Barycentric coordinates for the second point
	*
	* NB: it is assumed that the points are within the triangle, and the triangle is valid (i.e. lengths satisfy triangle inequality)
	*/
	double DistanceBetweenBarycentricPoints(const double L0, const double L1, const double L2, const FVector3d& BCPoint0, const FVector3d& BCPoint1)
	{
		// see Sharp, Soliman and Crane [2019, ACM Transactions on Graphics] or  Schindler and Chen [2012, Section 3.2]
		using Vector3Type = FVector3d;
		using ScalarType = double;
		const ScalarType Zero(0);

		const Vector3Type u = BCPoint1 - BCPoint0;
		const ScalarType dsqr = -(L0 * L0 * u[0] * u[1] + L1 * L1 * u[1] * u[2] + L2 * L2 * u[2] * u[0]);
		const ScalarType d = (dsqr < Zero) ? Zero : sqrt(dsqr);
		return d;
	}


	/**
	* Compute a vector (in polar form) from triangle corner to a point within the triangle.
	* @params L0, L1, L2 - triangle side lengths in CCW order
	* @param BarycetricPoint - interior point in triangle
	*
	* @return, a polar vector from the vertex shared by L0, L2 to the BarycentricPoint in the form
	*          Vector[0] = radial distance
	*          Vector[1] = angle measured from the L0 side of the triangle
	*/
	FVector2d VectorToPoint(const double L0, const double L1, const double L2, const FVector3d& BarycentricPoint)
	{
		FVector3d BCpi(1., 0., 0.);
		FVector3d BCpj(0., 1., 0.);
		double rpi = DistanceBetweenBarycentricPoints(L0, L1, L2, BCpi, BarycentricPoint);
		double rpj = DistanceBetweenBarycentricPoints(L0, L1, L2, BCpj, BarycentricPoint);
		double angle = InteriorAngle(L0, rpj, rpi);
		return FVector2d(rpi, angle);
	}

	/**
	* Heron's formula for computing the area of a triangle given the lengths of all three sides.  
	*/
	double TriangleArea(const double L0, const double L1, const double L2)
	{
		// note: since this is a triangle, the lengths should satisfy triangle inequality, meaning this should be positive
		double AreaSqrTimes16 = FMath::Max(0., (L0 + L1 + L2) * (-L0 + L1 + L2 ) * (L0 -L1 + L2) * (L0 + L1 -L2));
		return FMath::Sqrt(AreaSqrTimes16) * 0.25;
	}
	double TriangleArea(const FVector3d& Ls)
	{
		return TriangleArea(Ls[0], Ls[1], Ls[2]);
	}
	/**
	* Compute the cotangent of the interior angle opposite the edge L0, give a triangle with edges {L0, L1, L2} in CCW order
	*/
	double ComputeCotangent(const double L0, const double L1, const double L2)
	{
		const double Area = TriangleArea(L0, L1, L2); 
		const double CT = 0.25 * (-L0 * L0 + L1 * L1 + L2 * L2) / Area;
		return CT;
	}

	/**
	* Return the location of third vertex in a triangle in R2 with prescribed lengths
	* such that the resulting triangle could be constructed as { (0,0),  (L0, 0), Result}.
	* 
	* NB: this assumes, but does not check, that the lengths satisfy the triangle inequality
	*/
	FVector2d ComputeOpposingVert2d(const double L0, const double L1, const double L2)
	{
		const double Area = TriangleArea(L0, L1, L2);
		const double Y = 2. * Area / L0;
		const double X = FMath::Sqrt(FMath::Max(0., (L2 - Y) *  (L2 + Y)));
		FVector2d ResultPos(X, Y);
		// angle between L0 and L2 edges can be computed form 2 * L0 * L2 * Cos(theta) = L0^2 + L2^2 - L1^2
		if (L0 * L0 + L2 * L2 < L1 * L1)
		{
			ResultPos.X = -X; // angle > 90-degrees.
		}
		return ResultPos;
	}

	/**
	* Given three side lengths that satisfy the triangle inequality,
	* this updates the 2d positions to be the vertices of a triangle, 
	* the edge lengths of which ( when in CCW order)  match the prescribed lengths.  
	* with vertices { (0,0),  (L0, 0), (X, Y)} where Y > 0
	*/
	void TriangleFromLengths(const double L0, const double L1, const double L2, FVector2d& p0, FVector2d& p1, FVector2d& p3)
	{
		p0 = FVector2d(0., 0.);
		p1 = FVector2d(L0, 0.);
		p3 = ComputeOpposingVert2d(L0, L1, L2);
	}

	
};


/**
 *  Utilities and generic implementation for flipping a mesh to Delaunay  
 */
namespace
{
	/**
	* LIFO for unique DynamicMesh IDs, can be constructed from any FRefCountVector::IndexEnumerable
	* and supports a filtering function on construction.  If the filtering function is absent, all
	* unique elements will be added, otherwise only those selected by the filter.
	*    
	*	for example: 
	* 
	*	 auto AddToQueueFilter = [](int ID)->bool{...};
	* 
	*    FIndexLIFO EdgeQueueLIFO(Mesh.EdgeIndicesItr(), AddToQueueFilter);
	*/
	class FIndexLIFO
	{
	public:
		// constructor that adds all unique IDs from IndexEnumerable
		FIndexLIFO(FRefCountVector::IndexEnumerable IDEnumerable)
			: IsEnqueued(false, (int32)IDEnumerable.Vector->GetMaxIndex())
		{
			const int32 NumIDs = (int32)IDEnumerable.Vector->GetCount();
			IDs.Reserve(NumIDs);
			for (int32 ID : IDEnumerable)
			{
				Enqueue(ID);
			}
		}
		// filtering constructor that only adds IDs for which Filter(ID) is true
		FIndexLIFO(FRefCountVector::IndexEnumerable IDEnumerable, const TFunctionRef<bool(int32)>& Filter)
			: IsEnqueued(false, (int32)IDEnumerable.Vector->GetMaxIndex())
		{
			const int32 NumIDs = (int32)IDEnumerable.Vector->GetCount();
			IDs.Reserve(NumIDs);
			for (int32 ID : IDEnumerable)
			{
				if (Filter(ID))
				{
					Enqueue(ID);
				}
			}
		}
		bool Dequeue(int32& IDOut)
		{
			if (IDs.Num())
			{
				IDOut = IDs.Pop(EAllowShrinking::No);
				IsEnqueued[IDOut] = false;
				return true;
			}
			IDOut = -1;
			return false;
		}
		void Enqueue(int32 ID)
		{
			if (!IsEnqueued[ID])
			{
				IDs.Add(ID);
				IsEnqueued[ID] = true;
			}
		}
	private:
		FIndexLIFO();
		TBitArray<FDefaultBitArrayAllocator> IsEnqueued;
		TArray<int32> IDs;
	};

	/**
	* FIFO for unique Dynamic Mesh IDs - can be constructed from any FRefCountVector::IndexEnumerable
	* and supports a filtering function on construction.  If the filtering function is absent, all
	* unique elements will be added, otherwise only those selected by the filter.
	*
	*	for example:
	*
	*	 auto AddToQueueFilter = [](int ID)->bool{...};
	*
	*    FIndexFIFO EdgeQueueFIFO(Mesh.EdgeIndicesItr(), AddToQueueFilter);
	*/
	class FIndexFIFO
	{
	public:
		// constructor that adds all unique IDs from IndexEnumerable
		FIndexFIFO(FRefCountVector::IndexEnumerable IDEnumerable)
			: IsEnqueued(false, (int32)IDEnumerable.Vector->GetMaxIndex())
		{
			const int32 NumEdges = (int32)IDEnumerable.Vector->GetMaxIndex();
			for (int32 ID : IDEnumerable)
			{
				Enqueue(ID);
			}
		}

		// filtering constructor that only adds IDs for which Filter(ID) is true  Note: Filter is called in parallel.
		FIndexFIFO(FRefCountVector::IndexEnumerable IDEnumerable, const TFunctionRef<bool(int32)>& Filter)
			: IsEnqueued(false, (int32)IDEnumerable.Vector->GetMaxIndex())
		{
			// parallel evaluation that assumes Filter is expensive
			
			const int32 MaxID = (int32)IDEnumerable.Vector->GetMaxIndex();
			TArray<int32> ToInclude;
			ToInclude.AddZeroed(MaxID);
			ParallelFor(MaxID, [&ToInclude, &Filter](int32 ID)
			{
				if (Filter(ID))
				{
					ToInclude[ID] = 1;
				}
			});
			for (int32 ID = 0; ID < MaxID; ++ID)
			{
				if (ToInclude[ID] == 1)
				{
					Enqueue(ID);
				}
			}
		}

		bool Dequeue(int32& IDOut)
		{
			if (IDs.Dequeue(IDOut))
			{
				IsEnqueued[IDOut] = false;
				return true;
			}
			IDOut = -1;
			return false;
		}
		void Enqueue(int32 ID)
		{
			if (!IsEnqueued[ID])
			{
				IDs.Enqueue(ID);
				IsEnqueued[ID] = true;
			}
		}
	private:
		FIndexFIFO();
		TBitArray<FDefaultBitArrayAllocator> IsEnqueued;
		TQueue<int32> IDs;
	};


	/**
	* Carry out edge flips on intrinsic mesh until either the MaxFlipCount is reached or the mesh is fully Delaunay and returns
	* the number of flips.
	* Note: There can be cases where an edge can not flip (e.g. if the resulting edge already exists)
	*       - for this reason, there may be some "uncorrected" edges in the resulting mesh.
	* 
	* @param IntrinsicMesh - The mesh to be operated on.
	* @param Uncorrected   - contains on return, all the edges that could not be flipped.
	* @param Threshold     - edges with cotan weight less than threshold should be flipped.
	*/
	template <typename MeshType>
	int32 FlipToDelaunayImpl(MeshType& Mesh, TSet<int32>& Uncorrected, const int32 MaxFlipCount, double Threshold = -TMathUtilConstants<double>::ZeroTolerance)
	{
	Uncorrected.Empty();

	// returns true if an edge should be flipped.
	auto EdgeShouldFlipFilter = [&Mesh, Threshold](int32 EID)->bool
	{
		if (!Mesh.IsEdge(EID) || Mesh.IsBoundaryEdge(EID))
		{
			return false;
		}
	
		const double CotanWeightValue = Mesh.EdgeCotanWeight(EID);
		
		// Delaunay test
		return CotanWeightValue < Threshold;
	};

	// Enqueue the "bad" edges.
	FIndexFIFO EdgeQueue(Mesh.EdgeIndicesItr(), EdgeShouldFlipFilter);

	// flip away the bad edges
	int32 FlipCount = 0;
	int32 EID;
	while (EdgeQueue.Dequeue(EID) && FlipCount < MaxFlipCount)
	{
		if (EdgeShouldFlipFilter(EID))
		{
			FIntrinsicTriangulation::FEdgeFlipInfo EdgeFlipInfo;
			const EMeshResult Result = Mesh.FlipEdge(EID, EdgeFlipInfo);

			if (Result == EMeshResult::Ok)
			{
				FlipCount++;
				for (int32 t = 0; t < 2; ++t)
				{
					const FIndex3i TriEIDs = Mesh.GetTriEdges(EdgeFlipInfo.Triangles[t]);
					int32 IndexOf = TriEIDs.IndexOf(EID);
					EdgeQueue.Enqueue(TriEIDs[(IndexOf + 1) % 3]);
					EdgeQueue.Enqueue(TriEIDs[(IndexOf + 2) % 3]);
				}
			}
			else
			{
				Uncorrected.Add(EID);
			}
		}
	}

	return FlipCount;
	}

};

namespace SignpostSufaceTraceUtil
{
	using namespace IntrinsicCorrespondenceUtils;
	/**
	* helper code that uses FSignpost to trace from a specified intrinsic mesh vertex in a given direction.
	* 
	*Note: this doesn't check the validity of the start point - the calling code should do that
	*/
	UE::Geometry::FMeshGeodesicSurfaceTracer TraceFromIntrinsicVert( const FSignpost& SignpostData, const int32 TraceStartVID, 
		                                                             const double TracePolarAngle, const double TraceDist )
	{
		const FDynamicMesh3* SurfaceMesh                            = SignpostData.SurfaceMesh; 
		const FSurfacePoint& StartSurfacePoint                      = SignpostData.IntrinsicVertexPositions[TraceStartVID];
		const FSurfacePoint::FSurfacePositionUnion& SurfacePosition = StartSurfacePoint.Position;

		double ActualDist = 0.;
		UE::Geometry::FMeshGeodesicSurfaceTracer SurfaceTracer(*SurfaceMesh);
		switch (StartSurfacePoint.PositionType)
		{
		case FSurfacePoint::EPositionType::Vertex:
		{
			// convert vertex location
			const int32 ExtrinsicVID    = StartSurfacePoint.Position.VertexPosition.VID;
			const int32 ExtrinsicRefEID = SignpostData.VIDToReferenceEID[ExtrinsicVID];

			const FMeshGeodesicSurfaceTracer::FMeshTangentDirection TangentDirection = { ExtrinsicVID, ExtrinsicRefEID, TracePolarAngle };
			ActualDist = SurfaceTracer.TraceMeshFromVertex(TangentDirection, TraceDist);
		}
		break;
		case FSurfacePoint::EPositionType::Edge:
		{
			// convert edge location -  Not used, and might have bugs
			const int32 RefSurfaceEID   = SurfacePosition.EdgePosition.EdgeID;
			double Alpha                = SurfacePosition.EdgePosition.Alpha;
			// convert to BC
			const int32 SurfaceTID      = SurfaceMesh->GetEdgeT(RefSurfaceEID).A;
			const FIndex2i SurfaceEdgeV = SurfaceMesh->GetEdgeV(RefSurfaceEID);
			const int32 IndexOf         = SurfaceMesh->GetTriEdges(SurfaceTID).IndexOf(RefSurfaceEID);
			const bool bSameOrientation = (SurfaceMesh->GetTriangle(SurfaceTID)[IndexOf] == SurfaceEdgeV.A);
			FVector3d BaryPoint(0., 0., 0.);
			if (!bSameOrientation)
			{
				Alpha = 1. - Alpha;
			}
			BaryPoint[IndexOf] = Alpha;
			BaryPoint[(IndexOf + 1) % 3] = 1. - Alpha;

			FMeshGeodesicSurfaceTracer::FMeshSurfaceDirection DirectionOnSurface(RefSurfaceEID, TracePolarAngle);
			ActualDist = SurfaceTracer.TraceMeshFromBaryPoint(SurfaceTID, BaryPoint, DirectionOnSurface, TraceDist);
		}
		break;
		case FSurfacePoint::EPositionType::Triangle:
		{
			// trace from BC point in triangle
			const int32 SurfaceTID = SurfacePosition.TriPosition.TriID;

			const FVector3d BaryPoint = SurfacePosition.TriPosition.BarycentricCoords;
									
			const int32 RefSurfaceEID = SignpostData.TIDToReferenceEID[SurfaceTID];

			FMeshGeodesicSurfaceTracer::FMeshSurfaceDirection DirectionOnSurface(RefSurfaceEID, TracePolarAngle);
			SurfaceTracer.TraceMeshFromBaryPoint(SurfaceTID, BaryPoint, DirectionOnSurface, TraceDist);
		}
		break;
		default:
		{
			// not possible
			check(0);
		}
		}

		// RVO..
		return SurfaceTracer;
	}
};

namespace FNormalCoordSurfaceTraceImpl
{
	using namespace IntrinsicCorrespondenceUtils;
	

	// @return ID of a vertex-adjacent edge that will allow all adjacent edges to be visited in CCW order.
	template <typename MeshType>
	int32 IdentifyInitialAdjacentEdge(const MeshType& Mesh, const int32 VID)
	{
		if (!Mesh.IsVertex(VID))
		{
			return MeshType::InvalidID;
		}

		if (Mesh.IsBoundaryVertex(VID))
		{
			// find the clockwise-most edge
			for (int32 NbrEID : Mesh.VtxEdgesItr(VID))
			{
				if (Mesh.IsBoundaryEdge(NbrEID))
				{
					const int32 NbrTID  = Mesh.GetEdgeT(NbrEID).A;
					const int32 IndexOf = Mesh.GetTriEdges(NbrTID).IndexOf(NbrEID);
					if (Mesh.GetTriangle(NbrTID)[IndexOf] == VID)
					{
						return NbrEID;
					}
				}
			}

			return MeshType::InvalidID; // shouldn't reach 
		}
		else
		{
			for (int32 NbrEID : Mesh.VtxEdgesItr(VID))
			{
				// get the first
				return NbrEID;
			}
			return MeshType::InvalidID; // shouldn't reach here
		}
	}

	
	/**
	* Low-level method for use with normal-coordinate equipped intrinsic meshes.
	* 
	* Given a surface edge that intersects (but does not terminate at) an intrinsic vertex 'IntrinsicVID' 
	* this function identifies the adjacent intrinsic mesh elements (i.e. edges or faces ) also containing that surface edge.
	* 
	* This is used when following a surface mesh edge across an intrinsic mesh (using normal coordinates). 
	* Note: this situation is the result of an edge split of an intrinsic edge that was initially aligned with a surface edge.
	*/ 
	template <typename IntrinsicMeshType>
	bool GetAdjElementsContainingSurfaceEdge(const IntrinsicMeshType& IntrinsicMesh, const int32 IntrinsicVID, TArray<TTuple<int32, int32>>& IntrisicTIDs, TArray<int32>& IntrisicEIDs)
	{
		checkSlow( IsEdgePoint(IntrinsicMesh.GetVertexSurfacePoint(IntrinsicVID)) );

		const FNormalCoordinates& NCoords = IntrinsicMesh.GetNormalCoordinates();

		for (const int32 EID : IntrinsicMesh.VtxEdgesItr(IntrinsicVID))
		{
			if (NCoords.IsSurfaceEdgeSegment(EID))
			{
				IntrisicEIDs.Add(EID);
			}

			// triangle that has EID as an outgoing edge
			const int32 TID = [&]
								{
									const FIndex2i AdjTris = IntrinsicMesh.GetEdgeT(EID);
									for (int32 i = 0; i < 2; ++i)
									{
										const int32 TID = AdjTris[i];
										if (TID == IntrinsicMeshType::InvalidID)
										{
											continue;
										}

										const FIndex3i TriEIDs    = IntrinsicMesh.GetTriEdges(TID);
										const int32 IndexOf       = TriEIDs.IndexOf(EID);
										const bool bIsEIDOutgoing = (IntrinsicMesh.GetTriangle(TID)[IndexOf] == IntrinsicVID);
										if (bIsEIDOutgoing)
										{
											return TID;
										}
									}
									return IntrinsicMeshType::InvalidID;
								}();

			if (TID == IntrinsicMeshType::InvalidID)
			{
				continue;
			}

			const FIndex3i TriEIDs = IntrinsicMesh.GetTriEdges(TID);
			const int32 IndexOf = TriEIDs.IndexOf(EID);

			// permuted triangle is {a, b, c} where a is VID.
			// given a surface edge crosses 'a' it will traverse the face of this triangle iff it exits face bc.

			const FIndex3i PermutedEIDs = IntrinsicMesh.Permute(IndexOf, TriEIDs);
			const int32 Nbc = NCoords.NumEdgeCrossing(PermutedEIDs[1]);           // number of surface edges that exits face bc

			if (Nbc == 0) // a surface edge that crosses 'a' doesn't exit this triangle face bc
			{
				continue;
			}

			const int32 Eca_b = NCoords.NumCornerEmanatingRefEdges(PermutedEIDs, 1); // number of surface edges that exit corner b and exit ca
			if (Eca_b > 0) // a surface edge that crosses 'a' is blocked from exiting this triangle by other surface edges
			{
				continue;
			}
			const int32 Eab_c = NCoords.NumCornerEmanatingRefEdges(PermutedEIDs, 2); // number of surface edges that exit corner c and exit ab
			if (Eab_c > 0) // a surface edge that crosses 'a' is blocked from exiting this triangle by other surface edges
			{
				continue;
			}

			const int32 Cb = NCoords.NumCornerCrossingRefEdges(PermutedEIDs, 1); // number of surface edges that cross corner b
			const int32 Cc = NCoords.NumCornerCrossingRefEdges(PermutedEIDs, 2); // number of surface edges that cross corner c

			if (Nbc > Cb + Cc) // a surface edge that crosses 'a' must exit face bc
			{
				IntrisicTIDs.Add(TTuple<int32, int32>(TID, IndexOf));
			}

		}

		return (IntrisicTIDs.Num() > 0 || IntrisicEIDs.Num() > 0);
	}

	
	/**
	* Low-level method for use with normal-coordinate equipped intrinsic meshes.
	* 
	* Given an intrinsic mesh vertex that is situated on a surface mesh edge and information that specifies the incoming section of the surface edge,
	* this function computes the crossing point where the outgoing section of the surface edge exits the intrinsic vertex 1-ring.
	* 
	* if bIncomingIsEdge == true, then the incoming section of the surface edge is along the intrinsic edge EdgeID = IncomingID, 
	* otherwise it crossed the intrinsic triangle TriID = IncomingID. 
	* 
	* on return the IncomingID and bIncomingIsEdge have been updated with the surface mesh elements traversed by the path to next crossing,
	* and the crossing itself is encoded in the returned FEdgeAndCrossing struct.  for convenience VID is also updated if the next crossing is an intrinsic vertx.
	*/
	template <typename IntrinsicMeshType>   
	FNormalCoordinates::FEdgeAndCrossingIdx NextCrossingFromEdgePoint(const IntrinsicMeshType& IntrinsicMesh, int32& VID, int32& IncomingID, bool& bIncomingIsEdge)
	{

		checkSlow(IsEdgePoint(IntrinsicMesh.GetVertexSurfacePoint(VID)));
	

		// four cases: the outer product of   "enters {on intrinsic edge, across intrinsic tri}"   with "exits {on intrinsic edge, across intrinsic tri}"

		// find intrinsic mesh elements adjacent to VID that support the surface edge
		TArray<TTuple<int32, int32>> TIDs;   TArray<int32> EIDs;
		GetAdjElementsContainingSurfaceEdge(IntrinsicMesh, VID, TIDs, EIDs);

		// only one edge crosses the implicit vert, so there can only be 2 sides to that edge ..
		checkSlow(TIDs.Num() + EIDs.Num() == 2);

		

		auto EncodeExitsOnEdge = [&](const int32 ExitEID, FNormalCoordinates::FEdgeAndCrossingIdx& Xing )
							{
								const typename IntrinsicMeshType::FEdge ExitEdge = IntrinsicMesh.GetEdge(ExitEID);

								const int32 AdjTID       = ExitEdge.Tri[0];
								const int32 IndexOfe     = IntrinsicMesh.GetTriEdges(AdjTID).IndexOf(ExitEID);
								const int32 IndexOfNextV = (IntrinsicMesh.GetTriangle(AdjTID)[IndexOfe] == VID) ? (IndexOfe + 1) % 3 : IndexOfe;

								Xing.TID = ExitEdge.Tri[0];
								Xing.EID = IndexOfNextV;
								Xing.CIdx = 0;
								return Xing;
							};

		auto EncodeExitsAcrossTri = [&](const TTuple<int32, int32>& TridAndIdx, FNormalCoordinates::FEdgeAndCrossingIdx& Xing)
								{
									const FNormalCoordinates& NCoords = IntrinsicMesh.GetNormalCoordinates();
									const int32 ExitTID               = TridAndIdx.Get<0>();
									const int32 OutgoingIndex         = TridAndIdx.Get<1>();

									// when counting the surface edge that cross the triangle face opposite to VID, what number is the single edge that eminates from VID?
									const FIndex3i TriEIDs = IntrinsicMesh.GetTriEdges(ExitTID);

									// permuted triangle is {a, b, c} where a is VID.
									// given a surface edge crosses 'a' it will traverse the face of this triangle iff it exits face bc.

									const FIndex3i PermutedEIDs = IntrinsicMesh.Permute(OutgoingIndex, TriEIDs);
									const int32 Cc = NCoords.NumCornerCrossingRefEdges(PermutedEIDs, 2); // number of surface edges that cross corner c

									// counting from corner c towards b, this is the 'CC +1'-th surface edge to cross the intrinsic edge 
									const int32 CIdx = Cc + 1;

									Xing.TID = ExitTID;
									Xing.EID = PermutedEIDs[1];
									Xing.CIdx = CIdx;
									return Xing;
								};

		FNormalCoordinates::FEdgeAndCrossingIdx Xing({-1, -1, -1});

		if (bIncomingIsEdge) // entered on an intrinsic edge
		{
			const int32 EnterEID = IncomingID;

			const bool bExitsOnEdge    = (EIDs.Num() == 2);
			const bool bExitsAccrosTri = (TIDs.Num() > 0);

			if (bExitsOnEdge)
			{
				checkSlow(!bExitsAccrosTri);

				const int32 Id = (EIDs[0] == EnterEID) ? 1 : 0;
				checkSlow(EIDs[1 - Id] == EnterEID);

				const int32 ExitEID = EIDs[Id];
				EncodeExitsOnEdge(ExitEID, Xing);
				
				VID             = IntrinsicMesh.GetTriangle(Xing.TID)[Xing.EID];
				IncomingID      = ExitEID;
				bIncomingIsEdge = true;
			}
			else if (bExitsAccrosTri)
			{
				checkSlow(!bExitsOnEdge);

				checkSlow(TIDs.Num() == 1); // can only traverse a single tri ( we already know it entered by an edge)
				EncodeExitsAcrossTri(TIDs[0], Xing);

				IncomingID      = TIDs[0].Get<0>();
				bIncomingIsEdge = false;
			}
			else
			{
				checkSlow(0);
			}
		}
		else // entered across and intrinsic triangle
		{
			const int32 EnterTID       = IncomingID;
			const bool bExitsOnEdge    = (EIDs.Num() == 1);
			const bool bExitsAcrossTri = (TIDs.Num() == 2);

			if (bExitsOnEdge)
			{
				const int32 ExitEID = EIDs[0];
				EncodeExitsOnEdge(ExitEID, Xing);

				VID             = IntrinsicMesh.GetTriangle(Xing.TID)[Xing.EID];
				IncomingID      = ExitEID;
				bIncomingIsEdge = true;
			}
			else if (bExitsAcrossTri)
			{
				const int32 Id = (TIDs[0].Get<0>() == EnterTID) ? 1 : 0;
				EncodeExitsAcrossTri(TIDs[Id], Xing);

				IncomingID      = TIDs[0].Get<0>();
				bIncomingIsEdge = false;
			}
			else
			{
				checkSlow(0);
			}
		}

		return Xing;
	}

	template <typename IntrinsicMeshType>
	void ContinueFromEdgePointCrossing(const IntrinsicMeshType& IntrinsicMesh, int32 VID, int32 IncomingID, bool bIncomingIsEdge, TArray<FNormalCoordinates::FEdgeAndCrossingIdx>& Crossings)
	{
		int32 CIdx = 0;
		while (CIdx == 0 && IsEdgePoint(IntrinsicMesh.GetVertexSurfacePoint(VID)))
		{
			FNormalCoordinates::FEdgeAndCrossingIdx Xing = NextCrossingFromEdgePoint(IntrinsicMesh, VID, IncomingID, bIncomingIsEdge);
			Crossings.Add(Xing);
			CIdx = Xing.CIdx;
		}
	}

	/**
	* Low-level method for use with normal-coordinate equipped intrinsic meshes.
	* Traces the p-th surface edge crossing of the intrinsic edge edgeID across the adjacent intrinsic triangle TriID
	* here 'P' is counted is the CCW direction (relative to TriID).
	* 
	* on returns TTuple<in32, int32>  = {EdgeID,P}  
	* on return if 'P' == 0, the path exits a vertex otherwise it exits the updated EdgeID (with index P relative to the next triangle it enters)
	* 
	* This is adapted from Algorithm 2 of  Gillespi et al, 2020
	*/ 
	template <typename IntrinsicMeshType>
	TTuple<int32, int32> GetCrossingExit(const IntrinsicMeshType& IntrinsicMesh, const FNormalCoordinates& NCoords, const int32 TriID, const int32 EdgeIDin, const int32 Pin)
	{

		TTuple<int32, int32> ExitCoord(EdgeIDin, Pin);
		int32& EdgeID = ExitCoord.Get<0>();
		int32& P = ExitCoord.Get<1>();

		const FIndex3i TriEIDs = IntrinsicMesh.GetTriEdges(TriID);
		const int32 IndexOf = TriEIDs.IndexOf(EdgeID);

		const int32 Edge_ji = EdgeID;
		const int32 Edge_il = TriEIDs[(IndexOf + 1) % 3];
		const int32 Edge_lj = TriEIDs[(IndexOf + 2) % 3];

		const int32 N_ji = NCoords.NumEdgeCrossing(Edge_ji);
		const int32 N_il = NCoords.NumEdgeCrossing(Edge_il);
		const int32 N_lj = NCoords.NumEdgeCrossing(Edge_lj);

		

		if (N_ji > N_lj + N_il)  // case 1
		{
			// some of the N_ji edges that cross edge IJ must connect with the vertex at L 
			if (P <= N_il)
			{
				// exits side il
				EdgeID = Edge_il;
				P      = Pin;
			}
			else if ((N_il < P) && (P <= N_ji - N_lj))
			{
				// this path terminates at vertex
				// this is a slight abuse.  The convention, when CrossingIndex = 0, we specify the vertex by the edge that originates there.
				EdgeID = TriEIDs.IndexOf(Edge_lj); // GetTriange()[EdgeID] = vert.  note changed from Edge_lj where  i = GetTriangle()[TriEIDs.IndexOf(EdgeID_lj)] will be the vertex.
				P = 0;
			}
			else
			{
				EdgeID = Edge_lj;
				P      = Pin - (N_ji - N_lj);
			}

		}
		else if (N_lj > N_ji + N_il)  // case 2
		{
			EdgeID = Edge_lj;
			P      = Pin + (N_lj - N_ji);

		}
		else if (N_il > N_ji + N_lj) // case 3
		{
			EdgeID = Edge_il;
			P      = Pin;

		}
		else  // case 4
		{
			const int32 Cij_l = (N_lj + N_il - N_ji) / 2;
			if (P <= N_il - Cij_l)
			{
				EdgeID = Edge_il;
				P      = Pin;

			}
			else
			{
				const int32 Clj_i = (N_il + N_ji - N_lj) / 2;
				EdgeID = Edge_lj;
				P      = Pin - Clj_i + Cij_l;
			}
		}
		return ExitCoord;
	}
	
	/**
	* Low-level method: code that uses normal coordinates to continues tracing a surface edge across an intrinsic mesh until it encounters a vertex.
	*  
	* 
	* This function requires that the fist edge crossing has already been computed will populate the a sequence of intrinsic edge crossings for the surface mesh edge
	* until the path encounters a vertex.  The actual crossings are recorded as a list of the edges crossed, and not the location on the individual edges.  
	*/
	template <typename IntrinsicMeshType>
	void  ContinueTraceSurfaceEdgeAcrossFaces( const IntrinsicMeshType& IntrinsicMesh, TArray<FNormalCoordinates::FEdgeAndCrossingIdx>& Crossings )
	{
		const FNormalCoordinates::FEdgeAndCrossingIdx& LastXing = Crossings.Last();
		checkSlow(LastXing.CIdx != 0); // continuing a face crossing

		const int32 StartTID = LastXing.TID;
		const int32 StartEID = LastXing.EID;
		const int32 StartP   = LastXing.CIdx;

		const FNormalCoordinates& NCoords = IntrinsicMesh.GetNormalCoordinates();

		if (!IntrinsicMesh.IsTriangle(StartTID))
		{
			return;
		}

		const FIndex3i StartTriEIDs = IntrinsicMesh.GetTriEdges(StartTID);
		checkSlow(StartTriEIDs.IndexOf(StartEID) != -1);

		const int32 NumXStartEID = NCoords.NumEdgeCrossing(StartEID);
		if (StartP > NumXStartEID || StartP < 1) // note the actual permitted range of P is smaller.. 
		{
			checkSlow(0); // this shouldn't happen
			return;
		}

		// This is adapted from Algorithm 2 of  Gillespi et al, 2020
		auto GetNextCrossing = [&IntrinsicMesh, &NCoords](int32& TriID, int32& EdgeID, int32& P)
								{
									// get next tri
									TriID = [&]
										{
											const FIndex2i EdgeT = IntrinsicMesh.GetEdgeT(EdgeID);
											return (EdgeT.A == TriID) ? EdgeT.B : EdgeT.A;
										}();
									checkSlow(TriID != -1);

									const TTuple<int32, int32> ExitEIDandP = GetCrossingExit(IntrinsicMesh, NCoords, TriID, EdgeID, P);
									EdgeID = ExitEIDandP.Get<0>();
									P = ExitEIDandP.Get<1>();
								};

		int32 P = StartP;
		int32 EID = StartEID;
		int32 TID = StartTID;
		do
		{
			GetNextCrossing(TID, EID, P);
			auto& Xing = Crossings.AddZeroed_GetRef();
			Xing.TID   = TID;
			Xing.EID   = EID;
			Xing.CIdx  = P;
			checkSlow(P > -1);
		} while (P != 0); // P = 0 when the path terminates ( paths terminate at vertices only)


	}

	/**
	* Low-level method for use with normal-coordinate equipped intrinsic meshes.
	* 
	* Given a partial trace of a surface edge, continue the trace across the intrinsic mesh until the path terminates at one end
	* of the surface edge.
	* 
	* Crossings.Last() must be an edge crossing, and not a vertex point.
	*/ 
	template <typename IntrinsicMeshType>
	void ContinueTraceSurfaceEdge(const IntrinsicMeshType& IntrinsicMesh, TArray<FNormalCoordinates::FEdgeAndCrossingIdx>& Crossings)
	{
		if (Crossings.Num() == 0)
		{
			return;
		}

		auto IsIntrinsicVertex = [](const FNormalCoordinates::FEdgeAndCrossingIdx& Xing)->bool
									{
										return (Xing.CIdx == 0);
									};

		checkSlow(!IsIntrinsicVertex(Crossings.Last()));

		auto IsSurfaceVertex = [&IntrinsicMesh, &IsIntrinsicVertex](const FNormalCoordinates::FEdgeAndCrossingIdx& Xing)->bool
								{
									if (!IsIntrinsicVertex(Xing))
									{
										return false;
									}

									// convert to intrinsic VID
									const int32 IndexOfV = Xing.EID;
									const int32 IntrinsicVID = IntrinsicMesh.GetTriangle(Xing.TID)[IndexOfV];

									return IsVertexPoint(IntrinsicMesh.GetVertexSurfacePoint(IntrinsicVID));
								};
		
		
		while (!IsSurfaceVertex(Crossings.Last()))
		{
			const FNormalCoordinates::FEdgeAndCrossingIdx& LastXing = Crossings.Last();
			if (!IsIntrinsicVertex(LastXing))
			{
				// trace enters intrinsic triangle edge - continue across intrinsic triangle faces.
				ContinueTraceSurfaceEdgeAcrossFaces(IntrinsicMesh, Crossings);
				
			}
			else
			{
				// trace hit an intrinsic vertex, coming from an intrinsic triangle face. 

				// convert to intrinsic VID
				const int32 IndexOfV = LastXing.EID;
				const int32 IntrinsicVID = IntrinsicMesh.GetTriangle(LastXing.TID)[IndexOfV];

				checkSlow(IsEdgePoint(IntrinsicMesh.GetVertexSurfacePoint(IntrinsicVID)));

				// advance along the surface edge until it either enters an intrinsic triangle or 
				// follows on more intrinsic edges to terminate at a surface vertex. 
				const bool bEnterTypeEdge = false; // we entered across a triangle face
				const int32 EnterEID = LastXing.TID;
				ContinueFromEdgePointCrossing(IntrinsicMesh, IntrinsicVID, EnterEID, bEnterTypeEdge, Crossings);
			}
		}
	}

	/**
	* Low-level method for use with normal-coordinate equipped intrinsic meshes.
	* 
	* Traces a surface edge across an intrinsic mesh, the result is an array of intrinsic edges crossed and an integer coordinates
	* that specify the intrinsic edge specific index of the crossing.   
	* 
	* The surface edge to trace is specified by the triangle adjacent to the edge, and the local index of the edge within that triangle
	*/ 
	template <typename IntrinsicMeshType>
	TArray<FNormalCoordinates::FEdgeAndCrossingIdx> TraceSurfaceEdge( const IntrinsicMeshType& IntrinsicMesh,
		                                                              const int32 SurfaceTID, const int32 IndexOf )
	{
		const FNormalCoordinates& NCoords = IntrinsicMesh.GetNormalCoordinates();
		const FDynamicMesh3& SurfaceMesh  = *NCoords.SurfaceMesh;

		if (!SurfaceMesh.IsTriangle(SurfaceTID) || IndexOf > 2 || IndexOf < 0)
		{
			TArray< FNormalCoordinates::FEdgeAndCrossingIdx > EmptyCrossings;
			return EmptyCrossings;
		}


		// the origin vertex
		const int32 StartVID = SurfaceMesh.GetTriangle(SurfaceTID)[IndexOf];
		// the surface edge we are tracing
		const int32 TraceEID = SurfaceMesh.GetTriEdge(SurfaceTID, IndexOf);

		
		const int32 RefEID = NCoords.VIDToReferenceEID[StartVID];
		const int32 OrderOfTraceEID = NCoords.GetEdgeOrder(StartVID, TraceEID);
		const int32 ValenceOfStartVID = NCoords.RefVertDegree[StartVID];
		checkSlow(OrderOfTraceEID != -1);

		// data to gather about the intrinsic triangle that where the trace starts.
		struct
		{
			bool bIsAlsoSurfaceEdge = false;
			int32 TID               = -1;
			int32 EID               = -1;
			int32 IdxOf             = -1;
			int32 FirstRoundabout   = -1;
			int32 SecondRoundabout  = -1;
		} FinderInfo;

		// Identify the target intrinsic triangle where this edge trace starts.
		{

			// where to start when visiting the intrinsic triangles that are adjacent to the startVID
			const int32 VistorStartEID = IdentifyInitialAdjacentEdge(IntrinsicMesh, StartVID);

			// test for the special case when the StartVID has valence one on the intrinsic mesh (ie a single triangle is wrapped into a cone)
			const FIndex2i IntrinsicEdgeT  = IntrinsicMesh.GetEdgeT(VistorStartEID);
			if (IntrinsicEdgeT.A == IntrinsicEdgeT.B)
			{
				const int32 IntrinsicEID     = VistorStartEID;
				const int32 IntrinsicTID     = IntrinsicEdgeT.A;
				const FIndex3i IntrinsicVIDs = IntrinsicMesh.GetTriangle(IntrinsicTID);
				const int32 IdxOf            = IntrinsicVIDs.IndexOf(StartVID);
				const int32 ThisRoundabout   = NCoords.RoundaboutOrder[IntrinsicTID][IdxOf];
				const bool bOnSurfaceEdge    = NCoords.IsSurfaceEdgeSegment(IntrinsicEID);
				const bool bEquivalentToSurface = bOnSurfaceEdge  && (ThisRoundabout == OrderOfTraceEID);

				FinderInfo.bIsAlsoSurfaceEdge = bEquivalentToSurface;
				FinderInfo.TID                = IntrinsicTID;
				FinderInfo.EID                = IntrinsicEID;
				FinderInfo.IdxOf              = IdxOf;
				FinderInfo.FirstRoundabout    = ThisRoundabout;
				FinderInfo.SecondRoundabout   = ThisRoundabout;
			}
			else
			{ 
				// when visiting each adjacent intrinsic triangle (in CCW order) test if the surface edge we trace is inside this triangle.
				auto EdgeFinder = [&](int32 IntrinsicTID, int32 IntrinsicEID, int32 IdxOf)->bool
									{


										const FIndex3i IntrinsicTriEIDs = IntrinsicMesh.GetTriEdges(IntrinsicTID);
										const int32 ThisRoundabout      = NCoords.RoundaboutOrder[IntrinsicTID][IdxOf];
										const bool bOnSurfaceEdge       = NCoords.IsSurfaceEdgeSegment(IntrinsicEID);
										const bool bEquivalentToSurface = bOnSurfaceEdge  && (ThisRoundabout == OrderOfTraceEID);
										int32 NextRoundabout = -1;

										bool bShouldBreak = false;

										if (bEquivalentToSurface) // test if the current intrinsic edge is the same as the surface mesh trace edge
										{
											bShouldBreak = true;
										}
										else // test if the edge starts in this intrinsic triangle
										{
											const int32 NextIntrinsicEID      = IntrinsicTriEIDs[(IdxOf + 2) % 3];
											const FIndex2i NextIntrinsicEdgeT = IntrinsicMesh.GetEdgeT(NextIntrinsicEID);
											const int32 NextIntrinsicTID      = (NextIntrinsicEdgeT.A == IntrinsicTID) ? NextIntrinsicEdgeT.B : NextIntrinsicEdgeT.A;

											if (NextIntrinsicTID != -1)
											{
												const int32 NextIndexOf = IntrinsicMesh.GetTriEdges(NextIntrinsicTID).IndexOf(NextIntrinsicEID);
												NextRoundabout          = NCoords.RoundaboutOrder[NextIntrinsicTID][NextIndexOf];
												if (NextRoundabout < ThisRoundabout) // Order = {NextRO |zero cut | ThisRO}
												{
													// we have crossed the zero roundabout cut ( e.g. NextRo = 2oclock and ThisRo = 11oclock)
													if ( NextRoundabout > OrderOfTraceEID) // Order = {NextRO, Trace | zero cut | ThisRO}
													{
														bShouldBreak = true;
													}
													else 
													{ 
														NextRoundabout += ValenceOfStartVID; // Order = {NextRO, | zero cut |, Trace(?), ThisRO}
													}
												}
												checkSlow(IntrinsicMesh.GetTriangle(NextIntrinsicTID)[NextIndexOf] == StartVID);
												//checkSlow(NextRoundabout != 0 || ThisRoundabout != 0)
												if ((NextRoundabout > OrderOfTraceEID) && (OrderOfTraceEID >= ThisRoundabout))
												{
													bShouldBreak = true;
												}
											}
											else // mesh boundary case and we made it to the last triangle w/o finding this edge. it must be in this one.
											{
												bShouldBreak = true;
											}
										}

										if (bShouldBreak)
										{
											FinderInfo.bIsAlsoSurfaceEdge = bEquivalentToSurface;
											FinderInfo.TID                = IntrinsicTID;
											FinderInfo.EID                = IntrinsicEID;
											FinderInfo.IdxOf              = IdxOf;
											FinderInfo.FirstRoundabout    = ThisRoundabout;
											FinderInfo.SecondRoundabout   = NextRoundabout;
										}

										return bShouldBreak;
									};
				VisitVertexAdjacentElements(IntrinsicMesh, StartVID, VistorStartEID, EdgeFinder);

				checkSlow(FinderInfo.TID != -1); // should have found something!
			}
		}


		TArray<FNormalCoordinates::FEdgeAndCrossingIdx > Crossings;
		// add start vertex location
		auto& StartXing = Crossings.AddZeroed_GetRef();
		StartXing.TID   = FinderInfo.TID;
		StartXing.EID   = FinderInfo.IdxOf; // TriVIDs(IdxOf) = StartVID
		StartXing.CIdx  = 0;

		
		if (FinderInfo.bIsAlsoSurfaceEdge) 
		{
			// the surface edge we are tracing is initially coincident with an intrinsic edge. 
			
			// add end intrinsic vertex location
			auto& EndXing = Crossings.AddZeroed_GetRef();
			EndXing.TID   = FinderInfo.TID;
			EndXing.EID   = (FinderInfo.IdxOf + 1) % 3;  // TriVIDs( (IdxOf +1)%3) = EndVID
			EndXing.CIdx  = 0;

			const int32 IndexOfV        = (FinderInfo.IdxOf + 1) % 3;
			const int32 EndIntrinsicVID = IntrinsicMesh.GetTriangle(FinderInfo.TID)[IndexOfV];
			
			// the surface edge will continue if the end intrinsic vertex is not a surface vertex.
			
			// advance along the surface edge until it either enters an intrinsic triangle or 
			// follows on more intrinsic edges to terminate at a surface vertex. 
			if (IsEdgePoint(IntrinsicMesh.GetVertexSurfacePoint(EndIntrinsicVID)))
			{
				const bool bEnterTypeEdge  = true;
				const int32 EnterEID       = FinderInfo.EID;
				ContinueFromEdgePointCrossing(IntrinsicMesh,  EndIntrinsicVID, EnterEID, bEnterTypeEdge, Crossings);
				
				const bool bVertexTerminated = (Crossings.Last().CIdx == 0);
				if (bVertexTerminated)
				{
					// Made it to the end of the surface edge since the trace must have hit an intrinsic vertex that is also a surface vertex. 
					return MoveTemp(Crossings);
				}
				else
				{
					// continue across intrinsic faces
					ContinueTraceSurfaceEdge(IntrinsicMesh, Crossings);
				}
			}
			else
			{	
				// Made it to the end of the surface edge since the trace must have hit an intrinsic vertex that is also a surface vertex. 
				return MoveTemp(Crossings);
			}
	
		}
		else
		{
			// edge trace starts at StartVID and exits the opposite edge of intrinsic Tri TID (but it isn't an edge of the intrinsic tri).
			const FIndex3i TriEIDs  = IntrinsicMesh.GetTriEdges(FinderInfo.TID);
			const int32 OppEID      = TriEIDs[(FinderInfo.IdxOf + 1) % 3];

			const int32 CrossingIdx = [&]
										{
											const int32 AdvanceFromFirstRO = ( (ValenceOfStartVID + OrderOfTraceEID) - FinderInfo.FirstRoundabout) % ValenceOfStartVID;
											if (NCoords.IsSurfaceEdgeSegment(FinderInfo.EID))
											{
												return AdvanceFromFirstRO;
											}
											else
											{
												const int32 NumCrossings = NCoords.NumEdgeCrossing(FinderInfo.EID);
												return AdvanceFromFirstRO + 1 + NumCrossings;
											}
										}();

			checkSlow(CrossingIdx > 0); // would be zero if the edge was also a surface edge, but that case is handled above
			
			// the first intrinsic edge crossing.  Need this to jump-start the continuation.
			auto& Xing = Crossings.AddZeroed_GetRef();
			Xing.TID = FinderInfo.TID;
			Xing.EID = OppEID;
			Xing.CIdx = CrossingIdx;

			ContinueTraceSurfaceEdge(IntrinsicMesh, Crossings);
		
		}

		return MoveTemp(Crossings);
	}

	/**
	* Low-level method for use with normal-coordinate equipped intrinsic meshes.
	* 
	* Traces a surface edge across an intrinsic mesh, the result is an array of intrinsic edges crossed each with a corresponding 
	* integer. The integer is used to disambiguate when a single intrinsic edge is crossed by multiple surface edges. 
	*/ 
	template <typename IntrinsicMeshType>
	TArray<FNormalCoordinates::FEdgeAndCrossingIdx> TraceSurfaceEdge( const IntrinsicMeshType& IntrinsicMesh,
																	  const int32 SurfaceEID, const bool bReverse )
	{
		const FNormalCoordinates& NCoords = IntrinsicMesh.GetNormalCoordinates();
		const FDynamicMesh3& SurfaceMesh  = *NCoords.SurfaceMesh;
		if (!SurfaceMesh.IsEdge(SurfaceEID))
		{
			TArray< FNormalCoordinates::FEdgeAndCrossingIdx > Crossings;
			return MoveTemp(Crossings);
		}

		const FIndex2i EdgeV = SurfaceMesh.GetEdgeV(SurfaceEID);
		const int32 StartVID = (bReverse) ? EdgeV.B : EdgeV.A;
	

		// if this surface edge corresponds to an intrinsic edge, then the trace is trivial
		{
			// note this test relies on the fact that intrinsic vertices that sit on surface mesh vertices 
			// will have the same vertex IDs by construction. 
			int32 EquivalentIntrinsicEID = -1;
			const int32 EndVID = (bReverse) ? EdgeV.A : EdgeV.B;
			for (int32 IntrinsicEID : IntrinsicMesh.VtxEdgesItr(StartVID))
			{
				const FIndex2i IntrinsicEdgeV = IntrinsicMesh.GetEdgeV(IntrinsicEID);
				if (IntrinsicEdgeV.IndexOf(EndVID) != -1 && NCoords.IsSurfaceEdgeSegment(IntrinsicEID))
				{
					EquivalentIntrinsicEID = IntrinsicEID;
					break;
				}
			}

			if (EquivalentIntrinsicEID != -1)
			{
				checkSlow(IntrinsicMesh.IsEdge(EquivalentIntrinsicEID));
				const int32 IntrinsicEID      = EquivalentIntrinsicEID;
				const int32 IntrinsicTID      = IntrinsicMesh.GetEdgeT(IntrinsicEID).A;
				const int32 IndexOf           = IntrinsicMesh.GetTriEdges(IntrinsicTID).IndexOf(IntrinsicEID);
				const bool bRerverseIntrinsic = !(IntrinsicMesh.GetTriangle(IntrinsicTID)[IndexOf] == StartVID);


				TArray< FNormalCoordinates::FEdgeAndCrossingIdx > Crossings;
				Crossings.SetNumUninitialized(2);
				auto& XingStart = Crossings[0];
				XingStart.TID   = IntrinsicTID;
				XingStart.EID   = (bRerverseIntrinsic) ? (IndexOf + 1) % 3 : IndexOf;
				XingStart.CIdx  = 0;

				auto& XingEnd = Crossings[1];
				XingEnd.TID   = IntrinsicTID;
				XingEnd.EID   = (bRerverseIntrinsic) ? IndexOf : (IndexOf + 1) % 3;
				XingEnd.CIdx  = 0;

				return MoveTemp(Crossings);
			}
		}

		// the edge isn't equivalent to an intrinsic edge:  encode its direction by identifying it as an edge of a triangle and do the trace
		{

			const FIndex2i EdgeT = SurfaceMesh.GetEdgeT(SurfaceEID);
			int32 TID            = EdgeT.A;
			int32 IndexOf        = SurfaceMesh.GetTriEdges(TID).IndexOf(SurfaceEID);
			if (SurfaceMesh.GetTriangle(TID)[IndexOf] != StartVID)
			{
				if (EdgeT.B != -1)
				{ 
					TID = EdgeT.B;
					IndexOf = SurfaceMesh.GetTriEdges(TID).IndexOf(SurfaceEID);
					checkSlow(SurfaceMesh.GetTriangle(TID)[IndexOf] == StartVID);

					return TraceSurfaceEdge(IntrinsicMesh, TID, IndexOf);
				}
				else
				{
					TArray<FNormalCoordinates::FEdgeAndCrossingIdx> TraceResult = TraceSurfaceEdge(IntrinsicMesh, TID, IndexOf);
					Algo::Reverse(TraceResult);
					return MoveTemp(TraceResult);
				}
			}
			else
			{ 
				return TraceSurfaceEdge(IntrinsicMesh, TID, IndexOf);
			}
		}
	}


	/**
	* Low-level utility for use with normal-coordinate equipped intrinsic meshes.
	* 
	* Utility to trace an edge defined  on the TraceMesh across the HostMesh, given a list of host mesh edges intersected by the trace mesh.
	* this assumes the host mesh and trace mesh come from an intrinsic mesh, surface mesh pair.
	* 
	* From the list of edges being crossed, this creates a 2d triangle strip of the triangles being traversed, and translates Trace{Start,End}SurfacePosition
	* to this space so it can compute the actual intersection locations of the trace with the triangle edges (stored line-based barycentric coord, alpha).
	* 
	* Note: this assumes, that StartSurfacePoint and EndSurfacePoints are actually points on the edge, and HostXing lists the Ids of the edges
	* being crossed in order.
	*/
	template<typename HostMeshType, typename TraceMeshType, typename TraceVIDToSurfacePointFtor>
	void ConvertEdgesCrossed( const FSurfacePoint& StartSurfacePoint, const FSurfacePoint& EndSurfacePoint,
							  const TArray<int32>& HostEdgesCrossed, const HostMeshType& HostMesh,
		                      const TraceMeshType& TraceMesh, double CoalesceThreshold,
							  const TraceVIDToSurfacePointFtor& VIDToSurfacePoint,
							  TArray<FSurfacePoint>& EdgeTrace)
	{
	
		if (HostEdgesCrossed.Num() == 0)
		{
			// no edge crossings to convert.
			return;
		}

		// utility to keep from adding duplicate VIDs.  Duplicate VIDs could result when Coalescing..
		auto AddVIDToTrace = [&EdgeTrace](const int32 HostVID)
							{
								if (EdgeTrace.Num() == 0)
								{
									EdgeTrace.Emplace(HostVID);
								}
								else
								{
									const FSurfacePoint& LastPoint = EdgeTrace.Last();
									const bool bSameAsLast         = ( IsVertexPoint(LastPoint)
																	&& LastPoint.Position.VertexPosition.VID == HostVID );
									if (!bSameAsLast)
									{
										EdgeTrace.Emplace(HostVID);
									}
								}
							};
		
		
		// knowing the host mesh edges this trace edge crosses, we make a 2d triangle strip by unfolding the 3d triangles the edge crosses
		// and solve a 2x2 problem for each 2d edge crossed.

		// struct holds bare-bones 2d triangle strip and ability to map vertexIDs from the src mesh
		struct
		{
			TMap<int32, int32> ToStripIndexMap;    // maps from mesh VID to triangle strip VID.
			TArray<FVector2d> StripVertexBuffer;
		} TriangleStrip;
		TriangleStrip.StripVertexBuffer.Reserve(3 + HostEdgesCrossed.Num());


		// Functor to add a triangle to the triangle strip. This assumes that at least one shared edge of this triangle already exits in the strip
		auto AddTriangleToStrip = [&HostMesh, &TriangleStrip](const int32 HostTID)
				{

					FIndex3i TriVIDs = HostMesh.GetTriangle(HostTID);

					// two of the tri vertices are already in the buffer.  Identify the new one.
					int32 IndexOf = -1;
					for (int32 i = 0; i < 3; ++i)
					{
						const int32 VID       = TriVIDs[i];
						const int32* StripVID = TriangleStrip.ToStripIndexMap.Find(VID);
						if (!StripVID)
						{
							IndexOf = i;
							break;
						}
					}

					// no need to do anything if all vertices had previously been added, 
					if (IndexOf == -1)
					{
						return;
					}


					const FVector3d Verts[3] = { HostMesh.GetVertex(TriVIDs[0]),  HostMesh.GetVertex(TriVIDs[1]),  HostMesh.GetVertex(TriVIDs[2]) };
					// with new vert last (i.e. V2).
					const FIndex3i Reordered((IndexOf + 1) % 3, (IndexOf + 2) % 3, IndexOf);
					// the spanning vectors 
					const FVector3d E1 = Verts[Reordered[1]] - Verts[Reordered[0]];
					const FVector3d E2 = Verts[Reordered[2]] - Verts[Reordered[0]];

					// coordinates of V2 relative to the direction of E1, and its orthogonal complement.
					const double E1DotE2       = E2.Dot(E1);
					const double E1LengthSqr   = FMath::Max(E1.SizeSquared(), TMathUtilConstants<double>::ZeroTolerance);
					const double E1Length      = FMath::Sqrt(E1LengthSqr);
					const double E1Dist        = E2.Dot(E1) / E1Length;
					const double OrthE1DistSqr = FMath::Max(0., E2.SizeSquared() - E1DotE2 * E1DotE2 / E1LengthSqr);
					const double OrthE1Dist    = FMath::Sqrt(OrthE1DistSqr);

					// 2d version of E1
					const int32* StripTriV0 = TriangleStrip.ToStripIndexMap.Find(TriVIDs[Reordered[0]]);
					const int32* StripTriV1 = TriangleStrip.ToStripIndexMap.Find(TriVIDs[Reordered[1]]);
					checkSlow(StripTriV0); checkSlow(StripTriV1);

					const FVector2d& StripTriVert0 = TriangleStrip.StripVertexBuffer[*StripTriV0];
					const FVector2d& StripTriVert1 = TriangleStrip.StripVertexBuffer[*StripTriV1];
					const FVector2d StripE1        = (StripTriVert1 - StripTriVert0);

					const FVector2d StripE1Perp(-StripE1[1], StripE1[0]); // Rotate StripE1 90 CCW.

					// new vertex 2d position
					const FVector2d StripTriVert2 = StripTriVert0 + (StripE1)*E1DotE2 / E1LengthSqr + (StripE1Perp / E1Length) * OrthE1Dist;
					TriangleStrip.StripVertexBuffer.Add(StripTriVert2);
					const int32 StripVID          = TriangleStrip.StripVertexBuffer.Num() - 1;
					const int32 SurfaceVID        = TriVIDs[IndexOf];
					TriangleStrip.ToStripIndexMap.Add(SurfaceVID, StripVID);
				};

		

		// find host triangle that contains both the StartVID and the first host edge we cross.
		const int32 FirstHostTID = [&]
			{
				if (IsVertexPoint(StartSurfacePoint))
				{ 
					const int32 HostEID = HostEdgesCrossed[0];
					const FIndex2i HostEdgeT = HostMesh.GetEdgeT(HostEID);
					const FIndex3i TriAVIDs = HostMesh.GetTriangle(HostEdgeT.A);
					const FIndex3i TriBVIDs = HostMesh.GetTriangle(HostEdgeT.B);

					const int32 HostStartVID = StartSurfacePoint.Position.VertexPosition.VID;
					const int32 HostTID =  (TriAVIDs.IndexOf(HostStartVID) != -1) ? HostEdgeT.A : HostEdgeT.B;
					checkSlow(HostMesh.GetTriangle(HostTID).IndexOf(HostStartVID) != -1);
					return HostTID;
				}
				else if (IsEdgePoint(StartSurfacePoint))
				{
					const int32 StartEID = StartSurfacePoint.Position.EdgePosition.EdgeID;
					const int32 ExitEID  = HostEdgesCrossed[0];
					const FIndex2i AdjTris = HostMesh.GetEdgeT(StartEID);
					const int32 NumAdj = (AdjTris[1] == -1) ? 1 : 2;
					for (int i = 0; i < NumAdj; ++i)
					{
						int32 TID = AdjTris[i];
						if (HostMesh.GetTriEdges(TID).IndexOf(ExitEID) != -1)
						{
							return TID;
						}
					}
					return -1;
				}
				else
				{
					checkSlow(IsFacePoint(StartSurfacePoint));
					return StartSurfacePoint.Position.TriPosition.TriID;
				}
			}();

		// jump-start the process of making the triangle strip by adding the first 2 verts
		// that define the edge prior (in the ccw sense) to the first edge crossing.
		{
			const FIndex3i TriVIDs = HostMesh.GetTriangle(FirstHostTID);
			const FIndex3i TriEIDs = HostMesh.GetTriEdges(FirstHostTID);
			const int32 FirstEIDXed = HostEdgesCrossed[0];
			const int32 IndexOfe = TriEIDs.IndexOf(FirstEIDXed);
			const int32 FirstVID = TriVIDs[(IndexOfe + 2)%3];
			const int32 SecondVID = TriVIDs[(IndexOfe)];

			const FVector3d Vert0 = HostMesh.GetVertex(FirstVID);
			const FVector3d Vert1 = HostMesh.GetVertex(SecondVID);

			const double LSqr = FMath::Max((Vert0 - Vert1).SizeSquared(), TMathUtilConstants<double>::ZeroTolerance);
			const double L    = FMath::Sqrt(LSqr);

			const FVector2d StripVert0(0., 0.);
			const FVector2d StripVert1(L, 0.);

			TriangleStrip.StripVertexBuffer.Add(StripVert0);
			TriangleStrip.ToStripIndexMap.Add(FirstVID, 0);

			TriangleStrip.StripVertexBuffer.Add(StripVert1);
			TriangleStrip.ToStripIndexMap.Add(SecondVID, 1);
		}

		// unfold the triangle strip, add the first triangle and then all subsequent ones
			
		AddTriangleToStrip(FirstHostTID);
		{
			int32 LastHostTID = FirstHostTID;
			for (int32 HostEID : HostEdgesCrossed)
			{
				const FIndex2i EdgeT = HostMesh.GetEdgeT(HostEID);
				LastHostTID          = (EdgeT.A == LastHostTID) ? EdgeT.B : EdgeT.A;
				AddTriangleToStrip(LastHostTID);
			}
		}

		// translate a trace point on the surface of the host mesh, to Fvector2d relative to the 2d triangle strip 
		auto To2dStripPosition = [&](const FSurfacePoint& TraceSurfacePoint)->FVector2d
								{
									if (IsVertexPoint(TraceSurfacePoint))
									{
										// end point is a vertex in the host mesh
										const auto& HostVertexPosition = TraceSurfacePoint.Position.VertexPosition;
										const int32 HostVID            = HostVertexPosition.VID;
										const int32* StripVID          = TriangleStrip.ToStripIndexMap.Find(HostVID);
										checkSlow(StripVID);
										return TriangleStrip.StripVertexBuffer[*StripVID];
									}
									if (IsEdgePoint(TraceSurfacePoint))
									{
										// end point lies on an edge of the host mesh. identify the edge vertices in the 2d-triangle strip
										// and reconstruct the point from alpha

										const auto& HostEdgePosition = TraceSurfacePoint.Position.EdgePosition;
										const double Alpha		     = HostEdgePosition.Alpha;
										const FIndex2i HostEdgeV     = HostMesh.GetEdgeV(HostEdgePosition.EdgeID);
										const FIndex2i StripEdgeV( *TriangleStrip.ToStripIndexMap.Find(HostEdgeV.A),
																	*TriangleStrip.ToStripIndexMap.Find(HostEdgeV.B) );
										const FVector2d StripEdgePos[2] = { TriangleStrip.StripVertexBuffer[StripEdgeV.A],
																			TriangleStrip.StripVertexBuffer[StripEdgeV.B] };
										return Alpha * StripEdgePos[0] + (1. - Alpha) * StripEdgePos[1];
									}
									else
									{
										// end point lies in the face of the last triangle. identify the tri vertices in the 2d-triangle strip and 
										// reconstruct the point from the barycentric coordinates.

										checkSlow(IsFacePoint(TraceSurfacePoint));
										const auto& HostTriPosition = TraceSurfacePoint.Position.TriPosition;
										const FVector3d Barycentric = HostTriPosition.BarycentricCoords;
										const FIndex3i HostTri = HostMesh.GetTriangle(HostTriPosition.TriID);
										const FIndex3i StripTri( *TriangleStrip.ToStripIndexMap.Find(HostTri[0]),
																	*TriangleStrip.ToStripIndexMap.Find(HostTri[1]),
																	*TriangleStrip.ToStripIndexMap.Find(HostTri[2]) );
										const FVector2d StripTriPos[3] = { TriangleStrip.StripVertexBuffer[StripTri[0]],
																			TriangleStrip.StripVertexBuffer[StripTri[1]],
																			TriangleStrip.StripVertexBuffer[StripTri[2]] };

										return Barycentric[0] * StripTriPos[0] + Barycentric[1] * StripTriPos[1] + Barycentric[2] * StripTriPos[2];
									}
								};
			
			

		// translate the end of the trace edge to a position in the unfolded triangle strip
		
	
		const FVector2d TraceStartVertex = To2dStripPosition(StartSurfacePoint);
		const FVector2d TraceEndVertex   = To2dStripPosition(EndSurfacePoint);										
		const FVector2d TraceVector      = TraceEndVertex - TraceStartVertex;

		// loop over the two-d versions of the host edges the path crosses and find the intersection.
		for (int32 HostEID : HostEdgesCrossed)
		{
			const FIndex2i EdgeV = HostMesh.GetEdgeV(HostEID);
			const int32* StripA  = TriangleStrip.ToStripIndexMap.Find(EdgeV.A);
			const int32* StripB  = TriangleStrip.ToStripIndexMap.Find(EdgeV.B);
			checkSlow(StripA); checkSlow(StripB);
			const FIndex2i StripEdgeV(*StripA, *StripB);

			const FVector2d StripVertA = TriangleStrip.StripVertexBuffer[StripEdgeV.A];
			const FVector2d StripVertB = TriangleStrip.StripVertexBuffer[StripEdgeV.B];

			// solve Alpha*StripVertA + (1-Alpha)StripVertB = Gamma Start + (1-Gamma)End.
			// i.e. 
			// Alpha * (StripVertA - StripVertB) + Gamma * (End - Start) =  End - StripVertB.
			//     here End - Start = TraceVector
			// write this as 2x2 matrix problem M.x = b solving for unknown vector x=(alpha, gamma).

			// matrix
			double m[2][2];
			m[0][0] = (StripVertA - StripVertB)[0];  m[0][1] = TraceVector[0];
			m[1][0] = (StripVertA - StripVertB)[1];  m[1][1] = TraceVector[1];

			// b-vector
			const FVector2d b = TraceEndVertex - StripVertB;

			const double Det = m[0][0] * m[1][1] - m[0][1] * m[1][0];
			// inverse: not yet scaled by 1/det
			double invm[2][2];
			invm[0][0] =  m[1][1];   invm[0][1] = -m[0][1];
			invm[1][0] = -m[1][0];   invm[1][1] =  m[0][0];

			// solve for alpha only ( don't care about gamma ) 
			double alpha = invm[0][0] * b[0] + invm[0][1] * b[1];

			if (FMath::Abs(Det) < TMathUtilConstants<double>::ZeroTolerance)
			{
				// this surface edge and the intrinsic edge that intersects it are nearly parallel.  
				// Assume the crossing happens at the farther vertex.
				const double dA = StripVertA.SquaredLength();
				const double dB = StripVertB.SquaredLength();

				alpha = (dA > dB) ? 1. : 0.;
			}
			else
			{
				alpha = FMath::Clamp(alpha / Det, 0., 1.);
			}

			// may want to convert this edge crossing to a vertex crossing, if it is close enough.
			int32 CoalesceVID = -1;
			bool bSnapToVert = false;

			if (alpha < CoalesceThreshold)
			{
				CoalesceVID = EdgeV.B;
				bSnapToVert = true;
			}
			else if ((1. - alpha) < CoalesceThreshold)
			{
				CoalesceVID = EdgeV.A;
				bSnapToVert = true;
			}

			if (bSnapToVert)
			{
				AddVIDToTrace(CoalesceVID);
			}
			else
			{
				EdgeTrace.Emplace(HostEID, alpha);
			}
		}

	}

	/**
	* Utility to trace an edge defined (by TraceEID) on the TraceMesh across the HostMesh, given a list of host mesh edges intersected by the trace mesh.
	* this assumes the host mesh and trace mesh come from an intrinsic mesh, surface mesh pair.
	*/
	template<typename HostMeshType, typename TraceMeshType, typename TraceVIDToSurfacePointFtor>
	TArray<FSurfacePoint> TraceEdgeOverHost( const int32 TraceEID, const TArray<int32>& HostEdgesCrossed, const HostMeshType& HostMesh,
											 const TraceMeshType& TraceMesh, double CoalesceThreshold, bool bReverse, 
											 const TraceVIDToSurfacePointFtor& VIDToSurfacePoint)
	{
		TArray<FSurfacePoint> EdgeTrace;

		if (!TraceMesh.IsEdge(TraceEID))
		{
			// empty array..
			return MoveTemp(EdgeTrace);
		}


		EdgeTrace.Reserve(HostEdgesCrossed.Num()+2);

		// find start (.A) and end (.B) trace mesh vids for this edge
		const FIndex2i OrderedTraceEdgeV = [&]
											{
												// by convention the intrinsic edge direction will be defined by the first adj triangle
												const int32 AdjTID     = TraceMesh.GetEdgeT(TraceEID).A;
												const int32 IndexOf    = TraceMesh.GetTriEdges(AdjTID).IndexOf(TraceEID);
												const FIndex3i TriVIDs = TraceMesh.GetTriangle(AdjTID);
												return FIndex2i( TriVIDs[IndexOf], TriVIDs[(IndexOf + 1) % 3]);
											}();
		
		// translate to points on the surface of the host mesh
		const FSurfacePoint TraceStartSurfacePoint = VIDToSurfacePoint(OrderedTraceEdgeV.A);
		const FSurfacePoint TraceEndSurfacePoint   = VIDToSurfacePoint(OrderedTraceEdgeV.B);

		
		// add the first point in the trace.
		EdgeTrace.Add(TraceStartSurfacePoint);

		// compute (and add) the crossing locations relative to the edges of the host mesh
		ConvertEdgesCrossed(TraceStartSurfacePoint, TraceEndSurfacePoint, HostEdgesCrossed, HostMesh, TraceMesh, CoalesceThreshold, VIDToSurfacePoint, EdgeTrace);
		
		// maybe add end point, if it doesn't duplicate the last one already
		{
			const FSurfacePoint& LastPoint = EdgeTrace.Last();
			const bool bSameAsLast = IsVertexPoint(TraceEndSurfacePoint) 
			                      && IsVertexPoint(LastPoint) 
								  && (LastPoint.Position.VertexPosition.VID == TraceEndSurfacePoint.Position.VertexPosition.VID);
			if (!bSameAsLast)
			{
				EdgeTrace.Add(TraceEndSurfacePoint);
			}
		}

		// correct trace results order so it is either EdgeV order or reversed as requested
		const bool bHasEdgeVOrder = (TraceMesh.GetEdgeV(TraceEID).A == OrderedTraceEdgeV.A);
		const bool bNeedToReverse = (bHasEdgeVOrder && bReverse) || (!bHasEdgeVOrder && !bReverse);

		if (bNeedToReverse)
		{
			Algo::Reverse(EdgeTrace);
		}

		return MoveTemp(EdgeTrace);
	}
};

namespace FNormalCoordIntrinsicTraceImpl
{
	using namespace IntrinsicCorrespondenceUtils;

	template <typename IntrinsicMeshType>
	TArray<FSurfacePoint> TraceEdge(const IntrinsicMeshType& IntrinsicMesh, int32 IntrinsicEID, double CoalesceThreshold, bool bReverse)
	{
		using FEdgeAndCrossingIdx = FNormalCoordinates::FEdgeAndCrossingIdx;
		
		TArray<FSurfacePoint> Result;

		if (!IntrinsicMesh.IsEdge(IntrinsicEID))
		{
			return MoveTemp(Result);
		}

		const FNormalCoordinates& NormalCoordinates = IntrinsicMesh.GetNormalCoordinates();

		TArray<int32> SurfXings;
		const int32 NumSurfXings = NormalCoordinates.NumEdgeCrossing(IntrinsicEID);
		if (NumSurfXings > 0)
		{
			SurfXings.Reserve(NumSurfXings);
		}

		const FDynamicMesh3& SurfaceMesh = *IntrinsicMesh.GetExtrinsicMesh();

		const FIndex2i IntrinsicEdgeT = IntrinsicMesh.GetEdgeT(IntrinsicEID);

		// need to create a list of surface mesh edges that cross this edge.  To do this we need to follow
		// each curve that crosses this intrinsic edge to the vertices where it terminates ( thus identifying the surface edge )
		for (int32 p = 1; p < NumSurfXings + 1; ++p)
		{

			const FIndex2i SurfaceEdgeV = [&]
										{

											TArray<FEdgeAndCrossingIdx> Crossings;
											FIndex2i Verts;
											// follow surface curve (i.e. surface edge) forward to the end and identify the vertex
											{
												Crossings.Add(FEdgeAndCrossingIdx({IntrinsicEdgeT.A, IntrinsicEID, p}));
												FNormalCoordSurfaceTraceImpl::ContinueTraceSurfaceEdgeAcrossFaces(IntrinsicMesh, Crossings);
												const FEdgeAndCrossingIdx& LastXing = Crossings.Last();

												const FIndex3i TriEIDs = IntrinsicMesh.GetTriEdges(LastXing.TID);
												Verts.A = IntrinsicMesh.GetTriangle(LastXing.TID)[LastXing.EID];
											}
											Crossings.Reset();

											// follow surface curve (i.e. surface edge) backward to other end and identify the vertex
											{
												Crossings.Add(FEdgeAndCrossingIdx({IntrinsicEdgeT.B, IntrinsicEID, NumSurfXings + 1 - p}));
												FNormalCoordSurfaceTraceImpl::ContinueTraceSurfaceEdgeAcrossFaces(IntrinsicMesh, Crossings);
												const FEdgeAndCrossingIdx& LastXing = Crossings.Last();

												const FIndex3i TriEIDs = IntrinsicMesh.GetTriEdges(LastXing.TID);
												Verts.B = IntrinsicMesh.GetTriangle(LastXing.TID)[LastXing.EID];
											}
											return Verts;
										}();

			// identify the surface edge from its endpoints. 
			const int32 SurfaceEID = SurfaceMesh.FindEdge(SurfaceEdgeV.A, SurfaceEdgeV.B);
			checkSlow(SurfaceEID != IndexConstants::InvalidID);

			SurfXings.Add(SurfaceEID);
		}

		// do the actual trace - this identifies the surface mesh triangles crossed by the intrinsic edge and unfolds them into a triangle strip where the trace is performed
		Result = FNormalCoordSurfaceTraceImpl::TraceEdgeOverHost(IntrinsicEID, SurfXings, SurfaceMesh, IntrinsicMesh, CoalesceThreshold, bReverse, 
																[&IntrinsicMesh](int32 VID) { return IntrinsicMesh.GetVertexSurfacePoint(VID); } );

		return MoveTemp(Result);
	}
}

int32 UE::Geometry::FlipToDelaunay(FIntrinsicTriangulation& IntrinsicMesh, TSet<int32>& Uncorrected, const int32 MaxFlipCount)
{
	return FlipToDelaunayImpl(IntrinsicMesh, Uncorrected, MaxFlipCount);
}

int32 UE::Geometry::FlipToDelaunay(FSimpleIntrinsicEdgeFlipMesh& IntrinsicMesh, TSet<int32>& Uncorrected, const int32 MaxFlipCount)
{
	return FlipToDelaunayImpl(IntrinsicMesh, Uncorrected, MaxFlipCount);
}

int32 UE::Geometry::FlipToDelaunay(FSimpleIntrinsicMesh& IntrinsicMesh, TSet<int32>& Uncorrected, const int32 MaxFlipCount)
{
	return FlipToDelaunayImpl(IntrinsicMesh, Uncorrected, MaxFlipCount);
}
 
int32 UE::Geometry::FlipToDelaunay(FIntrinsicMesh& IntrinsicMesh, TSet<int32>& Uncorrected, const int32 MaxFlipCount)
{
	return FlipToDelaunayImpl(IntrinsicMesh, Uncorrected, MaxFlipCount);
}

int32 UE::Geometry::FlipToDelaunay(FIntrinsicEdgeFlipMesh& IntrinsicMesh, TSet<int32>& Uncorrected, const int32 MaxFlipCount)
{
	return FlipToDelaunayImpl(IntrinsicMesh, Uncorrected, MaxFlipCount);
}



/**------------------------------------------------------------------------------
* FSimpleIntrinsicEdgeFlipMesh Methods
*------------------------------------------------------------------------------ */

FSimpleIntrinsicEdgeFlipMesh::FSimpleIntrinsicEdgeFlipMesh(const FDynamicMesh3& SrcMesh)
{
	Reset(SrcMesh);
}
void FSimpleIntrinsicEdgeFlipMesh::Clear()
{
	Vertices.Clear();
	VertexRefCounts.Clear();
	VertexEdgeLists.Reset();

	Triangles.Clear();
	TriangleRefCounts.Clear();
	TriangleEdges.Clear();
	
	Edges.Clear();
	EdgeRefCounts.Clear();

	EdgeLengths.Clear();
	InternalAngles.Clear();
}

void FSimpleIntrinsicEdgeFlipMesh::Reset(const FDynamicMesh3& SrcMesh)
{
	Clear();

	Vertices        = SrcMesh.GetVerticesBuffer();
	VertexRefCounts = SrcMesh.GetVerticesRefCounts();
	VertexEdgeLists = SrcMesh.GetVertexEdges();

	Triangles         = SrcMesh.GetTrianglesBuffer();
	TriangleRefCounts = SrcMesh.GetTrianglesRefCounts();
	TriangleEdges     = SrcMesh.GetTriangleEdges();

	Edges         = SrcMesh.GetEdgesBuffer();
	EdgeRefCounts = SrcMesh.GetEdgesRefCounts();
	
	const int32 MaxEID = MaxEdgeID();
	EdgeLengths.SetNum(MaxEID);
	for (int32 EID = 0; EID < MaxEID; ++EID)
	{
		if (!IsEdge(EID))
		{
			continue;
		}
		const FIndex2i EdgeV = GetEdgeV(EID);
		const FVector3d Pos[2] = { GetVertex(EdgeV.A), GetVertex(EdgeV.B) };
		EdgeLengths[EID] = (Pos[1] - Pos[0]).Length();
	}

	const int32 MaxTriID = MaxTriangleID();
	InternalAngles.SetNum(MaxTriID);
	for (int32 TID = 0; TID < MaxTriID; ++TID)
	{
		if (!IsTriangle(TID))
		{
			continue;
		}
		// angles at v0, v1, v2, in that order
		InternalAngles[TID] = ComputeTriInternalAnglesR(TID);
	}
}


FIndex2i FSimpleIntrinsicEdgeFlipMesh::GetEdgeOpposingV(int32 EID) const
{
	const FEdge& Edge = Edges[EID];
	FIndex2i Result(InvalidID, InvalidID);

	for (int32 i = 0; i < 2; ++i)
	{
		int32 TriID = Edge.Tri[i];
		if (TriID == InvalidID) continue;

		const FIndex3i TriEIDs = GetTriEdges(TriID);
		const int32 IndexOf    = TriEIDs.IndexOf(EID);
		const FIndex3i TriVIDs = GetTriangle(TriID);
		Result[i] = TriVIDs[AddTwoModThree[IndexOf]];
	}

	return Result;
}


FIndex2i FSimpleIntrinsicEdgeFlipMesh::GetOrientedEdgeV(int32 EID, int32 TID) const
{
	int32 IndexOf = GetTriEdges(TID).IndexOf(EID);
	checkSlow(IndexOf != InvalidID);
	FIndex3i TriVIDs = GetTriangle(TID);
	return FIndex2i(TriVIDs[IndexOf], TriVIDs[AddOneModThree[IndexOf]]);
}


int32 FSimpleIntrinsicEdgeFlipMesh::ReplaceEdgeTriangle(int32 eID, int32 tOld, int32 tNew)
{
	FIndex2i& Tris = Edges[eID].Tri;
	int32 a = Tris[0], b = Tris[1];
	if (a == tOld) {
		if (tNew == InvalidID)
		{
			Tris[0] = b;
			Tris[1] = InvalidID;
		}
		else
		{
			Tris[0] = tNew;
		}
		return 0;
	}
	else if (b == tOld)
	{
		Tris[1] = tNew;
		return 1;
	}
	else
	{
		return -1;
	}
}


EMeshResult FSimpleIntrinsicEdgeFlipMesh::FlipEdgeTopology(int32 eab, FEdgeFlipInfo& FlipInfo)
{
	if (!IsEdge(eab))
	{
		return EMeshResult::Failed_NotAnEdge;
	}
	if (IsBoundaryEdge(eab))
	{
		return EMeshResult::Failed_IsBoundaryEdge;
	}

	// find oriented edge [a,b], tris t0,t1, and other verts c in t0, d in t1
	const FEdge Edge = Edges[eab];
	int32 t0 = Edge.Tri[0], t1 = Edge.Tri[1];

	FIndex2i oppV = GetEdgeOpposingV(eab);
	FIndex2i orientedV = GetOrientedEdgeV(eab, t0);
	int32 a = orientedV.A, b = orientedV.B;

	if (oppV[0] == InvalidID || oppV[1] == InvalidID)
	{
		return EMeshResult::Failed_BrokenTopology;
	}
	int32 c = oppV[0];
	int32 d = oppV[1];

	const FIndex3i T0te = GetTriEdges(t0);
	const FIndex3i T1te = GetTriEdges(t1);

	const int32 T0IndexOf = T0te.IndexOf(eab);
	const int32 T1IndexOf = T1te.IndexOf(eab);
	// find edges bc, ca, ad, db
	const int32 ebc = T0te[AddOneModThree[T0IndexOf]];
	const int32 eca = T0te[AddTwoModThree[T0IndexOf]];

	int32 ead = T1te[AddOneModThree[T1IndexOf]];
	int32 edb = T1te[AddTwoModThree[T1IndexOf]];
	if (!FlipInfo.bHadSameOrientations)
	{
		Swap(ead,edb);
	}
	

	// update triangles
	Triangles[t0] = FIndex3i(c, d, b);
	Triangles[t1] = FIndex3i(d, c, a);

	// update edge AB, which becomes flipped edge CD
	SetEdgeVerticesInternal(eab, c, d);     // Edge.Vert = (c, d) or (d, c)   
	SetEdgeTrianglesInternal(eab, t0, t1);  // Edge.Tri = (t0, t1)
	const int32 ecd = eab;

	// update the two other edges whose triangle nbrs have changed
	if (ReplaceEdgeTriangle(eca, t0, t1) == -1)
	{
		checkfSlow(false, TEXT("FSimpleIntrinsicEdgeFlipMesh.FlipEdge: first ReplaceEdgeTriangle failed"));
		return EMeshResult::Failed_UnrecoverableError;
	}
	if (ReplaceEdgeTriangle(edb, t1, t0) == -1)
	{
		checkfSlow(false, TEXT("FSimpleIntrinsicEdgeFlipMesh.FlipEdge: second ReplaceEdgeTriangle failed"));
		return EMeshResult::Failed_UnrecoverableError;
	}

	// update triangle nbr lists (these are edges)
	TriangleEdges[t0] = FIndex3i(ecd, edb, ebc);
	TriangleEdges[t1] = FIndex3i(ecd, eca, ead);

	// remove old eab from verts a and b, and Decrement ref counts
	if (VertexEdgeLists.Remove(a, eab) == false)
	{
		checkfSlow(false, TEXT("FSimpleIntrinsicEdgeFlipMesh.FlipEdge: first edge list remove failed"));
		return EMeshResult::Failed_UnrecoverableError;
	}
	VertexRefCounts.Decrement(a);
	if (a != b)
	{
		if (VertexEdgeLists.Remove(b, eab) == false)
		{
			checkfSlow(false, TEXT("FSimpleIntrinsicEdgeFlipMesh.FlipEdge: second edge list remove failed"));
			return EMeshResult::Failed_UnrecoverableError;
		}
		VertexRefCounts.Decrement(b);
	}
	if (IsVertex(a) == false || IsVertex(b) == false)
	{
		checkfSlow(false, TEXT("FSimpleIntrinsicEdgeFlipMesh.FlipEdge: either a or b is not a vertex?"));
		return EMeshResult::Failed_UnrecoverableError;
	}

	// add edge ecd to verts c and d, and increment ref counts
	VertexEdgeLists.Insert(c, ecd);
	VertexRefCounts.Increment(c);
	if (c != d)
	{
		VertexEdgeLists.Insert(d, ecd);
		VertexRefCounts.Increment(d);
	}

	// success! collect up results
	FlipInfo.EdgeID = eab;
	FlipInfo.OriginalVerts = FIndex2i(a, b);
	FlipInfo.OpposingVerts = FIndex2i(c, d);
	FlipInfo.Triangles = FIndex2i(t0, t1);
	return EMeshResult::Ok;
}


FVector3d FSimpleIntrinsicEdgeFlipMesh::ComputeTriInternalAnglesR(const int32 TID) const
{
	const FVector3d Lengths = GetTriEdgeLengths(TID);
	FVector3d Angles;
	for (int32 v = 0; v < 3; ++v)
	{
		Angles[v] = InteriorAngle(Lengths[v], Lengths[AddOneModThree[v]], Lengths[AddTwoModThree[v]]);
	}

	return Angles;
}


double FSimpleIntrinsicEdgeFlipMesh::GetOpposingVerticesDistance(int32 EID) const
{
	const bool bHasSameOrientation = AreSameOrientation(EID);

	const double OrgLength  = GetEdgeLength(EID);
	const FIndex2i EdgeTris = GetEdgeT(EID);
	checkSlow(EdgeTris[1] != FDynamicMesh3::InvalidID);

	// compute 2d locations of the verts opposite the EID edge.
	FVector2d Opp2dVerts[2];
	for (int32 i = 0; i < 2; ++i)
	{
		const FIndex3i TriEIDs = GetTriEdges(EdgeTris[i]);
		const FVector3d Ls = GetEdgeLengthTriple(TriEIDs);
		const int32 IndexOf = TriEIDs.IndexOf(EID);
		const FVector3d PermutedLs = Permute(IndexOf, Ls);
		Opp2dVerts[i] = ComputeOpposingVert2d(PermutedLs[0], PermutedLs[1], PermutedLs[2]);
	}
	// rotate second tri so the shared edge aligns
	Opp2dVerts[1].Y = -Opp2dVerts[1].Y;
	if (bHasSameOrientation)
	{ 
		Opp2dVerts[1].X = OrgLength - Opp2dVerts[1].X;
	}
	
	return (Opp2dVerts[0] - Opp2dVerts[1]).Length();
}


EMeshResult FSimpleIntrinsicEdgeFlipMesh::FlipEdge(int32 EID, FEdgeFlipInfo& EdgeFlipInfo)
{
	if (!IsEdge(EID))
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	if (IsBoundaryEdge(EID))
	{
		return EMeshResult::Failed_IsBoundaryEdge;
	}

	const bool bHasSameOrientation = AreSameOrientation(EID);
	EdgeFlipInfo.bHadSameOrientations = bHasSameOrientation;

	// prohibit case where the edge is shared by a non-convex pair of triangles.
	{
		// original triangles
		FDynamicMesh3::FEdge OrgEdge = GetEdge(EID);



		// total interior angles at the edge vertices as ordered by the first triangle.
		double TotalAngleAtOrgVert[2] = { 0., 0. };
		{
			// accumulate interior angles at edge vertices in first triangle. 
			{
				const int32 TriID = OrgEdge.Tri[0];
				const int32 IndexOf = GetTriEdges(TriID).IndexOf(EID);
				TotalAngleAtOrgVert[0] += InternalAngles[TriID][IndexOf];
				TotalAngleAtOrgVert[1] += InternalAngles[TriID][AddOneModThree[IndexOf]];
			}
			// accumulate interior angles at the edge vertices in the second triangle
			{
				const int32 TriID = OrgEdge.Tri[1];
				const int32 IndexOf = GetTriEdges(TriID).IndexOf(EID);
				if (bHasSameOrientation)
				{ 
					TotalAngleAtOrgVert[1] += InternalAngles[TriID][IndexOf];
					TotalAngleAtOrgVert[0] += InternalAngles[TriID][AddOneModThree[IndexOf]];
				}
				else
				{
					TotalAngleAtOrgVert[0] += InternalAngles[TriID][IndexOf];
					TotalAngleAtOrgVert[1] += InternalAngles[TriID][AddOneModThree[IndexOf]];
				}
			}
		}

		if (TotalAngleAtOrgVert[0] > TMathUtilConstants<double>::Pi - TMathUtilConstants<double>::ZeroTolerance || TotalAngleAtOrgVert[1] > TMathUtilConstants<double>::Pi - TMathUtilConstants<double>::ZeroTolerance)
		{
			return EMeshResult::Failed_Unsupported;
		}
	}

	// prohibit case where one of the ends of the edge has degree one (this looks like a triangle rolled into a cone ).
	{
		FIndex2i EdgeT = GetEdgeT(EID);
		
		if (EdgeT.A == EdgeT.B)
		{
			return EMeshResult::Failed_Unsupported;
		}
	}

	// compute the length of edge after flip
	const double PostFlipLength = GetOpposingVerticesDistance(EID);

	// flip edge in the underlying mesh
	const EMeshResult MeshFlipResult = FlipEdgeTopology(EID, EdgeFlipInfo);

	if (MeshFlipResult != EMeshResult::Ok)
	{
		// could fail for many reasons, e.g. a boundary edge or the new edge already exists.
		return MeshFlipResult;
	}

	// update the intrinsic edge length
	EdgeLengths[EID] = PostFlipLength;

	// update interior angles for the tris
	for (int32 t = 0; t < 2; ++t)
	{
		const int32 TID = EdgeFlipInfo.Triangles[t];
		InternalAngles[TID] = ComputeTriInternalAnglesR(TID);	
	}

	return MeshFlipResult;
}


FVector2d FSimpleIntrinsicEdgeFlipMesh::EdgeOpposingAngles(int32 EID) const
{
	FVector2d Result;
	FDynamicMesh3::FEdge Edge = GetEdge(EID);
	{
		const int32 IndexOf    = GetTriEdges(Edge.Tri.A).IndexOf(EID);
		const int32 IndexOfOpp = AddTwoModThree[IndexOf];
		const double Angle     = GetTriInternalAngleR(Edge.Tri.A, IndexOfOpp);
		Result[0] = Angle;
	}
	if (Edge.Tri.B != FDynamicMesh3::InvalidID)
	{
		const int32 IndexOf    = GetTriEdges(Edge.Tri.B).IndexOf(EID);
		const int32 IndexOfOpp = AddTwoModThree[IndexOf];
		const double Angle     = GetTriInternalAngleR(Edge.Tri.B, IndexOfOpp);
		Result[1] = Angle;
	}
	else
	{
		Result[1] = -TMathUtilConstants<double>::MaxReal;
	}

	return Result;
}


double FSimpleIntrinsicEdgeFlipMesh::EdgeCotanWeight(int32 EID) const
{
	auto ComputeCotanOppAngle = [this](int32 EID, int32 TID)->double
	{
		const FIndex3i TriEIDs = GetTriEdges(TID);
		const int32 IOf        = TriEIDs.IndexOf(EID);
		const FVector3d TriLs  = GetEdgeLengthTriple(TriEIDs);

		const FVector3d Ls(TriLs[IOf], TriLs[AddOneModThree[IOf]], TriLs[AddTwoModThree[IOf]]);
		return ComputeCotangent(Ls[0], Ls[1], Ls[2]);
	};

	FDynamicMesh3::FEdge Edge = GetEdge(EID);
	double Result = ComputeCotanOppAngle(EID, Edge.Tri.A);
	if (Edge.Tri.B != FDynamicMesh3::InvalidID)
	{
		Result += ComputeCotanOppAngle(EID, Edge.Tri.B);
		Result /= 2;
	}

	return Result;
}

/**------------------------------------------------------------------------------
*  FSimpleIntrinsicMesh Methods
*-------------------------------------------------------------------------------*/

UE::Geometry::EMeshResult UE::Geometry::FSimpleIntrinsicMesh::PokeTriangleTopology(int32 TriangleID, FPokeTriangleInfo& PokeResult)
{
	PokeResult = FPokeTriangleInfo();

	if (!IsTriangle(TriangleID))
	{
		return EMeshResult::Failed_NotATriangle;
	}

	FIndex3i tv = GetTriangle(TriangleID);
	FIndex3i te = GetTriEdges(TriangleID);

	// create vertex with averaged position.. 
	FVector3d vPos = (1. / 3.) * (GetVertex(tv[0]) + GetVertex(tv[1]) + GetVertex(tv[2]));

	int32 newVertID = AppendVertex(vPos);


	// add in edges to center vtx, do not connect to triangles yet
	int32 eAN = AddEdgeInternal(tv[0], newVertID, -1, -1);
	int32 eBN = AddEdgeInternal(tv[1], newVertID, -1, -1);
	int32 eCN = AddEdgeInternal(tv[2], newVertID, -1, -1);
	VertexRefCounts.Increment(tv[0]);
	VertexRefCounts.Increment(tv[1]);
	VertexRefCounts.Increment(tv[2]);
	VertexRefCounts.Increment(newVertID, 3);

	// old triangle becomes tri along first edge
	Triangles[TriangleID] = FIndex3i(tv[0], tv[1], newVertID);
	TriangleEdges[TriangleID] = FIndex3i(te[0], eBN, eAN);

	// add two triangles
	int32 t1 = AddTriangleInternal(tv[1], tv[2], newVertID, te[1], eCN, eBN);
	int32 t2 = AddTriangleInternal(tv[2], tv[0], newVertID, te[2], eAN, eCN);

	// second and third edges of original tri have neighbors
	ReplaceEdgeTriangle(te[1], TriangleID, t1);
	ReplaceEdgeTriangle(te[2], TriangleID, t2);

	// set the triangles for the edges we created above
	SetEdgeTrianglesInternal(eAN, TriangleID, t2);
	SetEdgeTrianglesInternal(eBN, TriangleID, t1);
	SetEdgeTrianglesInternal(eCN, t1, t2);


	PokeResult.OriginalTriangle = TriangleID;
	PokeResult.TriVertices = tv;
	PokeResult.NewVertex = newVertID;
	PokeResult.NewTriangles = FIndex2i(t1, t2);
	PokeResult.NewEdges = FIndex3i(eAN, eBN, eCN);
	PokeResult.BaryCoords = FVector3d(1. / 3., 1. / 3., 1. / 3.);

	return EMeshResult::Ok;
}

UE::Geometry::EMeshResult  UE::Geometry::FSimpleIntrinsicMesh::SplitEdgeTopology(int32 eab, FEdgeSplitInfo& SplitInfo)
{
	SplitInfo = FEdgeSplitInfo();

	if (!IsEdge(eab))
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	// look up primary edge & triangle
	const FEdge Edge = Edges[eab];
	const int32 t0 = Edge.Tri[0];
	if (t0 == InvalidID)
	{
		return EMeshResult::Failed_BrokenTopology;
	}
	const FIndex3i T0tv = GetTriangle(t0);
	const FIndex3i T0te = GetTriEdges(t0);
	const int32 IndexOfe = T0te.IndexOf(eab);
	const int32 a = T0tv[IndexOfe];
	const int32 b = T0tv[AddOneModThree[IndexOfe]];
	const int32 c = T0tv[AddTwoModThree[IndexOfe]];

	// look up edge bc, which needs to be modified
	const int32 ebc = T0te[AddOneModThree[IndexOfe]];

	// RefCount overflow check. Conservatively leave room for
	// extra increments from other operations.
	if (VertexRefCounts.GetRawRefCount(c) > FRefCountVector::INVALID_REF_COUNT - 3)
	{
		return EMeshResult::Failed_HitValenceLimit;
	}


	SplitInfo.OriginalEdge      = eab;
	SplitInfo.OriginalVertices  = FIndex2i(a, b);   // this is the oriented a,b
	SplitInfo.OriginalTriangles = FIndex2i(t0, InvalidID);
	SplitInfo.SplitT            = 0.5;

	// quite a bit of code is duplicated between boundary and non-boundary case, but it
	//  is too hard to follow later if we factor it out...
	if (IsBoundaryEdge(eab))
	{
		// create vertex
		const FVector3d vNew = 0.5 * (GetVertex(a) + GetVertex(b));
		const int32 f = AppendVertex(vNew);


		// rewrite existing triangle
		Triangles[t0][AddOneModThree[IndexOfe]] = f;

		// add second triangle
		const int32 t2 = AddTriangleInternal(f, b, c, InvalidID, InvalidID, InvalidID);


		// rewrite edge bc, create edge af
		ReplaceEdgeTriangle(ebc, t0, t2);
		const int32 eaf = eab;
		Edges[eaf].Vert = FIndex2i(FMath::Min(a, f), FMath::Max(a, f));
		//ReplaceEdgeVertex(eaf, b, f);
		if (a != b)
		{
			VertexEdgeLists.Remove(b, eab);
		}
		VertexEdgeLists.Insert(f, eaf);

		// create edges fb and fc
		const int32 efb = AddEdgeInternal(f, b, t2);
		const int32 efc = AddEdgeInternal(f, c, t0, t2);

		// update triangle edge-nbrs
		ReplaceTriangleEdge(t0, ebc, efc);
		TriangleEdges[t2] = FIndex3i(efb, ebc, efc);

		// update vertex refcounts
		VertexRefCounts.Increment(c);
		VertexRefCounts.Increment(f, 2);

		SplitInfo.bIsBoundary   = true;
		SplitInfo.OtherVertices = FIndex2i(c, InvalidID);
		SplitInfo.NewVertex     = f;
		SplitInfo.NewEdges      = FIndex3i(efb, efc, InvalidID);
		SplitInfo.NewTriangles  = FIndex2i(t2, InvalidID);

		return EMeshResult::Ok;

	}
	else 		// interior triangle branch
	{
		// look up other triangle
		const int32 t1 = Edges[eab].Tri[1];
		SplitInfo.OriginalTriangles.B = t1;
		const FIndex3i T1tv = GetTriangle(t1);
		const FIndex3i T1te = GetTriEdges(t1);
		const int32 T1IndexOfe = T1te.IndexOf(eab);
		checkSlow(T1tv[T1IndexOfe] == b);

		const int32 d = T1tv[AddTwoModThree[T1IndexOfe]];
		const int32 edb = T1te[AddTwoModThree[T1IndexOfe]];

		// RefCount overflow check. Conservatively leave room for
		// extra increments from other operations.
		if (VertexRefCounts.GetRawRefCount(d) > FRefCountVector::INVALID_REF_COUNT - 3)
		{
			return EMeshResult::Failed_HitValenceLimit;
		}

		// create vertex
		FVector3d vNew = 0.5 * (GetVertex(a) + GetVertex(b));
		int32 f = AppendVertex(vNew);


		// rewrite existing triangles, replacing b with f
		Triangles[t0][AddOneModThree[IndexOfe]] = f;
		Triangles[t1][T1IndexOfe] = f;


		// add two triangles to close holes we just created
		int32 t2 = AddTriangleInternal(f, b, c, InvalidID, InvalidID, InvalidID);
		int32 t3 = AddTriangleInternal(f, d, b, InvalidID, InvalidID, InvalidID);


		// update the edges we found above, to point to triangles
		ReplaceEdgeTriangle(ebc, t0, t2);
		ReplaceEdgeTriangle(edb, t1, t3);

		// edge eab became eaf
		int32 eaf = eab; //Edge * eAF = eAB;
		Edges[eaf].Vert = FIndex2i(FMath::Min(a, f), FMath::Max(a, f));

		// update a/b/f vertex-edges
		if (a != b)
		{
			VertexEdgeLists.Remove(b, eab);
		}
		VertexEdgeLists.Insert(f, eaf);

		// create edges connected to f  (also updates vertex-edges)
		int32 efb = AddEdgeInternal(f, b, t2, t3);
		int32 efc = AddEdgeInternal(f, c, t0, t2);
		int32 edf = AddEdgeInternal(d, f, t1, t3);

		// update triangle edge-nbrs
		ReplaceTriangleEdge(t0, ebc, efc);
		ReplaceTriangleEdge(t1, edb, edf);
		TriangleEdges[t2] = FIndex3i(efb, ebc, efc);
		TriangleEdges[t3] = FIndex3i(edf, edb, efb);

		// update vertex refcounts
		VertexRefCounts.Increment(c);
		VertexRefCounts.Increment(d);
		VertexRefCounts.Increment(f, 4);

		SplitInfo.bIsBoundary   = false;
		SplitInfo.OtherVertices = FIndex2i(c, d);
		SplitInfo.NewVertex     = f;
		SplitInfo.NewEdges      = FIndex3i(efb, efc, edf);
		SplitInfo.NewTriangles  = FIndex2i(t2, t3);

		return EMeshResult::Ok;
	}
}


UE::Geometry::EMeshResult UE::Geometry::FSimpleIntrinsicMesh::PokeTriangle(int32 TID, const FVector3d& BaryCoordinates, FPokeTriangleInfo& PokeInfo)
{
	if (!IsTriangle(TID))
	{
		return EMeshResult::Failed_NotATriangle;
	}

	// state before the poke
	const FIndex3i OriginalVIDs            = GetTriangle(TID);
	const FIndex3i OriginalEdges           = GetTriEdges(TID);
	const FVector3d OriginalTriEdgeLengths = GetTriEdgeLengths(TID);

	// Add a new vertex and faces to the IntrinsicMesh.   
	// Note: the r3 position will be wrong initially since this poke will just interpolate the corners of the tri.
	//       we fix this position as the last step in this function
	EMeshResult PokeResult = PokeTriangleTopology(TID, PokeInfo);
	if (PokeResult != EMeshResult::Ok)
	{
		return PokeResult;
	}


	FIndex3i NewTris(TID, PokeInfo.NewTriangles[0], PokeInfo.NewTriangles[1]);
	const int32 NewVID = PokeInfo.NewVertex;

	// Need to update intrinsic information for the 3 triangles that resulted from the poke
	// 1) update the edge lengths for the triangles
	// 2) update the internal angles for the triangles


	// (1) compute intrinsic edge lengths for the 3 new edges
	{
		FVector3d DistancesFromOldCorner;
		for (int32 v = 0; v < 3; ++v)
		{
			FVector3d BaryCoordVertex(0., 0., 0.); BaryCoordVertex[v] = 1.;
			const double Distance = DistanceBetweenBarycentricPoints( OriginalTriEdgeLengths[0],
																	  OriginalTriEdgeLengths[1],
																	  OriginalTriEdgeLengths[2],
																	  BaryCoordVertex, BaryCoordinates);
			DistancesFromOldCorner[v] = Distance;
		}

		{
			// original tri is and verts (a, b,c) and edges (ab, bc, ca)
			// the updated tri has verts (a, b, new) and edges (ab, bnew, newa)
			FIndex3i T0EIDs = GetTriEdges(NewTris[0]);
			EdgeLengths.InsertAt(DistancesFromOldCorner[1], T0EIDs[1]);
			EdgeLengths.InsertAt(DistancesFromOldCorner[0], T0EIDs[2]);

			// new tri[0] has verts (b, c, new) and edges (bc, cnew, newb)
			FIndex3i T1EIDs = GetTriEdges(PokeInfo.NewTriangles[0]);
			EdgeLengths.InsertAt(DistancesFromOldCorner[2], T1EIDs[1]);
		}
	}
	// (2) update internal angles for the triangles.
	{
		InternalAngles.InsertAt(ComputeTriInternalAnglesR(NewTris[2]), NewTris[2]);
		InternalAngles.InsertAt(ComputeTriInternalAnglesR(NewTris[1]), NewTris[1]);
		InternalAngles[NewTris[0]] = ComputeTriInternalAnglesR(NewTris[0]);
	}

	// record the coordinate actually used. 
	PokeInfo.BaryCoords = BaryCoordinates;

	return EMeshResult::Ok;
}


UE::Geometry::EMeshResult UE::Geometry::FSimpleIntrinsicMesh::SplitEdge(int32 EdgeAB, FEdgeSplitInfo& SplitInfo, double SplitParameterT)
{
	SplitParameterT = FMath::Clamp(SplitParameterT, 0., 1.);

	if (!IsEdge(EdgeAB))
	{
		return EMeshResult::Failed_NotAnEdge;
	}


	const FEdge OriginalEdge = GetEdge(EdgeAB);

	if (IsBoundaryEdge(EdgeAB))
	{
		// state before the split
		const int32 TID      = OriginalEdge.Tri[0];
		const int32 IndexOfe = GetTriEdges(TID).IndexOf(EdgeAB);

		// Info about the original T0 tri, reordered to make the split edge the first edge..
		const FVector3d OriginalT0EdgeLengths = Permute(IndexOfe, GetTriEdgeLengths(TID));    // as (|ab|, |bc|, |ca|)


		// Update the connectivity with the edge split
		EMeshResult Result = SplitEdgeTopology(EdgeAB, SplitInfo);

		if (Result != EMeshResult::Ok)
		{
			return Result;
		}
		const int32 NewVID = SplitInfo.NewVertex;
		const int32 NewTID = SplitInfo.NewTriangles[0]; // edges {fb, bc, cf}
		const int32 NewEdgeFB = SplitInfo.NewEdges[0];
		const int32 NewEdgeFC = SplitInfo.NewEdges[1];

		const double DistAF = SplitParameterT * OriginalT0EdgeLengths[0];
		const double DistFB = FMath::Max(0., OriginalT0EdgeLengths[0] - DistAF);
		// new edge length
		const double DistFC = DistanceBetweenBarycentricPoints( OriginalT0EdgeLengths[0],
																OriginalT0EdgeLengths[1],
																OriginalT0EdgeLengths[2],
																FVector3d(SplitParameterT, 1. - SplitParameterT, 0.),
																FVector3d(0., 0., 1.));

		EdgeLengths.InsertAt(DistFC, NewEdgeFC);
		EdgeLengths.InsertAt(DistFB, NewEdgeFB);
		EdgeLengths[EdgeAB] = DistAF;

		// update the internal angles
		InternalAngles.InsertAt(ComputeTriInternalAnglesR(NewTID), NewTID);
		InternalAngles[TID] = ComputeTriInternalAnglesR(TID);
	
		return Result;
	}
	else
	{
		// state before the split
		const int32 TID      = OriginalEdge.Tri[0];
		const int32 IndexOfe = GetTriEdges(TID).IndexOf(EdgeAB);

		// Info about the original T0 tri, reordered to make the split edge the first edge..
		const FVector3d OriginalT0EdgeLengths = Permute(IndexOfe, GetTriEdgeLengths(TID));    // as (|ab|, |bc|, |ca|)
		

	    // Info about the original T1 tri, reordered to make the split edge the first edge..
		const int32 TID1       = OriginalEdge.Tri[1];
		const int32 IndexOfT1e = GetTriEdges(TID1).IndexOf(EdgeAB);

		const FVector3d OriginalT1EdgeLengths = Permute(IndexOfT1e, GetTriEdgeLengths(TID1));   // as (|ba|, |ad|, |db})


		// Update the connectivity with the edge split
		EMeshResult Result = SplitEdgeTopology(EdgeAB, SplitInfo);

		if (Result != EMeshResult::Ok)
		{
			return Result;
		}
		const int32 NewVID = SplitInfo.NewVertex;
		const int32 NewTID0 = SplitInfo.NewTriangles[0]; // edges {fb, bc, cf}
		const int32 NewTID1 = SplitInfo.NewTriangles[1]; // edges {fd, db, bc}
		const int32 EdgeAF = EdgeAB;
		const int32 NewEdgeFB = SplitInfo.NewEdges[0];
		const int32 NewEdgeFC = SplitInfo.NewEdges[1];
		const int32 NewEdgeFD = SplitInfo.NewEdges[2];

		const double DistAF = SplitParameterT * OriginalT0EdgeLengths[0];
		const double DistFB = FMath::Max(0., OriginalT0EdgeLengths[0] - DistAF);
		// new edge length
		const double DistFC = DistanceBetweenBarycentricPoints( OriginalT0EdgeLengths[0],
																OriginalT0EdgeLengths[1],
																OriginalT0EdgeLengths[2],
																FVector3d(SplitParameterT, 1. - SplitParameterT, 0.),
																FVector3d(0., 0., 1.));

		// new edge length
		const double DistFD = DistanceBetweenBarycentricPoints( OriginalT1EdgeLengths[0],
																OriginalT1EdgeLengths[1],
																OriginalT1EdgeLengths[2],
																FVector3d(1. - SplitParameterT, SplitParameterT, 0.),
																FVector3d(0., 0., 1.));

		EdgeLengths.InsertAt(DistFD, NewEdgeFD);
		EdgeLengths.InsertAt(DistFC, NewEdgeFC);
		EdgeLengths.InsertAt(DistFB, NewEdgeFB);
		EdgeLengths[EdgeAF] = DistAF;

		// update the internal angles (this uses edge lengths)
		InternalAngles.InsertAt(ComputeTriInternalAnglesR(NewTID1), NewTID1);
		InternalAngles.InsertAt(ComputeTriInternalAnglesR(NewTID0), NewTID0);
		InternalAngles[TID] = ComputeTriInternalAnglesR(TID);
		InternalAngles[TID1] = ComputeTriInternalAnglesR(TID1);

		return Result;
	}
}


/**------------------------------------------------------------------------------
*  FIntrinsicTriangulation Methods
*-------------------------------------------------------------------------------*/


UE::Geometry::FIntrinsicTriangulation::FIntrinsicTriangulation(const FDynamicMesh3& SrcMesh)
	: MyBase(SrcMesh)
	, SignpostData(SrcMesh)
{
}

TArray<FIntrinsicTriangulation::FSurfacePoint> UE::Geometry::FIntrinsicTriangulation::TraceEdge(int32 EID, double CoalesceThreshold, bool bReverse) const
{
	TArray<FIntrinsicTriangulation::FSurfacePoint> SurfacePoints;
	if (!IsEdge(EID))
	{
		return SurfacePoints;
	}


	const FDynamicMesh3* SurfaceMesh = GetExtrinsicMesh();
	const double IntrinsicEdgeLength = GetEdgeLength(EID);
	const FIndex2i EdgeV = GetEdgeV(EID);
	int32 StartVID = EdgeV.A;
	int32 EndVID = EdgeV.B;
	if (bReverse)
	{
		StartVID = EdgeV.B;
		EndVID = EdgeV.A;
	}


	FIndex2i AdjTIDs = GetEdgeT(EID);

	// look-up the polar angle of this edge as it leaves StartVID ( this angle is relative to a specified reference mesh edge )
	const double PolarAngle = 
		[&]{
			const int32 AdjTID = AdjTIDs.A;
			const int32 IndexOf = GetTriEdges(AdjTID).IndexOf(EID);
			checkSlow(IndexOf > -1);
			if (GetTriangle(AdjTID)[IndexOf] == StartVID)
			{
				// IntrinsicEdgeAngles hold the local angle of each out-going edge relative to the vertex the edge exits
				return SignpostData.IntrinsicEdgeAngles[AdjTID][IndexOf];
			}
			else
			{
				const double ToRadians = SignpostData.GeometricVertexInfo[StartVID].ToRadians;
				// polar angle of prev edge just before (clockwise) the edge we want
				const double PrevPolarAngle = SignpostData.IntrinsicEdgeAngles[AdjTID][(IndexOf + 1) % 3];
				// angle between previous edge and this one
				const double InternalAngle = InternalAngles[AdjTID][(IndexOf + 1) % 3];
				// add the internal angle to rotate from Prev Edge to this edge
				return ToRadians * InternalAngle + PrevPolarAngle;
			}
		}();

	// Trace the surface mesh from the intrinsic StartVID, in the PolarAngle direction, a distance of IntrinsicEdgeLength.
	// 
	// Note: this intrinsic vertex may or may not correspond to a vertex in the surface mesh if vertices were added to the intrinsic mesh
	// by doing an edge split or a triangle poke.
	FMeshGeodesicSurfaceTracer SurfaceTracer =  SignpostSufaceTraceUtil::TraceFromIntrinsicVert(SignpostData, StartVID, PolarAngle, IntrinsicEdgeLength);

	// util to convert the trace result to a surface point, potentially snapping to a vertex if within the coalesce threshold 
	auto TraceResultToSurfacePoint = [CoalesceThreshold, SurfaceMesh](const FMeshGeodesicSurfaceTracer::FTraceResult& TraceResult)->FSurfacePoint
	{

	
		if (TraceResult.bIsEdgePoint)
		{
			// TraceResult alpha is defined as EdgeV.A * (1-Alpha) + EdgeV.B Alpha;  This is the complement of what we want
			const double Alpha = FMath::Clamp((1. - TraceResult.EdgeAlpha), 0., 1.);
			const int32 EID = TraceResult.EdgeID;

			if (Alpha <= 0.5 && Alpha < CoalesceThreshold)
			{
				return FSurfacePoint(SurfaceMesh->GetEdgeV(EID).B);
			}
			else if (Alpha > 0.5 && (1. - Alpha) < CoalesceThreshold)
			{
				return FSurfacePoint(SurfaceMesh->GetEdgeV(EID).A);
			}

			return FSurfacePoint(EID, Alpha);
		}
		else
		{
			// TODO - should snap to vertex / edge if close?
			const int32 TID = TraceResult.TriID;
			const FVector3d BC = TraceResult.Barycentric;
			return FSurfacePoint(TID, BC);
		}
	};

	// Add surface point to the outgoing array, but don't allow for duplicate vertex points
	auto AddSurfacePoint = [&SurfacePoints](const FSurfacePoint& PointA)
	{
		if (SurfacePoints.Num() == 0)
		{
			SurfacePoints.Add(PointA);
		}
		else
		{
			const FSurfacePoint& PointB = SurfacePoints.Last();
			const bool bAreSameVertexPoint = ( PointA.PositionType == FSurfacePoint::EPositionType::Vertex &&
				                               PointB.PositionType == FSurfacePoint::EPositionType::Vertex &&
				                               PointA.Position.VertexPosition.VID == PointB.Position.VertexPosition.VID );
			if (!bAreSameVertexPoint)
			{
				SurfacePoints.Add(PointA);
			}
		}
	};

	// package the surface trace results as series of surface points.  Note, because this is a trace along an intrinsic edge
	// the first and last result will be an intrinsic vertex (StartVID and EndVID)
	TArray<FMeshGeodesicSurfaceTracer::FTraceResult>& TraceResults = SurfaceTracer.GetTraceResults();
	{
		int32 NumTraceResults = TraceResults.Num();
		AddSurfacePoint(GetVertexSurfacePoint(StartVID));
		for (int32 i = 1; i < NumTraceResults - 1; ++i)
		{
			FMeshGeodesicSurfaceTracer::FTraceResult& TraceResult = TraceResults[i];
			FSurfacePoint SurfacePoint = TraceResultToSurfacePoint(TraceResult);
			AddSurfacePoint(SurfacePoint);
		}
		AddSurfacePoint(GetVertexSurfacePoint(EndVID));
	}


	return SurfacePoints;
}

EMeshResult UE::Geometry::FIntrinsicTriangulation::FlipEdge(int32 EID, FEdgeFlipInfo& EdgeFlipInfo)
{

	if (IsBoundaryEdge(EID))
	{
		return EMeshResult::Failed_IsBoundaryEdge;
	}
	// capture the state of the triangles before the flip.
	const FIndex2i Tris = GetEdgeT(EID);
	const FIndex2i PreFlipIndexOf(GetTriEdges(Tris[0]).IndexOf(EID), GetTriEdges(Tris[1]).IndexOf(EID));


	// flip edge in the underlying mesh
	// this updates the edge lengths and the interior angles.
	const EMeshResult MeshFlipResult = MyBase::FlipEdge(EID, EdgeFlipInfo);

	if (MeshFlipResult != EMeshResult::Ok)
	{
		// could fail for many reasons, e.g. a boundary edge 
		return MeshFlipResult;
	}

	// internal angles at the verts that are now connected by the flipped edge
	const double NewAngleAtC = InternalAngles[Tris[1]][1];
	const double NewAngleAtD = InternalAngles[Tris[0]][1];

	// update signpost
	SignpostData.OnFlipEdge(EID, Tris, EdgeFlipInfo.OpposingVerts, PreFlipIndexOf, NewAngleAtC, NewAngleAtD);

	return EMeshResult::Ok;
}


double UE::Geometry::FIntrinsicTriangulation::UpdateVertexByEdgeTrace(const int32 NewVID, const int32 TraceStartVID, const double TracePolarAngle, const double TraceDist)
{
	using FSurfaceTraceResult = FSignpost::FSurfaceTraceResult;

	FMeshGeodesicSurfaceTracer SurfaceTracer = SignpostSufaceTraceUtil::TraceFromIntrinsicVert(SignpostData, TraceStartVID, TracePolarAngle, TraceDist);

	TArray<FMeshGeodesicSurfaceTracer::FTraceResult>& TraceResultArray = SurfaceTracer.GetTraceResults();
	// need to convert the result of the trace into the correct form

	const FMeshGeodesicSurfaceTracer::FTraceResult& TraceResult = TraceResultArray.Last();
	const FSurfacePoint TraceResultPosition(TraceResult.TriID, TraceResult.Barycentric);
	// fix directions relative to local reference edge on the extrinsic mesh 
	// by finding direction of the TraceEID edge indecent on NewVID
	const double AngleOffset = [&]
	{
		FVector2d Dir = TraceResult.SurfaceDirection.Dir;

		// translate to Dir about the reference edge for this triangle
		const int32 EndRefEID = SignpostData.TIDToReferenceEID[TraceResult.TriID];

		const FMeshGeodesicSurfaceTracer::FTangentTri2& TangentTri2 = SurfaceTracer.GetLastTri();
		// convert to local basis for first edge of tri2
		if (TangentTri2.EdgeOrientationSign[0] == -1)
		{
			Dir = -Dir;
		}
		const int32 IndexOfEndRefEID = TangentTri2.PermutedTriEIDs.IndexOf(EndRefEID);
		FVector2d DirRelToRefEID = TangentTri2.ChangeBasis(Dir, IndexOfEndRefEID);
		if (TangentTri2.EdgeOrientationSign[IndexOfEndRefEID] == -1)
		{
			DirRelToRefEID = -DirRelToRefEID;
		}
		// angle of (new edge) path to NewVert relative to Ref edge
		const double AngleToNewVert = FMath::Atan2(DirRelToRefEID.Y, DirRelToRefEID.X);
		// reverse direction of path because we NewVert is the local origin for polar angles around new vert
		const double AngleFromNewVert = AngleToNewVert + TMathUtilConstants<double>::Pi;
		return AsZeroToTwoPi(AngleFromNewVert);
	}();

	// Trace result as position and angle.
	FSurfaceTraceResult SurfaceTraceResult = { TraceResultPosition, AngleOffset };

	// As R3
	bool bTmpIsValid;
	const FVector3d TraceResultPos = AsR3Position(SurfaceTraceResult.SurfacePoint, *SignpostData.SurfaceMesh, bTmpIsValid);

	// update the R3 for NewVID in the intrinsic mesh
	Vertices[NewVID] = TraceResultPos;

	// store the surface position for the intrinsic vertex in the signpost data
	// Currently we are storing every new intrinsic surface position as a point on a surface tri
	//  TODO: is there any advantage to classify as "Edge" position if very close to edge ?  
	SignpostData.IntrinsicVertexPositions.InsertAt(SurfaceTraceResult.SurfacePoint, NewVID);

	// return relative angle of trace
	return SurfaceTraceResult.Angle;
}


UE::Geometry::EMeshResult UE::Geometry::FIntrinsicTriangulation::PokeTriangle(int32 TID, const FVector3d& BaryCoordinates, FPokeTriangleInfo& PokeInfo)
{
	if (!IsTriangle(TID))
	{
		return EMeshResult::Failed_NotATriangle;
	}

	// state before the poke
	const FIndex3i OriginalVIDs = GetTriangle(TID);
	const FVector3d OriginalEdgeDirs = SignpostData.IntrinsicEdgeAngles[TID];
	
	// Add a new vertex and faces to the IntrinsicMesh.   
	// this operation will update the internal angles and the edge lengths, but the actual R3 positon 
	// of the vertex will need to be updated as will any signpost data.
	EMeshResult PokeResult = MyBase::PokeTriangle(TID, BaryCoordinates, PokeInfo);
	if (PokeResult != EMeshResult::Ok)
	{
		return PokeResult;
	}
	

	FIndex3i NewTris(TID, PokeInfo.NewTriangles[0], PokeInfo.NewTriangles[1]);
	const int32 NewVID = PokeInfo.NewVertex;

	// Need to update intrinsic information for the 3 triangles that resulted from the poke
	// 1) update Signpost data ( the edge directions for the triangles )
	// 2) update the surface point for the new vertex by tracing one of the new edges. 
	//    also update the edge directions leaving the new vertex to be relative to the direction defined on the extrinsic mesh
	
	// (1) update the edge directions 
	FVector3d TriEdgeDir[3]; // one for each tri in order (TID, NewTris[0], NewTris[1] )
	{
		// edges around the boundary of the original tri
		TriEdgeDir[0][0] = OriginalEdgeDirs[0]; // AtoB dir
		TriEdgeDir[1][0] = OriginalEdgeDirs[1]; // BtoC dir
		TriEdgeDir[2][0] = OriginalEdgeDirs[2]; // CtoA dir
	
		// edges from the corners of the original tri towards the new vertex at the "center"
		const double ToRadiansAtA = SignpostData.GeometricVertexInfo[OriginalVIDs[0]].ToRadians;
		const double ToRadiansAtB = SignpostData.GeometricVertexInfo[OriginalVIDs[1]].ToRadians;
		const double ToRadiansAtC = SignpostData. GeometricVertexInfo[OriginalVIDs[2]].ToRadians;
		TriEdgeDir[0][1] = AsZeroToTwoPi( OriginalEdgeDirs[1] + ToRadiansAtB * InternalAngles[NewTris[1]][0]);  // BtoNew dir
		TriEdgeDir[1][1] = AsZeroToTwoPi( OriginalEdgeDirs[2] + ToRadiansAtC * InternalAngles[NewTris[2]][0]);  // CtoNew dir
		TriEdgeDir[2][1] = AsZeroToTwoPi( OriginalEdgeDirs[0] + ToRadiansAtA * InternalAngles[NewTris[0]][0]);  // AtoNew dir   

		// edges from new vertex to a, to b, and to c, using new-to-A as the zero direction.  These will be updated later when we learn the correct angle for new-to-A
		TriEdgeDir[0][2] = 0.;                                                                // NewToA dir 
		TriEdgeDir[1][2] = InternalAngles[NewTris[0]][2];                                     // NewToB dir
		TriEdgeDir[2][2] = InternalAngles[NewTris[0]][2] + InternalAngles[NewTris[1]][2];     // NewToC dir
	}
	SignpostData.GeometricVertexInfo.InsertAt(FSignpost::FGeometricInfo(), NewVID); // we want the default of false, and 1.

	// (2) update the surface point for the new vertex by tracing one of the new edges. 
	// 
	// --- fix the r3 position of the new vertex and compute its SurfacePoint Attributes by doing a trace on the Extrinsic Mesh along one of the edges incident on NewVID

	// Use first incident edge and find the distance and direction (angle) to trace
	// Q: would picking the shortest of the new edges be better than just the first?
	const int32 AtoNewEID = PokeInfo.NewEdges[0]; // a-to-new edge
	const double AtoNewDist = EdgeLengths[AtoNewEID];

	// trace from A vertex in the direction toward the new vertex the edge length distance. This updates the surface position entry for the new vert.
	const int32 AVID = OriginalVIDs[0];
	const double AtoNewDir = TriEdgeDir[2][1];
	const double NewToAAngle = UpdateVertexByEdgeTrace(NewVID, AVID, AtoNewDir, AtoNewDist);

	// update the Newto{A,B,C}  directions such that NewToA has the angle LocalEIDAngle.
	for (int32 e = 0; e < 3; ++e)
	{ 
		TriEdgeDir[e][2] = AsZeroToTwoPi(TriEdgeDir[e][2] + NewToAAngle);
	}
	
	// record the new edge directions.
	SignpostData.IntrinsicEdgeAngles.InsertAt(TriEdgeDir[2], NewTris[2]);
	SignpostData.IntrinsicEdgeAngles.InsertAt(TriEdgeDir[1], NewTris[1]);
	SignpostData.IntrinsicEdgeAngles[NewTris[0]] = TriEdgeDir[0];
	
	// record the coordinate actually used. 
	PokeInfo.BaryCoords = BaryCoordinates;

	return EMeshResult::Ok;
}


UE::Geometry::EMeshResult  UE::Geometry::FIntrinsicTriangulation::SplitEdge(int32 EdgeAB, FEdgeSplitInfo& SplitInfo, double SplitParameterT)
{
	SplitParameterT = FMath::Clamp(SplitParameterT, 0., 1.);

	if (!IsEdge(EdgeAB))
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	
	const FEdge OriginalEdge = GetEdge(EdgeAB);

	if (IsBoundaryEdge(EdgeAB))
	{
		// state before the split
		const int32 TID      = OriginalEdge.Tri[0];
		const int32 IndexOfe = GetTriEdges(TID).IndexOf(EdgeAB);

		// Info about the original T0 tri, reordered to make the split edge the first edge..
		const FIndex3i OriginalT0VIDs         = Permute(IndexOfe, GetTriangle(TID));  // as (a, b, c)
		const FVector3d OriginalT0EdgeDirs    = Permute(IndexOfe, SignpostData.IntrinsicEdgeAngles[TID]);  // as ( aTob, bToc, cToa)
		const FVector3d OriginalT0EdgeLengths = Permute(IndexOfe, GetTriEdgeLengths(TID));    // as (|ab|, |bc|, |ca|)
		
		// Update the connectivity with the edge split
		EMeshResult Result = MyBase::SplitEdge(EdgeAB, SplitInfo, SplitParameterT);

		if (Result != EMeshResult::Ok)
		{
			return Result;
		}
		const int32 NewVID = SplitInfo.NewVertex;
		const int32 NewTID = SplitInfo.NewTriangles[0]; // edges {fb, bc, cf}

		const double DistAF = SplitParameterT * OriginalT0EdgeLengths[0];
		const FVector3d TIDInternalAngles = Permute(IndexOfe, InternalAngles[TID]);
		
		// update the directions.  Say DirFtoB is zero
		const double ToRadiansAtC = SignpostData.GeometricVertexInfo[OriginalT0VIDs[2]].ToRadians;
		SignpostData.GeometricVertexInfo.InsertAt(FSignpost::FGeometricInfo(), NewVID); // we want the default of false, and 1.

		FVector3d TriEdgeDir[2]; // one for each tri in order (TID, NewTID )
		// edges around the boundary of the original tri
		TriEdgeDir[0][0] = OriginalT0EdgeDirs[0];    // AtoF dir
		TriEdgeDir[1][1] = OriginalT0EdgeDirs[1];    // BtoC dir
		TriEdgeDir[0][2] = OriginalT0EdgeDirs[2];    // CtoA dir  

		TriEdgeDir[1][0] = 0.;  // FtoB dir
		TriEdgeDir[1][2] = AsZeroToTwoPi(OriginalT0EdgeDirs[2] + ToRadiansAtC * TIDInternalAngles[2]);  // CtoF dir
		TriEdgeDir[0][1] = AsZeroToTwoPi(InternalAngles[NewTID][0]);                                    // FtoC dir relative to FtoB
		const double FtoADir = AsZeroToTwoPi(InternalAngles[NewTID][0] + TIDInternalAngles[1]);         // FtoA dir relative to FtoB
		// update the surface position of the new vertex and the relative directions from it.

		// trace from A vertex in the direction toward the new vertex the edge length distance. This updates the surface position entry for the new vert.
		const int32 AVID = OriginalT0VIDs[0];
		const double AtoFDir    = TriEdgeDir[0][0];
		const double FToAAngle  = UpdateVertexByEdgeTrace(NewVID, AVID, AtoFDir, DistAF);
		const double ToLocalDir = (FToAAngle - FtoADir); 
		// update the Fto{B,C}  directions such that the direction from F to A would agree with FtoAAngle
		TriEdgeDir[0][1] = AsZeroToTwoPi(TriEdgeDir[0][1] + ToLocalDir);
		TriEdgeDir[1][0] = AsZeroToTwoPi(TriEdgeDir[1][0] + ToLocalDir);
		
		// store angle results
		SignpostData.IntrinsicEdgeAngles.InsertAt(TriEdgeDir[1], NewTID);
		SignpostData.IntrinsicEdgeAngles[TID] = Permute((3 - IndexOfe)%3,  TriEdgeDir[0]);


		return Result;
	}
	else
	{
		// state before the split
		const int32 TID = OriginalEdge.Tri[0];
		const int32 IndexOfe = GetTriEdges(TID).IndexOf(EdgeAB);

		// Info about the original T0 tri, reordered to make the split edge the first edge..
		const FIndex3i OriginalT0VIDs  = Permute(IndexOfe, GetTriangle(TID));  // as (a, b, c)
		const FVector3d OriginalT0EdgeDirs    = Permute(IndexOfe, SignpostData.IntrinsicEdgeAngles[TID]);  // as ( aTob, bToc, cToa)
		const FVector3d OriginalT0EdgeLengths = Permute(IndexOfe, GetTriEdgeLengths(TID));    // as (|ab|, |bc|, |ca|)

		// Info about the original T0 tri, reordered to make the split edge the first edge..
		const int32 TID1 = OriginalEdge.Tri[1];
		const int32 IndexOfT1e = GetTriEdges(TID1).IndexOf(EdgeAB);

		const FIndex3i OriginalT1VIDs      = Permute(IndexOfT1e, GetTriangle(TID1));  // as (b, a, d)
		const FVector3d OriginalT1EdgeDirs = Permute(IndexOfT1e, SignpostData.IntrinsicEdgeAngles[TID1]); // as (bToa, aTod, dTob) 
		

		// Update the connectivity and the intrinsic data with the edge split
		EMeshResult Result = MyBase::SplitEdge(EdgeAB, SplitInfo, SplitParameterT);

		if (Result != EMeshResult::Ok)
		{
			return Result;
		}
		const int32 NewVID = SplitInfo.NewVertex;
		const int32 NewTID0 = SplitInfo.NewTriangles[0]; // edges {fb, bc, cf}
		const int32 NewTID1 = SplitInfo.NewTriangles[1]; // edges {fd, db, bc}

		const double DistAF = SplitParameterT * OriginalT0EdgeLengths[0];
		
		// update the directions.  Say DirFtoB is zero
		const double ToRadiansAtC = SignpostData.GeometricVertexInfo[OriginalT0VIDs[2]].ToRadians;
		const double ToRadiansAtD = SignpostData.GeometricVertexInfo[OriginalT1VIDs[2]].ToRadians;
		SignpostData.GeometricVertexInfo.InsertAt(FSignpost::FGeometricInfo(), NewVID); // we want the default of false, and 1.

		const FVector3d TIDInternalAngles  = Permute(IndexOfe, InternalAngles[TID]);
		const FVector3d TID1InternalAngles = Permute(IndexOfT1e, InternalAngles[TID1]);

		FVector3d TriEdgeDir[4]; // one for each tri in order (TID, NewTID0, TID1,  NewTID1 )
		// edges around the boundary of the original tri
		TriEdgeDir[0][0] = OriginalT0EdgeDirs[0];    // AtoF dir
		TriEdgeDir[1][1] = OriginalT0EdgeDirs[1];    // BtoC dir
		TriEdgeDir[0][2] = OriginalT0EdgeDirs[2];    // CtoA dir  

		TriEdgeDir[3][2] = OriginalT1EdgeDirs[0];    // BtoF dir
		TriEdgeDir[2][1] = OriginalT1EdgeDirs[1];    // AtoD dir
		TriEdgeDir[3][1] = OriginalT1EdgeDirs[2];    // DtoB dir

		
		TriEdgeDir[1][2] = AsZeroToTwoPi(OriginalT0EdgeDirs[2] + ToRadiansAtC * TIDInternalAngles[2]);       // CtoF dir
		TriEdgeDir[2][2] = AsZeroToTwoPi(OriginalT1EdgeDirs[2] + ToRadiansAtD * InternalAngles[NewTID1][1]); // Dtof dir 

		TriEdgeDir[1][0] = 0.;                                                                          // FtoB dir
		TriEdgeDir[0][1] = AsZeroToTwoPi(InternalAngles[NewTID0][0]);                                   // FtoC dir relative to FtoB
		TriEdgeDir[2][0] = AsZeroToTwoPi(InternalAngles[NewTID0][0] + TIDInternalAngles[1]);            // FtoA dir relative to FtoB
		
		TriEdgeDir[3][0] = AsZeroToTwoPi(InternalAngles[NewTID0][0] + TIDInternalAngles[1] + TID1InternalAngles[0]); // FtoD dir relative to FtoB

		// trace from A vertex in the direction toward the new vertex the edge length distance. This updates the surface position entry for the new vert.
		const int32 AVID = OriginalT0VIDs[0];
		const double AtoFDir = TriEdgeDir[0][0];
		const double FToAAngle   = UpdateVertexByEdgeTrace(NewVID, AVID, AtoFDir, DistAF);
		const double ToLocalDir = (FToAAngle - TriEdgeDir[2][0]);

		// fix angles relative to local dir.
		TriEdgeDir[1][0] = AsZeroToTwoPi(ToLocalDir);
		TriEdgeDir[0][1] = AsZeroToTwoPi(TriEdgeDir[0][1] + ToLocalDir);
		TriEdgeDir[2][0] = AsZeroToTwoPi(TriEdgeDir[2][0] + ToLocalDir);
		TriEdgeDir[3][0] = AsZeroToTwoPi(TriEdgeDir[3][0] + ToLocalDir);

		// store angle results
		SignpostData.IntrinsicEdgeAngles.InsertAt(TriEdgeDir[3], NewTID1);
		SignpostData.IntrinsicEdgeAngles.InsertAt(TriEdgeDir[1], NewTID0);
		SignpostData.IntrinsicEdgeAngles[TID]  = Permute((3 - IndexOfe) % 3, TriEdgeDir[0]);
		SignpostData.IntrinsicEdgeAngles[TID1] = Permute((3 - IndexOfT1e) % 3, TriEdgeDir[2]);

		return Result;
	}
}

/**------------------------------------------------------------------------------
*  FIntrinsicEdgeFlipMesh Methods
*-------------------------------------------------------------------------------*/
FIntrinsicEdgeFlipMesh::FIntrinsicEdgeFlipMesh(const FDynamicMesh3& SurfaceMesh)
{
	// construct the intrinsic mesh directly from this surface mesh
	MyBase::Reset(SurfaceMesh);
	NormalCoordinates.Reset(SurfaceMesh);
}

EMeshResult FIntrinsicEdgeFlipMesh::FlipEdge(int32 EID, FEdgeFlipInfo& EdgeFlipInfo)
{
	if (!IsEdge(EID))
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	if (IsBoundaryEdge(EID))
	{
		return EMeshResult::Failed_IsBoundaryEdge;
	}

	// state before flip, needed when updating the normal coords after the flip
	const FIndex2i EdgeT  = GetEdgeT(EID);
	const FIndex3i TAEIDs = GetTriEdges(EdgeT.A);
	const FIndex3i TBEIDs = GetTriEdges(EdgeT.B);
	const FIndex2i OppVs  = GetEdgeOpposingV(EID);

	EMeshResult FlipResult = MyBase::FlipEdge(EID, EdgeFlipInfo);

	if (FlipResult == EMeshResult::Ok)
	{
		// update the normal coords
		NormalCoordinates.OnFlipEdge(EdgeT.A, TAEIDs, OppVs.A, EdgeT.B, TBEIDs, OppVs.B, EID);
	}

	return FlipResult;
}

TArray<FIntrinsicEdgeFlipMesh::FSurfacePoint> FIntrinsicEdgeFlipMesh::TraceEdge(int32 IntrinsicEID, double CoalesceThreshold, bool bReverse) const
{
	return FNormalCoordIntrinsicTraceImpl::TraceEdge(*this, IntrinsicEID, CoalesceThreshold, bReverse);
}

TArray<FIntrinsicEdgeFlipMesh::FEdgeAndCrossingIdx> FIntrinsicEdgeFlipMesh::GetImplicitEdgeCrossings(const int32 SurfaceEID, const bool bReverse) const
{
	return FNormalCoordSurfaceTraceImpl::TraceSurfaceEdge(*this, SurfaceEID, bReverse);
}



TArray<FIntrinsicEdgeFlipMesh::FSurfacePoint>
FIntrinsicEdgeFlipMesh::TraceSurfaceEdge(int32 SurfaceEID, double CoalesceThreshold, bool bReverse) const
{
	const FDynamicMesh3& HostMesh                   = *this->GetExtrinsicMesh();
	const FIntrinsicEdgeFlipMesh& TraceMesh         = *this;
	TArray<FEdgeAndCrossingIdx> EdgeAndCrossingIdxs = this->GetImplicitEdgeCrossings(SurfaceEID, bReverse);

	// NB: for this intrinsic mesh type, we know that the intrinsic verts are the same as the surface verts ( since splits and pokes aren't allowed)
	TArray<int32> HostEdgeCrossings;
	HostEdgeCrossings.Reserve(EdgeAndCrossingIdxs.Num() - 2); // EdgeAndCrossingsIdx include the start and end vertex.  don't need them.
	for (FEdgeAndCrossingIdx EdgeAndCrossing : EdgeAndCrossingIdxs)
	{
		if (EdgeAndCrossing.CIdx != 0) // skip the start and end vertex
		{
			HostEdgeCrossings.Add(EdgeAndCrossing.EID);
		}
	}

	return FNormalCoordSurfaceTraceImpl::TraceEdgeOverHost(SurfaceEID, HostEdgeCrossings, HostMesh, TraceMesh, CoalesceThreshold, bReverse, 
														  [](int32 VID) {return FSurfacePoint(VID);});
}

/**------------------------------------------------------------------------------
*  TEdgeCorrespondence 
*-------------------------------------------------------------------------------*/

template <typename IntrinsicMeshType>
void UE::Geometry::TEdgeCorrespondence<IntrinsicMeshType>::Setup(const IntrinsicMeshType& Mesh)
{
	SurfaceEdgesCrossed.Reset();
	IntrinsicMesh = &Mesh;
	SurfaceMesh = Mesh.GetNormalCoordinates().SurfaceMesh;

	const int32 IntrinsicMaxEID = IntrinsicMesh->MaxEdgeID();
	const int32 SurfaceMaxEID = SurfaceMesh->MaxEdgeID();

	SurfaceEdgesCrossed.SetNum(IntrinsicMaxEID);
	const IntrinsicCorrespondenceUtils::FNormalCoordinates& NormalCoords = IntrinsicMesh->GetNormalCoordinates();

	// allocate the SurfaceEdgesCrossed. From the normal coordinates we know how many surface edge crossings each intrinsic edge sees
	for (int32 IntrinsicEID = 0; IntrinsicEID < IntrinsicMaxEID; ++IntrinsicEID)
	{
		if (!IntrinsicMesh->IsEdge(IntrinsicEID))
		{
			continue;
		}
		// number of times a surface edge crosses this intrinsic edge
		const int32 NumXings = NormalCoords.NumEdgeCrossing(IntrinsicEID);

		if (NumXings > 0)
		{
			SurfaceEdgesCrossed[IntrinsicEID].SetNum(NumXings);

		} // else don't bother making an entry when NumXings == 0 since that means the edges are the same on both meshes.
	}

	// trace each surface edge across the intrinsic mesh and construct an ordered list of surface edges crossing each intrinsic edge.
	// the order of the crossings should be consistent with the edge direction relative to the first adjacent tri 
	// (i.e. starting at the corner Mesh.GetTriEdges(GetEdgeT(EID).A).IndexOf(EID) ) 

	for (int32 SurfaceEID = 0; SurfaceEID < SurfaceMaxEID; ++SurfaceEID)
	{
		if (!SurfaceMesh->IsEdge(SurfaceEID))
		{
			continue;
		}
		const TArray<FEdgeAndCrossingIdx> IntrinsicEdgeXings = IntrinsicMesh->GetImplicitEdgeCrossings(SurfaceEID, false /* = bReverseTrace*/);

		for (int32 i = 0; i < IntrinsicEdgeXings.Num(); ++i)
		{
			const FEdgeAndCrossingIdx& EdgeXing = IntrinsicEdgeXings[i];
			bool bIsEndVertex = (EdgeXing.CIdx == 0);

			if (!bIsEndVertex)
			{
				const int32 IntrinsicEID = EdgeXing.EID;
				const FIndex2i IntrinsicEdgeT = IntrinsicMesh->GetEdgeT(IntrinsicEID);
				checkSlow(IntrinsicEdgeT.A == EdgeXing.TID || IntrinsicEdgeT.B == EdgeXing.TID);

				// array of surface edges this intrinsic edge crosses, these should be ordered relative to the direction of TriA.
				TArray<int32>& XingSurfaceEdges = SurfaceEdgesCrossed[IntrinsicEID];

				// crossings count from the bottom of the edge to the top relative to EdgeXing.TID 
				const int32 XingID = (IntrinsicEdgeT.A == EdgeXing.TID) ? EdgeXing.CIdx - 1 : XingSurfaceEdges.Num() - EdgeXing.CIdx;

				checkSlow(XingID > -1);
				XingSurfaceEdges[XingID] = SurfaceEID;
			}
		}
	}
}

template <typename IntrinsicMeshType>
TArray<UE::Geometry::IntrinsicCorrespondenceUtils::FSurfacePoint> TEdgeCorrespondence<IntrinsicMeshType>::TraceEdge(int32 IntrinsicEID, double CoalesceThreshold, bool bReverse) const
{
	const FDynamicMesh3& HostMesh = *SurfaceMesh;
	const IntrinsicMeshType& TraceMesh = *IntrinsicMesh;
	const TArray<int32>& HostEdgeCrossings = SurfaceEdgesCrossed[IntrinsicEID];

	return FNormalCoordSurfaceTraceImpl::TraceEdgeOverHost(IntrinsicEID, HostEdgeCrossings, HostMesh, TraceMesh, CoalesceThreshold, bReverse, 
	                                                       [&TraceMesh](int32 VID) { return TraceMesh.GetVertexSurfacePoint(VID); });
}

template struct UE::Geometry::TEdgeCorrespondence<UE::Geometry::FIntrinsicEdgeFlipMesh>;
template struct UE::Geometry::TEdgeCorrespondence<UE::Geometry::FIntrinsicMesh>;

/**------------------------------------------------------------------------------
*  FIntrinsicMesh Helpers
*-------------------------------------------------------------------------------*/


namespace FIntrinsicMeshImplUtils
{
	using namespace UE::Geometry;

	/** 
	* Test if a given SurfacePoint is on the specified surface triangle of the surface mesh, this includes the triangle boundaries:
	* i.e. will return true, if the point is on the face of the triangle, one of the edges, or on a vertex of the triangle.
	*/
	bool IsOnSurfaceTriangle(const IntrinsicCorrespondenceUtils::FSurfacePoint& SurfacePoint, const int32 SurfaceTID, const FDynamicMesh3& SurfaceMesh)
	{
		bool bFoundTID = false;
		if (!SurfaceMesh.IsTriangle(SurfaceTID))
		{
			return bFoundTID;
		}
		
		if (IsFacePoint(SurfacePoint))
		{
			const int32 TID = SurfacePoint.Position.TriPosition.TriID;
			if (TID == SurfaceTID)
			{
				bFoundTID = true;
			}
		}
		else if (IsEdgePoint(SurfacePoint))
		{
			const int32 SurfaceEID      = SurfacePoint.Position.EdgePosition.EdgeID;
			const FIndex2i SurfaceEdgeT = SurfaceMesh.GetEdgeT(SurfaceEID);
			if (SurfaceEdgeT.IndexOf(SurfaceTID) != -1)
			{
				bFoundTID = true;
			}
		}
		else
		{
			checkSlow(IsVertexPoint(SurfacePoint));
			const int32 SurfaceVID = SurfacePoint.Position.VertexPosition.VID;
			for (int TID : SurfaceMesh.VtxTrianglesItr(SurfaceVID))
			{
				if (TID == SurfaceTID)
				{
					bFoundTID = true;
					break;
				}
			}

		}

		return bFoundTID;
	}

	/** 
	* Collect the surface triangles that are adjacent to the specified surface point 
	* this may be one, two, or many - depending on location of the point (eg a surface point that is on an edge may have two adjacent tris)
	*/
	TArray<int32> GetAdjacentTriangles(const IntrinsicCorrespondenceUtils::FSurfacePoint& SurfacePoint, const FDynamicMesh3& SurfaceMesh)
	{

		TArray<int32> AdjTIDs;
		if (IsEdgePoint(SurfacePoint))
		{
			const int32 SurfaceEID      = SurfacePoint.Position.EdgePosition.EdgeID;
			const FIndex2i SurfaceEdgeT = SurfaceMesh.GetEdgeT(SurfaceEID);
			AdjTIDs.Add(SurfaceEdgeT.A);
			if (SurfaceEdgeT.B != -1)
			{
				AdjTIDs.Add(SurfaceEdgeT.B);
			}
		}
		else if (IsFacePoint(SurfacePoint))
		{
			const int32 SurfaceTID = SurfacePoint.Position.TriPosition.TriID;
			AdjTIDs.Add(SurfaceTID);
		}
		else
		{
			checkSlow(IsVertexPoint(SurfacePoint));
			const int32 SurfaceVID = SurfacePoint.Position.VertexPosition.VID;
			for (int SurfaceTID : SurfaceMesh.VtxTrianglesItr(SurfaceVID))
			{
				AdjTIDs.Add(SurfaceTID);
			}
		}
		return MoveTemp(AdjTIDs);
	}

	/**  
	* Convert a surface point to barycentric coordinates relative to the specified surface triangle. 
	*     NB: this assumes (but does not check) that the surface point is on or adjacent to the surface triangle. 
	*/
	FVector3d AsBarycenteric(const IntrinsicCorrespondenceUtils::FSurfacePoint& SurfacePoint, const int32 SurfaceTID, const FDynamicMesh3& SurfaceMesh)
	{
		FVector3d BC(0., 0., 0.);


		if (IsEdgePoint(SurfacePoint))  // translate an edge point to barycentric coordinates
		{
			const int32 SurfaceEID = SurfacePoint.Position.EdgePosition.EdgeID;
			const double Alpha     = SurfacePoint.Position.EdgePosition.Alpha; // Pos(Edge.A) (Alpha) + (1-Alpha) Pos(Edge.B)

			checkSlow(SurfaceMesh.GetEdgeT(SurfaceEID).IndexOf(SurfaceTID) != -1);

			const FIndex2i SurfaceEdgeV = SurfaceMesh.GetEdgeV(SurfaceEID);
			const FIndex3i SurfaceTri   = SurfaceMesh.GetTriangle(SurfaceTID);

			for (int32 i = 0; i < 3; ++i)
			{
				if (SurfaceTri[i] == SurfaceEdgeV.A)
				{
					BC[i] = Alpha;
				}
				else if (SurfaceTri[i] == SurfaceEdgeV.B)
				{
					BC[i] = (1. - Alpha);
				}
			}
		}
		else if (IsFacePoint(SurfacePoint)) // translate a face point to barycentric coordinates
		{
			checkSlow(SurfacePoint.Position.TriPosition.TriID == SurfaceTID);
			BC = SurfacePoint.Position.TriPosition.BarycentricCoords;
		}
		else // translate a vertex point to barycentric coordinates
		{
			const int32 SurfaceVID    = SurfacePoint.Position.VertexPosition.VID;
			const FIndex3i SurfaceTri = SurfaceMesh.GetTriangle(SurfaceTID);
			bool bValid = false;
			for (int32 i = 0; i < 3; ++i)
			{
				if (SurfaceTri[i] == SurfaceVID)
				{
					BC[i] = 1.;
					bValid = true;
				}
			}
			checkSlow(bValid);
		}
		return BC;
	}

	bool AreOnSameSurfaceEdge(const IntrinsicCorrespondenceUtils::FSurfacePoint& PointA, const IntrinsicCorrespondenceUtils::FSurfacePoint&  PointB, const FDynamicMesh3& Mesh)
	{
		if (IsFacePoint(PointA) || IsFacePoint(PointB))
		{
			return false;
		}

		if (IsVertexPoint(PointA) && IsVertexPoint(PointB))
		{
			return Mesh.FindEdge(PointA.Position.VertexPosition.VID, PointB.Position.VertexPosition.VID) != FDynamicMesh3::InvalidID;
		}

		auto TestVertAndEdge = [&Mesh](const IntrinsicCorrespondenceUtils::FSurfacePoint& VertexPoint, const IntrinsicCorrespondenceUtils::FSurfacePoint& EdgePoint)->bool
										{
											const int32 VID = VertexPoint.Position.VertexPosition.VID;
											const int32 EID = EdgePoint.Position.EdgePosition.EdgeID;
											const FDynamicMesh3::FEdge& Edge = Mesh.GetEdgeRef(EID);
											return (Edge.Vert.IndexOf(VID) != -1);
										};
		if (IsVertexPoint(PointA))
		{
			// B must be an edge point.
			return TestVertAndEdge(PointA, PointB);
		}
		else
		{
			// B must be vert and A is edge
			return TestVertAndEdge(PointB, PointA);
		}
	}

	/** array given matrix-like access with row-major layout for a non-square matrix */
	struct FNonSqrMat
	{
		FNonSqrMat(int32 NumCols, int32 NumRows)
			: N(NumCols)
			, M(NumRows)
		{
			Data.AddZeroed(N * M);
		}

		double& operator()(int32 i, int32 j)
		{
			return Data[RowMajorOffset(i, j)];
		}
		const double& operator()(int32 i, int32 j) const
		{
			return Data[RowMajorOffset(i, j)];
		}
		int32 RowMajorOffset(int32 i, int32 j) const
		{
			return i * N + j;
		}

		int32 N;
		int32 M;
		TArray<double> Data;
	};

	/**
	* Given an underspecified matrix equation (more unknowns than equations) of the form  M * Soln = BVector, 
	* where M is a Nx3 matrix, Soln is an N-dimensional array, and BVector is a 3-dimensional array
	* 
	* this computes the solution that minimizes the quantity |Soln|
	* 
	* The matrix M is specified by providing the three rows of the matrix (Row0, Row1, Row2).
	* 
	* @return false on failure, otherwise the vector Soln will hold the result.
	*/
	bool ComputeMinNormSolution(const TArray<double>& Row0, const TArray<double>& Row1, const TArray<double>& Row2, const FVector3d& BVector, TArray<double>& Soln)
	{
		checkSlow(Row0.Num() == Row1.Num() && Row2.Num() == Row1.Num());

		const int32 NCols = Row0.Num();

		checkSlow(NCols > 2);


		Soln.SetNum(NCols);

		if (NCols == 3)
		{
			const FMatrix3d Mat3d(Row0[0], Row0[1], Row0[2],
								  Row1[0], Row1[1], Row1[2],
								  Row2[0], Row2[1], Row2[2]);

			const double Det = Mat3d.Determinant();
			if (TMathUtil<double>::Abs(Det) < TMathUtil<double>::Epsilon)
			{
				return false;
			}
			else
			{
				const FMatrix3d InvMat3d    = Mat3d.Inverse();
				const FVector3d SolVector3d = InvMat3d * BVector;

				Soln[0] = SolVector3d[0];
				Soln[1] = SolVector3d[1];
				Soln[2] = SolVector3d[2];
			}
		}
		else
		{
			FNonSqrMat Mat(NCols, 3);
			for (int j = 0; j < NCols; ++j)
			{
				Mat(0, j) = Row0[j];
				Mat(1, j) = Row1[j];
				Mat(2, j) = Row2[j];
			}

			// form MMt = Mat * Transpose(Mat)

			FMatrix3d MMt(0.);
			for (int32 i = 0; i < 3; ++i)
			{
				FVector3d& Row = (i==0) ? MMt.Row0 : (i==1) ? MMt.Row1 : MMt.Row2 ;
				for (int32 j = 0; j < 3; ++j)
				{
					for (int32 k = 0; k < NCols; ++k)
					{
						Row[j] += Mat(i, k) * Mat(j, k);
					}
				}
			}

			// solve MMt y = b, and compute x = Mt * y
			const double Det = MMt.Determinant();
			if (TMathUtil<double>::Abs(Det) < TMathUtil<double>::Epsilon)
			{
				return false;
			}
			else
			{
				const FMatrix3d InvMMt    = MMt.Inverse();
				const FVector3d YVector3d = InvMMt * BVector;

				//Soln = Transpose(Mat) * Yvector;
				for (int32 j = 0; j < NCols; ++j)
				{
					Soln[j] = Mat(0, j) * YVector3d[0] + Mat(1, j) * YVector3d[1] + Mat(2, j) * YVector3d[2];
				}
			}
		}

		return true;
	}

} // end namespace FIntrinsicMeshImplUtils


/**------------------------------------------------------------------------------
*  FIntrinsicMesh Methods
*-------------------------------------------------------------------------------*/

UE::Geometry::FIntrinsicMesh::FIntrinsicMesh(const FDynamicMesh3& SurfaceMesh)
	: MyBase(SurfaceMesh)
	, NormalCoordinates(SurfaceMesh)
{
	const int32 MaxVertexID = SurfaceMesh.MaxVertexID();

	// add surface position.

	IntrinsicVertexPositions.SetNum(MaxVertexID);

	// initialize: identify with vertex in Extrinsic Mesh
	for (int32 VID = 0; VID < MaxVertexID; ++VID)
	{
		if (SurfaceMesh.IsVertex(VID))
		{
			FSurfacePoint VertexSurfacePoint(VID);
			IntrinsicVertexPositions[VID] = VertexSurfacePoint;
		}
	}

}


TArray<UE::Geometry::FIntrinsicMesh::FSurfacePoint> FIntrinsicMesh::TraceEdge(int32 IntrinsicEID, double CoalesceThreshold, bool bReverse) const
{
	return FNormalCoordIntrinsicTraceImpl::TraceEdge(*this, IntrinsicEID, CoalesceThreshold, bReverse);
}

TArray<UE::Geometry::FIntrinsicMesh::FEdgeAndCrossingIdx> FIntrinsicMesh::GetImplicitEdgeCrossings(const int32 SurfaceEID, const bool bReverse) const
{
	return FNormalCoordSurfaceTraceImpl::TraceSurfaceEdge(*this, SurfaceEID, bReverse);
}


TArray<UE::Geometry::FIntrinsicMesh::FSurfacePoint> FIntrinsicMesh::TraceSurfaceEdge(int32 SurfaceEID, double CoalesceThreshold, bool bReverse) const
{
	const FDynamicMesh3& SurfaceMesh = *this->GetExtrinsicMesh();
	const FIntrinsicMesh& IntrinsicMesh = *this;
	TArray<FEdgeAndCrossingIdx> EdgeAndCrossingIdxs = this->GetImplicitEdgeCrossings(SurfaceEID, bReverse);

	const int32 NumEdgeAndXIdx = EdgeAndCrossingIdxs.Num();

	TArray<UE::Geometry::FIntrinsicMesh::FSurfacePoint>  ResultTraceArray;

	if (NumEdgeAndXIdx == 0)
	{
		return ResultTraceArray;
	}

	// find start (.A) and end (.B) trace mesh vids for this edge
	const FIndex2i OrderedTraceEdgeV = [&]
										{
											const FIndex2i EdgeV = SurfaceMesh.GetEdgeV(SurfaceEID);
											if (bReverse)
											{
												return FIndex2i(EdgeV.B, EdgeV.A);
											}
											else
											{
												return EdgeV;
											}
										}();

	// the start and end are vertices in both SurfaceMesh and IntrinsicMesh, furthermore when vertices exist on both meshes they have the same VID 
	const FSurfacePoint TraceStartSurfacePoint(OrderedTraceEdgeV.A);
	const FSurfacePoint TraceEndSurfacePoint(OrderedTraceEdgeV.B);


	ResultTraceArray.Add(TraceStartSurfacePoint);

	// due to (possible) intrinsic mesh edge splits, the trace of the surface edge across the intrinsic mesh has to be broken
	// into sections that follow split intrinsic mesh edges, and those that cross the faces of intrinsic mesh triangles.
	{
		FSurfacePoint LocalStartSurfacePoint = TraceStartSurfacePoint;
		TArray<int32> IntrinsicEdgesCrossed;
		for (int32 i = 1; i < NumEdgeAndXIdx; ++i)
		{
			const FEdgeAndCrossingIdx& EdgeIDandXIdx = EdgeAndCrossingIdxs[i];
			if (EdgeIDandXIdx.CIdx != 0)
			{
				IntrinsicEdgesCrossed.Add(EdgeIDandXIdx.EID);
			}
			else if (IntrinsicEdgesCrossed.Num() > 0)
			{
				// current point is the end point of local face crossing.
				// convert to surface point.
				const FSurfacePoint LocalEndSurfacePoint( IntrinsicMesh.GetTriangle(EdgeIDandXIdx.TID)[EdgeIDandXIdx.EID]);
				
				// compute crossings
				FNormalCoordSurfaceTraceImpl::ConvertEdgesCrossed( LocalStartSurfacePoint, LocalEndSurfacePoint, IntrinsicEdgesCrossed, 
					                                               IntrinsicMesh, SurfaceMesh, CoalesceThreshold,
															       [](int32 vid){return FSurfacePoint(vid);}, ResultTraceArray);
				
				// add surface point 
				ResultTraceArray.Add(LocalEndSurfacePoint);

				// empty the temp crossings.
				IntrinsicEdgesCrossed.Reset();

				LocalStartSurfacePoint = LocalEndSurfacePoint;
			}
			else
			{
				// convert to surface point.
				const FSurfacePoint LocalEndSurfacePoint(IntrinsicMesh.GetTriangle(EdgeIDandXIdx.TID)[EdgeIDandXIdx.EID]);
				// add surface point 
				ResultTraceArray.Add(LocalEndSurfacePoint);
				LocalStartSurfacePoint = LocalEndSurfacePoint;
			}
		}

		// should have processed all edge crossings.
		checkSlow(IntrinsicEdgesCrossed.Num() == 0);
		// ended on the last vertex.
		checkSlow(ResultTraceArray.Last().Position.VertexPosition.VID == OrderedTraceEdgeV.B); 
	}
	
	return MoveTemp(ResultTraceArray);
}

UE::Geometry::EMeshResult UE::Geometry::FIntrinsicMesh::FlipEdge(int32 EID, FEdgeFlipInfo& EdgeFlipInfo)
{
	using namespace FIntrinsicMeshImplUtils;

	if (!IsEdge(EID))
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	if (IsBoundaryEdge(EID))
	{
		return EMeshResult::Failed_IsBoundaryEdge;
	}

	// state before flip, needed when updating the normal coords after the flip
	const FIndex2i EdgeT  = GetEdgeT(EID);
	const FIndex3i TAEIDs = GetTriEdges(EdgeT.A);
	const FIndex3i TBEIDs = GetTriEdges(EdgeT.B);
	const FIndex2i OppVs  = GetEdgeOpposingV(EID);

	EMeshResult FlipResult = MyBase::FlipEdge(EID, EdgeFlipInfo);

	if (FlipResult == EMeshResult::Ok)
	{
		// update the normal coords
		NormalCoordinates.OnFlipEdge(EdgeT.A, TAEIDs, OppVs.A, EdgeT.B, TBEIDs, OppVs.B, EID);

		// if the flip produced an intrinsic edge that is a segment of a surface edge, update the 
		// coorindate to -1 to help tracing.
		if (NormalCoordinates.NormalCoord[EID] == 0 && OppVs.A != OppVs.B)
		{
			FSurfacePoint SurfacePoints[2] = {GetVertexSurfacePoint(OppVs.A), GetVertexSurfacePoint(OppVs.B)};
			if (AreOnSameSurfaceEdge(SurfacePoints[0], SurfacePoints[1], *NormalCoordinates.SurfaceMesh))
			{
				NormalCoordinates.NormalCoord[EID] = -1;
			}
		}
	}

	return FlipResult;
}


UE::Geometry::EMeshResult UE::Geometry::FIntrinsicMesh::PokeTriangle(int32 IntrinsicTID, const FVector3d& BaryCoordinates, FPokeTriangleInfo& PokeInfo)
{
	// The intrinsic poke  creates 3 new intrinsic edges and "normal coordinates"(the number of crossing surface edges) need to be created for each.
	// 
	// also, "roundabout" data is recored for any new intrinsic edge that is adjacent to an intrinsic vertex that is also a surface vertex.
	// this roundabout data indicates the next surface edge when traveling ccw about the vertex from the new edge.
	// 
	// lastly the surface position of the new intrinsic vertex is computed and recored.
	//
	// note: to compute the surface position of the new intrinsic vertex, the intersection (convex polygon) between the original intrinsic triangle and the surface triangle 
	// that supports the new vertex is first identified 

	using namespace FIntrinsicMeshImplUtils;

	if (!IsTriangle(IntrinsicTID))
	{
		return EMeshResult::Failed_NotATriangle;
	}

	// state before the poke
	const FIndex3i OriginalVIDs            = GetTriangle(IntrinsicTID);
	const FIndex3i OriginalEdges           = GetTriEdges(IntrinsicTID);
	const FVector3d OriginalTriEdgeLengths = GetTriEdgeLengths(IntrinsicTID);

	const int32 IndexOf = VectorUtil::Min3Index(OriginalTriEdgeLengths);

	// fail if the smallest side of the triangle is too small
	//[todo] consider permuting the triangle so the zero side is the longest.
	if (OriginalTriEdgeLengths[IndexOf] < TMathUtilConstants<double>::ZeroTolerance)
	{
		return EMeshResult::Failed_Unsupported;
	}

	
	// unwrap the intrinsic triangle to a 2d plane.
	// edge0 runs along the positive x-axis
	const FVector2d IntrinsicTri2D[3] = { FVector2d(0.,0.),
								          FVector2d(OriginalTriEdgeLengths[0], 0.),
								          ComputeOpposingVert2d(OriginalTriEdgeLengths[0], OriginalTriEdgeLengths[1], OriginalTriEdgeLengths[2]) };

	auto AsR2Position = [&IntrinsicTri2D](const FVector3d& BCoords)
						{
							return BCoords[0] * IntrinsicTri2D[0] + BCoords[1] * IntrinsicTri2D[1] + BCoords[2] * IntrinsicTri2D[2];
						};

	// position of the new vertex relative to the flattened intrinsic triangle.								
	const FVector2d PokedPos = AsR2Position(BaryCoordinates);

	// Utility: converts distance along a directed edge of an the intrinsic triangle to barycentric coords.
	auto EdgeDistanceToBaryCoords = [&OriginalTriEdgeLengths](int32 TriSide, double Distance)
									{
										const double EdgeLength = OriginalTriEdgeLengths[TriSide];
										const double Alpha = (EdgeLength > TMathUtilConstants<double>::ZeroTolerance) ? Distance / EdgeLength : 0.5;

										FVector3d Result(0., 0., 0.);
										Result[TriSide]           = (1. - Alpha);
										Result[(TriSide + 1) % 3] = Alpha;
										return Result;
									};


	// -- find the surface edges that cross the intrinsic triangle.

	// for each side of the intrinsic triangle, generate a list of places where the surfaces edge cross the intrinsic edge
	// storing the location as a FPointCorrespondence
	
	// point described both relative to the intrinsic triangle and to the surface mesh.
	struct FPointCorrespondence
	{
		FVector2d LocalPos;       // position relative to flattened intrinsic triangle
		FVector3d IntrinsicBC;    // barycentric coordinates relative to intrinsic triangle ( duplicates info in LocalPos, but easier to work with)
		FSurfacePoint SurfacePos; // position relative to surface mesh
	};

	// for each directed side the intrinsic triangle, recored all the surface edge crossings and their relative locations along the side.
	TMap<int32, FPointCorrespondence> SurfaceEdgeXings[3];
	{
		auto GetSurfaceEdgeCrossingLocations = [&](const int32 Side)
												{
													const int32 IntrinsicEID = OriginalEdges[Side];
													const FDynamicMesh3& SurfaceMesh = *NormalCoordinates.SurfaceMesh;

													const double CoalesceThreshold = 0.;
													const bool bReverse = (GetEdgeT(IntrinsicEID).A != IntrinsicTID);

													// sequence of surface points corresponding to surface edges intersecting this intrinsic edge. 
													const TArray<FSurfacePoint> EdgeAsSurfacePoints = TraceEdge(IntrinsicEID, CoalesceThreshold, bReverse);


													// record the surface point and the distance along this triangle side to the intersection 
													// key with the surface edge id.
													TMap<int32, FPointCorrespondence> CrossingLocations;
													const int32 NumSPoints = EdgeAsSurfacePoints.Num();
													if (NumSPoints > 0)
													{ 
														bool bTmp;
														FVector3d Positions[2];
														Positions[0] = AsR3Position(EdgeAsSurfacePoints[0], SurfaceMesh, bTmp);

														int32 Cur = 1;
														double AccumulatedDistance = 0.;

														for (int i = 1; i < NumSPoints; ++i)
														{
															const FSurfacePoint& SurfacePoint = EdgeAsSurfacePoints[i];
															Positions[Cur] = AsR3Position(SurfacePoint, SurfaceMesh, bTmp);
															const double LineElementLength = (Positions[Cur] - Positions[1 - Cur]).Length();
															AccumulatedDistance += LineElementLength;
															if (SurfacePoint.PositionType == IntrinsicCorrespondenceUtils::FSurfacePoint::EPositionType::Edge)
															{
																const int32 SurfaceEID = SurfacePoint.Position.EdgePosition.EdgeID;
																const FVector3d IntrinsicBaryCoords = EdgeDistanceToBaryCoords(Side, AccumulatedDistance);
																FPointCorrespondence XingCorrespondence = {AsR2Position(IntrinsicBaryCoords), IntrinsicBaryCoords, SurfacePoint};
																CrossingLocations.Add(SurfaceEID, XingCorrespondence);
															}

															Cur = 1 - Cur;
														}
													}
													return MoveTemp(CrossingLocations);
												};
		for (int32 i = 0; i < 3; ++i)
		{
			SurfaceEdgeXings[i] = GetSurfaceEdgeCrossingLocations(i);
		}
	}
	
	// the intersection of a surface edge with the intrinsic triangle.
	struct FSurfaceEdgeSegment
	{
		// end points where this surface edge intersects the boundary of the intrinsic triangle.
		FPointCorrespondence P0;
		FPointCorrespondence P1;

		// corresponding surface mesh edge id
		int32 SurfaceEID = -1;
	};

	// after the poke, new intrinsic edges will connect the new vertex to the corners of the original triangle.
	// number these as  {0, 1, 2} according to the corners of the IntrinsicTri2D
	
	
	// compute "normal coordinates", i.e. find the number of times each new intrinsic edge is intersected by surface mesh edge segments
	// additionally, (for each new intrinsic edge) identify the intersecting surface edge that was closest to the new vertex.
	
	int32 NewEdgeNormCoords[3] = { 0, 0, 0 };         
	FSurfaceEdgeSegment ClosestSurfaceEdgeSegment[3]; // per new intrinsic edge
	{
		auto UpdateNormCoordsAndClosest = [&NewEdgeNormCoords, &ClosestSurfaceEdgeSegment](const FPointCorrespondence& P0, const FPointCorrespondence& P1, int32 SurfaceEID, int32 ZeroOneOrTwo)
											{
												NewEdgeNormCoords[ZeroOneOrTwo] += 1;
												FSurfaceEdgeSegment& ClosestSegment = ClosestSurfaceEdgeSegment[ZeroOneOrTwo];
												const bool bHaveValidClosest = (ClosestSegment.SurfaceEID != -1);
												if (bHaveValidClosest)
												{
													// by construction the surface edge segments can not intersect in the face of an intrinsic triangle
													// so we need only check a single point of the old segment against the one defined by P0 to P1
													const FVector2d OldSegmentCenter = 0.5 * (ClosestSegment.P0.LocalPos + ClosestSegment.P1.LocalPos);
													const double SideTest = UE::Geometry::Orient(P0.LocalPos, P1.LocalPos, OldSegmentCenter);
													if (SideTest > 0) // old surface segment is to the right of this new one segment 
													{
														ClosestSegment.P0 = P0;
														ClosestSegment.P1 = P1;
														ClosestSegment.SurfaceEID = SurfaceEID;
													}
												}
												else
												{
													// just update since this is the first intersecting surface edge
													ClosestSegment.P0 = P0;
													ClosestSegment.P1 = P1;
													ClosestSegment.SurfaceEID = SurfaceEID;
												}
											};

		for (int32 Side = 0; Side < 3; ++Side)
		{

			const int32 NextSide     = (Side + 1) % 3;
			const int32 NextNextSide = (Side + 2) % 3;

			for (const TPair<int32,  FPointCorrespondence>& XingPair : SurfaceEdgeXings[Side])
			{
				const int32& SurfaceEID        = XingPair.Key;
				const FPointCorrespondence& P0 = XingPair.Value;

				if ( FPointCorrespondence* P1Ptr = SurfaceEdgeXings[NextSide].Find(SurfaceEID)) // the surface edge exits the next side
				{
					const FPointCorrespondence& P1 = *P1Ptr;
					const double SignedArea = UE::Geometry::Orient(P0.LocalPos, P1.LocalPos, PokedPos); //same as (P1-P0)X(PokedPos - P0)

					if (SignedArea <= 0) // surface segment crosses two new intrinsic edges.
					{
						// [todo] better treatment for == case?
						UpdateNormCoordsAndClosest(P0, P1, SurfaceEID, Side);
						UpdateNormCoordsAndClosest(P0, P1, SurfaceEID, NextNextSide);
					}
					if (SignedArea > 0) // surface segment crosses one new intrinsic edges.
					{
						UpdateNormCoordsAndClosest(P1, P0, SurfaceEID, NextSide);
					}

				}
				else if (SurfaceEdgeXings[NextNextSide].Contains(SurfaceEID) == false) // the surface edge must exit the opp vertex
				{
					FVector3d P1BC(0., 0., 0.);  P1BC[NextNextSide] = 1.;

					const FPointCorrespondence P1 = { IntrinsicTri2D[NextNextSide], P1BC, IntrinsicVertexPositions[OriginalVIDs[NextNextSide]] };;
					const double SignedArea = UE::Geometry::Orient(P0.LocalPos, P1.LocalPos, PokedPos); // same as (P1-P0)X(PokedPos - P0)

					if (SignedArea <= 0) // surface segment crosses one new intrinsic edges.
					{
						// [todo] better treatment for == case?
						UpdateNormCoordsAndClosest(P0, P1, SurfaceEID, Side);
					}
					if (SignedArea > 0) // surface segment crosses one new intrinsic edges.
					{
						UpdateNormCoordsAndClosest(P1, P0, SurfaceEID, NextSide);
					}
				}
			}
		}
	}


	// intersection of the intrinsic mesh triangle (prior to poke) and the surface mesh triangle that ultimately supports the new vertex
	// forms a convex polygon.   Convert the vertices of this polygon to FPointCorresponce. 
	const TArray<FPointCorrespondence> BoundingConvexPolyVerts = [&]
																{
																	TArray<FPointCorrespondence> TmpBoundaryPoints;
																	// method to add Correspondence points.  Makes sure we don't add a point twice
																	// note, only need to check surface verts due to our usage and the fact surface edges only meet at surface verts. 
																	TSet<int32> VisitedSurfaceVerts;
																	auto AddPointCorrespondence = [&](const FPointCorrespondence& PointCorrespondence)
																									{
																										const FSurfacePoint& SP = PointCorrespondence.SurfacePos;
																										if (IsVertexPoint(SP))
																										{
																											const int32 SurfaceVID = SP.Position.VertexPosition.VID;
																											if (!VisitedSurfaceVerts.Contains(SurfaceVID))
																											{
																												TmpBoundaryPoints.Add(PointCorrespondence);
																												VisitedSurfaceVerts.Add(SurfaceVID);
																											}
																										}
																										else
																										{
																											TmpBoundaryPoints.Add(PointCorrespondence);
																										}

																									};
																	
																	TSet<int32> VisitedSurfaceEdges;
																	for (int32 i = 0; i < 3; ++i)
																	{
																		// if it exists, this surface edge blocks
																		// the new intrinsic vertex's view of the i-th original tri corner.
																		const FSurfaceEdgeSegment& SurfaceEdgeSegment = ClosestSurfaceEdgeSegment[i];
																		const int32 SurfaceEdgeID = SurfaceEdgeSegment.SurfaceEID;

																		if (SurfaceEdgeID == -1) // no edge blocks..
																		{
																			const FVector2d& CornerLocalPos = IntrinsicTri2D[i];
																			const int32 VID = OriginalVIDs[i];
																			const FSurfacePoint& CornerSP = IntrinsicVertexPositions[VID];
																			FVector3d IntrinsicBC(0., 0., 0.);
																			IntrinsicBC[i] = 1.;

																			FPointCorrespondence PointCorrespondence = { CornerLocalPos, IntrinsicBC, CornerSP };
																			AddPointCorrespondence(PointCorrespondence);
																		}
																		else
																		{
																			if (!VisitedSurfaceEdges.Contains(SurfaceEdgeID))
																			{
																				VisitedSurfaceEdges.Add(SurfaceEdgeID);

																				AddPointCorrespondence(SurfaceEdgeSegment.P0);
																				AddPointCorrespondence(SurfaceEdgeSegment.P1);
																			}
																		}
																	}
																	return MoveTemp(TmpBoundaryPoints);
																}();


	
	// Identify the surface triangle that is common to the bounding convex polygon verts.
	const int32 SurfaceTID = [&]
							{
								const FDynamicMesh3& SurfaceMesh = *NormalCoordinates.SurfaceMesh;

								// special case: are the points just the corners of a surface triangle? 
								if (BoundingConvexPolyVerts.Num() == 3)
								{
									if (   IsVertexPoint(BoundingConvexPolyVerts[0].SurfacePos)
										&& IsVertexPoint(BoundingConvexPolyVerts[1].SurfacePos)
										&& IsVertexPoint(BoundingConvexPolyVerts[2].SurfacePos) )
									{
										int32 SurfaceVIDs[3] = { BoundingConvexPolyVerts[0].SurfacePos.Position.VertexPosition.VID,
											                     BoundingConvexPolyVerts[1].SurfacePos.Position.VertexPosition.VID,
											                     BoundingConvexPolyVerts[2].SurfacePos.Position.VertexPosition.VID
										};

										return SurfaceMesh.FindTriangle(SurfaceVIDs[0], SurfaceVIDs[1], SurfaceVIDs[2]);
									}
								}

								// find surface triangles corresponding to one of the surface points.
								// note: a surface point will correspond to 1, 2, or many surface triangles (face point, edge point, or vertex point).
								const TArray<int32> CandidateTIDs = [&]
																	{
																		TArray<int32> TmpTIDs;
																		// are any of the surface point already on a surface triangle face?
																		int32 TID = -1;
																		for (const FPointCorrespondence& BoundaryPoint : BoundingConvexPolyVerts)
																		{
																			const FSurfacePoint& SurfacePos = BoundaryPoint.SurfacePos;
																			if (IsFacePoint(SurfacePos))
																			{
																				TID = SurfacePos.Position.TriPosition.TriID;
																				break;
																			}
																		}

																		if (TID != -1)
																		{
																			TmpTIDs.Add(TID);
																		}
																		else
																		{
																			const FSurfacePoint& SurfacePos = BoundingConvexPolyVerts[0].SurfacePos;
																			// get all the surface triangles adjacent to this surface point
																			TmpTIDs = GetAdjacentTriangles(SurfacePos, SurfaceMesh);
																		}

																		return MoveTemp(TmpTIDs);
																	}();

								// determine which candidate triangle is actually adjacent to all the surface points.
								const int32 MutuallyAdjacentTID = [&]
																	{
																		for (int32 CandidateTID : CandidateTIDs)
																		{
																			bool bIsAdjacentToAll = true;
																			// check adjacency to the boundary points.
																			for (const FPointCorrespondence& BoundaryPoint : BoundingConvexPolyVerts)
																			{
																				const FSurfacePoint& SurfacePos = BoundaryPoint.SurfacePos;
																				bIsAdjacentToAll = bIsAdjacentToAll && IsOnSurfaceTriangle(SurfacePos, CandidateTID, SurfaceMesh);
																			}

																			if (bIsAdjacentToAll)
																			{
																				return CandidateTID;
																			}
																		}
																		return -1;
																	}();

								return MutuallyAdjacentTID;
							}();

	if (SurfaceTID == -1)
	{
		return EMeshResult::Failed_Unsupported;
	}



	// construct the surface barycentric coords for the poked position.
	const FVector3d PokedSurfaceBaryCoords = [&]
												{
													const int32 NumBoundaryPts = BoundingConvexPolyVerts.Num();
													// In the barycentric space of the intrinsic triangle:
													// find the (min norm) linear combination convex-poly vertices that is equivalent to the new vertex location
													TArray<double> MinNormSolution;
													TArray<double> MatRows[3];
													for (int32 i = 0; i < 3; ++i)
													{
														MatRows[i].AddUninitialized(NumBoundaryPts);

														for (int32 j = 0; j < NumBoundaryPts; ++j)
														{
															const FVector3d& IntrinsicBarycentric = BoundingConvexPolyVerts[j].IntrinsicBC;
															MatRows[i][j] = IntrinsicBarycentric[i];
														}
													}

													const bool bValidInterpolation = ComputeMinNormSolution(MatRows[0], MatRows[1], MatRows[2], BaryCoordinates, MinNormSolution);

													// convert the surface points to barycentric coords relative to the surface triangle.
													TArray<FVector3d> SurfaceBaryCoords;
													{
														const FDynamicMesh3& SurfaceMesh = *NormalCoordinates.SurfaceMesh;
														SurfaceBaryCoords.Reserve(NumBoundaryPts);
														for (const FPointCorrespondence& BoundaryPoint : BoundingConvexPolyVerts)
														{
															const FSurfacePoint& SurfacePos = BoundaryPoint.SurfacePos;
															SurfaceBaryCoords.Add(AsBarycenteric(SurfacePos, SurfaceTID, SurfaceMesh));
														}
													}

													// interpolate the surface barycentric coordinates (using the coefficients from the intrinsic barycenterics)
													// to construct the surface barycentric coordinates of the new vertex location.
													// note: if interpolation failed just use the center of the polygon.
													FVector3d SurfaceBarycoods(0., 0., 0.);
													if (bValidInterpolation)
													{
														for (int32 i = 0; i < NumBoundaryPts; ++i)
														{
															SurfaceBarycoods += MinNormSolution[i] * SurfaceBaryCoords[i];
														}
													}
													else
													{
														// just use the "center" of the convex polygon
														for (int32 i = 0; i < NumBoundaryPts; ++i)
														{
															SurfaceBarycoods += SurfaceBaryCoords[i];
														}
														SurfaceBarycoods *= 1. / double(NumBoundaryPts);
													}

													return SurfaceBarycoods;
												}();

	// surface point for the new (poked) vertex
	const FSurfacePoint PokedSurfacePoint(SurfaceTID, PokedSurfaceBaryCoords);

	// update the topology.

	// Add a new vertex and faces to the IntrinsicMesh.   
	// Note: the r3 position will be wrong initially since this poke will just interpolate the corners of the intrinsic tri.
	//       we fix this position as the last step in this function
	EMeshResult PokeResult = MyBase::PokeTriangle(IntrinsicTID, BaryCoordinates, PokeInfo);
	if (PokeResult != EMeshResult::Ok)
	{
		return PokeResult;
	}


	FIndex3i NewTris(IntrinsicTID, PokeInfo.NewTriangles[0], PokeInfo.NewTriangles[1]);
	const int32 NewVID = PokeInfo.NewVertex;

	// Need to update intrinsic information for the 3 triangles that resulted from the poke
	// 1) a)update the number of surface edge crossings for each new intrinsic edge
	//    b)update the roundabout data for any new intrinsic edge that is adj to a surface vertex
	// 2) update the surface position of the new vertex.

	{
		// set/update number of surface edge crossings for the new edges. 
		{
			// original tri is and verts (a, b,c) and edges (a2b, b2c, c2a)
			// is updated by the poke to have verts (a, b, new) and edges (a2b, b2new, new2a)
			const FIndex3i T0EIDs = GetTriEdges(NewTris[0]);
			const FIndex3i T1EIDs = GetTriEdges(NewTris[1]);
	
			// set the correct number of surface edges the cross each new intrinsic edge.
			NormalCoordinates.NormalCoord.InsertAt(NewEdgeNormCoords[0], T0EIDs[2]); // a2new
			NormalCoordinates.NormalCoord.InsertAt(NewEdgeNormCoords[2], T1EIDs[1]); // b2new
			NormalCoordinates.NormalCoord.InsertAt(NewEdgeNormCoords[1], T0EIDs[1]); // c2new
		}

		// update the roundabout information.
		{
			// edge order: a2b, b2c, c2a
			const FIndex3i  OriginalRO = NormalCoordinates.RoundaboutOrder[IntrinsicTID];

			// copy the roundabout information to the new triangles with default -1 for the new directed edges.

			FIndex3i NewTrisRO[3] = { FIndex3i(OriginalRO[0], -1, -1) // NewTri[0]   edge order: a2b, b2new, new2a
									 ,FIndex3i(OriginalRO[1], -1, -1) // NewTri[1]   edge order: b2c, c2new, new2b
									 ,FIndex3i(OriginalRO[2], -1, -1) // NewTri[2]   edge order: c2a, a2new, new2c
									};

			// update roundabout order for any new directed edge that starts at an implicit vertex that is also a surface vertex 	
			// Note: by construction the new intrinsic vertex does not correspond to a vertex on the surface mesh so we need only consider a2new, b2new, c2new.


			const FSurfacePoint OriginalTriSPs[3] = { IntrinsicVertexPositions[OriginalVIDs[0]],
													  IntrinsicVertexPositions[OriginalVIDs[1]],
													  IntrinsicVertexPositions[OriginalVIDs[2]] };

			for (int32 i = 0; i < 3; ++i)
			{
				if (IsVertexPoint(OriginalTriSPs[i]))
				{
					const FIndex3i TriEIDs = GetTriEdges(NewTris[i]);
					const int32 Radj = NewTrisRO[i][0];                                            // {R_ab,  R_bc,  R_ca }  for i = 0, 1, or 2
					const int32 Eopp = NormalCoordinates.NumCornerEmanatingRefEdges(TriEIDs, 0);  // {Ebn_a, Ecn_b, Ean_c} for i = 0, 1, or 2
					const int32 Rnew = (Radj + Eopp) % NormalCoordinates.RefVertDegree[OriginalVIDs[i]];

					NewTrisRO[(i + 2) % 3][1] = Rnew;
				}
			}

			NormalCoordinates.RoundaboutOrder.InsertAt(NewTrisRO[0], NewTris[0]);
			NormalCoordinates.RoundaboutOrder.InsertAt(NewTrisRO[1], NewTris[1]);
			NormalCoordinates.RoundaboutOrder.InsertAt(NewTrisRO[2], NewTris[2]);
		}
	}
	// update the position of the new vertex ( R3 position and the surface point )
	bool bIsValid;
	const FVector3d R3Pos = IntrinsicCorrespondenceUtils::AsR3Position(PokedSurfacePoint, *NormalCoordinates.SurfaceMesh, bIsValid);
	Vertices[NewVID] = R3Pos;
	IntrinsicVertexPositions.InsertAt(PokedSurfacePoint, NewVID);

	return EMeshResult::Ok;
}



UE::Geometry::EMeshResult UE::Geometry::FIntrinsicMesh::SplitEdge(int32 EdgeAB, FEdgeSplitInfo& SplitInfo, double SplitParameterT)
{

	using namespace FIntrinsicMeshImplUtils;

	if (!IsEdge(EdgeAB))
	{
		return EMeshResult::Failed_NotAnEdge;
	}

	const FDynamicMesh3& SurfaceMesh = *NormalCoordinates.SurfaceMesh;

	const FEdge OriginalEdge = GetEdge(EdgeAB);
	// is the target intrinsic edge equivalent to a segment of a surface edge?
	const bool bIsOnSurfaceEdge   = NormalCoordinates.IsSurfaceEdgeSegment(EdgeAB);
	// say new vertex is f.
	const int32 TID0 = OriginalEdge.Tri[0];
	const int32 IndexOfe = GetTriEdges(TID0).IndexOf(EdgeAB);
	const int32 TID1 = OriginalEdge.Tri[1];

	const bool bIsBoundary = (TID1 == -1);
	const int32 IndexOf1e = (!bIsBoundary) ? GetTriEdges(TID1).IndexOf(EdgeAB) : -1;

	// Info about the original T0 tri, reordered to make the split edge the first edge..
	const FIndex3i OriginalT0VIDs         = Permute(IndexOfe, GetTriangle(TID0));
	const FIndex3i OriginalT0Edges        = Permute(IndexOfe, GetTriEdges(TID0));
	const FVector3d OriginalT0EdgeLengths = Permute(IndexOfe, GetTriEdgeLengths(TID0));    // as (|ab|, |bc|, |ca|)
	
	// which, if any, of the original intrinsic edges are coincident with surface edges
	// in the order { edgeAB, edgeBC, edgeCA, edgeDB }
	const bool Kroneckers[4] = { NormalCoordinates.IsSurfaceEdgeSegment(OriginalT0Edges[0]), 
	                             NormalCoordinates.IsSurfaceEdgeSegment(OriginalT0Edges[1]), 
							     NormalCoordinates.IsSurfaceEdgeSegment(OriginalT0Edges[2]),
								 (bIsBoundary) ? false : NormalCoordinates.IsSurfaceEdgeSegment(Permute(IndexOf1e, GetTriEdges(TID1))[2])};

	

	const double IntrinsicEdgeSplitDistance = SplitParameterT * OriginalT0EdgeLengths[0];

	// description of surface edge intersection with intrinsic edge AB
	struct FEdgeDistanceCorrespondence
	{
		int32  SurfaceEID = -1;      // surface edge
		double DistOnEdge;           // distance measured along the intrinsic edge
		FSurfacePoint  SurfacePoint; // location on the surface mesh

	};

	auto AsR3Pos = [&SurfaceMesh](const FSurfacePoint& SP)
				{
					bool bTemp;
					return AsR3Position(SP, SurfaceMesh, bTemp);
				};

	// -- prior to actually doing the edge split, we find the two points (surface edge intersections) that bracket the proposed edge split location
	//    and pre-compute the normal coordinates for the 4 edges that are changed by the split

	// initialize the bounding points with the ends of the edge being split. 
	FEdgeDistanceCorrespondence BoundingPoints[2] = { {-1, 0., IntrinsicVertexPositions[OriginalT0VIDs[0]]}, 
													  {-1, OriginalT0EdgeLengths[0], IntrinsicVertexPositions[OriginalT0VIDs[1]]} };

	int32 UpdatedNormCoord = 0; // after split edgeAB becomes edge [a, f]													
	int32 NewEdgeNormCoords[3] = { 0, 0, 0}; //N_fb, N_fc, N_fd  
	
	{
		if (NormalCoordinates.NumEdgeCrossing(EdgeAB) == 0) // no surface edges cross the intrinsic edge being split
		{
			// potentially propagating -1 from NormalCoord[EdgeAB] may explicitly mark these as a segment of EdgeAB  
			UpdatedNormCoord     = NormalCoordinates.NormalCoord[EdgeAB];  // becomes edgeAF
			NewEdgeNormCoords[0] = NormalCoordinates.NormalCoord[EdgeAB];  // new edge, edgeFB


			NewEdgeNormCoords[1] = FMath::Max(NormalCoordinates.NumEdgeCrossing(OriginalT0Edges[1]), NormalCoordinates.NumEdgeCrossing(OriginalT0Edges[2]));

			// poke makes t0 = {a, f, c} and  t2 = { f, b, c}  ( note: t1 = {f, a, d}  t3 = f, d, b}  don't exist in the boundary edge case )
		
			if (!bIsBoundary)
			{
				// Info about the original T1 tri, reordered to make the split edge the first edge..
			
				const FIndex3i OriginalT1Edges        = Permute(IndexOf1e, GetTriEdges(TID1));
				const FVector3d OriginalT1EdgeLengths = Permute(IndexOf1e, GetTriEdgeLengths(TID1));   // as (|ba|, |ad|, |db})

				NewEdgeNormCoords[2] = FMath::Max(NormalCoordinates.NumEdgeCrossing(OriginalT1Edges[1]), NormalCoordinates.NumEdgeCrossing(OriginalT1Edges[2]));

			}
		} 
		else
		{
		

			const int32 IntrinsicEID = OriginalT0Edges[0];
		

			const FIndex3i OriginalT1Edges = (TID1 != -1) ? Permute(IndexOf1e, GetTriEdges(TID1)) : FIndex3i(-1, -1, -1);


		

			// sequence of surface points corresponding to surface edges intersecting intrinsic edge  ab. 
			// ordered relative to  OriginalEdge.Tri[0];
			const TArray<FSurfacePoint> ABEdgeAsSurfacePoints = [&] 
																{
																	const double CoalesceThreshold = 0.;
																	const bool bReverse = false;
																	return TraceEdge(IntrinsicEID, CoalesceThreshold, bReverse);
																}();


			auto GetExitEdge = [&](const int32 TriID, const int32 P)
									{
										int32 ExitEID = -1;
										if (TriID != -1)
										{
											const TTuple<int32, int32> ExitEIDandP = FNormalCoordSurfaceTraceImpl::GetCrossingExit(*this, NormalCoordinates, TriID, EdgeAB, P);
											ExitEID = (ExitEIDandP.Get<1>() != 0) ? ExitEIDandP.Get<0>() : -1;
										}
										return ExitEID;
									};

		
			// account for surface edges that cross the intrinsic edgeAB at the endpoints and the new edges, ie edgeFC (or edgeFD)
			{
				const int32 Ebc_a = NormalCoordinates.NumCornerEmanatingRefEdges(OriginalT0Edges, 0); // surface edges that start at 'a' and exit side bc
				const int32 Eca_b = NormalCoordinates.NumCornerEmanatingRefEdges(OriginalT0Edges, 1); // surface edges that start at 'b' and exit side ca

				NewEdgeNormCoords[1] += Eca_b + Eca_b;  // new edge - edgeFC

				if (!bIsBoundary)
				{
					const int32 Ead_b = NormalCoordinates.NumCornerEmanatingRefEdges(OriginalT1Edges, 0); // surface edges that start at 'b' and exit side ad
					const int32 Edb_a = NormalCoordinates.NumCornerEmanatingRefEdges(OriginalT1Edges, 1); // surface edges that start at 'a' and exit side db

					NewEdgeNormCoords[2] += Ead_b + Edb_a;  // new edge - edgeFD
				}
			}

			// count the number of surface edges that cross both edgeAB and the edgeFC, and those that cross edgeAB and edgeFD
			// Also identify the two surface edge crossing of edgeAB that bracket the vertex produced by the split.
			{
				// double buffer.
				int32 Cur = 1;
				FVector3d Positions[2];
		
				
				Positions[0] = AsR3Pos(ABEdgeAsSurfacePoints[0]);

				const int32 NumSPoints = ABEdgeAsSurfacePoints.Num();

		
		
				double AccumulatedDistance = 0;
				for (int32 i= 1; i < NumSPoints - 1; ++i) // the first and last points are the implicit edge start/end. 
				{
					const FSurfacePoint& SurfacePoint = ABEdgeAsSurfacePoints[i];
					Positions[Cur] = AsR3Pos(SurfacePoint);
					const double LineElementLength = (Positions[Cur] - Positions[1 - Cur]).Length();
					AccumulatedDistance += LineElementLength;
			
					Cur = 1 - Cur; // swap double buffer.

					// which side does this surface edge exit?
			
					const int32 T0ExitEID = GetExitEdge(TID0, i);
					const int32 T1ExitEID = GetExitEdge(TID1, NumSPoints - i);
			
					if (AccumulatedDistance < IntrinsicEdgeSplitDistance) // crossing to left of split
					{	
						UpdatedNormCoord += 1; // edgeAF

						checkSlow(IsEdgePoint(SurfacePoint));
						int32 SurfaceEID = SurfacePoint.Position.EdgePosition.EdgeID;
						BoundingPoints[0] = {SurfaceEID, AccumulatedDistance, SurfacePoint};
				
						if (OriginalT0Edges[1] == T0ExitEID)
						{ 
							NewEdgeNormCoords[1] +=1; // new edge - edgeFC
						}
						if (T1ExitEID != -1 && OriginalT1Edges[2] == T1ExitEID)
						{
							NewEdgeNormCoords[2] += 1; // new edge - edgeFD
						}
					}
					else // crossing to right of split
					{ 
						NewEdgeNormCoords[0] += 1;  // edgeFB

						if (AccumulatedDistance < BoundingPoints[1].DistOnEdge)
						{
							checkSlow(IsEdgePoint(SurfacePoint));
							int32 SurfaceEID = SurfacePoint.Position.EdgePosition.EdgeID;
							BoundingPoints[1] = { SurfaceEID, AccumulatedDistance, SurfacePoint };
						}

						if (OriginalT0Edges[2] == T0ExitEID)
						{
							NewEdgeNormCoords[1] += 1; // new edge - edgeFC
						}
						if (T1ExitEID != -1 && OriginalT1Edges[1] == T1ExitEID)
						{
							NewEdgeNormCoords[2] += 1; // new edge - edgeFD
						}

					}
				}
			}
		}
	}
	
	// do the actual edge split
	UE::Geometry::EMeshResult SplitResult = MyBase::SplitEdge(EdgeAB, SplitInfo, SplitParameterT);

	if (SplitResult != EMeshResult::Ok)
	{
		return SplitResult;
	}
	
	// update the number of edge crossings for the edge configuration resulting from the split
	// original edge[a,b] is now [a,f] new edges are [f,b], [f,c] and [f,d]
	// Note: this must be done before updating the roundabout information
	NormalCoordinates.NormalCoord[SplitInfo.OriginalEdge] = UpdatedNormCoord;
	NormalCoordinates.NormalCoord.InsertAt(NewEdgeNormCoords[0], SplitInfo.NewEdges[0]);
	NormalCoordinates.NormalCoord.InsertAt(NewEdgeNormCoords[1], SplitInfo.NewEdges[1]);
	if (!bIsBoundary)
	{
		NormalCoordinates.NormalCoord.InsertAt(NewEdgeNormCoords[2], SplitInfo.NewEdges[2]);
	}

	// fix the roundabout orders
	if (!bIsBoundary) // for 4 triangles. 
	{ 
		
		const FIndex3i OriginalT0RO   = Permute(IndexOfe, NormalCoordinates.RoundaboutOrder[TID0]);  // {ROa2b, ROb2c, ROc2a}
		const FIndex3i OriginalT1RO   = Permute(IndexOf1e, NormalCoordinates.RoundaboutOrder[TID1]); // {ROb2a, ROa2d, ROd2b}
		const FIndex3i OriginalT1VIDs = Permute(IndexOf1e, GetTriangle(TID1)); // {b, a, d} 

		// initialize with the correct RO information for the outer diamond c2a, a2d, b2c, d2b
		const int32 Rc2a = OriginalT0RO[2];
		const int32 Ra2d = OriginalT1RO[1];
		const int32 Rd2b = OriginalT1RO[2];
		const int32 Rb2c = OriginalT0RO[1];
		const int32 Ra2b = OriginalT0RO[0];

		// T0 and T1 after split {a,f, c} and {f, a, d}.. although permuted to the original order.
		FIndex3i UpdatedRO[2] = { FIndex3i(-1, -1, Rc2a), 
								  FIndex3i(-1, Ra2d, -1) };
		// tris {f,b, c}  and {f, d, b}
		FIndex3i NewTrisRO[2] = { FIndex3i(-1, Rb2c, -1),  
								  FIndex3i(-1, Rd2b, -1) };
		

		// correct the roundabout order consistent with the new point f being just inside the original T0

		// directed edge a2f RO:  UpdatedRO[0][0]
		{
			const int32 A = OriginalT0VIDs[0];
			if (IsVertexPoint(IntrinsicVertexPositions[A]))
			{
				UpdatedRO[0][0] = Ra2b;
			}
		}
		// directed edge b2f RO: NewTriRO[1][2] 
		{
			const int32 B = OriginalT0VIDs[1];
			if (IsVertexPoint(IntrinsicVertexPositions[B]))
			{
				const FIndex3i NewTri0Edges = GetTriEdges(SplitInfo.NewTriangles[0]);
				const int32 Kronecker_bc = (int32)Kroneckers[1];
				const int32 Ecf_b = NormalCoordinates.NumCornerEmanatingRefEdges(NewTri0Edges, 1); // num surface edges coming from b and crossing edgeCF.
				NewTrisRO[1][2] = (Rb2c + Ecf_b + Kronecker_bc)  % NormalCoordinates.RefVertDegree[B];
			}
		}
		// directed edges c2f RO: NewTriRO[0][2]
		{
			const int32 C = OriginalT0VIDs[2];
			if (IsVertexPoint(IntrinsicVertexPositions[C]))
			{
				const FIndex3i T0UpdatedEdges = Permute(IndexOfe, GetTriEdges(TID0));
				const int32 Kronecker_ca  = (int32)Kroneckers[2];
				const int32 Eaf_c = NormalCoordinates.NumCornerEmanatingRefEdges(T0UpdatedEdges, 2); // num surface edges coming from c and crossing edgeAF.
				const int32 Rc2f = (Rc2a + Eaf_c + Kronecker_ca) % NormalCoordinates.RefVertDegree[C];
				NewTrisRO[0][2] = Rc2f;
			}
		}
		// directed edge d2f RO: UpdatedRO[1][2]
		{
			const int32 D = OriginalT1VIDs[2];
			if (IsVertexPoint(IntrinsicVertexPositions[D]))
			{
				const FIndex3i NewTri1Edges = GetTriEdges(SplitInfo.NewTriangles[1]);
				const int32 Kronecker_bd = (int32)Kroneckers[3];
				const int32 Ebf_d = NormalCoordinates.NumCornerEmanatingRefEdges(NewTri1Edges, 1); // num surface edges coming from d and crossing edgeBF
				const int32 Rd2f  = (Rd2b + Ebf_d + Kronecker_bd) % NormalCoordinates.RefVertDegree[D];
			
				UpdatedRO[1][2] = Rd2f;
			}
		}
		NormalCoordinates.RoundaboutOrder[TID0] = Permute( (3-IndexOfe)%3,  UpdatedRO[0]);  
		NormalCoordinates.RoundaboutOrder[TID1] = Permute( (3-IndexOf1e)%3, UpdatedRO[1]); 
		NormalCoordinates.RoundaboutOrder.InsertAt(NewTrisRO[0], SplitInfo.NewTriangles[0]);
		NormalCoordinates.RoundaboutOrder.InsertAt(NewTrisRO[1], SplitInfo.NewTriangles[1]);
	}
	else // only 2 triangles.
	{
		const FIndex3i OriginalT0RO = Permute(IndexOfe, NormalCoordinates.RoundaboutOrder[TID0]);
		// initialize with the correct RO information for the outer diamond c2a, a2d, b2c, d2b
		
		const int32 Ra2b = OriginalT0RO[0];
		const int32 Rb2c = OriginalT0RO[1];
		const int32 Rc2a = OriginalT0RO[2];
		

		FIndex3i UpdatedRO = FIndex3i(-1, -1, Rc2a);  // {ROa2f, ROf2c, ROc2a}
		FIndex3i NewTriRO  = FIndex3i(-1, Rb2c, -1);  // {Rof2b, ROb2c, ROc2f} tris {f,b, c}
		
		// directed edge a2f RO:  UpdatedRO[0][0]
		{
			const int32 A = OriginalT0VIDs[0];
			if (IsVertexPoint(IntrinsicVertexPositions[A]))
			{
				UpdatedRO[0] = Ra2b;
			}
		}
		// directed edges c2f RO: NewTriRO[2]
		{
			const int32 C = OriginalT0VIDs[2];
			if(IsVertexPoint(IntrinsicVertexPositions[C]))
			{
				const FIndex3i T0UpdatedEdges = Permute(IndexOfe, GetTriEdges(TID0));
				const int32 Kronecker_ca  = (int32)Kroneckers[2];
				const int32 Eaf_c = NormalCoordinates.NumCornerEmanatingRefEdges(T0UpdatedEdges, 2); // num surface edges coming from c and crossing edgeAF.
				const int32 Rc2f  = (Rc2a + Eaf_c + Kronecker_ca) % NormalCoordinates.RefVertDegree[C];
				NewTriRO[2] = Rc2f;
			}
		}

		NormalCoordinates.RoundaboutOrder[TID0] = Permute((3-IndexOfe)%3, UpdatedRO); 
		NormalCoordinates.RoundaboutOrder.InsertAt(NewTriRO, SplitInfo.NewTriangles[0]);
	}

	// update the location of the new vertex, both as surface position and R3.
	{
		// the two surface points on the split edge that bracket the new vertex
		const FSurfacePoint& LeftSP  = BoundingPoints[0].SurfacePoint;
		const FSurfacePoint& RightSP = BoundingPoints[1].SurfacePoint;


		if (bIsOnSurfaceEdge)
		{
			//    the bounding surface points are either surface vert points or surface edge points

			// find the surface edge
			const int32 SurfaceEID = [&]
									{
										int32 SurfaceEID = -1;
										if (IsEdgePoint(LeftSP))
										{
											SurfaceEID = LeftSP.Position.EdgePosition.EdgeID;
										} 
										else if (IsEdgePoint(RightSP))
										{
											SurfaceEID = RightSP.Position.EdgePosition.EdgeID;
										}
										else
										{
											// neither bounding point was an "edge" point, 
											// so they must both be "vertex" points
											checkSlow(IsVertexPoint(LeftSP) && IsVertexPoint(RightSP));
											const int32 SurfaceVIDs[2] = { LeftSP.Position.VertexPosition.VID,  RightSP.Position.VertexPosition.VID};
											SurfaceEID = SurfaceMesh.FindEdge(SurfaceVIDs[0], SurfaceVIDs[1]);
											checkSlow(SurfaceEID != -1);
										}
										return SurfaceEID;
									}();

			// compute alpha location relative to the surface edge.
			const FEdge  SurfaceEdge    = SurfaceMesh.GetEdge(SurfaceEID);
			const FIndex2i SurfaceEdgeV = SurfaceEdge.Vert;
			
			// orient the surface edge in the same direction as the intrinsic edge ( the intrinsic edge was ordered by its T0)
			// note: the intrinsic edge vertices can not be the same because the intrinsic edge is a segment of the surface edge

			const int32 StartV = [&]
								{
									if (IsVertexPoint(LeftSP))
									{
										return (int32)(LeftSP.Position.VertexPosition.VID == SurfaceEdgeV.B);
									}
									else if (IsVertexPoint(RightSP))
									{
										return (int32)(RightSP.Position.VertexPosition.VID != SurfaceEdgeV.B);
									}
									
									checkSlow(IsEdgePoint(RightSP) && IsEdgePoint(LeftSP));
									return (int32)(RightSP.Position.EdgePosition.Alpha > LeftSP.Position.EdgePosition.Alpha); 
								}();

			// surface edge end points, ordered correctly.	
			const FVector3d SurfaceEdgeEndPts[2] = {SurfaceMesh.GetVertex(SurfaceEdgeV[StartV]), SurfaceMesh.GetVertex(SurfaceEdgeV[1-StartV])};

			const double SurfaceEdgeLength  = ( SurfaceEdgeEndPts[1] - SurfaceEdgeEndPts[0] ).Length();
			const double SurfaceDistToSplit = ( AsR3Pos(LeftSP) - SurfaceEdgeEndPts[0] ).Length()  + IntrinsicEdgeSplitDistance;
			const double R3Alpha            = 1. - SurfaceDistToSplit / SurfaceEdgeLength;
			checkSlow(SurfaceDistToSplit <= SurfaceEdgeLength);

		

			// compute the R3 position of the new vertex
			const FVector3d R3Pos = R3Alpha * SurfaceEdgeEndPts[0] + (1. - R3Alpha) * SurfaceEdgeEndPts[1];

			// record the R3 position and the surface position.
			Vertices[SplitInfo.NewVertex] = R3Pos;
			const double SurfaceAlpha     = (StartV == 0) ? R3Alpha : 1. - R3Alpha;
			IntrinsicVertexPositions.InsertAt(FSurfacePoint(SurfaceEID, SurfaceAlpha), SplitInfo.NewVertex);
		
		}
		else
		{
			// compute the r3 positions of the boundary points
			const FVector3d LeftR3  = AsR3Pos(LeftSP);
			const FVector3d RightR3 = AsR3Pos(RightSP);
		
			// compute the local alpha of the new vertex between the boundary points
			const double BoundingPtsSeperation = (LeftR3 - RightR3).Length();
			const double SplitToRightBoundary  = BoundingPoints[1].DistOnEdge - IntrinsicEdgeSplitDistance;
			const double Alpha = SplitToRightBoundary / BoundingPtsSeperation;
			checkSlow(Alpha >= 0. && Alpha <=1.);

	

			// find the surface triangle the new vertex lives on
			const int32 SurfaceTID = [&]
									{
										int32 TID = -1;
										if (IsFacePoint(LeftSP))
										{
											TID = LeftSP.Position.TriPosition.TriID;
										}
										else if (IsFacePoint(RightSP))
										{
											TID = RightSP.Position.TriPosition.TriID;
										}
										if (TID == -1)
										{
											TArray<int32> AdjSurfaceTIDs = GetAdjacentTriangles(LeftSP, SurfaceMesh);
											for (const int32 CandidtateTID : AdjSurfaceTIDs)
											{
												if (IsOnSurfaceTriangle(RightSP, CandidtateTID, SurfaceMesh))
												{
													TID = CandidtateTID;
													break;
												}
											}
										}
										return TID;
									}();

			checkSlow(SurfaceTID != -1);

			// compute the barycentric coordinates of the boundary points relative to the surface triangle
			const FVector3d LeftBC  = AsBarycenteric(LeftSP, SurfaceTID, SurfaceMesh);
			const FVector3d RightBC = AsBarycenteric(RightSP, SurfaceTID, SurfaceMesh);
	
			// compute the R3 and Barycentric location of the new vertex
			const FVector3d R3Pos        = Alpha * LeftR3 + (1.-Alpha) * RightR3;
			const FVector3d SurfaceBCPos = Alpha * LeftBC + (1.-Alpha) * RightBC;

			// record the R3 position and the surface position.
			Vertices[SplitInfo.NewVertex] = R3Pos;
			IntrinsicVertexPositions.InsertAt(FSurfacePoint(SurfaceTID, SurfaceBCPos), SplitInfo.NewVertex);
		
		}
	}

	
	return EMeshResult::Ok;
}

