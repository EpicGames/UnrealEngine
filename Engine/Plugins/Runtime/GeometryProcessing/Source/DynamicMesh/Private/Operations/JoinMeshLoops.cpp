// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/JoinMeshLoops.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/MeshIndexUtil.h"

using namespace UE::Geometry;

FJoinMeshLoops::FJoinMeshLoops(FDynamicMesh3* MeshIn) 
	: Mesh(MeshIn)
{
}

FJoinMeshLoops::FJoinMeshLoops(FDynamicMesh3* MeshIn, const TArray<int32>& LoopAIn, const TArray<int32>& LoopBIn)
	: Mesh(MeshIn), LoopA(LoopAIn), LoopB(LoopBIn)
{

}



static int32 GetFirstEdgeTri(const FDynamicMesh3* Mesh, int32 VertexA, int32 VertexB)
{
	int32 EdgeID = Mesh->FindEdge(VertexA, VertexB);
	if (EdgeID != FDynamicMesh3::InvalidID)
	{
		return Mesh->GetEdgeT(EdgeID).A;
	}
	return FDynamicMesh3::InvalidID;
}


template<typename TriangleIDFuncType>
static double CalculateAverageUVScale(const FDynamicMeshUVOverlay* UVOverlay, int32 NumTriangles, TriangleIDFuncType GetTriangleIDFunc)
{
	if (!UVOverlay)
	{
		return 1.0;
	}

	const FDynamicMesh3* Mesh = UVOverlay->GetParentMesh();

	double AverageScale = 0;
	int32 Count = 0;
	for (int32 k = 0; k < NumTriangles; ++k)
	{
		int32 TriIndex = GetTriangleIDFunc(k);
		if (Mesh->IsTriangle(TriIndex))
		{
			FVector3d Pos[3];
			Mesh->GetTriVertices(TriIndex, Pos[0], Pos[1], Pos[2]);
			FVector2f UV[3];
			UVOverlay->GetTriElements(TriIndex, UV[0], UV[1], UV[2]);

			for (int32 j = 0; j < 3; ++j)
			{
				int32 jn = (j + 1) % 3;
				double Len3D = Distance(Pos[j], Pos[jn]);
				double LenUV = Distance(UV[j], UV[jn]);
				double Scale = LenUV / FMathd::Max(FMathf::ZeroTolerance, Len3D);
				AverageScale += Scale;
				Count++;
			}
		}
	}

	if (AverageScale < FMathf::ZeroTolerance || Count == 0)
	{
		return 1.0;
	}
	return AverageScale / (double)Count;
}


bool FJoinMeshLoops::Apply()
{
	FDynamicMeshEditor Editor(Mesh);

	int32 NV = LoopA.Num();
	FDynamicMeshUVOverlay* UVOverlay = Mesh->Attributes()->PrimaryUV();
	double BorderUVScaleA = CalculateAverageUVScale(UVOverlay, NV, [&](int32 i) { return GetFirstEdgeTri(Mesh, LoopA[i], LoopA[(i+1)% NV]); });
	double BorderUVScaleB = CalculateAverageUVScale(UVOverlay, NV, [&](int32 i) { return GetFirstEdgeTri(Mesh, LoopB[i], LoopB[(i+1)% NV]); });
	double UseUVScale = (BorderUVScaleA + BorderUVScaleB) / 2.0;

	FDynamicMeshEditResult StitchResult;
	bool bStitchSuccess = Editor.StitchVertexLoopsMinimal(LoopA, LoopB, StitchResult);

	JoinQuads = MoveTemp(StitchResult.NewQuads);
	int32 NumQuads = JoinQuads.Num();
	QuadGroups.SetNum(NumQuads);

	int32 NewGroupID = Mesh->AllocateTriangleGroup();
	NewGroups.Add(NewGroupID);

	for ( int32 qi = 0; qi < NumQuads; ++qi)
	{
		FIndex2i& Quad = JoinQuads[qi];
		QuadGroups[qi] = NewGroupID;
		if (Mesh->IsTriangle(Quad.A))
		{
			JoinTriangles.Add(Quad.A);
			Mesh->SetTriangleGroup(Quad.A, NewGroupID);
		}
		if (Mesh->IsTriangle(Quad.B))
		{
			JoinTriangles.Add(Quad.B);
			Mesh->SetTriangleGroup(Quad.B, NewGroupID);
		}
	}


	if (Mesh->Attributes() && Mesh->Attributes()->PrimaryNormals() != nullptr)
	{
		FMeshNormals::InitializeOverlayRegionToPerVertexNormals(
			Mesh->Attributes()->PrimaryNormals(), JoinTriangles);
	}


	if (Mesh->Attributes() && UVOverlay != nullptr)
	{
		TArray<int32> UVLoopA, UVLoopB;
		UVLoopA.SetNum(NV + 1);
		UVLoopB.SetNum(NV + 1);

		// calculate arc length of join strip
		double TotalArcLen = 0;
		double AverageWidth = 0;
		TArray<double> AccumArcLen;
		AccumArcLen.SetNum(NV+1);
		TArray<double> RowWidth;
		RowWidth.SetNum(NV + 1);
		for (int32 i = 0; i < NV; ++i)
		{
			int32 j = (i + 1) % NV;
			AccumArcLen[i] = TotalArcLen;
			RowWidth[i] = Distance(Mesh->GetVertex(LoopA[i]), Mesh->GetVertex(LoopB[i]));
			AverageWidth += RowWidth[i];
			FVector3d MidCur = (Mesh->GetVertex(LoopA[i]) + Mesh->GetVertex(LoopB[i])) * 0.5;
			FVector3d MidNext = (Mesh->GetVertex(LoopA[j]) + Mesh->GetVertex(LoopB[j])) * 0.5;
			TotalArcLen += Distance(MidCur, MidNext);
		}
		AccumArcLen[NV] = TotalArcLen;
		RowWidth[NV] = RowWidth[0];
		AverageWidth /= (double)NV;

		// create UV elements for strip
		for (int32 ii = 0; ii <= NV; ++ii)
		{
			double ScaledArcLen = AccumArcLen[ii] * UseUVScale;
			// less area distortion but looks kinda weird?
			//double ScaledRowWidth = RowWidth[ii] * UseUVScale;
			double ScaledRowWidth = AverageWidth * UseUVScale;
			FVector2d UVA(ScaledArcLen, 0.0);
			FVector2d UVB(ScaledArcLen, ScaledRowWidth);
			UVLoopA[ii] = UVOverlay->AppendElement((FVector2f)UVA);
			UVLoopB[ii] = UVOverlay->AppendElement((FVector2f)UVB);
		}

		for (int32 qi = 0; qi < NumQuads; ++qi)
		{
			// from StitchVertexLoopsMinimal comment:
			// 	  If loop edges are [a,b] and [c,d], then tris added are [b,a,d] and [a,c,d]
			FIndex2i& Quad = JoinQuads[qi];
			int32 uva = UVLoopA[qi];
			int32 uvb = UVLoopA[qi+1];
			int32 uvc = UVLoopB[qi];
			int32 uvd = UVLoopB[qi + 1];
			if (Mesh->IsTriangle(Quad.A))
			{
				UVOverlay->SetTriangle(Quad.A, FIndex3i(uvb, uva, uvd));
			}
			if (Mesh->IsTriangle(Quad.B))
			{
				UVOverlay->SetTriangle(Quad.B, FIndex3i(uva, uvc, uvd));
			}
		}

	}


	return bStitchSuccess;
}
