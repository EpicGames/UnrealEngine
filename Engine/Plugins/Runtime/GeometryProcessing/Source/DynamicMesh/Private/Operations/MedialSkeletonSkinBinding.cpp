// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MedialSkeletonSkinBinding.h"

#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"

namespace MedialSkeletonSkinBindingLocals
{

	static TArray<TArray<int32, TInlineAllocator<16>>> BuildMedialSkeletonBoneGroups(
		const UE::Geometry::MedialAxis::FMedialSkeleton& InMedialSkeleton,
		TConstArrayView<int32> MedialIndexToBoneIndex,
		int32 NumBones,
		int32 ClusterNeighborSearchRange)
	{
		using namespace UE::Geometry;

		const int32 NumClusters = InMedialSkeleton.ClusterNeighbors.Num();
		TArray<TArray<int32, TInlineAllocator<16>>> BoneGroups;
		BoneGroups.SetNum(NumClusters);

		// For each cluster, BFS-expand by ClusterNeighborSearchRange steps to get the cluster group
		for (int32 ClusterIdx = 0; ClusterIdx < NumClusters; ++ClusterIdx)
		{
			TArray<int32> CurrentFrontier;
			TArray<int32> NextFrontier;
			TSet<int32> Visited;

			CurrentFrontier.Add(ClusterIdx);
			Visited.Add(ClusterIdx);

			for (int32 Step = 0; Step < ClusterNeighborSearchRange; ++Step)
			{
				NextFrontier.Reset();
				for (int32 Src : CurrentFrontier)
				{
					for (int32 Nbr : InMedialSkeleton.ClusterNeighbors[Src])
					{
						bool bFound;
						Visited.FindOrAdd(Nbr, &bFound);
						if (!bFound)
						{
							NextFrontier.Add(Nbr);
						}
					}
				}
				Swap(CurrentFrontier, NextFrontier);
			}

			// Remap visited cluster indices to bone indices
			for (int32 VisitedCluster : Visited)
			{
				check(MedialIndexToBoneIndex.IsValidIndex(VisitedCluster));
				int32 BoneIdx = MedialIndexToBoneIndex[VisitedCluster];
				if (BoneIdx >= 0 && BoneIdx < NumBones)
				{
					BoneGroups[ClusterIdx].Add(BoneIdx);
				}
			}
		}

		return BoneGroups;
	}
}

namespace UE::Geometry::SkinBinding
{


	
bool CreateSkinWeightsFromMedialSkeleton(
	const MedialAxis::FMedialSkeleton& InMedialSkeleton,
	FDynamicMesh3& TargetMesh,
	bool& bOutMeshWasCompatible,
	FName WeightProfileName,
	const FBindMedialSkeletonSettings& Settings,
	int32 RootIndex)
{
	using namespace MedialSkeletonSkinBindingLocals;

	bOutMeshWasCompatible = InMedialSkeleton.IsCompatibleWithMesh(TargetMesh);

	// Step 1: Convert medial skeleton to hierarchical skeleton
	TArray<int32> Parents;
	TArray<FTransform> Poses;
	TArray<int32> BoneIndexToMedialIndex;
	TArray<int32> MedialIndexToBoneIndex;

	MedialAxis::FMedialSkeletonToTreeSkeletonOptions::ToHierarchy(
		InMedialSkeleton, Parents, Poses, Settings.AnimationSkeletonOptions, RootIndex,
		&BoneIndexToMedialIndex, &MedialIndexToBoneIndex);

	const int32 NumBones = Parents.Num();
	if (NumBones == 0)
	{
		return false;
	}

	// Step 2: Prepare mesh attributes — clear any (likely-incompatible) existing skin weights
	TargetMesh.EnableAttributes();
	TargetMesh.Attributes()->RemoveAllSkinWeightsAttributes();

	// Step 3: Build FBonePoseInfo array from hierarchy
	TArray<FBonePoseInfo> Bones;
	Bones.SetNum(Parents.Num());
	for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
	{
		Bones[BoneIdx].Name = FName("Bone", BoneIdx);
		Bones[BoneIdx].ParentIndex = Parents[BoneIdx];
		Bones[BoneIdx].LocalTransform = Poses[BoneIdx];
	}

	if (bOutMeshWasCompatible)
	{
		// Step 4: Build bone groups per-cluster from medial skeleton cluster graph
		TArray<TArray<int32, TInlineAllocator<16>>> BoneGroups = BuildMedialSkeletonBoneGroups(
			InMedialSkeleton, MedialIndexToBoneIndex, NumBones,
			Settings.ClusterNeighborSearchRange);

		// Step 5: Apply constrained skin binding
		CreateConstrainedSkinWeights(
			TargetMesh, Bones, WeightProfileName, Settings.BindSettings,
			BoneGroups, InMedialSkeleton.VIDtoClusterIndex);
	}
	else
	{
		CreateSkinWeights(TargetMesh, Bones, WeightProfileName, Settings.BindSettings);
	}

	return true;
}

} // namespace UE::Geometry::SkinBinding
