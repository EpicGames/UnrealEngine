// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MeshAttributeTransfer.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "TriangleTypes.h"
#include "Util/IndexUtil.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;


FMeshAttributeTransfer::FMeshAttributeTransfer(const FDynamicMesh3* SourceMeshIn, FDynamicMesh3* TargetMeshIn)
	: SourceMesh(SourceMeshIn), TargetMesh(TargetMeshIn)
{
	ensure(SourceMeshIn);
	ensure(TargetMeshIn);
}


bool FMeshAttributeTransfer::Apply()
{
	if (SourceMesh == nullptr)
	{
		Errors.Add(EMeshAttributeTransferError::InvalidSource);
		return false;
	}
	if (TargetMesh == nullptr)
	{
		Errors.Add(EMeshAttributeTransferError::InvalidTarget);
		return false;
	}

	if ( TransferType == EMeshAttributeTransferType::MaterialID)
	{
		return Apply_MaterialID();
	}
	else
	{
		Errors.Add(EMeshAttributeTransferError::InvalidOperationForInputs);
		return false;
	}

}



template<typename AllocType>
int32 SelectMostFrequentElement(TArray<int32, AllocType>& Samples)
{
	Samples.Sort();
	int32 N = Samples.Num();
	int32 MaxVotes = 0;
	int32 MaxVoteValue = Samples[0];
	int32 CurValue = Samples[0];
	int32 CurCount = 1;
	for (int32 k = 1; k < N; ++k)
	{
		if (Samples[k] == CurValue)
		{
			CurCount++;
		}
		else
		{
			if (CurCount > MaxVotes)
			{
				MaxVotes = CurCount;
				MaxVoteValue = CurValue;
			}
			CurCount = 1;
			CurValue = Samples[k];
		}
	}
	return MaxVoteValue;
}



bool FMeshAttributeTransfer::Apply_MaterialID()
{
	const FDynamicMeshMaterialAttribute* SourceMaterialID = SourceMesh->HasAttributes() ? SourceMesh->Attributes()->GetMaterialID() : nullptr;
	if (SourceMaterialID == nullptr)
	{
		Errors.Add(EMeshAttributeTransferError::SourceMissingAttribute);
		return false;
	}
	FDynamicMeshMaterialAttribute* TargetMaterialID = TargetMesh->HasAttributes() ? TargetMesh->Attributes()->GetMaterialID() : nullptr;
	if (TargetMaterialID == nullptr)
	{
		Errors.Add(EMeshAttributeTransferError::TargetMissingAttribute);
		return false;
	}
	
	SourceSpatial = MakeUnique<FDynamicMeshAABBTree3>(SourceMesh, true);

	static const FVector3d BarySamplePoints[7] = {
		{1, 0, 0}, { 0,1,0 }, { 0,0,1 },
		{ 0.5,0.5,0 }, {0.5,0,0.5}, {0,0.5,0.5},
		{1.0/3.0, 1.0/3.0, 1.0/3.0}
	};
	constexpr int32 NumSamplePoints = 7;		// must change Samples array below too!

	// generate initial assignments
	ParallelFor(TargetMesh->MaxTriangleID(), [&](int32 tid)
	{
		FTriangle3d TargetTriangle;
		TargetMesh->GetTriVertices(tid, TargetTriangle.V[0], TargetTriangle.V[1], TargetTriangle.V[2]);

		TArray<int32, TFixedAllocator<7>> Samples;
		for (int32 k = 0; k < NumSamplePoints; ++k)
		{
			FVector3d SamplePoint = TargetTriangle.BarycentricPoint(BarySamplePoints[k]);
			double DistSqr;
			int32 SourceTriID = SourceSpatial->FindNearestTriangle(SamplePoint, DistSqr);
			Samples.Add(SourceMaterialID->GetValue(SourceTriID));
		}

		int32 BestValue = SelectMostFrequentElement(Samples);
		TargetMaterialID->SetValue(tid, BestValue);
	});

	return true;
}