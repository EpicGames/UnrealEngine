// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MeshBevel.h"

#include "GroupTopology.h"
#include "EdgeLoop.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "MeshWeights.h"
#include "CompGeom/PolygonTriangulation.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "MeshRegionBoundaryLoops.h"
#include "MeshBoundaryLoops.h"
#include "Operations/PolyEditingEdgeUtil.h"
#include "Operations/PolyEditingUVUtil.h"
#include "Algo/Count.h"
#include "Algo/RemoveIf.h"
#include "Distance/DistLine3Line3.h"
#include "Operations/UniformTessellate.h"
#include "DynamicSubmesh3.h"
#include "Solvers/ConstrainedMeshSmoother.h"
#include "Solvers/ConstrainedMeshDeformer.h"
#include "Selections/MeshFaceSelection.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Polygon2.h"
#include "VectorUtil.h"
#include "Spatial/DenseGrid2.h"


using namespace UE::Geometry;

// Uncomment to enable various checks and ensures in the Bevel code.
// We cannot default-enable these checks even in Debug builds, because 
// Bevel still frequently hits some of these cases, and those checks/ensures 
// make it very difficult to debug a game using Bevel in Geometry Script (eg Lyra sample)
//#define ENABLE_MESH_BEVEL_DEBUG

#ifdef ENABLE_MESH_BEVEL_DEBUG
#define MESH_BEVEL_DEBUG_CHECK(Expr) checkSlow(Expr)
#define MESH_BEVEL_DEBUG_ENSURE(Expr) ensure(Expr)
#else
#define MESH_BEVEL_DEBUG_CHECK(Expr) 
#define MESH_BEVEL_DEBUG_ENSURE(Expr) !!(Expr)
#endif


namespace UELocal
{
	static void QuadsToTris(const FDynamicMesh3& Mesh, const TArray<FIndex2i>& Quads, TArray<int32>& TrisOut, bool bReset)
	{
		int32 N = Quads.Num();
		if (bReset)
		{
			TrisOut.Reset();
			TrisOut.Reserve(2 * N);
		}
		for (const FIndex2i& Quad : Quads)
		{
			if (Mesh.IsTriangle(Quad.A))
			{
				TrisOut.Add(Quad.A);
			}
			if (Mesh.IsTriangle(Quad.B))
			{
				TrisOut.Add(Quad.B);
			}
		}
	};

}




void FMeshBevel::InitializeFromGroupTopology(const FDynamicMesh3& Mesh, const FGroupTopology& Topology)
{
	ResultInfo = FGeometryResult(EGeometryResultType::InProgress);

	// set up initial problem inputs
	for (int32 TopoEdgeID = 0; TopoEdgeID < Topology.Edges.Num(); ++TopoEdgeID)
	{
		if (Topology.IsIsolatedLoop(TopoEdgeID))
		{
			FEdgeLoop NewLoop;
			NewLoop.InitializeFromEdges(&Mesh, Topology.Edges[TopoEdgeID].Span.Edges);
			AddBevelEdgeLoop(Mesh, NewLoop);
		}
		else
		{
			AddBevelGroupEdge(Mesh, Topology, TopoEdgeID);
		}

		if (ResultInfo.CheckAndSetCancelled(Progress))
		{
			return;
		}
	}

	// precompute topological information necessary to apply bevel to vertices/edges/loops
	BuildVertexSets(Mesh);
}


void FMeshBevel::InitializeFromGroupTopologyEdges(const FDynamicMesh3& Mesh, const FGroupTopology& Topology, const TArray<int32>& GroupEdges)
{
	ResultInfo = FGeometryResult(EGeometryResultType::InProgress);

	// set up initial problem inputs
	for (int32 TopoEdgeID : GroupEdges)
	{
		if (Topology.IsIsolatedLoop(TopoEdgeID))
		{
			FEdgeLoop NewLoop;
			NewLoop.InitializeFromEdges(&Mesh, Topology.Edges[TopoEdgeID].Span.Edges);
			AddBevelEdgeLoop(Mesh, NewLoop);
		}
		else
		{
			AddBevelGroupEdge(Mesh, Topology, TopoEdgeID);
		}

		if (ResultInfo.CheckAndSetCancelled(Progress))
		{
			return;
		}
	}

	// precompute topological information necessary to apply bevel to vertices/edges/loops
	BuildVertexSets(Mesh);
}

void FMeshBevel::InitializeFromTriangleEdges(const FDynamicMesh3& Mesh, TConstArrayView<int32> TriangleEdges, TFunctionRef<bool(int32)> IsCornerVertex)
{
	ResultInfo = FGeometryResult(EGeometryResultType::InProgress);

	// VID to TriangleEdges Index map, used for walking chains/loops
	TMultiMap<int32, int32> V2EIdx;
	TArray<uint8> EdgeUsed;
	EdgeUsed.SetNumZeroed(TriangleEdges.Num());
	TSet<int32> AllEdgeVIDs;

	for (int32 Idx = 0; Idx < TriangleEdges.Num(); ++Idx)
	{
		int32 EID = TriangleEdges[Idx];
		if (Mesh.IsBoundaryEdge(EID))
		{
			// bevel doesn't support boundary edges; mark them as processed without adding them to the bevel data
			EdgeUsed[Idx] = 1;
			continue;
		}
		FIndex2i VIDs = Mesh.GetEdgeV(EID);
		V2EIdx.Add(VIDs.A, Idx);
		V2EIdx.Add(VIDs.B, Idx);
		AllEdgeVIDs.Add(VIDs.A);
		AllEdgeVIDs.Add(VIDs.B);
	}

	// edge chains, stored as [VID,EID] pairs and a EndChain or EndLoop as the EID at the end of each continguous chain or loop respectively
	constexpr int32 EndChain = -1, EndLoop = -2;
	TArray<TPair<int32,int32>> Chains;

	auto ProcessEdgeIdx = [&EdgeUsed, &TriangleEdges, &Mesh, &IsCornerVertex, &V2EIdx, &Chains, EndChain, EndLoop](int32 VID, int32 EdgeIdx)
	{
		EdgeUsed[EdgeIdx] = 1;
		int32 WalkEID = TriangleEdges[EdgeIdx];
		Chains.Emplace(VID, WalkEID);
		int32 StartVID = VID;
		int32 WalkVID = VID;
		int32 MaxIters = TriangleEdges.Num() + 2; // safety to prevent infinite loop
		while (--MaxIters > 0)
		{
			FIndex2i EdgeV = Mesh.GetEdgeV(WalkEID);
			// step WalkVID to next vertex
			WalkVID = EdgeV[1 - EdgeV.IndexOf(WalkVID)];
			// detect loops
			if (WalkVID == StartVID)
			{
				Chains.Emplace(WalkVID, EndLoop);
				break;
			}
			// stop chains at specified corner vertices
			if (IsCornerVertex(WalkVID))
			{
				Chains.Emplace(WalkVID, EndChain);
				break;
			}
			// non-loop case, look for a next edge
			int32 WalkEIdx = -1;
			int32 ECount = 0;
			for (TMultiMap<int32, int32>::TConstKeyIterator It = V2EIdx.CreateConstKeyIterator(WalkVID); It; ++It)
			{
				ECount++;
				if (EdgeUsed[It.Value()] != 0)
				{
					continue;
				}
				WalkEIdx = It.Value();
			}
			// if next vertex had valence != 2, it's the end of this chain
			if (ECount != 2)
			{
				Chains.Emplace(WalkVID, EndChain);
				break;
			}
			// walk to the next edge and mark it as processed
			EdgeUsed[WalkEIdx] = 1;
			WalkEID = TriangleEdges[WalkEIdx];
			Chains.Emplace(WalkVID, WalkEID);
		}
		check(MaxIters > 0);
	};

	// Process all loops and chains that have at least one vertex on multiple selected edges
	for (int32 VID : AllEdgeVIDs)
	{
		int32 NumEdges = V2EIdx.Num(VID);
		if (IsCornerVertex(VID) || NumEdges != 2)
		{
			for (TMultiMap<int32, int32>::TConstKeyIterator It = V2EIdx.CreateConstKeyIterator(VID); It; ++It)
			{
				if (!EdgeUsed[It.Value()])
				{
					ProcessEdgeIdx(VID, It.Value());
				}
			}
		}
	}

	// Process remaining (isolated) loops where all vertices had only two adjacent selected edges
	for (int32 Idx = 0; Idx < EdgeUsed.Num(); ++Idx)
	{
		if (!EdgeUsed[Idx])
		{
			ProcessEdgeIdx(Mesh.GetEdgeV(TriangleEdges[Idx]).A, Idx);
		}
	}

	// Add the edge chains and loops to the bevel data structures
	TArray<int32> EdgeLoop, VertexLoop;
	for (int32 ChainIdx = 0, ChainLen = 0; ChainIdx < Chains.Num(); ChainIdx += ChainLen + 1)
	{
		for (ChainLen = 0; ChainLen + ChainIdx < Chains.Num() && Chains[ChainIdx + ChainLen].Value >= 0; ++ChainLen)
		{}
		check(ChainLen > 0 && ChainIdx + ChainLen < Chains.Num());
		
		bool bIsLoop = Chains[ChainIdx + ChainLen].Value == EndLoop;
		if (bIsLoop)
		{
			EdgeLoop.Reset(ChainLen);
			VertexLoop.Reset(ChainLen);
			for (int32 Idx = ChainIdx; Idx < ChainIdx + ChainLen; ++Idx)
			{
				EdgeLoop.Add(Chains[Idx].Value);
				VertexLoop.Add(Chains[Idx].Key);
			}
			// Note: Could initialize the bevel edge loop data directly rather than going via FEdgeLoop and avoid some copying
			FEdgeLoop NewLoop;
			NewLoop.Initialize(&Mesh, VertexLoop, EdgeLoop);
			AddBevelEdgeLoop(Mesh, NewLoop);
		}
		else
		{
			FIndex2i CornerVIDs(Chains[ChainIdx].Key, Chains[ChainIdx + ChainLen].Key);

			FBevelEdge Edge;
			int32 NewBevelEdgeIndex = Edges.Num();

			for (int32 ci = 0; ci < 2; ++ci)
			{
				int32 VertexID = CornerVIDs[ci];
				Edge.bEndpointBoundaryFlag[ci] = Mesh.IsBoundaryVertex(VertexID);
				int32 IncomingEdgeID = (ci == 0) ? Chains[ChainIdx].Value : Chains[ChainIdx + ChainLen - 1].Value;

				int32 BevelVertexIndex = -1;
				FBevelVertex* VertInfo = GetBevelVertexFromVertexID(VertexID, &BevelVertexIndex);
				if (VertInfo == nullptr)
				{
					FBevelVertex NewVertex;
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					NewVertex.CornerID = INDEX_NONE;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					NewVertex.VertexID = VertexID;
					BevelVertexIndex = Vertices.Num();
					Vertices.Add(NewVertex);
					VertexIDToIndexMap.Add(VertexID, BevelVertexIndex);
					VertInfo = &Vertices[BevelVertexIndex];
				}
				VertInfo->IncomingBevelMeshEdges.Add(IncomingEdgeID);
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				VertInfo->IncomingBevelTopoEdges.Add(INDEX_NONE);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				VertInfo->IncomingBevelEdgeIndices.Add(NewBevelEdgeIndex);
				Edge.BevelVertices[ci] = BevelVertexIndex;
			}

			Edge.MeshEdges.Reset(ChainLen);
			for (int32 Idx = ChainIdx; Idx < ChainIdx + ChainLen; ++Idx)
			{
				Edge.MeshEdges.Add(Chains[Idx].Value);
			}
			Edge.MeshVertices.Reset(ChainLen + 1);
			for (int32 Idx = ChainIdx; Idx < ChainIdx + ChainLen + 1; ++Idx)
			{
				Edge.MeshVertices.Add(Chains[Idx].Key);
			}
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Edge.GroupEdgeID = INDEX_NONE;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			Edge.MeshEdgeTris.Reserve(Edge.MeshEdges.Num());
			for (int32 eid : Edge.MeshEdges)
			{
				Edge.MeshEdgeTris.Add(Mesh.GetEdgeT(eid));
			}

			Edge.InitialPositions.Reserve(Edge.MeshVertices.Num());
			for (int32 vid : Edge.MeshVertices)
			{
				Edge.InitialPositions.Add(Mesh.GetVertex(vid));
			}

			Edge.EdgeIndex = NewBevelEdgeIndex;
			Edges.Add(MoveTemp(Edge));
		}

		if (ResultInfo.CheckAndSetCancelled(Progress))
		{
			return;
		}
	}

	// precompute topological information necessary to apply bevel to vertices/edges/loops
	BuildVertexSets(Mesh);
}


bool FMeshBevel::InitializeFromGroupTopologyFaces(const FDynamicMesh3& Mesh, const FGroupTopology& Topology, const TArray<int32>& GroupFaces)
{
	TSet<int32> GroupSelection;
	GroupSelection.Append(GroupFaces);
	bool bFoundAnythingToBevel = false;
	for (int32 GroupID : GroupFaces)
	{
		const FGroupTopology::FGroup* Group = Topology.FindGroupByID(GroupID);
		for (const FGroupTopology::FGroupBoundary& Boundary : Group->Boundaries)
		{
			for (int32 GroupEdgeID : Boundary.GroupEdges)
			{
				const FGroupTopology::FGroupEdge& GroupEdge = Topology.Edges[GroupEdgeID];
				int32 OtherGroupID = GroupEdge.OtherGroupID(GroupID);
				// Do not bevel edges where both sides are selected, or boundary edges
				if (OtherGroupID == IndexConstants::InvalidID || GroupSelection.Contains(OtherGroupID))
				{
					continue;
				}
				if (Topology.IsIsolatedLoop(GroupEdgeID))
				{
					FEdgeLoop NewLoop;
					NewLoop.InitializeFromEdges(&Mesh, Topology.Edges[GroupEdgeID].Span.Edges);
					AddBevelEdgeLoop(Mesh, NewLoop);
					bFoundAnythingToBevel = true;
				}
				else
				{
					AddBevelGroupEdge(Mesh, Topology, GroupEdgeID);
					bFoundAnythingToBevel = true;
				}
			}
		}
	}
	if (bFoundAnythingToBevel)
	{
		// precompute topological information necessary to apply bevel to vertices/edges/loops
		BuildVertexSets(Mesh);
	}
	return bFoundAnythingToBevel;
}

bool FMeshBevel::InitializeFromTriangleSet(const FDynamicMesh3& Mesh, const TArray<int32>& Triangles)
{
	ResultInfo = FGeometryResult(EGeometryResultType::InProgress);

	FMeshRegionBoundaryLoops RegionLoops(&Mesh, Triangles, true);
	if (RegionLoops.bFailed)
	{
		ResultInfo.SetFailed();
		return false;
	}

	// cannot bevel a selection-bowtie vertex, so we have to check for those and fail here
	TSet<int32> AllVertices;
	for (const FEdgeLoop& Loop : RegionLoops.Loops)
	{
		for (int32 vid : Loop.Vertices)
		{
			if (AllVertices.Contains(vid))
			{
				return false;
			}
			AllVertices.Add(vid);
		}
	}

	for (const FEdgeLoop& Loop : RegionLoops.Loops)
	{
		AddBevelEdgeLoop(Mesh, Loop);
	}

	// precompute topological information necessary to apply bevel to vertices/edges/loops
	BuildVertexSets(Mesh);

	return true;
}

void FMeshBevel::FixBowties(FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker)
{
	TMultiMap<int32, int32> SplitMeshVIDs; // map from original VID to new VIDs for any bowtie vertices that were split
	auto SplitBowtie = [this, &Mesh, &SplitMeshVIDs, &ChangeTracker](int32 VID, int32 EID) -> int32
	{
		FIndex2i EdgeV = Mesh.GetEdgeV(EID);
		int32 SubIdx = EdgeV.IndexOf(VID);
		check(SubIdx != INDEX_NONE);
		int32 OtherVID = EdgeV[1 - SubIdx];

		if (ChangeTracker)
		{
			ChangeTracker->SaveVertexOneRingTriangles(VID, true);
		}
		FDynamicMeshEditor Edit(&Mesh);
		FDynamicMeshEditResult EditResult;
		Edit.SplitBowties(VID, EditResult);
		FIndex2i UpdatedEdgeV = Mesh.GetEdgeV(EID);
		int32 OtherVIDSubIdx = UpdatedEdgeV.IndexOf(OtherVID);
		check(OtherVIDSubIdx != INDEX_NONE);
		int32 NewVID = UpdatedEdgeV[1 - OtherVIDSubIdx];
		
		for (int32 AddedVID : EditResult.NewVertices)
		{
			SplitMeshVIDs.Add(VID, AddedVID);
		}
		return NewVID;
	};

	auto RemapVertexID = [this, &Mesh, &SplitMeshVIDs, &SplitBowtie](int32 OrigVID, int32 EID, int32& RemapVID) -> bool
	{
		bool bWasBowtie = SplitMeshVIDs.Contains(OrigVID);
		RemapVID = OrigVID;
		if (bWasBowtie)
		{
			FIndex2i EdgeV = Mesh.GetEdgeV(EID);
			if (!EdgeV.Contains(OrigVID))
			{
				for (auto KeyIter = SplitMeshVIDs.CreateConstKeyIterator(OrigVID); KeyIter; ++KeyIter)
				{
					int32 NewVID = KeyIter.Value();
					if (EdgeV.Contains(NewVID))
					{
						RemapVID = NewVID;
						return true;
					}
				}
				// the edge should always include one of the vertices that the bowtie was split in to
				checkSlow(false);
			}
			return true;
		}
		else if (Mesh.IsBowtieVertex(OrigVID))
		{
			 RemapVID = SplitBowtie(OrigVID, EID);
			 return true;
		}
		return false;
	};

	for (FBevelLoop& Loop : Loops)
	{
		checkSlow(Loop.MeshVertices.Num() == Loop.MeshEdges.Num());
		for (int32 VertPathIdx = 0; VertPathIdx < Loop.MeshVertices.Num(); ++VertPathIdx)
		{
			int32 OrigVID = Loop.MeshVertices[VertPathIdx];
			int32 EdgePathIdx = VertPathIdx;
			int32 EID = Loop.MeshEdges[EdgePathIdx];
			int32 RemapVID = OrigVID;
			if (RemapVertexID(OrigVID, EID, RemapVID))
			{
				Loop.MeshVertices[VertPathIdx] = RemapVID;
			}
		}
	}

	int32 OrigNumVertices = Vertices.Num();
	TSet<int32> BevelVerticesToRelink;
	TMap<int32, int32> MeshVIDToBevelVertexIdx;
	for (FBevelEdge& Edge : Edges)
	{
		checkSlow(Edge.MeshVertices.Num() == Edge.MeshEdges.Num() + 1);
		for (int32 VertPathIdx = 0; VertPathIdx < Edge.MeshVertices.Num(); ++VertPathIdx)
		{
			int32 OrigVID = Edge.MeshVertices[VertPathIdx];
			int32 EdgePathIdx = FMath::Min(VertPathIdx, Edge.MeshEdges.Num() - 1);
			int32 EID = Edge.MeshEdges[EdgePathIdx];
			int32 RemapVID = OrigVID;
			if (RemapVertexID(OrigVID, EID, RemapVID))
			{
				Edge.MeshVertices[VertPathIdx] = RemapVID;

				// if we're at an endpoint of the bevel edge, also update the bevel vertex, and make the edge point to the updated bevel vertex
				if (VertPathIdx == 0 || VertPathIdx + 1 == Edge.MeshVertices.Num())
				{
					int32 BevelVertexSubIdx = VertPathIdx == 0 ? 0 : 1;
					int32 BevelVertexIdx = Edge.BevelVertices[BevelVertexSubIdx];
					VertexIDToIndexMap.Remove(OrigVID);
					bool bWasAlreadyRelinked;
					BevelVerticesToRelink.Add(BevelVertexIdx, &bWasAlreadyRelinked);

					// after splitting bowties, recompute whether the split vertex is still a boundary vertex
					Edge.bEndpointBoundaryFlag[BevelVertexSubIdx] = Mesh.IsBoundaryVertex(RemapVID);

					if (!bWasAlreadyRelinked)
					{
						// First time encountering this bevel vertex; just remap it to the split vertex on the current edge
						Vertices[BevelVertexIdx].VertexID = RemapVID;
						MeshVIDToBevelVertexIdx.Add(RemapVID, BevelVertexIdx);
					}
					else
					{
						// We've encountered this bevel vertex before; use a previously-found mapping, or copy the vertex to create a new one
						int32* FoundBevelVert = MeshVIDToBevelVertexIdx.Find(RemapVID);
						int32 NewBevelVertIdx = INDEX_NONE;
						if (FoundBevelVert)
						{
							NewBevelVertIdx = *FoundBevelVert;
						}
						else
						{
							// Add a copy of the existing bevel vertex
							NewBevelVertIdx = Vertices.Add(FBevelVertex(Vertices[BevelVertexIdx]));
							Vertices[NewBevelVertIdx].VertexID = RemapVID;
							BevelVerticesToRelink.Add(NewBevelVertIdx);
							MeshVIDToBevelVertexIdx.Add(RemapVID, NewBevelVertIdx);
						}
						Edge.BevelVertices[BevelVertexSubIdx] = NewBevelVertIdx;
					}
				}
			}
		}
	}

	// Fix up edge and triangle references on any relinked bevel vertices
	for (int32 BevelVertexIdx : BevelVerticesToRelink)
	{
		FBevelVertex& Vertex = Vertices[BevelVertexIdx];
		VertexIDToIndexMap.Add(Vertices[BevelVertexIdx].VertexID, BevelVertexIdx);
		Vertex.VertexType = EBevelVertexType::Unknown; // reset type to unknown; will be set by InitVertexSet below
		Vertex.IncomingBevelEdgeIndices.SetNum(
			Algo::RemoveIf(Vertex.IncomingBevelEdgeIndices, [this, BevelVertexIdx](int32 BevelEdgeIdx)
			{
				return !Edges[BevelEdgeIdx].BevelVertices.Contains(BevelVertexIdx);
			}));
		int32 MeshVertexID = Vertex.VertexID;
		Vertex.IncomingBevelMeshEdges.SetNum(
			Algo::RemoveIf(Vertex.IncomingBevelMeshEdges, [this, MeshVertexID, &Mesh](int32 EdgeID)
			{
				return !Mesh.GetEdgeV(EdgeID).Contains(MeshVertexID);
			}));
		InitVertexSet(Mesh, Vertices[BevelVertexIdx]);
	}

	for (int32 BevelVertexIdx : BevelVerticesToRelink)
	{
		FBevelVertex& Vertex = Vertices[BevelVertexIdx];
		FinalizeTerminatorVertex(Mesh, Vertex);
	}
}

bool FMeshBevel::Apply(FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker)
{
	FixBowties(Mesh, ChangeTracker);

	// disconnect along bevel graph edges/vertices and save necessary info
	UnlinkEdges(Mesh, ChangeTracker);
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}
	UnlinkLoops(Mesh, ChangeTracker);
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}
	UnlinkVertices(Mesh, ChangeTracker);
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}
	FixUpUnlinkedBevelEdges(Mesh);
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}

	// update vertex positions
	DisplaceVertices(Mesh, InsetDistance);
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}

	// mesh the bevel corners and edges
	if (NumSubdivisions <= 0)
	{
		CreateBevelMeshing(Mesh);
	}
	else
	{
		CreateBevelMeshing_Multi(Mesh);
	}

	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}

	// build output list of triangles so it can be re-used in operations below if useful
	NewTriangles.Reset();
	for (FBevelVertex& Vertex : Vertices)
	{
		NewTriangles.Append(Vertex.NewTriangles);
	}
	for (FBevelEdge& Edge : Edges)
	{
		UELocal::QuadsToTris(Mesh, Edge.StripQuads, NewTriangles, false);
	}
	for (FBevelLoop& Loop : Loops)
	{
		UELocal::QuadsToTris(Mesh, Loop.StripQuads, NewTriangles, false);
	}


	// compute normals
	ComputeNormals(Mesh);
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}

	// compute UVs
	ComputeUVs(Mesh);
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}

	ComputeMaterialIDs(Mesh);
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return false;
	}

	// todo: compute other attribs

	ResultInfo.SetSuccess(true, Progress);
	return true;
}



FMeshBevel::FBevelVertex* FMeshBevel::GetBevelVertexFromVertexID(int32 VertexID, int32* IndexOut)
{
	int32* FoundIndex = VertexIDToIndexMap.Find(VertexID);
	if (FoundIndex == nullptr)
	{
		return nullptr;
	}
	if (IndexOut != nullptr)
	{
		*IndexOut = *FoundIndex;
	}
	return &Vertices[*FoundIndex];
}



void FMeshBevel::AddBevelGroupEdge(const FDynamicMesh3& Mesh, const FGroupTopology& Topology, int32 GroupEdgeID)
{
	const TArray<int32>& MeshEdgeList = Topology.Edges[GroupEdgeID].Span.Edges;

	// currently cannot handle boundary edges
	if ( Algo::CountIf(MeshEdgeList, [&Mesh](int EdgeID) { return Mesh.IsBoundaryEdge(EdgeID); }) > 0 )
	{
		return;
	}

	FIndex2i EdgeCornerIDs = Topology.Edges[GroupEdgeID].EndpointCorners;

	FBevelEdge Edge;
	int32 NewBevelEdgeIndex = Edges.Num();

	for (int32 ci = 0; ci < 2; ++ci)
	{
		int32 CornerID = EdgeCornerIDs[ci];
		FGroupTopology::FCorner Corner = Topology.Corners[CornerID];
		int32 VertexID = Corner.VertexID;
		Edge.bEndpointBoundaryFlag[ci] = Mesh.IsBoundaryVertex(VertexID);
		int32 IncomingEdgeID = (ci == 0) ? MeshEdgeList[0] : MeshEdgeList.Last();

		int32 BevelVertexIndex = -1;
		FBevelVertex* VertInfo = GetBevelVertexFromVertexID(VertexID, &BevelVertexIndex);
		if (VertInfo == nullptr)
		{
			FBevelVertex NewVertex;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			NewVertex.CornerID = CornerID;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			NewVertex.VertexID = VertexID;
			BevelVertexIndex = Vertices.Num();
			Vertices.Add(NewVertex);
			VertexIDToIndexMap.Add(VertexID, BevelVertexIndex);
			VertInfo = &Vertices[BevelVertexIndex];
		}
		VertInfo->IncomingBevelMeshEdges.Add(IncomingEdgeID);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		VertInfo->IncomingBevelTopoEdges.Add(GroupEdgeID);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		VertInfo->IncomingBevelEdgeIndices.Add(NewBevelEdgeIndex);
		Edge.BevelVertices[ci] = BevelVertexIndex;
	}

	Edge.MeshEdges.Append(MeshEdgeList);
	Edge.MeshVertices.Append(Topology.Edges[GroupEdgeID].Span.Vertices);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Edge.GroupEdgeID = GroupEdgeID;
	Edge.GroupIDs = Topology.Edges[GroupEdgeID].Groups;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Edge.MeshEdgeTris.Reserve(Edge.MeshEdges.Num());
	for (int32 eid : Edge.MeshEdges)
	{
		Edge.MeshEdgeTris.Add(Mesh.GetEdgeT(eid));
	}

	Edge.InitialPositions.Reserve(Edge.MeshVertices.Num());
	for (int32 vid : Edge.MeshVertices)
	{
		Edge.InitialPositions.Add(Mesh.GetVertex(vid));
	}

	Edge.EdgeIndex = Edges.Num();
	Edges.Add(MoveTemp(Edge));
}





void FMeshBevel::AddBevelEdgeLoop(const FDynamicMesh3& Mesh, const FEdgeLoop& MeshEdgeLoop)
{
	// currently cannot handle boundary edges
	if ( Algo::CountIf(MeshEdgeLoop.Edges, [&Mesh](int EdgeID) { return Mesh.IsBoundaryEdge(EdgeID); }) > 0 )
	{
		return;
	}

	FBevelLoop Loop;
	Loop.MeshEdges = MeshEdgeLoop.Edges;
	Loop.MeshVertices = MeshEdgeLoop.Vertices;

	Loop.MeshEdgeTris.Reserve(Loop.MeshEdges.Num());
	for (int32 eid : Loop.MeshEdges)
	{
		Loop.MeshEdgeTris.Add(Mesh.GetEdgeT(eid));
	}

	Loop.InitialPositions.Reserve(Loop.MeshVertices.Num());
	for (int32 vid : Loop.MeshVertices)
	{
		Loop.InitialPositions.Add(Mesh.GetVertex(vid));
	}

	Loops.Add(Loop);
}


void FMeshBevel::InitVertexSet(const FDynamicMesh3& Mesh, FMeshBevel::FBevelVertex& Vertex)
{
	// get sorted list of triangles around the vertex
	TArray<int> GroupLengths;
	TArray<bool> bGroupIsLoop;
	EMeshResult Result = Mesh.GetVtxContiguousTriangles(Vertex.VertexID, Vertex.SortedTriangles, GroupLengths, bGroupIsLoop);
	if (Result != EMeshResult::Ok || GroupLengths.Num() != 1 || Vertex.SortedTriangles.Num() < 2)
	{
		Vertex.VertexType = EBevelVertexType::Unknown;
		return;
	}

	// GetVtxContiguousTriangles does not return triangles sorted in a consistent direction. This check will
	// reverse the ordering such that it is consistently walking counter-clockwise around the vertex (I think...)
	FIndex3i Tri0 = Mesh.GetTriangle(Vertex.SortedTriangles[0]).GetCycled(Vertex.VertexID);
	FIndex3i Tri1 = Mesh.GetTriangle(Vertex.SortedTriangles[1]).GetCycled(Vertex.VertexID);
	if (Tri0.C == Tri1.B)
	{
		Algo::Reverse(Vertex.SortedTriangles);
	}

	if (Mesh.IsBoundaryVertex(Vertex.VertexID))
	{
		Vertex.VertexType = EBevelVertexType::BoundaryVertex;
		// TODO: we should have a BuildBoundaryVertex function here that correctly populates the 
		// Wedges for the boundary vertex. The currently BuildJunctionVertex will not be able to do
		// this because it assumes it can just walk forward from any edge
		return;
	}


	MESH_BEVEL_DEBUG_CHECK(Vertex.IncomingBevelMeshEdges.Num() != 0);		// shouldn't ever happen
	if (Vertex.IncomingBevelMeshEdges.Num() == 1)
	{
		BuildTerminatorVertex(Vertex, Mesh);
	}
	else
	{
		BuildJunctionVertex(Vertex, Mesh);
	}
}

void FMeshBevel::FinalizeTerminatorVertex(const FDynamicMesh3& Mesh, FBevelVertex& Vertex)
{
	if (Vertex.VertexType == EBevelVertexType::TerminatorVertex)
	{
		int32 OtherVertexID = Vertex.TerminatorInfo.B;
		int32* OtherBevelVtxIdx = VertexIDToIndexMap.Find(OtherVertexID);
		if (OtherBevelVtxIdx != nullptr)
		{
			// does other vertex have to be a terminator? or can this also happen w/ a junction?
			FBevelVertex& OtherVertex = Vertices[*OtherBevelVtxIdx];
			if (OtherVertex.VertexType == EBevelVertexType::TerminatorVertex)
			{
				// want to skip this if the ring-split edge is already a bevel edge
				int32 MeshEdgeID = Mesh.FindEdge(Vertex.VertexID, OtherVertex.VertexID);
				MESH_BEVEL_DEBUG_CHECK(MeshEdgeID >= 0);
				if (Mesh.IsEdge(MeshEdgeID) &&
					Vertex.TerminatorInfo.A == MeshEdgeID &&
					OtherVertex.TerminatorInfo.A == MeshEdgeID &&		// do we need the other vertex to use the same edge here?  (is this actually a hard constraint on that edge that we should be enforcing??)
					Vertex.IncomingBevelMeshEdges.Contains(MeshEdgeID) == false)
				{
					Vertex.ConnectedBevelVertex = *OtherBevelVtxIdx;
				}
			}

		}
	}
}

void FMeshBevel::BuildVertexSets(const FDynamicMesh3& Mesh)
{
	// can be parallel
	for (FBevelVertex& Vertex : Vertices)
	{
		InitVertexSet(Mesh, Vertex);

		if (ResultInfo.CheckAndSetCancelled(Progress))
		{
			return;
		}
	}

	// At a 'Terminator' vertex we are going to split the one-ring and fill the bevel-side with a quad, which
	// will usually leave a single-triangle hole. However if two Terminator vertices are directly connected
	// via a non-bevel mesh edge, the two triangles will be connected, ie the hole is quad-shaped. It's easier
	// to detect this case here /before/ we split the mesh up into disconnected parts...
	for (FBevelVertex& Vertex : Vertices)
	{
		FinalizeTerminatorVertex(Mesh, Vertex);
	}

}



void FMeshBevel::BuildJunctionVertex(FBevelVertex& Vertex, const FDynamicMesh3& Mesh)
{
	//
	// Now split up the single contiguous one-ring into "Wedges" created by the incoming split-edges
	//

	// find first split edge, and the triangle/index "after" that first split edge
	int32 NT = Vertex.SortedTriangles.Num();
	int32 StartTriIndex = -1;
	for (int32 k = 0; k < NT; ++k)
	{
		if (FindSharedEdgeInTriangles(Mesh, Vertex.SortedTriangles[k], Vertex.SortedTriangles[(k + 1) % NT]) == Vertex.IncomingBevelMeshEdges[0])
		{
			StartTriIndex = (k + 1) % NT;		// start at second tri, so that bevel-edge is first edge in wedge
			break;
		}
	}
	if (StartTriIndex == -1)
	{
		Vertex.VertexType = EBevelVertexType::Unknown;
		return;
	}

	// now walk around the one-ring tris, accumulating into current Wedge until we hit another split-edge,
	// at which point a new Wedge is spawned
	int32 CurTriIndex = StartTriIndex;
	FOneRingWedge CurWedge;
	CurWedge.WedgeVertex = Vertex.VertexID;
	CurWedge.Triangles.Add(Vertex.SortedTriangles[CurTriIndex]);
	CurWedge.BorderEdges.A = Vertex.IncomingBevelMeshEdges[0];
	for (int32 k = 0; k < NT; ++k)
	{
		int32 CurTri = Vertex.SortedTriangles[CurTriIndex % NT];
		int32 NextTri = Vertex.SortedTriangles[(CurTriIndex + 1) % NT];
		int32 SharedEdge = FindSharedEdgeInTriangles(Mesh, CurTri, NextTri);
		MESH_BEVEL_DEBUG_CHECK(SharedEdge != -1);
		if (Vertex.IncomingBevelMeshEdges.Contains(SharedEdge))
		{
			// if we found a bevel-edge, close the current wedge and start a new one
			CurWedge.BorderEdges.B = SharedEdge;
			Vertex.Wedges.Add(CurWedge);
			CurWedge = FOneRingWedge();
			CurWedge.WedgeVertex = Vertex.VertexID;
			CurWedge.BorderEdges.A = SharedEdge;
		}
		CurWedge.Triangles.Add(NextTri);
		CurTriIndex++;
	}
	// ?? is there a chance that we have a final open wedge here? we iterate one extra time so it shouldn't happen (or could we get an extra wedge then??)

	for (FOneRingWedge& Wedge : Vertex.Wedges)
	{
		Wedge.BorderEdgeTriEdgeIndices.A = Mesh.GetTriEdges(Wedge.Triangles[0]).IndexOf(Wedge.BorderEdges.A);
		Wedge.BorderEdgeTriEdgeIndices.B = Mesh.GetTriEdges(Wedge.Triangles.Last()).IndexOf(Wedge.BorderEdges.B);
	}

	if (Vertex.Wedges.Num() > 1)
	{
		Vertex.VertexType = EBevelVertexType::JunctionVertex;
	}
	else
	{
		Vertex.VertexType = EBevelVertexType::Unknown;
	}
}


void FMeshBevel::BuildTerminatorVertex(FBevelVertex& Vertex, const FDynamicMesh3& Mesh)
{
	Vertex.VertexType = EBevelVertexType::Unknown;
	if (MESH_BEVEL_DEBUG_ENSURE(Vertex.IncomingBevelMeshEdges.Num() == 1) == false)
	{
		return;
	}

	int32 IncomingEdgeID = Vertex.IncomingBevelMeshEdges[0];
	int32 NumTris = Vertex.SortedTriangles.Num();

	// We have one edge coming into the vertex one ring, our main problem is to pick a second
	// edge to split the one-ring with. There is no obvious right answer in many cases. 

	// "other" split edge that we pick
	int32 RingSplitEdgeID = -1;

	// NOTE: code below follows polygroups on the mesh when available; otherwise, just tries to pick a 'furthest' edge

	// Find the ordered set of triangles that are not in either of the groups connected to IncomingEdgeID. Eg imagine
	// a cube corner, if we want to bevel IncomingEdgeID along one of the cube edges, we want to add the new edge
	// in the "furthest" face (perpendicular to the edge)
	FIndex2i IncomingEdgeT = Mesh.GetEdgeT(IncomingEdgeID);

	// Logic to find the RingSplitEdgeID (in its own control block just to clarify local variables logical scope)
	{
		FIndex2i IncomingEdgeGroups(Mesh.GetTriangleGroup(IncomingEdgeT.A), Mesh.GetTriangleGroup(IncomingEdgeT.B));

		// Find the first index of Vertex.SortedTriangles that we want to remove.
		// This should not be able to fail but if it does, we will just start at 0 
		// and maybe end up with some bevel failures below
		int32 NumTriangles = Vertex.SortedTriangles.Num();
		int32 StartIndex = 0;
		for (int32 k = 0; k < NumTriangles; ++k)
		{
			int32 gid = Mesh.GetTriangleGroup(Vertex.SortedTriangles[k]);
			if (IncomingEdgeGroups.Contains(gid))
			{
				StartIndex = k;
				break;
			}
		}

		TArray<int32> OtherGroupTris;	// sorted wedge of triangles that are not in either of the groups connected to incoming edge
		TArray<int32> OtherGroups;		// list of group IDs encountered, in-order
		for (int32 k = 0; k < NumTriangles; ++k)
		{
			// Vertex.SortedTriangles was ordered, ie sequential triangles were connected, but if we filter
			// out some triangles it may no longer be sequential in OtherGroupTris, leading to badness. 
			// But the two groups we are removing should be contiguous, so if we start there, then the
			// remaining tris should be contiguous. StartIndex found above should give us that triangle.
			int32 ShiftedIndex = (StartIndex + k) % NumTriangles;
			int32 tid = Vertex.SortedTriangles[ShiftedIndex];
			int32 gid = Mesh.GetTriangleGroup(tid);
			if (IncomingEdgeGroups.Contains(gid) == false)
			{
				OtherGroupTris.Add(tid);
				OtherGroups.AddUnique(gid);
			}
		}
		int32 NumRemainingTris = OtherGroupTris.Num();

		// If we didn't find other groups at all, then pick the outgoing edge
		// that is most aligned to the incoming edge
		if (OtherGroups.Num() == 0)
		{
			FIndex2i IncomingEdgeV = Mesh.GetEdgeV(IncomingEdgeID);
			int32 OtherVID = IncomingEdgeV.OtherElement(Vertex.VertexID);
			if (!ensure(OtherVID != IndexConstants::InvalidID))
			{
				// incoming edge is not attached to the expected vertex, fail out
				return;
			}
			FVector3d CenterPos = Mesh.GetVertex(Vertex.VertexID);
			FVector3d IncomingPos = Mesh.GetVertex(OtherVID);
			FVector3d IncomingDir = (CenterPos - IncomingPos);
			IncomingDir.Normalize();
			// Note: If the incoming edge is degenerate, this will just pick the first different edge.
			// We could instead try to use a different heuristic in this case.
			double BestAlignmentScore = -FMathd::MaxReal;
			Mesh.EnumerateVertexEdges(Vertex.VertexID, 
				[&BestAlignmentScore, &RingSplitEdgeID, &Mesh, &IncomingDir, &CenterPos, &Vertex, IncomingEdgeID]
				(int32 EID)
				{
					if (EID == IncomingEdgeID)
					{
						return;
					}
					FIndex2i OutEdgeV = Mesh.GetEdgeV(EID);
					int32 OutVID = OutEdgeV.OtherElement(Vertex.VertexID);
					if (ensure(OutVID != IndexConstants::InvalidID))
					{
						FVector3d OutgoingPos = Mesh.GetVertex(OutVID);
						FVector3d OutgoingDir = OutgoingPos - CenterPos;
						OutgoingDir.Normalize();
						double AlignmentScore = OutgoingDir.Dot(IncomingDir);
						if (AlignmentScore > BestAlignmentScore)
						{
							RingSplitEdgeID = EID;
							BestAlignmentScore = AlignmentScore;
						}
					}
				}
			);
		}
		// If we found edges on other groups, use them to guide the split --
		// Determine which edge to split at in the "other" group triangles. If we only have one group
		// then we can try to pick the "middlest" edge. The worst case is when there is only one triangle,
		// then we are picking a not-very-good edge no matter what (potentially we should do an edge split or
		// face poke in that situation). If we have multiple groups then we probably want to pick one
		// of the group-boundary edges inside the triangle-span, ideally the "middlest" but currently we
		// are just picking one arbitrarily...
		else if (OtherGroups.Num() == 1)
		{
			Vertex.NewGroupID = OtherGroups[0];
			if (OtherGroupTris.Num() == 1)
			{
				FIndex3i TriEdges = Mesh.GetTriEdges(OtherGroupTris[0]);
				for (int32 j = 0; j < 3; ++j)
				{
					if (Mesh.GetEdgeV(TriEdges[j]).Contains(Vertex.VertexID))
					{
						RingSplitEdgeID = TriEdges[j];
						break;
					}
				}
			}
			else if (OtherGroupTris.Num() == 2)
			{
				RingSplitEdgeID = FindSharedEdgeInTriangles(Mesh, OtherGroupTris[0], OtherGroupTris[1]);
			}
			else
			{
				// try using the 'middlest' triangle as the 'middlest' edge
				// TODO: should compute opening angles here and pick the edge closest to the middle of the angular span!!
				int32 j = OtherGroupTris.Num() / 2;
				RingSplitEdgeID = FindSharedEdgeInTriangles(Mesh, OtherGroupTris[j], OtherGroupTris[j + 1]);

				// If the OtherGroupTris list ended up being not contiguous (see above for how that can happen), then
				// it's possible that (j) and (j+1) are not connected and the share-edge search will fail. In that case
				// we will just linear-search for two connected triangles. If this fails then this vertex will not be bevelled.
				if (RingSplitEdgeID == -1)
				{
					for (int32 k = 0; k < NumRemainingTris; ++k)
					{
						RingSplitEdgeID = FindSharedEdgeInTriangles(Mesh, OtherGroupTris[k], OtherGroupTris[(k + 1) % NumRemainingTris]);
						if (RingSplitEdgeID != -1)
						{
							break;
						}
					}
				}
			}
		}
		else
		{
			// Search for two adjacent triangles in different groups that have a shared edge. This search
			// may need to wrap around if we got unlucky in the triangle ordering in OtherGroupTris
			for (int32 k = 0; k < OtherGroupTris.Num(); ++k)
			{
				int32 TriangleA = OtherGroupTris[k], TriangleB = OtherGroupTris[(k + 1) % NumRemainingTris];
				if (Mesh.GetTriangleGroup(TriangleA) != Mesh.GetTriangleGroup(TriangleB))
				{
					RingSplitEdgeID = FindSharedEdgeInTriangles(Mesh, TriangleA, TriangleB);
					Vertex.NewGroupID = -1;		// allocate a new group for this triangle, this is usually what one would want
					break;
				}
			}
		}
	}

	if (RingSplitEdgeID == -1)
	{
		return;
	}

	// save edgeid/vertexid for the 'terminator edge' that we will disconnect at
	FIndex2i SplitEdgeV = Mesh.GetEdgeV(RingSplitEdgeID);
	Vertex.TerminatorInfo = FIndex2i(RingSplitEdgeID, SplitEdgeV.OtherElement(Vertex.VertexID));

	// split the terminator vertex into two wedges
	TArray<int32> SplitTriSets[2];
	if (SplitInteriorVertexTrianglesIntoSubsets(&Mesh, Vertex.VertexID, IncomingEdgeID, RingSplitEdgeID, SplitTriSets[0], SplitTriSets[1]) == false)
	{
		return;
	}

	// make the wedge list
	Vertex.Wedges.SetNum(2);
	Vertex.Wedges[0].WedgeVertex = Vertex.VertexID;
	Vertex.Wedges[0].Triangles.Append(SplitTriSets[0]);
	Vertex.Wedges[1].WedgeVertex = Vertex.VertexID;
	Vertex.Wedges[1].Triangles.Append(SplitTriSets[1]);

	// We need to know the two border edges of each wedge, because we will use this information
	// in later stages. Note that this block is not specific to the TerminatorVertex case
	for (FOneRingWedge& Wedge : Vertex.Wedges)
	{
		int32 NumWedgeTris = Wedge.Triangles.Num();

		FIndex2i VtxEdges0 = IndexUtil::FindVertexEdgesInTriangle(Mesh, Wedge.Triangles[0], Vertex.VertexID);
		if (NumWedgeTris == 1)
		{
			// If the wedge only has one tri, both edges are the border edges. 
			// Note that these are not sorted, ie B might not be the edge shared with the next Wedge. 
			// Currently this does not matter but it might in the future?
			Wedge.BorderEdges.A = VtxEdges0.A;
			Wedge.BorderEdges.B = VtxEdges0.B;
		}
		else
		{
			// the wedge-border-edge is *not* the edge connected to the next triangle in the wedge-triangle-list
			Wedge.BorderEdges.A = ( Mesh.GetEdgeT(VtxEdges0.A).Contains(Wedge.Triangles[1]) ) ? VtxEdges0.B : VtxEdges0.A;
			// final wedge-border-edge is the same case, with the last and second-last tris
			FIndex2i VtxEdges1 = IndexUtil::FindVertexEdgesInTriangle(Mesh, Wedge.Triangles[NumWedgeTris-1], Vertex.VertexID);
			Wedge.BorderEdges.B = ( Mesh.GetEdgeT(VtxEdges1.A).Contains(Wedge.Triangles[NumWedgeTris-2]) ) ? VtxEdges1.B : VtxEdges1.A;
		}
		// save the index of the border edge in it's triangle, so that when we disconnect the wedges later,
		// we can find the new edge ID
		Wedge.BorderEdgeTriEdgeIndices.A = Mesh.GetTriEdges(Wedge.Triangles[0]).IndexOf(Wedge.BorderEdges.A);
		Wedge.BorderEdgeTriEdgeIndices.B = Mesh.GetTriEdges(Wedge.Triangles.Last()).IndexOf(Wedge.BorderEdges.B);
	}

	Vertex.VertexType = EBevelVertexType::TerminatorVertex;
}


void FMeshBevel::UnlinkEdges(FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker)
{
	for (FBevelEdge& Edge : Edges)
	{
		UnlinkBevelEdgeInterior(Mesh, Edge, ChangeTracker);
	}
}



namespace UELocal
{
	// decomposition of a vertex one-ring into two connected triangle subsets
	struct FVertexSplit
	{
		int32 VertexID;
		bool bOK;
		TArray<int32> TriSets[2];
	};

	// walk along a sequence of vertex-splits and make sure that the split triangle sets
	// maintain consistent "sides" (see call in UnlinkBevelEdgeInterior for more details)
	static void ReconcileTriangleSets(TArray<FVertexSplit>& SplitSequence)
	{
		int32 N = SplitSequence.Num();
		TArray<int32> PrevTriSet0;
		for (int32 k = 0; k < N; ++k)
		{
			if (PrevTriSet0.Num() == 0 && SplitSequence[k].TriSets[0].Num() > 0)
			{
				PrevTriSet0 = SplitSequence[k].TriSets[0];
			}
			else
			{
				bool bFoundInSet0 = false;
				for (int32 tid : SplitSequence[k].TriSets[0])
				{
					if (PrevTriSet0.Contains(tid))
					{
						bFoundInSet0 = true;
						break;
					}
				}
				if (!bFoundInSet0)
				{
					Swap(SplitSequence[k].TriSets[0], SplitSequence[k].TriSets[1]);
				}
				PrevTriSet0 = SplitSequence[k].TriSets[0];
			}
		}
	}

};


void FMeshBevel::UnlinkBevelEdgeInterior(
	FDynamicMesh3& Mesh,
	FBevelEdge& BevelEdge,
	FDynamicMeshChangeTracker* ChangeTracker)
{
	// figure out what sets of triangles to split each vertex into
	int32 N = BevelEdge.MeshVertices.Num();

	TArray<UELocal::FVertexSplit> SplitsToProcess;
	SplitsToProcess.SetNum(N);

	// precompute triangle sets for each vertex we want to split, by "cutting" the one ring into two halves based
	// on edges - 2 edges for interior vertices, and 1 edge for a boundary vertex at the start/end of the edge-span

	SplitsToProcess[0] = UELocal::FVertexSplit{ BevelEdge.MeshVertices[0], false };
	if (BevelEdge.bEndpointBoundaryFlag[0])
	{
		SplitsToProcess[0].bOK = SplitBoundaryVertexTrianglesIntoSubsets(&Mesh, SplitsToProcess[0].VertexID, BevelEdge.MeshEdges[0], 
			SplitsToProcess[0].TriSets[0], SplitsToProcess[0].TriSets[1]);
	}
	for (int32 k = 1; k < N - 1; ++k)
	{
		SplitsToProcess[k].VertexID = BevelEdge.MeshVertices[k];
		if (Mesh.IsBoundaryVertex(SplitsToProcess[k].VertexID))
		{
			SplitsToProcess[k].bOK = false;
		}
		else
		{
			SplitsToProcess[k].bOK = SplitInteriorVertexTrianglesIntoSubsets(&Mesh, SplitsToProcess[k].VertexID,
				BevelEdge.MeshEdges[k-1], BevelEdge.MeshEdges[k], SplitsToProcess[k].TriSets[0], SplitsToProcess[k].TriSets[1]);
		}
	}
	SplitsToProcess[N-1] = UELocal::FVertexSplit{ BevelEdge.MeshVertices[N - 1], false };
	if (BevelEdge.bEndpointBoundaryFlag[1])
	{
		SplitsToProcess[N-1].bOK = SplitBoundaryVertexTrianglesIntoSubsets(&Mesh, SplitsToProcess[N-1].VertexID, BevelEdge.MeshEdges[N-2], 
			SplitsToProcess[N-1].TriSets[0], SplitsToProcess[N-1].TriSets[1]);
	}

	// SplitInteriorVertexTrianglesIntoSubsets does not consistently order its output sets, ie, if you imagine [Edge0,Edge1] as a path
	// cutting through the one ring, the "side" that Set0 and Set1 end up is arbitrary, and depends on the ordering of edges in the triangles of Edge1.
	// This might ideally be fixed in the future, but for the time being, all we need is consistency. So we walk from the start of the 
	// edge to the end, checking for overlap between each tri-one-ring-wedge. If Split[k].TriSet0 does not overlap with Split[k-1].TriSet0, then
	// we want to swap TriSet0 and TriSet1 at Split[k].
	UELocal::ReconcileTriangleSets(SplitsToProcess);

	// apply vertex splits and accumulate new list
	N = SplitsToProcess.Num();
	for (int32 k = 0; k < N; ++k)
	{
		const UELocal::FVertexSplit& Split = SplitsToProcess[k];
		if (ChangeTracker)
		{
			ChangeTracker->SaveVertexOneRingTriangles(Split.VertexID, true);
		}

		bool bDone = false;
		if (Split.bOK)
		{
			FDynamicMesh3::FVertexSplitInfo SplitInfo;
			EMeshResult Result = Mesh.SplitVertex(Split.VertexID, Split.TriSets[0], SplitInfo);
			if (MESH_BEVEL_DEBUG_ENSURE(Result == EMeshResult::Ok))
			{
				BevelEdge.NewMeshVertices.Add(SplitInfo.NewVertex);
				bDone = true;
			}
		}
		if (!bDone)
		{
			BevelEdge.NewMeshVertices.Add(Split.VertexID);
		}
	}

	// now build edge correspondences
	N = BevelEdge.MeshVertices.Num();
	MESH_BEVEL_DEBUG_CHECK(N == BevelEdge.NewMeshVertices.Num());
	for (int32 k = 0; k < N-1; ++k)
	{
		int32 Edge0 = BevelEdge.MeshEdges[k];
		int32 Edge1 = Mesh.FindEdge(BevelEdge.NewMeshVertices[k], BevelEdge.NewMeshVertices[k + 1]);
		BevelEdge.NewMeshEdges.Add(Edge1);
		MESH_BEVEL_DEBUG_CHECK(Edge1 >= 0);
		if ( Mesh.IsEdge(Edge1) && Edge0 != Edge1 && MeshEdgePairs.Contains(Edge0) == false )
		{
			MeshEdgePairs.Add(Edge0, Edge1);
			MeshEdgePairs.Add(Edge1, Edge0);
		}
	}
}




void FMeshBevel::UnlinkBevelLoop(FDynamicMesh3& Mesh, FBevelLoop& BevelLoop, FDynamicMeshChangeTracker* ChangeTracker)
{
	int32 N = BevelLoop.MeshVertices.Num();

	TArray<UELocal::FVertexSplit> SplitsToProcess;
	SplitsToProcess.SetNum(N);

	// precompute triangle sets for each vertex we want to split
	for (int32 k = 0; k < N; ++k)
	{
		SplitsToProcess[k].VertexID = BevelLoop.MeshVertices[k];
		if (Mesh.IsBoundaryVertex(SplitsToProcess[k].VertexID))
		{
			// cannot split boundary vertex
			SplitsToProcess[k].bOK = false;
		}
		else
		{
			int32 PrevEdge = (k == 0) ? BevelLoop.MeshEdges.Last() : BevelLoop.MeshEdges[k-1];
			int32 CurEdge = BevelLoop.MeshEdges[k];
			SplitsToProcess[k].bOK = SplitInteriorVertexTrianglesIntoSubsets(&Mesh, SplitsToProcess[k].VertexID,
				PrevEdge, CurEdge, SplitsToProcess[k].TriSets[0], SplitsToProcess[k].TriSets[1]);
		}
	}

	// fix up triangle sets - see call in UnlinkBevelEdgeInterior() for more info
	UELocal::ReconcileTriangleSets(SplitsToProcess);

	// apply vertex splits and accumulate new list
	N = SplitsToProcess.Num();
	for (int32 k = 0; k < N; ++k)
	{
		const UELocal::FVertexSplit& Split = SplitsToProcess[k];
		if (ChangeTracker)
		{
			ChangeTracker->SaveVertexOneRingTriangles(Split.VertexID, true);
		}

		bool bDone = false;
		if (Split.bOK)
		{
			FDynamicMesh3::FVertexSplitInfo SplitInfo;
			EMeshResult Result = Mesh.SplitVertex(Split.VertexID, Split.TriSets[1], SplitInfo);
			if (MESH_BEVEL_DEBUG_ENSURE(Result == EMeshResult::Ok))
			{
				BevelLoop.NewMeshVertices.Add(SplitInfo.NewVertex);
				bDone = true;
			}
		}
		if (!bDone)
		{
			BevelLoop.NewMeshVertices.Add(Split.VertexID);		// failed to split, so we have a shared vertex on both "sides"
		}
	}

	// now build edge correspondences
	N = BevelLoop.MeshVertices.Num();
	MESH_BEVEL_DEBUG_CHECK(N == BevelLoop.NewMeshVertices.Num());
	for (int32 k = 0; k < N; ++k)
	{
		int32 Edge0 = BevelLoop.MeshEdges[k];
		int32 Edge1 = Mesh.FindEdge(BevelLoop.NewMeshVertices[k], BevelLoop.NewMeshVertices[(k + 1)%N]);
		BevelLoop.NewMeshEdges.Add(Edge1);
		MESH_BEVEL_DEBUG_CHECK(Edge1 >= 0);
		if (Mesh.IsEdge(Edge1) && Edge0 != Edge1 && MeshEdgePairs.Contains(Edge0) == false )
		{
			MeshEdgePairs.Add(Edge0, Edge1);
			MeshEdgePairs.Add(Edge1, Edge0);
		}
	}
}

void FMeshBevel::UnlinkLoops(FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker)
{
	for (FBevelLoop& Loop : Loops)
	{
		UnlinkBevelLoop(Mesh, Loop, ChangeTracker);
	}
}




void FMeshBevel::UnlinkVertices(FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker)
{
	// TODO: currently have to do terminator vertices first because we do some of the 
	// determination inside the unlink code...

	for (FBevelVertex& Vertex : Vertices)
	{
		if (Vertex.VertexType == EBevelVertexType::TerminatorVertex)
		{
			UnlinkTerminatorVertex(Mesh, Vertex, ChangeTracker);
		}
	}

	for (FBevelVertex& Vertex : Vertices)
	{
		if (Vertex.VertexType == EBevelVertexType::JunctionVertex)
		{
			UnlinkJunctionVertex(Mesh, Vertex, ChangeTracker);
		}
	}
}


void FMeshBevel::UnlinkJunctionVertex(FDynamicMesh3& Mesh, FBevelVertex& Vertex, FDynamicMeshChangeTracker* ChangeTracker)
{
	MESH_BEVEL_DEBUG_CHECK(Vertex.VertexType == EBevelVertexType::JunctionVertex);

	if (ChangeTracker)
	{
		ChangeTracker->SaveVertexOneRingTriangles(Vertex.VertexID, true);
	}

	int32 NumWedges = Vertex.Wedges.Num();
	MESH_BEVEL_DEBUG_CHECK(NumWedges > 1);

	// Split triangles around vertex into separate tri-sets based on wedges.
	// This will create a new vertex for each wedge.
	for (int32 k = 1; k < NumWedges; ++k)
	{
		FOneRingWedge& Wedge = Vertex.Wedges[k];

		FDynamicMesh3::FVertexSplitInfo SplitInfo;
		EMeshResult Result = Mesh.SplitVertex(Vertex.VertexID, Wedge.Triangles, SplitInfo);
		if (MESH_BEVEL_DEBUG_ENSURE(Result == EMeshResult::Ok))
		{
			Wedge.WedgeVertex = SplitInfo.NewVertex;
		}
	}

	// update end start/end pairs for each wedge. If we created new edges above, this is
	// the first time we will encounter them, so save in edge correspondence map
	for (int32 k = 0; k < NumWedges; ++k)
	{
		FOneRingWedge& Wedge = Vertex.Wedges[k];
		for (int32 j = 0; j < 2; ++j)
		{
			int32 OldWedgeEdgeID = Wedge.BorderEdges[j];
			int32 OldWedgeEdgeIndex = Wedge.BorderEdgeTriEdgeIndices[j];
			int32 TriangleID = (j == 0) ? Wedge.Triangles[0] : Wedge.Triangles.Last();
			int32 CurWedgeEdgeID = Mesh.GetTriEdges(TriangleID)[OldWedgeEdgeIndex];
			if (OldWedgeEdgeID != CurWedgeEdgeID)
			{
				if (MeshEdgePairs.Contains(OldWedgeEdgeID) == false)
				{
					MeshEdgePairs.Add(OldWedgeEdgeID, CurWedgeEdgeID);
					MeshEdgePairs.Add(CurWedgeEdgeID, OldWedgeEdgeID);
				}
				Wedge.BorderEdges[j] = CurWedgeEdgeID;
			}
		}
	}

}




void FMeshBevel::UnlinkTerminatorVertex(FDynamicMesh3& Mesh, FBevelVertex& BevelVertex, FDynamicMeshChangeTracker* ChangeTracker)
{
	MESH_BEVEL_DEBUG_CHECK(BevelVertex.VertexType == EBevelVertexType::TerminatorVertex);
	MESH_BEVEL_DEBUG_CHECK(BevelVertex.Wedges.Num() == 2);

	if (ChangeTracker)
	{
		ChangeTracker->SaveVertexOneRingTriangles(BevelVertex.VertexID, true);
	}

	// split the vertex
	FDynamicMesh3::FVertexSplitInfo SplitInfo;
	EMeshResult Result = Mesh.SplitVertex(BevelVertex.VertexID, BevelVertex.Wedges[1].Triangles, SplitInfo);
	if (MESH_BEVEL_DEBUG_ENSURE(Result == EMeshResult::Ok))
	{
		BevelVertex.Wedges[1].WedgeVertex = SplitInfo.NewVertex;

		// update end start/end pairs for each wedge, and save in the edge correspondence map.
		// Note that this is the same block as in UnlinkJunctionVertex
		int32 NumWedges = BevelVertex.Wedges.Num();
		for (int32 k = 0; k < NumWedges; ++k)
		{
			FOneRingWedge& Wedge = BevelVertex.Wedges[k];
			for (int32 j = 0; j < 2; ++j)
			{
				int32 OldWedgeEdgeID = Wedge.BorderEdges[j];
				int32 OldWedgeEdgeIndex = Wedge.BorderEdgeTriEdgeIndices[j];
				int32 TriangleID = (j == 0) ? Wedge.Triangles[0] : Wedge.Triangles.Last();
				int32 CurWedgeEdgeID = Mesh.GetTriEdges(TriangleID)[OldWedgeEdgeIndex];
				if (OldWedgeEdgeID != CurWedgeEdgeID)
				{
					if (MeshEdgePairs.Contains(OldWedgeEdgeID) == false)
					{
						MeshEdgePairs.Add(OldWedgeEdgeID, CurWedgeEdgeID);
						MeshEdgePairs.Add(CurWedgeEdgeID, OldWedgeEdgeID);
					}
					Wedge.BorderEdges[j] = CurWedgeEdgeID;
				}
			}
		}

	}


}


void FMeshBevel::FixUpUnlinkedBevelEdges(FDynamicMesh3& Mesh)
{
	// Rewrite vertex IDs in BevelEdge vertex lists to correctly match the vertices in the new unlinked wedges.
	// We did not know these new vertices in UnlinkBevelEdgeInterior() because we didn't unlink the vertices
	// into wedges until afterwards. 
	for (FBevelEdge& Edge : Edges)
	{
		// In some cases, like single edges, Edge.NewMeshEdges is incorrect (mapped to itself); e.g., because we
		// could not actually unlink the edge in UnlinkBevelEdgeInterior (if there were no interior vertices).
		// Now that all vertices are unlinked, take one more pass through and fix the single-edge or self-mapped edges.
		bool bSingleEdge = Edge.MeshEdges.Num() == 1;
		bool bFailedEdgePairing = false;
		for (int32 Idx = 0; Idx < Edge.MeshEdges.Num(); ++Idx)
		{
			bool bNewEdgeIsOld = Edge.MeshEdges[Idx] == Edge.NewMeshEdges[Idx];
			if (!bSingleEdge && !bNewEdgeIsOld)
			{
				continue;
			}

			int32* FoundOtherEdge = MeshEdgePairs.Find(Edge.MeshEdges[Idx]);
			MESH_BEVEL_DEBUG_CHECK(FoundOtherEdge != nullptr);
			if (FoundOtherEdge != nullptr)
			{
				Edge.NewMeshEdges[Idx] = *FoundOtherEdge;
			}
			else
			{
				bFailedEdgePairing = true;		// something went wrong, loop below will break things
				break;
			}
		}
		if (bFailedEdgePairing)
		{
			continue;
		}

		// process start and end vertices of the path
		for (int32 j = 0; j < 2; ++j)
		{
			int32 vi = (j == 0) ? 0 : (Edge.MeshVertices.Num()-1);
			int32 ei = (j == 0) ? 0 : (Edge.MeshEdges.Num()-1);
			const FBevelVertex* BevelVertex = GetBevelVertexFromVertexID(Edge.MeshVertices[vi]);
			int32& V0 = Edge.MeshVertices[vi];
			int32& V1 = Edge.NewMeshVertices[vi];
			int32 E0 = Edge.MeshEdges[ei], E1 = Edge.NewMeshEdges[ei];
			bool bFoundV0 = false, bFoundV1 = false;
			for (const FOneRingWedge& Wedge : BevelVertex->Wedges)
			{
				for (int32 tid : Wedge.Triangles)
				{
					FIndex3i TriEdges = Mesh.GetTriEdges(tid);
					if (TriEdges.Contains(E0) && bFoundV0 == false)
					{
						MESH_BEVEL_DEBUG_CHECK( Mesh.GetEdgeV(E0).Contains(Wedge.WedgeVertex) );
						V0 = Wedge.WedgeVertex;
						bFoundV0 = true;
						break;
					}
					else if (TriEdges.Contains(E1) && bFoundV1 == false)
					{
						MESH_BEVEL_DEBUG_CHECK( Mesh.GetEdgeV(E1).Contains(Wedge.WedgeVertex) );
						V1 = Wedge.WedgeVertex;
						bFoundV1 = true;
						break;
					}
				}
			}
		}
	}
}



void FMeshBevel::DisplaceVertices(FDynamicMesh3& Mesh, double Distance)
{
	// fallback (very bad) technique to compute an inset vertex
	auto GetDisplacedVertexPos = [Distance](const FDynamicMesh3& Mesh, int32 VertexID) -> FVector3d
	{
		FVector3d CurPos = Mesh.GetVertex(VertexID);
		FVector3d Centroid = FMeshWeights::MeanValueCentroid(Mesh, VertexID);
		return CurPos + Distance * Normalized(Centroid - CurPos);
	};

	// Basically want to inset any beveled edges inwards into the existing poly-faces. 
	// To do this we will solve using our standard inset technique, eg similar to FInsetMeshRegion,
	// which involves computing 'inset lines' for each edge and then finding nearest-points between
	// pairs of lines (which will be the intersection point if the face is planar).

	// Need to keep track of the inset line sets for open paths because at the corner vertices we
	// will need to combine data from multiple path-line-sets (possibly could do this more efficiently 
	// as we only ever need the first and last...but small in context)
	struct FEdgePathInsetLines
	{
		TArray<FLine3d> InsetLines0;
		TArray<FLine3d> InsetLines1;
	};
	TArray<FEdgePathInsetLines> AllInsetLines;
	AllInsetLines.SetNum(Edges.Num());

	// solve open paths
	for ( int32 k = 0; k < Edges.Num(); ++k)
	{
		FBevelEdge& Edge = Edges[k];
		UE::Geometry::ComputeInsetLineSegmentsFromEdges(Mesh, Edge.MeshEdges, InsetDistance, AllInsetLines[k].InsetLines0);
		UE::Geometry::SolveInsetVertexPositionsFromInsetLines(Mesh, AllInsetLines[k].InsetLines0, Edge.MeshVertices, Edge.NewPositions0, false);

		UE::Geometry::ComputeInsetLineSegmentsFromEdges(Mesh, Edge.NewMeshEdges, InsetDistance, AllInsetLines[k].InsetLines1);
		UE::Geometry::SolveInsetVertexPositionsFromInsetLines(Mesh, AllInsetLines[k].InsetLines1, Edge.NewMeshVertices, Edge.NewPositions1, false);
	}

	// solve loops
	for (FBevelLoop& Loop : Loops)
	{
		TArray<FLine3d> InsetLines;
		UE::Geometry::ComputeInsetLineSegmentsFromEdges(Mesh, Loop.MeshEdges, InsetDistance, InsetLines);
		UE::Geometry::SolveInsetVertexPositionsFromInsetLines(Mesh, InsetLines, Loop.MeshVertices, Loop.NewPositions0, true);

		UE::Geometry::ComputeInsetLineSegmentsFromEdges(Mesh, Loop.NewMeshEdges, InsetDistance, InsetLines);
		UE::Geometry::SolveInsetVertexPositionsFromInsetLines(Mesh, InsetLines, Loop.NewMeshVertices, Loop.NewPositions1, true);
	}


	// Now solve corners. For corners, we want to find the 1 or 2 inset-lines corresponding
	// to the outgoing bevel-edges at each bevel-vertex-wedge. 
	for (FBevelVertex& Vertex : Vertices)
	{
		if (Vertex.VertexType == EBevelVertexType::Unknown)
		{
			continue;
		}

		int32 NumWedges = Vertex.Wedges.Num();
		for (int32 k = 0; k < NumWedges; ++k)
		{
			FOneRingWedge& Wedge = Vertex.Wedges[k];
			FVector3d CurPos = Mesh.GetVertex(Wedge.WedgeVertex);

			// collect up set of inset lines relevant to this vertex
			TArray<FLine3d> SolveLines;
			for (int32 j : Vertex.IncomingBevelEdgeIndices)
			{
				if (Edges[j].MeshVertices[0] == Wedge.WedgeVertex)
				{
					SolveLines.Add(AllInsetLines[j].InsetLines0[0]);
				}
				else if (Edges[j].MeshVertices.Last() == Wedge.WedgeVertex)
				{
					SolveLines.Add(AllInsetLines[j].InsetLines0.Last());
				}
				else if (Edges[j].NewMeshVertices[0] == Wedge.WedgeVertex)
				{
					SolveLines.Add(AllInsetLines[j].InsetLines1[0]);
				}
				else if (Edges[j].NewMeshVertices.Last() == Wedge.WedgeVertex)
				{
					SolveLines.Add(AllInsetLines[j].InsetLines1.Last());
				}
			}

			// todo: BoundaryVertex case never actually gets here because currently we do not initialize wedges of BoundaryVertex!
			bool bIsSimpleBoundary = (Vertex.VertexType == EBevelVertexType::BoundaryVertex && SolveLines.Num() == 1);

			if (Vertex.VertexType == EBevelVertexType::TerminatorVertex || bIsSimpleBoundary)
			{
				// TODO: this ensure has been sporadically hit in Lyra. Needs to be investigated.
				//ensure(SolveLines.Num() == 1);
				if (SolveLines.Num() == 1)
				{
					// This will be on the inset edge-line but possibly pulled away from the face the incoming terminating edge is 'hitting'
					// It's fine on right-angles but you can see the problem on the front edge of a cube shaped like:
					// 
					//         *---*
					//          *-*
					FVector3d InsetLinePosition = SolveLines[0].NearestPoint(CurPos);
					Wedge.NewPosition = InsetLinePosition;

					// What we ought to do is determine which group topology edges each wedge vertex should 'slide along'. 
					// However this is a bit complex to figure out and so for the shorter term we are just going to
					// do a hack by finding the wedge-mesh-edge most aligned w/ the line inset edge.
					// This will obviously fail if there are sliver triangles in the wedge that it can get confused by...

					FVector3d BaseInsetDir = Normalized(InsetLinePosition - CurPos);
					double MaxDot = -1; 
					FLine3d MaxDotEdgeLine;
					Mesh.EnumerateVertexVertices(Wedge.WedgeVertex, [&](int32 othervid)
					{
						FLine3d EdgeLine = FLine3d::FromPoints(CurPos, Mesh.GetVertex(othervid));
						double DirDot = EdgeLine.Direction.Dot(BaseInsetDir);
						if (DirDot > MaxDot )
						{
							MaxDot = DirDot;
							MaxDotEdgeLine = EdgeLine;
						}
					});

					if (MaxDot > -1)
					{
						FDistLine3Line3d LineIntersection(SolveLines[0], MaxDotEdgeLine);
						LineIntersection.Get();
						Wedge.NewPosition = LineIntersection.Line2ClosestPoint;
					}

					Wedge.bHaveNewPosition = true;
				}
			}
			else 
			{
				MESH_BEVEL_DEBUG_CHECK(SolveLines.Num() >= 2);
				if (SolveLines.Num() >= 2)
				{
					Wedge.NewPosition = UE::Geometry::SolveInsetVertexPositionFromLinePair(CurPos, SolveLines[0], SolveLines[1]);
					Wedge.bHaveNewPosition = true;
				}
			}

		}
	}


	auto SetDisplacedPositions = [&GetDisplacedVertexPos](FDynamicMesh3& Mesh, TArray<int32>& VerticesIn, TArray<FVector3d>& PositionsIn, int32 InsetStart, int32 InsetEnd)
	{
		int32 NumVertices = VerticesIn.Num();
		if (PositionsIn.Num() == NumVertices)
		{
			int32 Stop = NumVertices - InsetEnd;
			for (int32 k = InsetStart; k < Stop; ++k)
			{
				Mesh.SetVertex(VerticesIn[k], PositionsIn[k]);
			}
		}
	};


	// now bake in new positions
	for (FBevelEdge& Edge : Edges)
	{
		SetDisplacedPositions(Mesh, Edge.MeshVertices, Edge.NewPositions0, Edge.bEndpointBoundaryFlag[0]?0:1, Edge.bEndpointBoundaryFlag[1]?0:1);
		SetDisplacedPositions(Mesh, Edge.NewMeshVertices, Edge.NewPositions1, Edge.bEndpointBoundaryFlag[0]?0:1, Edge.bEndpointBoundaryFlag[1]?0:1);
	}
	for (FBevelLoop& Loop : Loops)
	{
		SetDisplacedPositions(Mesh, Loop.MeshVertices, Loop.NewPositions0, 0, 0);
		SetDisplacedPositions(Mesh, Loop.NewMeshVertices, Loop.NewPositions1, 0, 0);
	}
	for (FBevelVertex& Vertex : Vertices)
	{
		for (FOneRingWedge& Wedge : Vertex.Wedges)
		{
			if (Wedge.bHaveNewPosition)
			{
				Mesh.SetVertex(Wedge.WedgeVertex, Wedge.NewPosition);
			}
		}
	}
}




void FMeshBevel::AppendJunctionVertexPolygon(FDynamicMesh3& Mesh, FBevelVertex& Vertex)
{
	MESH_BEVEL_DEBUG_CHECK(Vertex.VertexType == EBevelVertexType::JunctionVertex);

	// UnlinkJunctionVertex() split the terminator vertex into N vertices, one for each
	// (now disconnected) triangle-wedge. The wedges are ordered such that their wedge-vertices
	// define a polygon with correct winding, so we can just mesh it and append the triangles

	TArray<FVector3d> PolygonPoints;
	for (FOneRingWedge& Wedge : Vertex.Wedges)
	{
		PolygonPoints.Add(Mesh.GetVertex(Wedge.WedgeVertex));
	}

	TArray<FIndex3i> Triangles;
	PolygonTriangulation::TriangulateSimplePolygon<double>(PolygonPoints, Triangles);
	Vertex.NewGroupID = Mesh.AllocateTriangleGroup();
	for (FIndex3i Tri : Triangles)
	{
		int32 A = Vertex.Wedges[Tri.A].WedgeVertex;
		int32 B = Vertex.Wedges[Tri.B].WedgeVertex;
		int32 C = Vertex.Wedges[Tri.C].WedgeVertex;
		int32 tid = Mesh.AppendTriangle(A, B, C, Vertex.NewGroupID);
		if (Mesh.IsTriangle(tid))
		{
			Vertex.NewTriangles.Add(tid);
		}
	}
}


void FMeshBevel::AppendTerminatorVertexTriangle(FDynamicMesh3& Mesh, FBevelVertex& Vertex)
{
	MESH_BEVEL_DEBUG_CHECK(Vertex.VertexType == EBevelVertexType::TerminatorVertex);

	// UnlinkTerminatorVertex() opened up a triangle-shaped hole adjacent to the incoming edge quad-strip
	// at the terminator vertex. The Wedges of the terminator vertex contain the vertex IDs of the two
	// verts on the quad-strip edge. We need the third vertex. We stored [SplitEdge, FarVertexID] in
	// .TerminatorInfo, however FarVertexID may have become a different vertex when we unlinked other
	// vertices. So, we will try to use SplitEdge to find it.
	// If this turns out to have problems, basically the QuadEdgeID is on the boundary of a 3-edge hole,
	// and so it should be straightforward to find the two other boundary edges and that gives the vertex.
	int32 RingSplitEdgeID = Vertex.TerminatorInfo.A;
	if (Mesh.IsEdge(RingSplitEdgeID))
	{
		FIndex2i SplitEdgeV = Mesh.GetEdgeV(RingSplitEdgeID);
		int32 FarVertexID = SplitEdgeV.OtherElement(Vertex.VertexID);

		int32 QuadEdgeID = Mesh.FindEdge(Vertex.Wedges[0].WedgeVertex, Vertex.Wedges[1].WedgeVertex);
		if (Mesh.IsEdge(QuadEdgeID) && Mesh.IsBoundaryEdge(QuadEdgeID))
		{
			FIndex2i QuadEdgeV = Mesh.GetOrientedBoundaryEdgeV(QuadEdgeID);
			// should have computed this GroupID in initial setup
			int32 UseGroupID = (Vertex.NewGroupID >= 0) ? Vertex.NewGroupID : Mesh.AllocateTriangleGroup();
			int32 tid = Mesh.AppendTriangle(QuadEdgeV.B, QuadEdgeV.A, FarVertexID, UseGroupID);
			if (Mesh.IsTriangle(tid))
			{
				Vertex.NewTriangles.Add(tid);
			}
			else
			{
				MESH_BEVEL_DEBUG_CHECK(false);
			}
		}
		else
		{
			MESH_BEVEL_DEBUG_CHECK(false);
		}
	}
}



void FMeshBevel::AppendTerminatorVertexPairQuad(FDynamicMesh3& Mesh, FBevelVertex& Vertex0, FBevelVertex& Vertex1)
{
	MESH_BEVEL_DEBUG_CHECK(Vertex0.VertexType == EBevelVertexType::TerminatorVertex);
	MESH_BEVEL_DEBUG_CHECK(Vertex1.VertexType == EBevelVertexType::TerminatorVertex);

	// This is a variant of AppendTerminatorVertexTriangle that handles the case where basically two
	// Terminator Vertices are directly connected by a non-beveled mesh edge that was used as the ring-split-edge.
	// Since both sides were opened, we have a quad-shaped hole instead of a triangle-shaped hole, with a quad-edge
	// at each end. The quad can be filled directly, we just need to sort out ordering/etc

	// does not seem like we need to do anything w/ the TerminatorInfo here, we can get everything from wedges
	int32 QuadEdgeID0 = Mesh.FindEdge(Vertex0.Wedges[0].WedgeVertex, Vertex0.Wedges[1].WedgeVertex);
	int32 QuadEdgeID1 = Mesh.FindEdge(Vertex1.Wedges[0].WedgeVertex, Vertex1.Wedges[1].WedgeVertex);
	if (!MESH_BEVEL_DEBUG_ENSURE(
		Mesh.IsEdge(QuadEdgeID0) && Mesh.IsEdge(QuadEdgeID1) &&
		Mesh.IsBoundaryEdge(QuadEdgeID0) && Mesh.IsBoundaryEdge(QuadEdgeID1)))
	{
		// if we failed to find an open boundary on edge, we can't build the connecting quad
		return;
	}
	FIndex2i QuadEdgeV0 = Mesh.GetOrientedBoundaryEdgeV(QuadEdgeID0);
	FIndex2i QuadEdgeV1 = Mesh.GetOrientedBoundaryEdgeV(QuadEdgeID1);

	// make sure that the two opposing/connecting edges exist
	MESH_BEVEL_DEBUG_CHECK(
		Mesh.FindEdge(QuadEdgeV0.A, QuadEdgeV1.B) != IndexConstants::InvalidID &&
		Mesh.FindEdge(QuadEdgeV0.B, QuadEdgeV1.A) != IndexConstants::InvalidID);

	// BIASED? should have computed this GroupID in initial setup
	int32 UseGroupID = (Vertex0.NewGroupID >= 0) ? Vertex0.NewGroupID : Mesh.AllocateTriangleGroup();

	// quad order is V0.B, V0.A, V1.B, V1.A
	int32 tid0 = Mesh.AppendTriangle(QuadEdgeV0.B, QuadEdgeV0.A, QuadEdgeV1.B, UseGroupID);
	MESH_BEVEL_DEBUG_CHECK(tid0 >= 0);
	if (Mesh.IsTriangle(tid0))
	{
		Vertex0.NewTriangles.Add(tid0);
	}
	int32 tid1 = Mesh.AppendTriangle(QuadEdgeV0.B, QuadEdgeV1.B, QuadEdgeV1.A, UseGroupID);
	MESH_BEVEL_DEBUG_CHECK(tid1 >= 0);
	if (Mesh.IsTriangle(tid1))
	{
		Vertex1.NewTriangles.Add(tid1);
	}
}


void FMeshBevel::AppendEdgeQuads(FDynamicMesh3& Mesh, FBevelEdge& Edge)
{
	int32 NumEdges = Edge.MeshEdges.Num();
	if (NumEdges != Edge.NewMeshEdges.Num())
	{
		return;
	}

	Edge.NewGroupID = Mesh.AllocateTriangleGroup();

	// At this point each edge-span should be fully disconnected into a set of paired edges, 
	// so we can trivially join each edge pair with a quad.
	for (int32 k = 0; k < NumEdges; ++k)
	{
		int32 EdgeID0 = Edge.MeshEdges[k];
		int32 EdgeID1 = Edge.NewMeshEdges[k];

		// in certain cases, like bevel topo-edges with a single mesh-edge, we would not
		// have been able to construct the "other" mesh edge when processing the topo-edge
		// (where .NewMeshEdges is computed), it would only have been created when processing the
		// junction vertex. Currently we do not go back and update .NewMeshEdges in that case, but
		// we do store the edge-pair-correspondence in the MeshEdgePairs map. 
		if (EdgeID0 == EdgeID1)
		{
			int32* FoundEdgeID1 = MeshEdgePairs.Find(EdgeID0);
			if (FoundEdgeID1 != nullptr)
			{
				EdgeID1 = *FoundEdgeID1;
			}
		}

		FIndex2i QuadTris(IndexConstants::InvalidID, IndexConstants::InvalidID);
		if (EdgeID0 != EdgeID1 && Mesh.IsEdge(EdgeID1) )
		{
			FIndex2i EdgeV0 = Mesh.GetOrientedBoundaryEdgeV(EdgeID0);
			FIndex2i EdgeV1 = Mesh.GetOrientedBoundaryEdgeV(EdgeID1);
			if (EdgeV0.Contains(EdgeV1.A) || EdgeV0.Contains(EdgeV1.B))
			{
				// If we hit this case, it means that Edge0 and Edge1 are still connected at one end, and
				// so cannot be connected by a Quad, they can only be connected by a single triangle.
				// It is unclear how we end up in this situation, it does occur somewhat regularly in complex
				// geometry scripts though (eg see UE-157531 for a potential repro).
				int32 OtherV = EdgeV0.Contains(EdgeV1.A) ? EdgeV1.B : EdgeV1.A;
				QuadTris.A = Mesh.AppendTriangle(EdgeV0.B, EdgeV0.A, OtherV, Edge.NewGroupID);
				QuadTris.B = IndexConstants::InvalidID;
			}
			else
			{
				QuadTris.A = Mesh.AppendTriangle(EdgeV0.B, EdgeV0.A, EdgeV1.B, Edge.NewGroupID);
				QuadTris.B = Mesh.AppendTriangle(EdgeV1.B, EdgeV1.A, EdgeV0.B, Edge.NewGroupID);
			}
		}
		Edge.StripQuads.Add(QuadTris);
	}
}



void FMeshBevel::AppendLoopQuads(FDynamicMesh3& Mesh, FBevelLoop& Loop)
{
	int32 NumEdges = Loop.MeshEdges.Num();
	if (NumEdges != Loop.NewMeshEdges.Num())
	{
		return;
	}

	auto GetGroupKey = [&Mesh, &Loop](int32 k)
	{
		FIndex2i EdgeTris = Loop.MeshEdgeTris[k];
		int32 Group0 = Mesh.GetTriangleGroup(EdgeTris.A);
		int32 Group1 = Mesh.IsTriangle(EdgeTris.B) ? Mesh.GetTriangleGroup(EdgeTris.B) : -1;
		return FIndex2i(FMath::Max(Group0, Group1), FMath::Min(Group0, Group1));
	};

	TMap<FIndex2i, int> NewGroupIDs;
	for (int32 k = 0; k < NumEdges; ++k)
	{
		FIndex2i GroupKey = GetGroupKey(k);
		if (NewGroupIDs.Contains(GroupKey) == false)
		{
			NewGroupIDs.Add(GroupKey, Mesh.AllocateTriangleGroup());
			Loop.NewGroupIDs.Add(NewGroupIDs[GroupKey]);
		}
	}

	// At this point each edge-span should be fully disconnected into a set of paired edges, 
	// so we can trivially join each edge pair with a quad.
	for (int32 k = 0; k < NumEdges; ++k)
	{
		int32 EdgeID0 = Loop.MeshEdges[k];
		int32 EdgeID1 = Loop.NewMeshEdges[k];

		// case that happens in AppendEdgeQuads() should never happen for loops...

		FIndex2i QuadTris(IndexConstants::InvalidID, IndexConstants::InvalidID);
		if (EdgeID0 != EdgeID1 && Mesh.IsEdge(EdgeID1))
		{
			FIndex2i GroupKey = GetGroupKey(k);
			int32 NewGroupID = NewGroupIDs[GroupKey];

			FIndex2i EdgeV0 = Mesh.GetOrientedBoundaryEdgeV(EdgeID0);
			FIndex2i EdgeV1 = Mesh.GetOrientedBoundaryEdgeV(EdgeID1);
			QuadTris.A = Mesh.AppendTriangle(EdgeV0.B, EdgeV0.A, EdgeV1.B, NewGroupID);
			if (MESH_BEVEL_DEBUG_ENSURE(!EdgeV1.Contains(EdgeV0.B)))
			{
				QuadTris.B = Mesh.AppendTriangle(EdgeV1.B, EdgeV1.A, EdgeV0.B, NewGroupID);
			}
		}
		Loop.StripQuads.Add(QuadTris);
	}
}




void FMeshBevel::CreateBevelMeshing(FDynamicMesh3& Mesh)
{
	for (FBevelVertex& Vertex : Vertices)
	{
		if (Vertex.VertexType == EBevelVertexType::JunctionVertex)
		{
			if (Vertex.Wedges.Num() > 2)
			{
				AppendJunctionVertexPolygon(Mesh, Vertex);
			}
		}
	}

	for (FBevelEdge& Edge : Edges)
	{
		AppendEdgeQuads(Mesh, Edge);
	}
	for (FBevelLoop& Loop : Loops)
	{
		AppendLoopQuads(Mesh, Loop);
	}

	// easier to do terminators last so that we can use quad edge to orient the triangle
	TSet<FIndex2i> HandledQuadVtxPairs;
	for (FBevelVertex& Vertex : Vertices)
	{
		if (Vertex.VertexType == EBevelVertexType::TerminatorVertex)
		{
			if (Vertex.ConnectedBevelVertex >= 0)
			{
				FBevelVertex& OtherVertex = Vertices[Vertex.ConnectedBevelVertex];
				FIndex2i VtxPair(Vertex.VertexID, OtherVertex.VertexID);
				VtxPair.Sort();
				if (HandledQuadVtxPairs.Contains(VtxPair) == false)
				{
					AppendTerminatorVertexPairQuad(Mesh, Vertex, OtherVertex);
					HandledQuadVtxPairs.Add(VtxPair);
				}
			}
			else
			{
				AppendTerminatorVertexTriangle(Mesh, Vertex);
			}
		}
	}
}













void FMeshBevel::AppendEdgeQuads_Multi(FDynamicMesh3& Mesh, FBevelEdge& Edge)
{
	int32 NumEdges = Edge.MeshEdges.Num();
	if (NumEdges != Edge.NewMeshEdges.Num())
	{
		return;
	}

	Edge.NewGroupID = Mesh.AllocateTriangleGroup();

	struct FEdgePair
	{
		FIndex2i EdgeV0;
		FIndex2i EdgeV1;
	};
	TArray<FEdgePair> SequentialQuadEdges;

	bool bFoundInvalidCase = false;

	// At this point each edge-span should be fully disconnected into a set of paired edges, 
	// so we can trivially join each edge pair with a quad. (In the multi-case we will further
	// subdivide this quad instead of adding it directly)
	for (int32 k = 0; k < NumEdges && bFoundInvalidCase == false; ++k)
	{
		int32 EdgeID0 = Edge.MeshEdges[k];
		int32 EdgeID1 = Edge.NewMeshEdges[k];

		// in certain cases, like bevel topo-edges with a single mesh-edge, we would not
		// have been able to construct the "other" mesh edge when processing the topo-edge
		// (where .NewMeshEdges is computed), it would only have been created when processing the
		// junction vertex. Currently we do not go back and update .NewMeshEdges in that case, but
		// we do store the edge-pair-correspondence in the MeshEdgePairs map. 
		if (EdgeID0 == EdgeID1)
		{
			int32* FoundEdgeID1 = MeshEdgePairs.Find(EdgeID0);
			if (FoundEdgeID1 != nullptr)
			{
				EdgeID1 = *FoundEdgeID1;
			}
		}

		if (EdgeID0 != EdgeID1 && Mesh.IsEdge(EdgeID1))
		{
			FIndex2i EdgeV0 = Mesh.GetOrientedBoundaryEdgeV(EdgeID0);
			FIndex2i EdgeV1 = Mesh.GetOrientedBoundaryEdgeV(EdgeID1);
			if (EdgeV0.Contains(EdgeV1.A) || EdgeV0.Contains(EdgeV1.B))
			{
				// If we hit this case, it means that Edge0 and Edge1 are still connected at one end, and
				// so cannot be connected by a Quad, they can only be connected by a single triangle.
				// It is unclear how we end up in this situation, it does occur somewhat regularly in complex
				// geometry scripts though (eg see UE-157531 for a potential repro).
				// In this case we cannot add subdivisions currently, and will fall back to simpler geometry
				bFoundInvalidCase = true;
			}
			else
			{
				SequentialQuadEdges.Add({ EdgeV0, EdgeV1 });
			}
		}
		else
		{
			bFoundInvalidCase = true;
		}
	}


	// if something went wrong above, fall back to adding a single quad to avoid holes/etc
	// (it's possible that this path may still result in catastrophic failure...)
	if (bFoundInvalidCase || SequentialQuadEdges.Num() != NumEdges)
	{
		AppendEdgeQuads(Mesh, Edge);
		return;
	}

	// all the code below is going to generate the bevel edge geometry and populate this list of 
	// vertex-rows, which will be used to assemble the final FQuadGridPatch for the bevel edge-strip
	int32 N = NumSubdivisions;
	TArray<TArray<int32>> VertexSpans;
	VertexSpans.SetNum(N + 2);

	// this function appends a new column to the rows in VertexSpans, by generating new
	// vertices on the interior of the edge between StartVID and EndVID
	auto AppendNewVertColumn = [&Mesh, &VertexSpans, N](int32 StartVID, int32 EndVID)
	{
		FVector3d StartPos = Mesh.GetVertex(StartVID);
		FVector3d EndPos = Mesh.GetVertex(EndVID);
		VertexSpans[0].Add(StartVID);
		for (int32 j = 0; j < N; ++j)
		{
			double T = (double)(j+1) / (double)(N + 1);
			int32 NewVertID = Mesh.AppendVertex(Lerp(StartPos, EndPos, T));
			VertexSpans[j+1].Add(NewVertID);
		}
		VertexSpans[N+1].Add(EndVID);
	};

	// this function appends a new column to the rows in VertexSpans by copying
	// the column from an existing adjacent FQuadGridPatch. This happens if we are 
	// meshing a bevel-edge that is connected to another bevel-edge that was already
	// generated, and the connecting bevel-vertex only has 2 bevel-edges (ie no polygon will be inserted)
	auto AppendExistingVertColumn = [&VertexSpans, N](const FQuadGridPatch* AdjacentQuadPatch, int32 CornerVertexID) -> bool
	{
		int32 ColumnIdx = AdjacentQuadPatch->FindColumnIndex(CornerVertexID);
		if (ColumnIdx >= 0)
		{
			TArray<int32> ColumnVerts;
			AdjacentQuadPatch->GetVertexColumn(ColumnIdx, ColumnVerts);

			// Our column should start w/ CornerVertexID, but it may be reversed due to different orientations when building the adjacent patch...
			// (could that be fixed further upstream? possibly it should be!)
			if (ColumnVerts.Last() == CornerVertexID)
			{
				Algo::Reverse(ColumnVerts);
			}

			for (int32 j = 0; j <= (N + 1); ++j)
			{
				VertexSpans[j].Add(ColumnVerts[j]);
			}

			return true;
		}
		return false;
	};

	// start and end vertex columns may already exist in some other FBevelEdge that has already been generated. 
	// In that case, instead of appending new vertices, we want to look up the existing vertices and stitch to them. 
	// This case only happens for BevelVertices connected to exactly 2 BevelEdges (<2 => Terminator, >2 => Polygon at vertex). 
	auto GetConnectedQuadStripRef = [this](int BevelVertexIdx, int CurBevelEdgeIdx)
	{
		if (BevelVertexIdx == -1) return (const FQuadGridPatch*)nullptr;
		const FBevelVertex& Vtx = Vertices[BevelVertexIdx];
		if (Vtx.VertexType != EBevelVertexType::JunctionVertex || Vtx.Wedges.Num() != 2) return (const FQuadGridPatch*)nullptr;

		for (int EdgeIdx : Vtx.IncomingBevelEdgeIndices)
		{
			if (EdgeIdx != CurBevelEdgeIdx)
			{
				const FBevelEdge& OtherEdge = Edges[EdgeIdx];
				if (OtherEdge.StripQuadPatch.IsEmpty() == false)
				{
					return (const FQuadGridPatch*)&OtherEdge.StripQuadPatch;
				}
				return (const FQuadGridPatch*)nullptr;
			}
		}
		return (const FQuadGridPatch*)nullptr;
	};

	// Below code expects SequentialQuadEdges to be ordered s.t. for edges spanning 
	// vertices 0,1,2 the edges should be (1,0),(2,1) not (0,1),(1,2) ...
	// We enforce this by reversing the array if needed
	if (SequentialQuadEdges.Num() > 1 && SequentialQuadEdges[0].EdgeV0.B == SequentialQuadEdges[1].EdgeV0.A)
	{
		Algo::Reverse(SequentialQuadEdges);
	}

	// SequentialQuadEdges ordering may not be in agreement with the [BevelVertices.A, BevelVertices.B] 
	// ordering of the BevelEdge. If so, then the Prev/Next QuadGridPatch
	// search below would return the Prev & Next flipped from where we need them, causing the later AppendExistingVertColumn
	// to fail. The simplest fix for this here is to swap PrevQuadPatch/NextQuadPatch...
	int OriginalPrevVtxID = (Edge.BevelVertices.A >= 0) ? Vertices[Edge.BevelVertices.A].VertexID : -1;
	int OriginalNextVtxID = (Edge.BevelVertices.B >= 0) ? Vertices[Edge.BevelVertices.B].VertexID : -1;
	FIndex2i EdgeStartQuadVerts(SequentialQuadEdges[0].EdgeV0.B, SequentialQuadEdges[0].EdgeV1.A);
	FIndex2i EdgeEndQuadVerts(SequentialQuadEdges[NumEdges-1].EdgeV0.A, SequentialQuadEdges[NumEdges-1].EdgeV1.B);
	FIndex2i BevelEdgeVerts(Edge.BevelVertices.A, Edge.BevelVertices.B);
	if (EdgeStartQuadVerts.Contains(OriginalNextVtxID) || EdgeEndQuadVerts.Contains(OriginalPrevVtxID))
	{
		Swap(BevelEdgeVerts.A, BevelEdgeVerts.B);
	}

	// find quadgrid patches connected to start and end of current bevel edge, if the yexist
	const FQuadGridPatch* PrevQuadPatch = GetConnectedQuadStripRef(BevelEdgeVerts.A, Edge.EdgeIndex);
	const FQuadGridPatch* NextQuadPatch = GetConnectedQuadStripRef(BevelEdgeVerts.B, Edge.EdgeIndex);

	// now add columns of vertices for each initial vertex along the bevel-edge (by iterating along mesh-edges) 
	for (int32 k = 0; k < NumEdges; ++k)
	{
		FIndex2i EdgeV0 = SequentialQuadEdges[k].EdgeV0;
		FIndex2i EdgeV1 = SequentialQuadEdges[k].EdgeV1;

		int32 QuadA = EdgeV0.B;
		int32 QuadB = EdgeV0.A;
		int32 QuadC = EdgeV1.B;
		int32 QuadD = EdgeV1.A;

		// for first and last edges, we have to add an extra row of vertices
		if (k == 0)
		{
			if (PrevQuadPatch == nullptr || AppendExistingVertColumn(PrevQuadPatch, QuadA) == false)
			{
				AppendNewVertColumn(QuadA, QuadD);
			}
		}

		if (k != NumEdges-1 || NextQuadPatch == nullptr || AppendExistingVertColumn(NextQuadPatch, QuadB) == false)
		{
			AppendNewVertColumn(QuadB, QuadC);
		}
	
	}

	// now append the quads that connect up the vertices
	TArray<TArray<FIndex2i>> QuadSpans;
	QuadSpans.SetNum(N + 1);

	bool bSuccess = true;
	int32 NumStrips = VertexSpans.Num() - 1;
	for (int32 k = 0; k < NumEdges; ++k)
	{
		for ( int32 j = 0; j < NumStrips; ++j)
		{
			FIndex2i QuadTris(IndexConstants::InvalidID, IndexConstants::InvalidID);
			QuadTris.A = Mesh.AppendTriangle(VertexSpans[j][k], VertexSpans[j][k+1], VertexSpans[j+1][k], Edge.NewGroupID);
			QuadTris.B = Mesh.AppendTriangle(VertexSpans[j+1][k+1], VertexSpans[j+1][k], VertexSpans[j][k+1], Edge.NewGroupID);
			bSuccess &= (QuadTris.A >= 0 && QuadTris.B >= 0);
			QuadSpans[j].Add(QuadTris);

			Edge.StripQuads.Add(QuadTris);
		}
	}

	// finally save the quad-patch we generated in the FBevelEdge
	if (MESH_BEVEL_DEBUG_ENSURE(bSuccess))
	{
		Edge.StripQuadPatch.InitializeFromQuadPatch(Mesh, QuadSpans, VertexSpans);
	}
}



void FMeshBevel::AppendLoopQuads_Multi(FDynamicMesh3& Mesh, FBevelLoop& Loop)
{
	// As in other places, the Loop case is a simplified version of the Edge case, where
	// many special cases do not have to be handled. See the comments in 
	// AppendEdgeQuads_Multi() for anything non-obvious below

	int32 NumEdges = Loop.MeshEdges.Num();
	if (NumEdges != Loop.NewMeshEdges.Num())
	{
		return;
	}

	auto GetGroupKey = [&Mesh, &Loop](int32 k)
	{
		FIndex2i EdgeTris = Loop.MeshEdgeTris[k];
		int32 Group0 = Mesh.GetTriangleGroup(EdgeTris.A);
		int32 Group1 = Mesh.IsTriangle(EdgeTris.B) ? Mesh.GetTriangleGroup(EdgeTris.B) : -1;
		return FIndex2i(FMath::Max(Group0, Group1), FMath::Min(Group0, Group1));
	};

	TMap<FIndex2i, int> NewGroupIDs;
	for (int32 k = 0; k < NumEdges; ++k)
	{
		FIndex2i GroupKey = GetGroupKey(k);
		if (NewGroupIDs.Contains(GroupKey) == false)
		{
			NewGroupIDs.Add(GroupKey, Mesh.AllocateTriangleGroup());
			Loop.NewGroupIDs.Add(NewGroupIDs[GroupKey]);
		}
	}

	struct FEdgePair
	{
		FIndex2i EdgeV0;
		FIndex2i EdgeV1;
	};
	TArray<FEdgePair> SequentialQuadEdges;

	bool bFoundInvalidCase = false;

	// At this point each edge-span should be fully disconnected into a set of paired edges, 
	// so we can trivially join each edge pair with a quad.
	for (int32 k = 0; k < NumEdges && bFoundInvalidCase == false; ++k)
	{
		int32 EdgeID0 = Loop.MeshEdges[k];
		int32 EdgeID1 = Loop.NewMeshEdges[k];

		// case that happens in AppendEdgeQuads() should never happen for loops...

		FIndex2i QuadTris(IndexConstants::InvalidID, IndexConstants::InvalidID);
		if (EdgeID0 != EdgeID1 && Mesh.IsEdge(EdgeID1))
		{
			FIndex2i GroupKey = GetGroupKey(k);
			int32 NewGroupID = NewGroupIDs[GroupKey];

			FIndex2i EdgeV0 = Mesh.GetOrientedBoundaryEdgeV(EdgeID0);
			FIndex2i EdgeV1 = Mesh.GetOrientedBoundaryEdgeV(EdgeID1);
			SequentialQuadEdges.Add({ EdgeV0, EdgeV1 });
		}
		else
		{
			bFoundInvalidCase = true;
		}
	}

	if (bFoundInvalidCase || SequentialQuadEdges.Num() != NumEdges)
	{
		AppendLoopQuads(Mesh, Loop);
		return;
	}

	// Test the edge ordering -- i.e. for vertices 0,1,2, if bEdgeHasReverseOrder == true, 
	// edge vertices will be (0,1),(1,2); otherwise it will be (1,0),(2,1)
	bool bEdgeHasReverseOrder = (SequentialQuadEdges.Num() > 1 && SequentialQuadEdges[0].EdgeV0.B == SequentialQuadEdges[1].EdgeV0.A);

	int32 N = NumSubdivisions;
	TArray<TArray<int32>> VertexSpans;
	VertexSpans.SetNum(N + 2);

	auto AppendVertColumn = [&Mesh, &VertexSpans, &N](int32 StartVID, int32 EndVID)
	{
		FVector3d StartPos = Mesh.GetVertex(StartVID);
		FVector3d EndPos = Mesh.GetVertex(EndVID);
		VertexSpans[0].Add(StartVID);
		for (int32 j = 0; j < N; ++j)
		{
			double T = (double)(j + 1) / (double)(N + 1);
			int32 NewVertID = Mesh.AppendVertex(Lerp(StartPos, EndPos, T));
			VertexSpans[j + 1].Add(NewVertID);
		}
		VertexSpans[N + 1].Add(EndVID);
	};

	for (int32 k = 0; k < NumEdges; ++k)
	{
		FIndex2i EdgeV0 = SequentialQuadEdges[k].EdgeV0;
		FIndex2i EdgeV1 = SequentialQuadEdges[k].EdgeV1;

		if (bEdgeHasReverseOrder)
		{
			EdgeV0.Swap();
			EdgeV1.Swap();
		}

		int32 QuadA = EdgeV0.B;
		int32 QuadB = EdgeV0.A;
		int32 QuadC = EdgeV1.B;
		int32 QuadD = EdgeV1.A;

		if (k == 0)
		{
			AppendVertColumn(QuadA, QuadD);
		}

		if (k != NumEdges - 1)
		{
			AppendVertColumn(QuadB, QuadC);
		}
		else
		{
			for (int32 j = 0; j <= (N+1); ++j)
			{
				int FirstColVal = VertexSpans[j][0];
				VertexSpans[j].Add(FirstColVal);
			}
		}

	}

	TArray<TArray<FIndex2i>> QuadSpans;
	QuadSpans.SetNum(N + 1);

	// Offsets for flipping triangles in cases where the edges had the opposite ordering from our expectation
	int32 SwapOffset0 = (int32)(bEdgeHasReverseOrder);
	int32 SwapOffset1 = (int32)(!bEdgeHasReverseOrder);

	bool bSuccess = true;
	int32 NumStrips = VertexSpans.Num() - 1;
	for (int32 k = 0; k < NumEdges; ++k)
	{
		FIndex2i GroupKey = GetGroupKey(k);
		int32 NewGroupID = NewGroupIDs[GroupKey];

		for (int32 j = 0; j < NumStrips; ++j)
		{
			FIndex2i QuadTris(IndexConstants::InvalidID, IndexConstants::InvalidID);
			QuadTris.A = Mesh.AppendTriangle(VertexSpans[j][k + SwapOffset0], VertexSpans[j][k + SwapOffset1], VertexSpans[j+1][k], NewGroupID);
			QuadTris.B = Mesh.AppendTriangle(VertexSpans[j+1][k + SwapOffset1], VertexSpans[j+1][k + SwapOffset0], VertexSpans[j][k+1], NewGroupID);
			bSuccess &= QuadTris.A >= 0 && QuadTris.B >= 0;
			QuadSpans[j].Add(QuadTris);

			Loop.StripQuads.Add(QuadTris);
		}
	}

	if (MESH_BEVEL_DEBUG_ENSURE(bSuccess))
	{
		Loop.StripQuadPatch.InitializeFromQuadPatch(Mesh, QuadSpans, VertexSpans);
	}
}




void FMeshBevel::AppendJunctionVertexPolygon_Multi(FDynamicMesh3& Mesh, FBevelVertex& Vertex)
{
	// This function appends triangulations for junction vertices, ie where 3+ bevel-edges meet and
	// so a polygonal face must be inserted. For a multi-segment bevel, we need to tessellate the
	// polygon with interior vertices so that the interior vertices can be displaced later based
	// on the bevel profile shape. And this is also where we will precompute certain information about
	// the interior vertices, like (eg) barycentric coords of interior vertices, so that we can later
	// deform them relative to the profile shapes on the boundary. 
	//
	// This is probably the most complex part of doing the bevel, and there are special cases for 
	// valence-3 and valence-4, with lots of similar-but-slightly-different code.

	MESH_BEVEL_DEBUG_CHECK(Vertex.VertexType == EBevelVertexType::JunctionVertex);

	// TODO: all the setup here could be done in parallel, which might matter given that
	// we are doing a tessellation, conformal unwrap, etc. Would need to cache temporary data.
	// Junction vertices can never be directly connected so there should not be any order effects.

	// UnlinkJunctionVertex() split the junction vertex into N vertices, one for each
	// (now disconnected) triangle-wedge. The wedges are ordered such that their wedge-vertices
	// define a polygon with correct winding. However in this case we have additional interior
	// vertices along the polygon edges. So we need to construct a mesh for the junction vertex
	// that has a compatible boundary and (in most cases) additional interior vertices, which
	// is done in this function via a regular tessellation of the polygon

	int32 NumEdgeVerts = 0;		// this should be determined by NumSubdivisions but we will compute from actual mesh here
	bool bAllEdgesHaveSameNumVerts = true;	// current code only handles the case where all edges have same # of subdivisions.

	// First step is to find the loop of vertices around the border of the junction vertex.
	// Each incoming source-mesh edge became a "wedge", with start vertex A and end B, and
	// then a quad-patch was appended connecting A to B. So we need to get the right "column"
	// of the quad patch, which will give us the vertex span [A, ..., B]. These are accumulated
	// in-wedge-sequence into PolygonPoints/Vertices. The start/end vertices A and B are accumulated 
	// in-order in the PolygonCorners/Indices/VIDs lists. 
	// (In both sets, the 'B' vertex is always skipped because it will be added as A in the next span)
	TArray<FVector3d> PolygonPoints, PolygonCorners;
	TArray<int32> PolygonVertices, PolygonCornerVIDs;
	int32 NumWedges = Vertex.Wedges.Num();
	for (int32 wi = 0; wi < NumWedges; ++wi)
	{
		FOneRingWedge& CurWedge = Vertex.Wedges[wi];
		FOneRingWedge& NextWedge = Vertex.Wedges[(wi+1)%NumWedges];

		int32 A = CurWedge.WedgeVertex;
		int32 B = NextWedge.WedgeVertex;

		// We don't know which Edge will contain the [A,B] vertices, and whether it is the 'start'
		// or 'end' of the Edge. So currently just linear searching through all of them and checking
		// each end. Maybe Vertex.IncomingBevelEdgeIndices would work here instead
		TArray<int32> BevelEdgeQuadStripEndVertices;
		bool bFound = false;
		for (FBevelEdge& Span : Edges)
		{
			// this helper appends the vertex-column of an adjacent edge-quadstrip to the current vertex polygon loop
			auto AppendStrip = [&](TArray<int32>& OrderedVertices)
			{
				PolygonCorners.Add(Mesh.GetVertex(OrderedVertices[0]));
				PolygonCornerVIDs.Add(OrderedVertices[0]);
				for (int32 j = 0; j < OrderedVertices.Num()-1; ++j)
				{
					int32 VertexID = OrderedVertices[j];
					PolygonVertices.Add(VertexID);
					PolygonPoints.Add(Mesh.GetVertex(VertexID));
				}
				if (NumEdgeVerts == 0)
				{
					NumEdgeVerts = OrderedVertices.Num();
				}
				else if (OrderedVertices.Num() != NumEdgeVerts)
				{
					bAllEdgesHaveSameNumVerts = false;
				}
				bFound = true;
			};

			if (Span.StripQuadPatch.VertexSpans.IsEmpty())
			{
				// The no-vertex-span case can happen for cases where the smooth bevel has failed and fallen back to non-smooth bevel
				continue;
			}
			
			// try start and end columns, and handle case where vertex ordering might be reversed  (should this be possible, because of consistent mesh winding??)
			Span.StripQuadPatch.GetVertexColumn(0, BevelEdgeQuadStripEndVertices);
			if (BevelEdgeQuadStripEndVertices[0] == A && BevelEdgeQuadStripEndVertices.Last() == B)
			{
				AppendStrip(BevelEdgeQuadStripEndVertices);
				break;
			}
			if (BevelEdgeQuadStripEndVertices[0] == B && BevelEdgeQuadStripEndVertices.Last() == A)
			{
				Algo::Reverse(BevelEdgeQuadStripEndVertices);
				AppendStrip(BevelEdgeQuadStripEndVertices);
				break;
			}

			Span.StripQuadPatch.GetVertexColumn(Span.StripQuadPatch.NumVertexCols() - 1, BevelEdgeQuadStripEndVertices);
			if (BevelEdgeQuadStripEndVertices[0] == A && BevelEdgeQuadStripEndVertices.Last() == B)
			{
				AppendStrip(BevelEdgeQuadStripEndVertices);
				break;
			}
			if (BevelEdgeQuadStripEndVertices[0] == B && BevelEdgeQuadStripEndVertices.Last() == A)
			{
				Algo::Reverse(BevelEdgeQuadStripEndVertices);
				AppendStrip(BevelEdgeQuadStripEndVertices);
				break;
			}
		}
		if (!bFound)
		{
			PolygonVertices.Add(CurWedge.WedgeVertex);
			PolygonPoints.Add(Mesh.GetVertex(CurWedge.WedgeVertex));
			bAllEdgesHaveSameNumVerts = false;
		}
	}

	// code below won't work unless each edge has same vert count, in that case just triangulate the polygon (ie fail)
	if (bAllEdgesHaveSameNumVerts == false || NumEdgeVerts == 0)
	{
		TArray<FIndex3i> Triangles;
		PolygonTriangulation::TriangulateSimplePolygon<double>(PolygonPoints, Triangles);
		Vertex.NewGroupID = Mesh.AllocateTriangleGroup();
		for (FIndex3i Tri : Triangles)
		{
			int32 tid = Mesh.AppendTriangle(PolygonVertices[Tri.A], PolygonVertices[Tri.B], PolygonVertices[Tri.C], Vertex.NewGroupID);
			if (Mesh.IsTriangle(tid))
			{
				Vertex.NewTriangles.Add(tid);
			}
		}
		return;
	}

	// find centroid of polygon
	FVector3d Centroid = FVector3d::Zero();
	for (FVector3d Pos : PolygonPoints)
	{
		Centroid += Pos;
	}
	Centroid /= (double)PolygonPoints.Num();


	// Now we have enough information to tessellate and fill the hole. 
	// Apologies for the convoluted code here. The general case was implemented
	// first, and then the 3/4 special-cases were added. Probably they should
	// be refactored out into 3 separate functions. Sorry! -rms

	// Append corners of the polygon into a temporary mesh, and then triangles
	// For the case of > 4 vertices, add the centroid and triangulate as a fan
	FDynamicMesh3 TmpMesh;
	int32 NCorners = PolygonCorners.Num();
	for (int32 k = 0; k < NCorners; ++k)
	{
		int NewVID = TmpMesh.AppendVertex(PolygonCorners[k]);
		ensure(NewVID == k);
	}
	if (NCorners == 3)	// single triangle
	{
		TmpMesh.AppendTriangle(0, 1, 2);
	}
	else if (NCorners == 4)	// quad
	{
		TmpMesh.AppendTriangle(0, 1, 2);
		TmpMesh.AppendTriangle(2, 3, 0);
	}
	else  // make triangle fan
	{
		int32 CentroidID = TmpMesh.AppendVertex(Centroid);
		for (int32 i = 0; i < NCorners; ++i)
		{
			int NewTID = TmpMesh.AppendTriangle(i, ((i + 1) % NCorners), CentroidID);
		}
	}

	// Ok, for the case of a triangle, we will use barycentric coordinates of the initial
	// triangle rather than trying to do general polygon barycentrics. To avoid writing custom
	// triangle-tess code here, FUniformTessellate is still used, but it doesn't return the barycentrics.
	// So instead we store the 3 vertex-coordinate-functions in vertex colors and allow them to be lerp'd
	if (NCorners == 3)
	{
		// note that this construction of TriTess will be the same for every valence-3 corner, and
		// probably should be cached unless NumEdgeVerts changes... 
		FDynamicMesh3 TriTess;
		TriTess = TmpMesh;
		TriTess.SetVertex(0, FVector3d(1, 0, 0));
		TriTess.SetVertex(1, FVector3d(0, 1, 0));
		TriTess.SetVertex(2, FVector3d(0.5, 0.75, 0));
		TriTess.EnableVertexColors(FVector3f::Zero());
		TriTess.SetVertexColor(0, FVector3f(1, 0, 0));
		TriTess.SetVertexColor(1, FVector3f(0, 1, 0));
		TriTess.SetVertexColor(2, FVector3f(0, 0, 1));

		FUniformTessellate Tesselator(&TriTess);
		Tesselator.TessellationNum = NumEdgeVerts - 2;
		bool bTesselateOK = Tesselator.Compute();
		checkSlow(bTesselateOK);

		// need mapping between the boundary of this tessellated mesh and the triangle-shaped polygon
		// surrounding the hole in the original mesh. Derive this from the known correspondence
		// between the first vertex in each
		FMeshBoundaryLoops BoundaryLoops(&TriTess, true);
		TArray<int32> TriLoopVerts = BoundaryLoops.Loops[0].Vertices;
		int32 LoopN = TriLoopVerts.Num();
		checkSlow(PolygonVertices[0] == PolygonCornerVIDs[0]);
		checkSlow(PolygonVertices.Num() == LoopN);
		int32 StartIndex = TriLoopVerts.IndexOfByKey(0);		// index of first vertex in loop

		TArray<int32> TriTessVertIDToPolygonIndexMap;
		TriTessVertIDToPolygonIndexMap.SetNum(TriTess.MaxVertexID());
		for (int32 k = 0; k < LoopN; ++k)
		{
			int32 LoopVertID = TriLoopVerts[(k + StartIndex) % LoopN];
			TriTessVertIDToPolygonIndexMap[LoopVertID] = k;
		}

		FVector3d PosA = TmpMesh.GetVertex(0);
		FVector3d PosB = TmpMesh.GetVertex(1);
		FVector3d PosC = TmpMesh.GetVertex(2);

		// for this special case we only store the 3 corners in the InteriorBorderLoop, this will be detected in
		// the profile code and used correctly there
		Vertex.InteriorVertices.Reserve(FMath::Max((NumEdgeVerts-1)*(NumEdgeVerts-1), 0));		// this is for quad so it's more than needed...
		Vertex.InteriorBorderLoop = TArray<int32>({ PolygonCornerVIDs[0], PolygonCornerVIDs[1], PolygonCornerVIDs[2] });

		TArray<int32> VertexMap;
		VertexMap.SetNum(TriTess.MaxVertexID());

		// Append the new interior vertices, boundary vertices already exist and are just updated in the map.
		// For interior vertices the barycentric coords are stored in the BorderFrameWeights
		for (int32 vid : TriTess.VertexIndicesItr())
		{
			if (TriTess.IsBoundaryVertex(vid))
			{
				int32 LoopIndex = TriTessVertIDToPolygonIndexMap[vid];
				int32 ExistingVertID = PolygonVertices[LoopIndex];
				VertexMap[vid] = ExistingVertID;
			}
			else
			{
				FVector3f BaryCoords = TriTess.GetVertexColor(vid);
				BaryCoords /= (BaryCoords.X + BaryCoords.Y + BaryCoords.Z);
				FVector3d InterpPos = BaryCoords.X * PosA + BaryCoords.Y * PosB + BaryCoords.Z * PosC;
				int32 NewVID = Mesh.AppendVertex(InterpPos);

				FBevelVertex_InteriorVertex& InteriorVertex = Vertex.InteriorVertices.Emplace_GetRef();
				InteriorVertex.VertexID = NewVID;
				InteriorVertex.BorderFrameWeight.Add(FVector3d(BaryCoords.X, BaryCoords.Y, BaryCoords.Z));
				VertexMap[vid] = NewVID;
			}
		}

		// finally append the triangles
		TArray<FIndex3i> Triangles;
		Vertex.NewGroupID = Mesh.AllocateTriangleGroup();
		for (FIndex3i Triangle : TriTess.TrianglesItr())
		{
			// NOTE FLIP HERE! This is because outer loop is oriented for outside tris, so we need to flip
			int T0 = Mesh.AppendTriangle(VertexMap[Triangle.A], VertexMap[Triangle.C], VertexMap[Triangle.B], Vertex.NewGroupID);
			if (Mesh.IsTriangle(T0))
			{
				Vertex.NewTriangles.Add(T0);
			}
		}

		// we are done the valnce=3 case, exit this function
		return;
	}

	// For a quad, the tessellation is simple and is just done directly here
	if (NCorners == 4)
	{
		FDenseGrid2i VertexGrid;
		VertexGrid.Resize(NumEdgeVerts, NumEdgeVerts);
		VertexGrid.Assign(-1);

		// transfer vertex polygon into border of grid
		int32 PolygonIdx = 0;
		for (int32 xi = 0; xi < NumEdgeVerts; ++xi)
		{
			VertexGrid.At(xi,0) = PolygonVertices[PolygonIdx++];
		}
		for ( int32 yi = 1; yi < NumEdgeVerts-1; ++yi )
		{
			VertexGrid.At(NumEdgeVerts-1, yi) = PolygonVertices[PolygonIdx++];
		}
		for (int32 xi = NumEdgeVerts-1; xi >= 0; --xi)
		{
			VertexGrid.At(xi, NumEdgeVerts-1) = PolygonVertices[PolygonIdx++];
		}
		for (int32 yi = NumEdgeVerts - 2; yi >= 1; --yi)
		{
			VertexGrid.At(0, yi) = PolygonVertices[PolygonIdx++];
		}
		ensure(PolygonIdx == PolygonVertices.Num());

		int c00 = VertexGrid.At(0, 0);
		int c10 = VertexGrid.At(NumEdgeVerts-1, 0);
		int c01 = VertexGrid.At(0, NumEdgeVerts-1);
		int c11 = VertexGrid.At(NumEdgeVerts-1, NumEdgeVerts-1);
		FVector3d V00 = Mesh.GetVertex(c00);
		FVector3d V10 = Mesh.GetVertex(c10);
		FVector3d V01 = Mesh.GetVertex(c01);
		FVector3d V11 = Mesh.GetVertex(c11);

		// for this special case we only store the 4 corners in the InteriorBorderLoop, this will be detected in
		// the profile code and used correctly there
		Vertex.InteriorVertices.Reserve(FMath::Max((NumEdgeVerts - 1)* (NumEdgeVerts - 1), 0));
		Vertex.InteriorBorderLoop = TArray<int32>({ c00, c10, c01, c11 });

		// append new vertices for the interior, while storing their U/V coords in the BorderFrameWeights
		for (int yi = 1; yi < NumEdgeVerts-1; ++yi)
		{
			double ty = (double)yi / (double)(NumEdgeVerts - 1);
			FVector3d A = Lerp(V00, V01, ty);
			FVector3d B = Lerp(V10, V11, ty);
			for (int xi = 1; xi < NumEdgeVerts-1; ++xi)
			{
				ensure(VertexGrid.At(xi, yi) == -1);
				double tx = (double)xi / (double)(NumEdgeVerts - 1);
				FVector3d InterpPos = Lerp(A, B, tx);
				int32 NewVID = Mesh.AppendVertex(InterpPos);
				VertexGrid.At(xi, yi) = NewVID;

				FBevelVertex_InteriorVertex& InteriorVertex = Vertex.InteriorVertices.Emplace_GetRef();
				InteriorVertex.VertexID = NewVID;
				InteriorVertex.BorderFrameWeight.Add(FVector3d(tx, ty, 0));
			}
		}

		// create new vertices for any that don't already have a mapping back to boundary ring
		TArray<FIndex3i> Triangles;
		Vertex.NewGroupID = Mesh.AllocateTriangleGroup();

		for (int y0 = 0; y0 < NumEdgeVerts - 1; ++y0)
		{
			for (int x0 = 0; x0 < NumEdgeVerts - 1; ++x0)
			{
				int i00 = VertexGrid.At(x0, y0);
				int i10 = VertexGrid.At(x0+1, y0);
				int i01 = VertexGrid.At(x0, y0+1);
				int i11 = VertexGrid.At(x0+1, y0+1);
				int T0 = Mesh.AppendTriangle(i10, i00, i01, Vertex.NewGroupID);
				if (Mesh.IsTriangle(T0))
				{
					Vertex.NewTriangles.Add(T0);
				}
				int T1 = Mesh.AppendTriangle(i01, i11, i10, Vertex.NewGroupID);
				if (Mesh.IsTriangle(T1))
				{
					Vertex.NewTriangles.Add(T1);
				}

			}
		}

		return;
	}

	// Ok, now we are in the general case (currently used for >= 5-vertex polygons)
	// We will use Uniform Tessellation to subdivide the initial polygon triangulation up to
	// the target polycount. This is a bit annoying because we have to reconstruct the boundary
	// correspondence, perhaps FUniformTessellate could be improved to return this information
	FUniformTessellate Tesselator(&TmpMesh);
	Tesselator.TessellationNum = NumEdgeVerts - 2;
	bool bTesselateOK = Tesselator.Compute();

	// create flattened 2D version of this tessellated mesh  (maybe should optimize in 2D for inner fairness??)
	FDynamicMeshUVEditor UVEditor(&TmpMesh, 0, true);
	TArray<int32> AllTriangles;
	for (int32 tid : TmpMesh.TriangleIndicesItr()) AllTriangles.Add(tid);
	TArray<int32> TmpMeshToUVMap; bool bMapIsCompact = false;
	UVEditor.SetToPerVertexUVs(TmpMeshToUVMap, bMapIsCompact);
	check(bMapIsCompact == true);
	UVEditor.SetTriangleUVsFromFreeBoundarySpectralConformal(AllTriangles, true, true);
	UVEditor.ScaleUVAreaTo3DArea(AllTriangles, true);
	FDynamicMeshUVOverlay* UVOverlay = UVEditor.GetOverlay();

	// Now we need to append this triangulated patch back into the main mesh, ie "fill the hole"
	// that exists for this junction vertex. Ideally we will stitch at the compatible patch border, via this map
	TArray<int32> VertexMap;
	VertexMap.Init(-1, TmpMesh.MaxVertexID());

	// We should have an exact match between the existing loop of boundary vertices (in PolygonVertices) and 
	// the boundary loop of the patch. And we know the correspondences at the Corners as we constructed them 
	// above in the TmpMesh.AppendVertex calls. So we should be able to find Corner 0 in the boundary loop,
	// and walk around the two loops from that initial correspondence. This block will do that and store it in VertexMap, if possible.
	int32 NV = PolygonVertices.Num();
	FMeshBoundaryLoops BoundaryLoops(&TmpMesh, true);	// should only ever be one loop...
	bool bFoundLoop = false;
	TArray<int32> BoundaryLoopVerts;
	for (int32 k = 0; k < BoundaryLoops.GetLoopCount() && bFoundLoop == false; ++k)
	{
		if (BoundaryLoops[k].Vertices.Num() == NV)
		{
			BoundaryLoopVerts = BoundaryLoops[k].Vertices;
			bFoundLoop = true;	// conceivably it's possible this isn't the correct loop, somehow. That case is not handled

			// find our known-correspondence vertices, from the initial polygon corners
			int32 CornerVID = PolygonCornerVIDs[0];
			int32 FoundIdx = BoundaryLoopVerts.IndexOfByKey(0);
			int32 SecondCornerVID = PolygonCornerVIDs[1];
			int32 FoundSecondIdx = BoundaryLoopVerts.IndexOfByKey(1);

			if (FoundIdx != INDEX_NONE && FoundSecondIdx != INDEX_NONE)
			{
				// detect if the boundary loop is reversed, and if so, flip it
				if (BoundaryLoopVerts[(FoundIdx+(NumEdgeVerts-1)) % NV] != BoundaryLoopVerts[FoundSecondIdx] )
				{
					Algo::Reverse(BoundaryLoopVerts);
					FoundIdx = BoundaryLoopVerts.IndexOfByKey(0);
					FoundSecondIdx = BoundaryLoopVerts.IndexOfByKey(1);
				}

				if (ensure(BoundaryLoopVerts[(FoundIdx+(NumEdgeVerts-1)) % NV] == BoundaryLoopVerts[FoundSecondIdx]))
				{
					// walk around loop at corresponding indices and populate VertexMap
					for (int32 j = 0; j < NV; ++j)
					{
						int32 ExistingID = PolygonVertices[j];
						int32 NewID = BoundaryLoopVerts[(FoundIdx + j) % NV];
						VertexMap[NewID] = ExistingID;
					}
				}
			}
		}
	}

	Vertex.InteriorVertices.Reserve(FMath::Max(TmpMesh.VertexCount() - NV, 0));

	int32 NumLoopVerts = BoundaryLoopVerts.Num();
	FPolygon2d BorderPolygon;
	for (int32 vid : BoundaryLoopVerts)
	{
		BorderPolygon.AppendVertex( (FVector2d)UVOverlay->GetElement(vid) );
	}

	// Create new vertices for any that don't already have a mapping back to boundary ring.
	// For those interior vertices we also want to compute 2D Mean Value Coordinates (MVC)
	// relative to the boundary polygon. We will use these later to interpolate 3D positions from
	// the 3D border.
	TArray<FIndex3i> Triangles;
	Vertex.NewGroupID = Mesh.AllocateTriangleGroup();
	for (int32 vid : TmpMesh.VertexIndicesItr())
	{
		if (VertexMap[vid] == -1)
		{
			int32 NewVID = Mesh.AppendVertex(TmpMesh.GetVertex(vid));
			VertexMap[vid] = NewVID;

			// construct interior MVC weights from 2D mesh
			FBevelVertex_InteriorVertex& InteriorVertex = Vertex.InteriorVertices.Emplace_GetRef();
			InteriorVertex.VertexID = NewVID;
			InteriorVertex.BorderFrameWeight.Reserve(BoundaryLoopVerts.Num());
			FVector2d UVPosition = (FVector2d)UVOverlay->GetElement(vid);
			double WeightSum = 0;
			for (int32 k = 0; k < NumLoopVerts; ++k)
			{
				int32 BoundaryVID = BoundaryLoopVerts[k];
				int32 NewBoundaryVID = VertexMap[BoundaryVID];
				checkSlow(NewBoundaryVID >= 0);
				FVector2d BoundaryUVPosition = BorderPolygon[k];
				FVector2d Prev = BorderPolygon[(k-1+NumLoopVerts) % NumLoopVerts];
				FVector2d Next = BorderPolygon[(k+1) % NumLoopVerts];

				FVector2d BoundaryFrameX = Normalized(Next - Prev);
				FVector2d BoundaryFrameY = PerpCW(BoundaryFrameX);

				FVector2d DeltaUV = UVPosition - BoundaryUVPosition;
				double DeltaX = BoundaryFrameX.Dot(DeltaUV);
				double DeltaY = BoundaryFrameY.Dot(DeltaUV);

				// compute Polygon MVC weight ( https://cgvr.cs.uni-bremen.de/teaching/cg_literatur/barycentric_floater.pdf )
				double Dist = Distance(UVPosition, BoundaryUVPosition);
				double Weight = 1.0;
				if ( Dist > FMathd::ZeroTolerance )
				{
					FVector2d DeltaP = Normalized(BoundaryUVPosition - UVPosition);
					double T1 = UE::Geometry::VectorUtil::VectorTanHalfAngle(Normalized(Prev - UVPosition), DeltaP);
					double T2 = UE::Geometry::VectorUtil::VectorTanHalfAngle(Normalized(Next - UVPosition), DeltaP);
					Weight = (T1 + T2) / Dist;
				}

				InteriorVertex.BorderFrameWeight.Add( FVector3d(DeltaX, DeltaY, Weight) );
				WeightSum += Weight;
			}
			for (FVector3d& Weight : InteriorVertex.BorderFrameWeight)
			{
				Weight.Z /= WeightSum;		// normalize MVC weights
			}
		}
	}

	Vertex.InteriorBorderLoop.Reserve(NumLoopVerts);
	for (int32 vid : BoundaryLoopVerts)
	{
		Vertex.InteriorBorderLoop.Add(VertexMap[vid]);
	}

	// PolygonVertices array is always constructed in reversed orientation, so patch will be flipped otherwise
	TmpMesh.ReverseOrientation();

	// append new triangles
	for (FIndex3i Tri : TmpMesh.TrianglesItr())
	{
		int32 A = VertexMap[Tri.A];
		int32 B = VertexMap[Tri.B];
		int32 C = VertexMap[Tri.C];
		int32 tid = Mesh.AppendTriangle(A, B, C, Vertex.NewGroupID);
		if (Mesh.IsTriangle(tid))
		{
			Vertex.NewTriangles.Add(tid);
		}
	}

}



void FMeshBevel::AppendTerminatorVertexTriangles_Multi(FDynamicMesh3& Mesh, FBevelVertex& Vertex)
{
	// A Terminator Vertex occurs at the end of an non-loop bevel-edge, where the bevel "ends". 
	// Disconnecting the mesh to insert the bevel edge-strip will have created a triangle-shaped hole
	// in the mesh that must be filled. In the Multi-case, we have already meshed the adjacent bevel-edge,
	// so it's not a simple triangle but one with subdivisions along the edge adajacent to the bevel-edge.
	// So basically we have to find that already-meshed edge, and extract the column from it's quadpatch
	// that we want to stitch to, and then add a triangle-fan to fill the hole polygon

	// TODO: maybe could do delaunay triangulation if the polygon below is nearly coplanar...

	MESH_BEVEL_DEBUG_CHECK(Vertex.VertexType == EBevelVertexType::TerminatorVertex);

	check(Vertex.IncomingBevelEdgeIndices.Num() == 1);

	int32 RingSplitEdgeID = Vertex.TerminatorInfo.A;
	if (Mesh.IsEdge(RingSplitEdgeID))
	{
		int32 BevelEdgeIndex = Vertex.IncomingBevelEdgeIndices[0];
		const FBevelEdge& IncomingEdge = Edges[BevelEdgeIndex];

		// If we failed to create a strip quad patch (due to failure earlier in the pipeline)
		// try to append a non-multi terminator instead
		if (IncomingEdge.StripQuadPatch.IsEmpty())
		{
			AppendTerminatorVertexTriangle(Mesh, Vertex);
			return;
		}

		FIndex2i SplitEdgeV = Mesh.GetEdgeV(RingSplitEdgeID);
		int32 FarVertexID = SplitEdgeV.OtherElement(Vertex.VertexID);

		int32 WedgeVertexA = Vertex.Wedges[0].WedgeVertex;
		int32 WedgeVertexB = Vertex.Wedges[1].WedgeVertex;

		// figure out which side of the incoming-edge quad-strip the terminator connects to
		int32 ColumnIndex = IncomingEdge.StripQuadPatch.FindColumnIndex(WedgeVertexA);
		if (!MESH_BEVEL_DEBUG_ENSURE(ColumnIndex == 0 || ColumnIndex == IncomingEdge.StripQuadPatch.NumVertexCols() - 1))
		{
			return;
		}
		TArray<int32> QuadStripEdgeVerts;
		IncomingEdge.StripQuadPatch.GetVertexColumn(ColumnIndex, QuadStripEdgeVerts);
		if (!MESH_BEVEL_DEBUG_ENSURE(QuadStripEdgeVerts.Contains(WedgeVertexB)))
		{
			return;
		}

		// make sure it is oriented consistently
		int32 FirstEdgeID = Mesh.FindEdge(QuadStripEdgeVerts[0], QuadStripEdgeVerts[1]);
		if (!MESH_BEVEL_DEBUG_ENSURE(FirstEdgeID != IndexConstants::InvalidID && Mesh.IsBoundaryEdge(FirstEdgeID)))
		{
			return;
		}
		FIndex2i QuadEdgeV = Mesh.GetOrientedBoundaryEdgeV(FirstEdgeID);
		if (QuadEdgeV.A != QuadStripEdgeVerts[0])
		{
			Algo::Reverse(QuadStripEdgeVerts);
		}

		// should have computed this GroupID in initial setup
		int32 UseGroupID = (Vertex.NewGroupID >= 0) ? Vertex.NewGroupID : Mesh.AllocateTriangleGroup();

		int32 NumEdges = QuadStripEdgeVerts.Num()-1;
		for (int32 k = 0; k < NumEdges; ++k)
		{
			int32 tid = Mesh.AppendTriangle(QuadStripEdgeVerts[k+1], QuadStripEdgeVerts[k], FarVertexID, UseGroupID);
			MESH_BEVEL_DEBUG_CHECK(tid >= 0);
			if (Mesh.IsTriangle(tid))
			{
				Vertex.NewTriangles.Add(tid);
			}
		}
	}
}



void FMeshBevel::AppendTerminatorVertexPairQuad_Multi(FDynamicMesh3& Mesh, FBevelVertex& Vertex0, FBevelVertex& Vertex1)
{
	MESH_BEVEL_DEBUG_CHECK(Vertex0.VertexType == EBevelVertexType::TerminatorVertex);
	MESH_BEVEL_DEBUG_CHECK(Vertex1.VertexType == EBevelVertexType::TerminatorVertex);



	// This is a variant of AppendTerminatorVertexTriangle that handles the case where basically two
	// Terminator Vertices are directly connected by a non-beveled mesh edge that was used as the ring-split-edge.
	// Since both sides were opened, we have a quad-shaped hole instead of a triangle-shaped hole, with a 
	// (subdivided, in the multi-case) quad-edge at each end. So this hole can be filled with a quad-patch that
	// stitches together the existing edges (which must have already been meshed)
	bool bFallbackToQuad = false, bInvalidInfo = false;
	auto CollectTerminatorVtxInfo = [this, &bFallbackToQuad, &bInvalidInfo](FDynamicMesh3& Mesh, FBevelVertex& Vertex, TArray<int32>& QuadStripEdgeVertsOut) -> void
	{
		int32 BevelEdgeIndex = Vertex.IncomingBevelEdgeIndices[0];
		const FBevelEdge& IncomingEdge = Edges[BevelEdgeIndex];
		if (IncomingEdge.StripQuadPatch.IsEmpty())
		{
			bFallbackToQuad = true;
			return;
		}

		int32 ColumnIndex = IncomingEdge.StripQuadPatch.FindColumnIndex(Vertex.Wedges[0].WedgeVertex);
		if (!(ColumnIndex == 0 || ColumnIndex == IncomingEdge.StripQuadPatch.NumVertexCols() - 1))
		{
			bInvalidInfo = true;
			return;
		}
		IncomingEdge.StripQuadPatch.GetVertexColumn(ColumnIndex, QuadStripEdgeVertsOut);
		if (!(QuadStripEdgeVertsOut.Contains(Vertex.Wedges[1].WedgeVertex)))
		{
			bInvalidInfo = true;
			return;
		}

		int32 FirstEdgeID = Mesh.FindEdge(QuadStripEdgeVertsOut[0], QuadStripEdgeVertsOut[1]);
		if (FirstEdgeID == IndexConstants::InvalidID || !Mesh.IsBoundaryEdge(FirstEdgeID))
		{
			bInvalidInfo = true;
			return;
		}
		FIndex2i QuadEdgeV = Mesh.GetOrientedBoundaryEdgeV(FirstEdgeID);
		if (QuadEdgeV.A != QuadStripEdgeVertsOut[0])
		{
			Algo::Reverse(QuadStripEdgeVertsOut);
		}
	};

	TArray<int32> QuadStripEdgeVerts0, QuadStripEdgeVerts1;
	CollectTerminatorVtxInfo(Mesh, Vertex0, QuadStripEdgeVerts0);
	CollectTerminatorVtxInfo(Mesh, Vertex1, QuadStripEdgeVerts1);
	if (bInvalidInfo)
	{
		MESH_BEVEL_DEBUG_CHECK(false);
		return;
	}
	if (bFallbackToQuad)
	{
		// no quad-strip fallback case
		AppendTerminatorVertexPairQuad(Mesh, Vertex0, Vertex1);
		return;
	}
	if (QuadStripEdgeVerts0.Num() != QuadStripEdgeVerts1.Num() || QuadStripEdgeVerts0.Num() == 0)
	{
		// this could happen if the quadstrip failed on one or both sides? will leave a hole.
		MESH_BEVEL_DEBUG_CHECK(false);
		return;
	}

	// BIASED? should have computed this GroupID in initial setup
	int32 UseGroupID = (Vertex0.NewGroupID >= 0) ? Vertex0.NewGroupID : Mesh.AllocateTriangleGroup();

	int32 NumEdges = QuadStripEdgeVerts0.Num() - 1;
	for (int32 k = 0; k < NumEdges; ++k)
	{
		int32 A0 = QuadStripEdgeVerts0[k];
		int32 B0 = QuadStripEdgeVerts0[k+1];
		int32 A1 = QuadStripEdgeVerts1[NumEdges-(k)];
		int32 B1 = QuadStripEdgeVerts1[NumEdges-(k+1)];

		int32 tid0 = Mesh.AppendTriangle(B0, A0, A1, UseGroupID);
		MESH_BEVEL_DEBUG_CHECK(tid0 >= 0);
		if (Mesh.IsTriangle(tid0))
		{
			Vertex0.NewTriangles.Add(tid0);
		}

		int32 tid1 = Mesh.AppendTriangle(A1, B1, B0, UseGroupID);
		MESH_BEVEL_DEBUG_CHECK(tid1 >= 0);
		if (Mesh.IsTriangle(tid1))
		{
			Vertex0.NewTriangles.Add(tid1);
		}
	}

}


void FMeshBevel::CreateBevelMeshing_Multi(FDynamicMesh3& Mesh)
{
	// This is the top-level driver function that is called after the mesh has been pulled apart along
	// the bevelled edges. There are four cases - open edge spans, edge loops, corners/vertices with bevel-valence > 3 (become polygons),
	// and "terminator" vertices of bevel-valence 1 (become triangles, except if directly connected, then they become quads).
	// First we figure out the mesh connectivity, ie the 'stitching' between the pulled-apart geometry,
	// and then we (optionally) apply a profile-curve shape along the bevel strips

	// We will later need normals along each exterior "side" of the bevel edge to define the arcs along 
	// rounded bevel edges, ie basically these are the smooth boundary conditions.
	// It ought to be possible to compute this after adding the bevel geometry, by filtering out the bevel-edge tris, 
	// but for now it's simpler to just do it here before we add the bevel tris and cache it...
	for (FBevelEdge& Edge : Edges)
	{
		Edge.NormalsA.SetNum(Edge.MeshVertices.Num());
		Edge.NormalsB.SetNum(Edge.MeshVertices.Num());
		for (int32 k = 0; k < Edge.MeshVertices.Num(); ++k)
		{
			Edge.NormalsA[k] = FMeshNormals::ComputeVertexNormal(Mesh, Edge.MeshVertices[k]);
			Edge.NormalsB[k] = FMeshNormals::ComputeVertexNormal(Mesh, Edge.NewMeshVertices[k]);
		}
	}
	for (FBevelLoop& Loop : Loops)
	{
		Loop.NormalsA.SetNum(Loop.MeshVertices.Num());
		Loop.NormalsB.SetNum(Loop.MeshVertices.Num());
		for (int32 k = 0; k < Loop.MeshVertices.Num(); ++k)
		{
			Loop.NormalsA[k] = FMeshNormals::ComputeVertexNormal(Mesh, Loop.MeshVertices[k]);
			Loop.NormalsB[k] = FMeshNormals::ComputeVertexNormal(Mesh, Loop.NewMeshVertices[k]);
		}
	}

	for (FBevelEdge& Edge : Edges)
	{
		AppendEdgeQuads_Multi(Mesh, Edge);
	}

	for (FBevelLoop& Loop : Loops)
	{
		AppendLoopQuads_Multi(Mesh, Loop);
	}


	for (FBevelVertex& Vertex : Vertices)
	{
		if (Vertex.VertexType == EBevelVertexType::JunctionVertex)
		{
			if (Vertex.Wedges.Num() > 2)
			{
				AppendJunctionVertexPolygon_Multi(Mesh, Vertex);
			}
		}
	}

	// easier to do terminators last so that we can use quad edge to orient the triangle
	TSet<FIndex2i> HandledQuadVtxPairs;
	for (FBevelVertex& Vertex : Vertices)
	{
		if (Vertex.VertexType == EBevelVertexType::TerminatorVertex)
		{
			if (Vertex.ConnectedBevelVertex >= 0)
			{
				FBevelVertex& OtherVertex = Vertices[Vertex.ConnectedBevelVertex];
				FIndex2i VtxPair(Vertex.VertexID, OtherVertex.VertexID);
				VtxPair.Sort();
				if (HandledQuadVtxPairs.Contains(VtxPair) == false)
				{
					AppendTerminatorVertexPairQuad_Multi(Mesh, Vertex, OtherVertex);
					HandledQuadVtxPairs.Add(VtxPair);
				}
			}
			else
			{
				AppendTerminatorVertexTriangles_Multi(Mesh, Vertex);
			}
		}
	}

	
	// At this point, all topology is determined. Now if desired a profile-curve shape can be applied as a postprocess.
	// (Not entirely certain this is the right strategy for all profile types but it works OK for a simple round profile)
	if (FMathd::Abs(RoundWeight) > FMathf::ZeroTolerance)
	{
		ApplyProfileShape_Round(Mesh);
	}
}




FInterpCurveVector FMeshBevel::MakeArcSplineCurve(const FVector3d& PosA, FVector3d& NormalA, const FVector3d& PosB, FVector3d& NormalB) const
{
	FInterpCurveVector Curve;

	// basic idea here is to approximate an arc w/ a bezier curve. We know A and B,
	// and we need TangentA (T_A) and TangentB. We have a surface Normal at A and B, so
	// for T_A we can project B onto the surface tangent plane at A (ie with NormalA),
	// and then B-A gives us a direction for T_A, and the same works for for B
	// 
	//         T_A
	//       <-----
	//    ^    __-*A
	//    |   /
	// T_B|  / (this is the arc, sorry)
	//    | |
	//      *B
	//
	// The lengths of the Tangents in this approach end up being the lengths of the sides
	// of the square, if this were a 2D situation. Which could effectively define a 2D arc.
	// For now, we are use a FInterpCurve instead, which is not quite a Bezier. With those
	// lengths, the curve will end up too flat, so it is scaled by a parameter (default sqrt 2).
	// This could perhaps be exposed as an option...
	//
	// The advantage of using a curve here instead of an arc is that it allows something reasonable
	// to happen for messy cases, like where the curve might not be planar. But likely some
	// issues are going to arise with ugly projections...

	// project the vector (B-A) onto the plane (A,NormalA). 
	FFrame3d PlaneNormalA(PosA, NormalA);
	FVector3d TangentA = PlaneNormalA.ToPlane(PosB) - PosA;

	// same for A-B onto (B, NormalB)
	FFrame3d PlaneNormalB(PosB, NormalB);
	FVector3d TangentB = PlaneNormalB.ToPlane(PosA) - PosB;

	Curve.AddPoint(0, PosA);
	Curve.AddPoint(1, PosB);
	Curve.Points[0].InterpMode = EInterpCurveMode::CIM_CurveUser;
	Curve.Points[1].InterpMode = EInterpCurveMode::CIM_CurveUser;
	double TangentScale = FMathd::Abs(RoundWeight) * FMathd::Sqrt2;
	if (RoundWeight >= 0)
	{
		Curve.Points[0].ArriveTangent = Curve.Points[0].LeaveTangent = TangentScale * TangentA;
		Curve.Points[1].ArriveTangent = Curve.Points[1].LeaveTangent = -TangentScale * TangentB;
	}
	else
	{
		Curve.Points[0].ArriveTangent = Curve.Points[0].LeaveTangent = -TangentScale * TangentB;
		Curve.Points[1].ArriveTangent = Curve.Points[1].LeaveTangent = TangentScale * TangentA;
	}
	return Curve;
}


void FMeshBevel::ApplyProfileShape_Round(FDynamicMesh3& Mesh)
{
	// This function applies round profile curves to the bevel edge quadstrips & 
	// bevel vertex tessellated-polygons that were computed in the _Multi() functions.
	// All the geometry already exists so the job here is just to deform it into rounder shapes.
	//
	// The current code is based on various different strategies but the basic idea is to
	// compute a arc-shaped 3D splines between correspondeng edge-vertex pairs , and then derive
	// the shape of the vertex patches using the now-curved borders. For 4-sided patches
	// this is relatively simple, 5+-sided we use a sort of MVC-based hole-filling strategy, and
	// for 3-sided currently using a 3-sided spline batch taken from PN-tessellation. 
	// The 3-sided case currently does not work very well though.	

	// Loops are an easy case as there are no junction vertices, so each vertex-pair gets a separate
	// spline and everything is done. 
	for (FBevelLoop& Loop : Loops)
	{
		FQuadGridPatch& Patch = Loop.StripQuadPatch;
		int32 NumCols = Patch.NumVertexCols();
		for (int32 Col = 0; Col < NumCols-1; ++Col)
		{
			TArray<int32> ColVerts;
			Patch.GetVertexColumn(Col, ColVerts);
			int32 NV = ColVerts.Num();

			int32 A = ColVerts[0], B = ColVerts.Last();
			FVector3d PosA = Mesh.GetVertex(A), PosB = Mesh.GetVertex(B);
			FVector3d NormalA = Loop.NormalsA[Col], NormalB = Loop.NormalsB[Col];

			// project normals onto section plane defined by original and inset vertex positions.
			// (do we even need the normals anymore? could they be defined by the 2D perp operator in the section plane?)
			FVector3d InitialPosition = Loop.InitialPositions[Col];
			FVector3d Direction1 = Normalized(PosA - InitialPosition);
			FVector3d Direction2 = Normalized(PosB - InitialPosition);
			FVector3d PlaneNormal = Normalized(Cross(Direction1, Direction2));
			NormalA = Normalized(NormalA - NormalA.Dot(PlaneNormal) * PlaneNormal);
			NormalB = Normalized(NormalB - NormalB.Dot(PlaneNormal) * PlaneNormal);

			FInterpCurveVector Curve = MakeArcSplineCurve(PosA, NormalA, PosB, NormalB);

			// TODO: should have special case here for flat patches? they get squished out a bit...

			for (int32 k = 1; k < (NV - 1); ++k)
			{
				int32 VID = ColVerts[k];
				FVector3d Pos = Mesh.GetVertex(VID);
				double T = (double)k / (double)(NV-1);
				FVector3d CurvePos = Curve.Eval((float)T, Lerp(PosA, PosB, T));
				Mesh.SetVertex(VID, CurvePos);
			}
		}
	}


	// kinda gross approach to keeping track of the normals along the ends of edge spans,
	// ie after we make them curved, we want the normals along those curves, ie what we 
	// want to use as the normal/tangent constraint for the bevelvertex-polygon. At this point
	// the mesh topo should be finalized so we can just use an array...
	TArray<FVector3d> DeformNormals;
	DeformNormals.Init(FVector3d::Zero(), Mesh.MaxVertexID());

	// we need to keep track of the curve used between each split pair of beveledge-vertices,
	// and we need to be able to loop them up based on the vertex-pair.
	TMap<FIndex2i, FInterpCurveVector> BorderCurves;

	// search function for BorderCurves list
	auto FindCurve = [&BorderCurves](int i0, int i1, bool& bReversed)
	{
		FInterpCurveVector* FoundCurve = BorderCurves.Find(FIndex2i(i0, i1));
		bReversed = false;
		if (FoundCurve == nullptr)
		{
			FoundCurve = BorderCurves.Find(FIndex2i(i1, i0));
			bReversed = true;
		}
		return FoundCurve;
	};

	// Each edge is processed similar to a loop, ie the column of vertices bewteen each vertex-pair on 
	// either side of the quad-strip gets deformed by fitting a spline curve based on the endpoints and end-normals. 
	// 
	for (FBevelEdge& Edge : Edges)
	{
		FQuadGridPatch& Patch = Edge.StripQuadPatch;

		bool bPatchIsFlippedX = false;

		int32 NumCols = Patch.NumVertexCols();
		for (int32 Col = 0; Col < NumCols; ++Col)
		{
			TArray<int32> ColVerts;
			Patch.GetVertexColumn(Col, ColVerts);
			int32 NumColVerts = ColVerts.Num();

			// The Edge contains various vertex lists like Edge.MeshVertices that have a consistent ordering.
			// However because of how the patches are constructed, the columns may be reversed  (unclear why...)
			// Detect that case so that the indexing Col/Array indexing can be inverted where necessary
			if (Col == 0 && ColVerts.Contains(Edge.MeshVertices[0]) == false)
			{
				bPatchIsFlippedX = true;
			}

			int32 A = ColVerts[0];
			int32 B = ColVerts.Last();
			FVector3d PosA = Mesh.GetVertex(A);
			FVector3d PosB = Mesh.GetVertex(B);

			int UseEdgeVertIndex = (bPatchIsFlippedX) ? (NumCols-Col-1) : Col;
			FVector3d NormalA = Edge.NormalsA[UseEdgeVertIndex];
			FVector3d NormalB = Edge.NormalsB[UseEdgeVertIndex];

			// Ok for each column along a bevel edge-strip, we want to bend the column into a curve.
			// We have the endpoints we want to interpolate, and we have normals at those points that we
			// computed elsewhere and are going to assume are good, ie those are the normals we want the
			// perfect rounded bevel to have, if it had infinite sections.
			//
			// The issue is that we have 2 points and 2 normals and they may not all lie in the same plane.
			// For most cases, the original vertex position has been inset in two directions (to PosA and PosB), and 
			// so that gives us the plane they all should lie in. However at valence3+ junction vertices this is not true,
			// as the one corner vertex was inset along 3+ edges and so we don't have the 'InitialPosition'
			// value below for each of those edges, that we would need to define the simple arc-sections.
			// 
			// (todo: maybe keeping track of those positions would be better than what we do now

			bool bInferTangentPlaneFromInitialPosition = true;
			bool bIsEndpoint = (UseEdgeVertIndex == 0) || (UseEdgeVertIndex == NumCols - 1);
			if ( bIsEndpoint )
			{
				int32 BevelVertexIndex = (UseEdgeVertIndex == 0) ? Edge.BevelVertices.A : Edge.BevelVertices.B;
				const FBevelVertex& BevelVtx = Vertices[BevelVertexIndex];
				if (BevelVtx.VertexType == EBevelVertexType::JunctionVertex && BevelVtx.IncomingBevelEdgeIndices.Num() > 2)
				{
					bInferTangentPlaneFromInitialPosition = false;
				}
			}

			FVector3d InitialPosition = Edge.InitialPositions[UseEdgeVertIndex];
			FVector3d Direction1 = Normalized(PosA - InitialPosition);
			FVector3d Direction2 = Normalized(PosB - InitialPosition);
			FVector3d SectionPlaneNormal = Normalized(Cross(Direction1, Direction2));

			if (!bInferTangentPlaneFromInitialPosition)
			{
				FVector3d InitialEdgeDirection = (UseEdgeVertIndex == 0) ?
					(Edge.InitialPositions[1] - InitialPosition) : (InitialPosition - Edge.InitialPositions[NumCols-2]);
				if (InitialEdgeDirection.Normalize())
				{
					FFrame3d TempFrame(PosA, Normalized(PosB - PosA));
					TempFrame.ConstrainedAlignAxis(1, InitialEdgeDirection, TempFrame.Z());
					SectionPlaneNormal = TempFrame.Y();
				}
			}

			// project normals onto section plane
			NormalA = Normalized(NormalA - NormalA.Dot(SectionPlaneNormal) * SectionPlaneNormal);
			NormalB = Normalized(NormalB - NormalB.Dot(SectionPlaneNormal) * SectionPlaneNormal);

			// TODO: should have special case here for flat patches? they get squished out a bit...
			// This may update NormalA/NormalB for 'inverted' bevels (ie w/ negative RoundWeight)
			FInterpCurveVector Curve = MakeArcSplineCurve(PosA, NormalA, PosB, NormalB);

			// accumulate normal at endpoints
			DeformNormals[A] += NormalA;
			DeformNormals[B] += NormalB;

			// save this curve in the curves list
			BorderCurves.Add(FIndex2i(A, B), Curve);

			// map linear-interpolation to curve parameter and evaluate curve
			TArray<FVector3d> CurveVerts;
			CurveVerts.SetNum(NumColVerts);
			CurveVerts[0] = PosA; CurveVerts[NumColVerts-1] = PosB;
			for (int32 k = 1; k < (NumColVerts - 1); ++k)
			{
				int32 VID = ColVerts[k];
				FVector3d Pos = Mesh.GetVertex(VID);
				double T = (double)k / (double)(NumColVerts-1);
				FVector3d CurvePos = Curve.Eval((float)T, Lerp(PosA, PosB, T));
				Mesh.SetVertex(VID, CurvePos);
				CurveVerts[k] = CurvePos;
			}

			// accumulate new interior normals (only triangles inside the edge are considered)
			// (do we need these anywhere??)
			for (int32 k = 1; k < (NumColVerts - 1); ++k)
			{
				int32 VID = ColVerts[k];
				FVector3d CurveEdgeNormal = FMeshNormals::ComputeVertexNormal(Mesh, VID,
						[&](int32 TriangleID) { return Mesh.GetTriangleGroup(TriangleID) == Edge.NewGroupID; }, true, true);

				DeformNormals[VID] += CurveEdgeNormal;
			}
		}
	}

	// re-normalize here because we accumulated normals above...
	for (FVector3d& Normal : DeformNormals)
	{
		Normal.Normalize();
	}

	// Process vertices. This code has special-cases for valence 3 and 4, which should be factored out into separate functions.
	for (FBevelVertex& Vertex : Vertices)
	{
		int32 LoopN = Vertex.InteriorBorderLoop.Num();
		for (const FBevelVertex_InteriorVertex& InteriorVtx : Vertex.InteriorVertices)
		{
			// Special case for valence-3 vertex, ie triangular patch. In this case we use
			// a triangular spline, the math is based on PN triangulation, where the vertex/normal
			// pairs at the endpoints are used. However a center-point/normal is also necessary.
			// This does an OK job but seems to come out too flat in many cases...ultimately it is
			// not going to even be C0 with the boundary spline curves we already decided on.
			// Perhaps a better option would be to do it similar to the valence-4 case, where basically
			// we construct interpolated splines based on the barycentrics and border splines
			if (InteriorVtx.BorderFrameWeight.Num() == 1 && Vertex.InteriorBorderLoop.Num() == 3)
			{
				// Nearly all of this work below is identical for every interior vertex!!! It should be done once and cached. 

				double w = InteriorVtx.BorderFrameWeight[0].X, u = InteriorVtx.BorderFrameWeight[0].Y, v = InteriorVtx.BorderFrameWeight[0].Z;

				int i300 = Vertex.InteriorBorderLoop[0];
				FVector3d b300 = Mesh.GetVertex(i300);
				int i030 = Vertex.InteriorBorderLoop[1];
				FVector3d b030 = Mesh.GetVertex(i030);
				int i003 = Vertex.InteriorBorderLoop[2];
				FVector3d b003 = Mesh.GetVertex(i003);

				FVector3d N300 = DeformNormals[i300];
				FVector3d N030 = DeformNormals[i030];
				FVector3d N003 = DeformNormals[i003];

				bool bReversedA = false, bReversedB = false, bReversedC = false;
				FInterpCurveVector* CurveA = FindCurve(i300, i030, bReversedA);
				FInterpCurveVector* CurveB = FindCurve(i030, i003, bReversedB);
				FInterpCurveVector* CurveC = FindCurve(i003, i300, bReversedC);
				if ( !ensure(CurveA != nullptr && CurveB != nullptr && CurveC != nullptr )) continue;

				// PN triangle sides are defined by bezier curves, however FInterpCurve is not actually a Bezier, it's missing a 3
				// on the tangents in the math. So scaling by 1/3 here corrects for this (...ish?)
				const double TangentScale = 1.0 / 3.0;

				FVector3d b210 = TangentScale * ((bReversedA) ? -CurveA->Points[1].LeaveTangent : CurveA->Points[0].LeaveTangent);
				b210 += b300;
				FVector3d b120 = TangentScale * ((bReversedA) ? CurveA->Points[0].LeaveTangent : -CurveA->Points[1].LeaveTangent);
				b120 += b030;

				FVector3d b021 = TangentScale * ((bReversedB) ? -CurveB->Points[1].LeaveTangent : CurveB->Points[0].LeaveTangent);
				b021 += b030;
				FVector3d b012 = TangentScale * ((bReversedB) ? CurveB->Points[0].LeaveTangent : -CurveB->Points[1].LeaveTangent);
				b012 += b003;

				FVector3d b102 = TangentScale * ((bReversedC) ? -CurveC->Points[1].LeaveTangent : CurveC->Points[0].LeaveTangent);
				b102 += b003;
				FVector3d b201 = TangentScale * ((bReversedC) ? CurveC->Points[0].LeaveTangent : -CurveC->Points[1].LeaveTangent);
				b201 += b300;

				// this computed midpoint tends to be too flat...
				FVector3d E = (b210 + b120 + b021 + b012 + b102 + b201) / 6.0;
				FVector3d V = (b300 + b030 + b003) / 3.0;
				FVector3d b111 = E + RoundWeight * (E - V) / 2.0;

				FVector InterpPos = b300*w*w*w + b030*u*u*u + b003*v*v*v
					+ b210*3*w*w*u + b120*3*w*u*u + b201*3*w*w*v
					+ b021*3*u*u*v + b102*3*w*v*v + b012*3*u*v*v
					+ b111*6*w*u*v;

				Mesh.SetVertex(InteriorVtx.VertexID, InterpPos);

				continue;
			}


			// this block handles special case of valence-4 vertex that can be done w/ a quad patch interpolating boundary curves
			// TODO: refactor this to a separate first pass so that we can pull the curve setup out of each iteration
			if (InteriorVtx.BorderFrameWeight.Num() == 1 && Vertex.InteriorBorderLoop.Num() == 4)
			{
				// Nearly all of this work below is identical for every interior vertex!!! It should be done once and cached. 

				// here is the situation: We have a tessellated quad patch, when we tessellated it, it was planar (or
				// at least the edges were straight). At tessellation time we computed barycentric coordinates for each interior vertex.
				// Now we have curved the edges of the patch along spline curves (above). So basically we want to compute new interior
				// positions by blending the four edge curve positions. If we think of the quad as having 2 axes X and Y, we are
				// going to blend the two X curves along the two Y curves, and then evaluate the blended curve to get the vertex position.
				// This is messy because (1) we have to look up the curves and (2) each one might have swapped end-vertices with a reversed parameter space.
				// Most of the code below is about handling that weirdness.
				// 
				// Note that since these are (almost) Beziers, the 'curve blending' is strictly about the tangents, as the endpoints are fixed/known
				//
				//c01  X2            c11            (diagram corresponds to variable naming below)
				//   *--------------*    ty=1
				// Y1|              |Y2
				//   |  XInterp     |
				//  A*==============*B
				//   |              |
				//   *--------------*    ty=0
				//c00  X1            c10
				// 
				// tx=0           tx=1

				// these are the UV (XY) coords of the vertex inside the initial planar quad
				double tx = InteriorVtx.BorderFrameWeight[0].X, ty = InteriorVtx.BorderFrameWeight[0].Y;

				// four ordered corner vertices of the initial planar quad
				int c00 = Vertex.InteriorBorderLoop[0];
				int c10 = Vertex.InteriorBorderLoop[1];
				int c01 = Vertex.InteriorBorderLoop[2];
				int c11 = Vertex.InteriorBorderLoop[3];

				// locate the two "Y" edge curves from their corner vertices, handling case of curve being reversed
				FInterpCurveVector* CurveY1 = BorderCurves.Find(FIndex2i(c00, c01));
				bool bReversed1 = false;
				if (CurveY1 == nullptr)
				{
					CurveY1 = BorderCurves.Find(FIndex2i(c01, c00));
					bReversed1 = true;
				}
				if ( ! ensure(CurveY1 != nullptr) ) continue;

				FInterpCurveVector* CurveY2 = BorderCurves.Find(FIndex2i(c10, c11));
				bool bReversed2 = false;
				if (CurveY2 == nullptr)
				{
					CurveY2 = BorderCurves.Find(FIndex2i(c11, c10));
					bReversed2 = true;
				}
				if (!ensure(CurveY2 != nullptr)) continue;

				// evaluate each edge curve at the Y coord
				double tA = bReversed1 ? (1.0 - ty) : ty;
				FVector A = CurveY1->Eval((float)tA, FVector::Zero());		// TODO: should lerp here instead of using ::Zero() obviously...
				double tB = bReversed2 ? (1.0 - ty) : ty;
				FVector B = CurveY2->Eval((float)tB, FVector::Zero());		// TODO: should lerp here instead of using ::Zero() obviously...

				// now find the two "X" edge curves from their corner vertices, again handling case of reversed edge
				FInterpCurveVector* CurveX1 = BorderCurves.Find(FIndex2i(c00, c10));
				bool bReversedX1 = false;
				if (CurveX1 == nullptr)
				{
					CurveX1 = BorderCurves.Find(FIndex2i(c10, c00));
					bReversedX1 = true;
				}
				if (!ensure(CurveX1 != nullptr)) continue;

				FInterpCurveVector* CurveX2 = BorderCurves.Find(FIndex2i(c01, c11));
				bool bReversedX2 = false;
				if (CurveX2 == nullptr)
				{
					CurveX2 = BorderCurves.Find(FIndex2i(c11, c01));
					bReversedX2 = true;
				}
				if (!ensure(CurveX2 != nullptr)) continue;

				// extract tangents of X curves, need to blend these to compute interpolated tangents at Y-curve points A/B
				FVector3d Tangent00 = (bReversedX1) ? CurveX1->Points[1].LeaveTangent : -CurveX1->Points[0].LeaveTangent;
				FVector3d Tangent10 = (bReversedX1) ? -CurveX1->Points[0].LeaveTangent : CurveX1->Points[1].LeaveTangent;
				FVector3d Tangent01 = (bReversedX2) ? CurveX2->Points[1].LeaveTangent : -CurveX2->Points[0].LeaveTangent;
				FVector3d Tangent11 = (bReversedX2) ? -CurveX2->Points[0].LeaveTangent : CurveX2->Points[1].LeaveTangent;

				// should perhaps be based on interpolating tangents along Y splines instead of just lerp? is it equivalent?
				FVector TangentA = Lerp(Tangent00, Tangent01, ty);
				FVector TangentB = Lerp(Tangent10, Tangent11, ty);

				// construct interpolated X curve, has endpoints evaluated along Y curves, with X curve endpoint tangents
				// blended along Y curves
				FInterpCurveVector InterpolatedXCurve;
				InterpolatedXCurve.AddPoint(0, A);
				InterpolatedXCurve.AddPoint(1, B);
				InterpolatedXCurve.Points[0].ArriveTangent = InterpolatedXCurve.Points[0].LeaveTangent = -TangentA;
				InterpolatedXCurve.Points[1].ArriveTangent = InterpolatedXCurve.Points[1].LeaveTangent = TangentB;
				InterpolatedXCurve.Points[0].InterpMode = InterpolatedXCurve.Points[1].InterpMode = EInterpCurveMode::CIM_CurveUser;

				// evaluate this new X curve at the vertex X position
				FVector InterpPos = InterpolatedXCurve.Eval((float)tx, Lerp(A, B, tx));
				Mesh.SetVertex(InteriorVtx.VertexID, InterpPos);

				continue;
			}

			// This should only happen for valence 3 and 4 that we already handled. But if it happens
			// elsewhere, it means we will just leave that patch flat
			if (InteriorVtx.BorderFrameWeight.Num() != LoopN) continue;

			// ok the general-case solution is basically to compute the new position for this vertex in the rotated
			// frame of each boundary-polygon vertex, and blend those positions using the MVC coordinates. This produces
			// a smooth interpolating surface but there is no continuity at the border, and the 'far' vertices will tend
			// to exert too much influence
			FVector3d BlendedPos = FVector3d::Zero();
			double WeightSum = 0;
			for (int32 k = 0; k < LoopN; ++k)
			{
				int32 BorderVID = Vertex.InteriorBorderLoop[k];
				FVector3d BorderPos = Mesh.GetVertex(BorderVID);
				FVector3d BorderFrameX = Mesh.GetVertex(Vertex.InteriorBorderLoop[(k+1) % LoopN]) - 
					Mesh.GetVertex(Vertex.InteriorBorderLoop[(k-1 + LoopN) % LoopN]);
				BorderFrameX = Normalized(BorderFrameX);
				FVector3d BorderFrameN = DeformNormals[BorderVID];

				if (RoundWeight < 0)
				{
					FQuaterniond RotQuat(BorderFrameX, -45, true);
					BorderFrameN = RotQuat * BorderFrameN;
				}

				FVector3d BorderFrameY = Cross(BorderFrameN, BorderFrameX);

				FVector3d FrameDeltaWeight = InteriorVtx.BorderFrameWeight[k];
				FVector3d ReconstructedPos = BorderPos + (FrameDeltaWeight.X*BorderFrameX) + (FrameDeltaWeight.Y*BorderFrameY);

				BlendedPos += FrameDeltaWeight.Z * ReconstructedPos;
				WeightSum += FrameDeltaWeight.Z;
			}
			BlendedPos *= (1.0 / WeightSum);
			Mesh.SetVertex(InteriorVtx.VertexID, BlendedPos);
		}
	}
}





void FMeshBevel::ComputeNormals(FDynamicMesh3& Mesh)
{
	if (Mesh.HasAttributes() == false)
	{
		return;
	}
	FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();

	auto SetNormalsOnTriRegion = [NormalOverlay](const TArray<int32>& Triangles)
	{
		if (Triangles.Num() > 0)
		{
			FMeshNormals::InitializeOverlayRegionToPerVertexNormals(NormalOverlay, Triangles);
		}
	};

	for (FBevelVertex& Vertex : Vertices)
	{
		SetNormalsOnTriRegion(Vertex.NewTriangles);
	}

	TArray<int32> TriList;
	for (FBevelEdge& Edge : Edges)
	{
		UELocal::QuadsToTris(Mesh, Edge.StripQuads, TriList, true);
		SetNormalsOnTriRegion(TriList);
	}
	for (FBevelLoop& Loop : Loops)
	{
		UELocal::QuadsToTris(Mesh, Loop.StripQuads, TriList, true);
		SetNormalsOnTriRegion(TriList);
	}
}



void FMeshBevel::ComputeUVs(FDynamicMesh3& Mesh)
{
	if (Mesh.HasAttributes() == false)
	{
		return;
	}
	FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->PrimaryUV();

	auto SetUVsOnTriRegion = [&Mesh, UVOverlay](const TArray<int32>& Triangles)
	{
		if (Triangles.Num() > 0)
		{
			UE::Geometry::ComputeArbitraryTrianglePatchUVs(Mesh, *UVOverlay, Triangles);
		}
	};


	TArray<int32> TriList;
	for (FBevelEdge& Edge : Edges)
	{
		UELocal::QuadsToTris(Mesh, Edge.StripQuads, TriList, true);
		SetUVsOnTriRegion(TriList);
	}
	for (FBevelLoop& Loop : Loops)
	{
		UELocal::QuadsToTris(Mesh, Loop.StripQuads, TriList, true);
		SetUVsOnTriRegion(TriList);
	}

	// do vertices last because until edges have UVs, the vertex polygons have no neighbour UVs islands
	for (FBevelVertex& Vertex : Vertices)
	{
		SetUVsOnTriRegion(Vertex.NewTriangles);
	}
}



void FMeshBevel::ComputeMaterialIDs(FDynamicMesh3& Mesh)
{
	if (Mesh.HasAttributes() == false || Mesh.Attributes()->HasMaterialID() == false)
	{
		return;
	}
	FDynamicMeshMaterialAttribute* MaterialIDs = Mesh.Attributes()->GetMaterialID();

	if (MaterialIDMode == EMaterialIDMode::ConstantMaterialID)
	{
		for (int32 tid : NewTriangles)
		{
			MaterialIDs->SetValue(tid, SetConstantMaterialID);
		}
	}
	else
	{
		auto SetQuadMaterial = [&Mesh, MaterialIDs](const FIndex2i& Quad, int32 MaterialID)
		{
			if (Quad.A >= 0)
			{
				MaterialIDs->SetValue(Quad.A, MaterialID);
			}
			if (Quad.B >= 0)
			{
				MaterialIDs->SetValue(Quad.B, MaterialID);
			}
		};

		// Try to set materials on the new triangles along a beveled edge based on the adjacent pre-bevel triangles.
		// Could be improved to at least be more consistent in ambiguous cases
		auto SetEdgeMaterials = [&Mesh, MaterialIDs, this, SetQuadMaterial](const TArray<FIndex2i>& StripQuads, const TArray<FIndex2i>& EdgeTris)
		{
			int32 NumEdges = EdgeTris.Num();
			if (StripQuads.Num() == NumEdges)
			{
				TArray<int32> SawMaterialIDs;
				TArray<int32> AmbiguousEdges;
				for (int32 k = 0; k < StripQuads.Num(); ++k)
				{
					FIndex2i NbrTris = EdgeTris[k];
					int MatIDA = MaterialIDs->GetValue(NbrTris.A);
					int MatIDB = (NbrTris.A >= 0) ? MaterialIDs->GetValue(NbrTris.B) : MatIDA;
					SawMaterialIDs.AddUnique(MatIDA);
					SawMaterialIDs.AddUnique(MatIDB);
					int SetMaterialID = (MatIDA == MatIDB) ? MatIDA : SetConstantMaterialID;
					if (MatIDA != MatIDB)
					{
						if (MaterialIDMode == EMaterialIDMode::InferMaterialID_ConstantIfAmbiguous)
						{
							SetMaterialID = SetConstantMaterialID;
						}
						else
						{
							AmbiguousEdges.Add(k);
						}
					}
					SetQuadMaterial(StripQuads[k], SetMaterialID);
				}

				if (AmbiguousEdges.Num() > 0)
				{
					SawMaterialIDs.Sort();
					if (AmbiguousEdges.Num() == NumEdges)		// if all ambigous, just pick one
					{
						for (int32 k : AmbiguousEdges)
						{
							SetQuadMaterial(StripQuads[k], SawMaterialIDs[0]);
						}
					}
					else
					{
						// TODO: what we probably want to do here is "infill" from known neighbours. 
						// for now we will just punt and pick one
						for (int32 k : AmbiguousEdges)
						{
							SetQuadMaterial(StripQuads[k], SawMaterialIDs[0]);
						}
					}
				}

			}
			else
			{
				for (const FIndex2i& Quad : StripQuads)
				{
					SetQuadMaterial(Quad, SetConstantMaterialID);
				}
			}
		};

		for (FBevelEdge& Edge : Edges)
		{
			SetEdgeMaterials(Edge.StripQuads, Edge.MeshEdgeTris);
		}
		for (FBevelLoop& Loop : Loops)
		{
			SetEdgeMaterials(Loop.StripQuads, Loop.MeshEdgeTris);
		}


		// find all the unique material IDs of neighbours of the Triangles list (that are not in Triangles list) and
		// return (MaterialID, NbrTriCount) tuples as a pair of lists
		auto CountUniqueBorderMaterialIDs = [&](const FDynamicMesh3& Mesh, const FDynamicMeshMaterialAttribute& MaterialAttrib, const TArray<int32>& Triangles, TArray<int32>& MaterialIDs, TArray<int32>& Counts)
		{
			MaterialIDs.Reset(); Counts.Reset();
			for (int32 tid : Triangles)
			{
				FIndex3i TriNbrs = Mesh.GetTriNeighbourTris(tid);
				for (int32 j = 0; j < 3; ++j)
				{
					int32 NbrTriangleID = TriNbrs[j];
					if ( Mesh.IsTriangle(NbrTriangleID) == false || Triangles.Contains(NbrTriangleID) )
					{
						continue;
					}
					int MatID = MaterialAttrib.GetValue(NbrTriangleID);
					int32 Index = MaterialIDs.AddUnique(MatID);
					if (Counts.Num() != MaterialIDs.Num())
					{
						Counts.Add(0);
						Counts[Index]++;
					}
				}
			}
		};

		// For each bevel-vertex-polygon, pick the nbr material ID that was most frequent.
		// Terminator vertices are also handled this way, which is not ideal, should probably
		// ignore the new 'edge' faces for the terminator vertex
		for (FBevelVertex& Vertex : Vertices)
		{
			TArray<int32> NbrMaterialIDs, NbrMaterialIDCounts;
			CountUniqueBorderMaterialIDs(Mesh, *MaterialIDs, Vertex.NewTriangles, NbrMaterialIDs, NbrMaterialIDCounts);
			int32 SetMaterialID = SetConstantMaterialID;
			if (NbrMaterialIDs.Num() > 0)
			{
				int32 MinIndex = 0;
				for (int32 k = 1; k < NbrMaterialIDs.Num(); ++k)
				{
					if (NbrMaterialIDCounts[k] < NbrMaterialIDCounts[MinIndex])
					{
						MinIndex = k;
					}
				}
				SetMaterialID = NbrMaterialIDs[MinIndex];
			}
			for (int32 tid : Vertex.NewTriangles)
			{
				MaterialIDs->SetValue(tid, SetMaterialID);
			}

		}
	}

}
