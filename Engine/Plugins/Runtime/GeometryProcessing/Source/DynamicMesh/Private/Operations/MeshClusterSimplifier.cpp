// Copyright Epic Games, Inc. All Rights Reserved.

//#include "Operations/UniformTessellate.h"
#include "Operations/MeshClusterSimplifier.h"

#include "Algo/MinElement.h"
#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "IndexTypes.h"
#include "VectorTypes.h"

#include <type_traits>

namespace UE::MeshClusterSimplifyLocals
{
	using namespace UE::Geometry::MeshClusterSimplify;
	using namespace UE::Geometry;

	using EElemTag = FSimplifyOptions::EConstraintLevel;
	using FVertexBuckets = TStaticArray<TArray<int32>, (uint32)EElemTag::MAX>;

	// Constraint levels are ordered from Most to Least constrained, so we can compare their integer values to determine constraint priority
	// @return true if constraint level of A is more constrained (lower value) vs the constraint level of B
	template<typename TagType>
	inline bool IsMoreConstrained(EElemTag A, TagType B)
	{
		// TagType should either be EElemTag or uint8
		static_assert(std::is_same_v<TagType, EElemTag> || std::is_same_v<TagType, uint8>);
		return (uint8)A < (uint8)B;
	}

	template<typename ElemType, int Dim, typename AttributeType>
	void CopyAttribs(AttributeType* Result, const AttributeType* Source, TConstArrayView<int32> ResToSource, int32 Num)
	{
		ParallelFor(Num, [&ResToSource, &Source, &Result](int32 ResID)
			{
				int32 SourceID = ResToSource[ResID];
				ElemType ToCopy[Dim];
				Source->GetValue(SourceID, ToCopy);
				Result->SetValue(ResID, ToCopy);
			}
		);
	}
	
	template<typename MeshType, typename MeshConnType>
	static void TagVerticesAndEdges(const MeshType& InMesh, const MeshConnType& InMeshConnectivity,
		TFunctionRef<EElemTag(int32 EID, EElemTag InitialTag)> GetEdgeConstraintLevel,
		TFunctionRef<bool(int32 VID)> IsSeamIntersectionVertex,
		const FSimplifyOptions& SimplifyOptions,
		TArray<EElemTag>& OutEdgeTags, TArray<EElemTag>& OutVertexTags)
	{
		// Compute an Edge ID -> Constraint Level mapping
		OutEdgeTags.SetNumUninitialized(InMeshConnectivity.MaxEdgeID());
		ParallelFor(InMeshConnectivity.MaxEdgeID(), [&InMesh, &InMeshConnectivity, &GetEdgeConstraintLevel, &SimplifyOptions, &OutEdgeTags](int32 EID)
			{
				if (!InMeshConnectivity.IsEdge(EID))
				{
					return;
				}

				EElemTag UseTag = EElemTag::Free;
				if (IsMoreConstrained(SimplifyOptions.PreserveEdges.Boundary, UseTag) && InMeshConnectivity.IsBoundaryEdge(EID))
				{
					UseTag = SimplifyOptions.PreserveEdges.Boundary;
				}
				UseTag = GetEdgeConstraintLevel(EID, UseTag);

				OutEdgeTags[EID] = UseTag;
			});

		OutVertexTags.SetNumUninitialized(InMesh.MaxVertexID());

		double CosBoundaryEdgeAngleTolerance = FMath::Cos(FMath::DegreesToRadians(FMath::Clamp(SimplifyOptions.FixBoundaryAngleTolerance, 0, 180)));

		ParallelFor(InMesh.MaxVertexID(),
			[&InMesh, &InMeshConnectivity, &SimplifyOptions,
			&OutVertexTags, &OutEdgeTags,
			&IsSeamIntersectionVertex,
			CosBoundaryEdgeAngleTolerance]
			(int32 VID)
			{
				if (!InMesh.IsVertex(VID))
				{
					return;
				}
				int32 FixedCount = 0;
				int32 ConstrainedCount = 0;

				FVector3d BoundaryEdgeVert[2];
				int32 FoundBoundaryEdgeVerts = 0;
				InMeshConnectivity.EnumerateVertexEdges(VID,
					[&OutEdgeTags, VID, &FixedCount, &ConstrainedCount, &BoundaryEdgeVert,
					&FoundBoundaryEdgeVerts, &SimplifyOptions, &InMesh, &InMeshConnectivity]
					(int32 EID)
					{
						FixedCount += int32(OutEdgeTags[EID] == EElemTag::Fixed);
						if (OutEdgeTags[EID] == EElemTag::Constrained)
						{
							ConstrainedCount++;
							if (SimplifyOptions.FixBoundaryAngleTolerance > 0)
							{
								if (InMeshConnectivity.IsBoundaryEdge(EID))
								{
									if (FoundBoundaryEdgeVerts < 2)
									{
										FIndex2i EdgeV = InMeshConnectivity.GetEdgeV(EID);
										int32 OtherV = EdgeV.A == VID ? EdgeV.B : EdgeV.A;

										BoundaryEdgeVert[FoundBoundaryEdgeVerts] = InMesh.GetVertex(OtherV);
									}
									FoundBoundaryEdgeVerts++;
								}
							}
						}
					}
				);


				if (FixedCount > 0)
				{
					OutVertexTags[VID] = EElemTag::Fixed;
					return;
				}
				if (FoundBoundaryEdgeVerts == 2)
				{
					FVector3d CenterV = InMesh.GetVertex(VID);
					FVector3d E1 = Normalized(BoundaryEdgeVert[0] - CenterV);
					FVector3d E2 = Normalized(CenterV - BoundaryEdgeVert[1]);
					if (E1.Dot(E2) < CosBoundaryEdgeAngleTolerance)
					{
						OutVertexTags[VID] = EElemTag::Fixed;
						return;
					}
				}
				if (ConstrainedCount > 0)
				{
					if (ConstrainedCount == 2 &&
						// seams are a special case where we can have two constrained edges but still be at a seam intersection 
						// (e.g. at a vertex that joins two different types of seam)
						!IsSeamIntersectionVertex(VID)
						)
					{
						// constrain vertices along contiguous constrained edge paths
						OutVertexTags[VID] = EElemTag::Constrained;
						return;
					}
					else
					{
						// fix vertices at constraint intersections
						OutVertexTags[VID] = EElemTag::Fixed;
						return;
					}
				}

				OutVertexTags[VID] = EElemTag::Free;
			});
	}

	template<typename MeshType>
	static TArray<float> InitTargetLengths(const MeshType& InMesh, float& OutMinTargetLength, const FSimplifyOptions& SimplifyOptions)
	{
		TArray<float> TargetLengths;
		OutMinTargetLength = FMathf::MaxReal;
		if (SimplifyOptions.OptionalTargetEdgeLengthScale)
		{
			TargetLengths.SetNumUninitialized(InMesh.MaxVertexID());
			TArray<float> MinLengthContexts;
			ParallelForWithTaskContext(MinLengthContexts, InMesh.MaxVertexID(),
				[](int32, int32) -> float {return FMathf::MaxReal;},
				[&InMesh, &TargetLengths, &SimplifyOptions](float& MinLength, int32 VID)
				{
					if (!InMesh.IsVertex(VID))
					{
						return;
					}
					float TargetLength = float(SimplifyOptions.TargetEdgeLength * SimplifyOptions.OptionalTargetEdgeLengthScale(VID));
					MinLength = FMath::Min(MinLength, TargetLength);
					TargetLengths[VID] = TargetLength;
				}
			);
			OutMinTargetLength = *Algo::MinElement(MinLengthContexts);
		}

		return TargetLengths;
	}

	template<typename MeshType, typename MeshConnType>
	void ClusterVerticesByRegionGrowth(const MeshType& InMesh, const MeshConnType& InMeshConnectivity, TArray<int32>& Source, TArray<float>& SourceDist, const FSimplifyOptions& SimplifyOptions,
		const TArray<EElemTag>& EdgeTags, const TArray<EElemTag>& VertexTags, const TArray<float>& TargetLengths, float MinTargetLength,
		const FVertexBuckets& VertexIDBuckets)
	{
		// add all the fixed vertices as sources first, so they can't be claimed by other verts
		for (int32 VID : VertexIDBuckets[(int32)EElemTag::Fixed])
		{
			Source[VID] = VID;
			SourceDist[VID] = 0.f;
		}

		struct FWalk
		{
			int32 VID;
			float Dist;

			bool operator<(const FWalk& Other) const
			{
				return Dist < Other.Dist;
			}
		};
		TArray<FWalk> HeapV;

		// process buckets in two phases:
		// first, starting from fixed/constrained verts, and disallowing free->(fixed/constrained) walks
		// second, starting-from and walking-to remaining unclaimed free vertices
		uint8 BucketIdx = 0;
		for (uint8 TagIdx = 1; TagIdx < (uint8)EElemTag::MAX; ++TagIdx)
		{
			for (; BucketIdx <= TagIdx; ++BucketIdx)
			{
				const TArray<int32>& CurBucket = VertexIDBuckets[BucketIdx];
				for (int32 InBucketIdx = 0; InBucketIdx < CurBucket.Num(); ++InBucketIdx)
				{
					int32 GrowFromVID = CurBucket[InBucketIdx];

					int32& CurSourceVID = Source[GrowFromVID];
					float& CurSourceDist = SourceDist[GrowFromVID];
					// the vertex is fixed or unclaimed, claim it as a new source/kept vertex
					if (BucketIdx == (uint8)EElemTag::Fixed || CurSourceVID == INDEX_NONE)
					{
						CurSourceVID = GrowFromVID;
						CurSourceDist = 0.f;
					}
					// the vertex was claimed by another source, no need to process it further
					else
					{
						continue;
					}

					float CurrentMaxDist = TargetLengths.IsEmpty() ? (float)SimplifyOptions.TargetEdgeLength : TargetLengths[CurSourceVID];

					// if we're on the unconstrained (free) vertex bucket, don't need to check vertex/edge tags anymore
					bool bIsFreePass = TagIdx == (uint8)EElemTag::Free;

					// helper to add candidate verts to a heap
					HeapV.Reset();
					auto AddCandidates = [
						&HeapV, &InMesh, &InMeshConnectivity, &SourceDist, &Source, &EdgeTags, &VertexTags, TagIdx, bIsFreePass]
						(const FWalk& From, float UseMaxDist)
						{
							uint8 FromTagIdx = bIsFreePass ? (uint8)EElemTag::Free : (uint8)VertexTags[From.VID];
							// expand to one-ring
							InMeshConnectivity.EnumerateVertexEdges(From.VID,
								[&From, UseMaxDist,
								&HeapV, &InMesh, &InMeshConnectivity, &SourceDist, &Source, &EdgeTags, &VertexTags, TagIdx, FromTagIdx, bIsFreePass]
								(int32 EID)
								{
									FIndex2i EdgeV = InMeshConnectivity.GetEdgeV(EID);
									int32 ToVID = EdgeV.A == From.VID ? EdgeV.B : EdgeV.A;
									
									uint8 UseFromTagIdx = FromTagIdx;
									// in free pass we're only visiting free verts which only have free edges by construction
									// but in constrained pass, we need to also test the tag on the edge
									if (!bIsFreePass)
									{
										uint8 EdgeTagIdx = (uint8)EdgeTags[EID];
										// can't go from (fixed/constrained)->constrained through an unconstrained edge
										UseFromTagIdx = FMath::Max(EdgeTagIdx, FromTagIdx);
									}
									if (IsMoreConstrained(VertexTags[ToVID], UseFromTagIdx) || From.Dist >= SourceDist[ToVID])
									{
										// vertex was already claimed by more-constrained context, or is already as close (or closer) to another source
										return;
									}
									// possible candidate, compute the actual distance and grow if close enough
									FVector3d Pos = InMesh.GetVertex(ToVID);
									FVector3d FromPos = InMesh.GetVertex(From.VID);
									float NewDist = From.Dist + (float)FVector3d::Dist(Pos, FromPos);
									if (NewDist < UseMaxDist && NewDist < SourceDist[ToVID])
									{
										// Viable candidate distance; add to heap
										HeapV.HeapPush(FWalk{ ToVID, NewDist });
									}
								}
							);
						};

					// initialize the heap w/ the neighbors of the initial grow-from vertex
					FWalk Start{ GrowFromVID, CurSourceDist };
					AddCandidates(Start, CurrentMaxDist);

					// In uniform target length case, or if we're already at the min target length, can skip the MaxDist updates
					bool bNeedToUpdateMaxDist = !TargetLengths.IsEmpty() && CurrentMaxDist > MinTargetLength;

					while (!HeapV.IsEmpty())
					{
						FWalk CurWalk;
						HeapV.HeapPop(CurWalk, EAllowShrinking::No);

						// If we have smaller TargetLengths elsewhere, test if we are walking over
						// a smaller length and clamp max dist + stop walking if needed
						if (bNeedToUpdateMaxDist)
						{
							CurrentMaxDist = FMath::Min(CurrentMaxDist, TargetLengths[CurWalk.VID]);
							
							if (CurWalk.Dist >= CurrentMaxDist)
							{
								break;
							}
						}

						// we already got to this vert from another place
						if (SourceDist[CurWalk.VID] <= CurWalk.Dist)
						{
							continue;
						}

						// claim the vertex
						SourceDist[CurWalk.VID] = CurWalk.Dist;
						Source[CurWalk.VID] = CurSourceVID;

						// search its (current-tag-level) edges for more verts to claim
						AddCandidates(CurWalk, CurrentMaxDist);
					}
				}
			}
		}
	}

	template<typename MeshType>
	static void FindClustersInOutput(const MeshType& InMesh, const TArray<int32>& Source, TArray<bool>& OutClusterInOutput)
	{
		OutClusterInOutput.Reset();
		OutClusterInOutput.SetNumZeroed(InMesh.MaxVertexID());
		ParallelFor(InMesh.MaxTriangleID(), [&InMesh, &Source, &OutClusterInOutput](int32 TID)
			{
				if (!InMesh.IsTriangle(TID))
				{
					return;
				}
				FIndex3i Tri = InMesh.GetTriangle(TID);
				FIndex3i SourceTri(Source[Tri.A], Source[Tri.B], Source[Tri.C]);
				if (SourceTri.A != SourceTri.B && SourceTri.A != SourceTri.C && SourceTri.B != SourceTri.C)
				{
					for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
					{
						OutClusterInOutput[Source[Tri[SubIdx]]] = true;
					}
				}
			}
		);
	}

	template<typename MeshType, typename MeshConnType>
	static void PreserveCollapsedClusters(const MeshType& InMesh, const MeshConnType& InMeshConnectivity,
		TArray<bool>& ClusterInOutput,
		TArray<int32>& Source, TArray<float>& SourceDist, const FSimplifyOptions& SimplifyOptions,
		const TArray<EElemTag>& EdgeTags, const TArray<EElemTag>& VertexTags, TArray<float>& TargetLengths, float MinTargetLength,
		const FVertexBuckets& VertexIDBuckets)
	{
		for (int32 PassIdx = 0; PassIdx < SimplifyOptions.MaxPreserveCollapsedClusterPasses; ++PassIdx)
		{
			// factor used to set new Target Length for collapsed clusters, as a factor of max distance
			constexpr float TargetLengthAsMaxClusterLengthFactor = .5f;
			
			TArray<int32> NeedRecluster;
			TMap<int32, int32> VIDtoNeedReclusterIdx;
			for (int32 VID = 0, MaxVID = InMesh.MaxVertexID(); VID < MaxVID; ++VID)
			{
				if (InMesh.IsVertex(VID) && VID == Source[VID] && !ClusterInOutput[VID])
				{
					VIDtoNeedReclusterIdx.Add(VID, NeedRecluster.Add(VID));
				}
			}
			if (NeedRecluster.IsEmpty())
			{
				break;
			}

			// Create buckets for vertices that will need re-processing due to not having any triangles
			FVertexBuckets ReProcessBuckets;
			// Make sure target lengths exist, so we can reduce them as needed for re-clustering
			if (TargetLengths.IsEmpty())
			{
				TargetLengths.Init((float)SimplifyOptions.TargetEdgeLength, InMesh.MaxVertexID());
			}

			TArray<float> MaxSourceDistInCluster;
			MaxSourceDistInCluster.SetNumZeroed(NeedRecluster.Num());
			for (int32 VID = 0, MaxVID = InMesh.MaxVertexID(); VID < MaxVID; ++VID)
			{
				if (!InMesh.IsVertex(VID))
				{
					continue;
				}
				int32 SourceVID = Source[VID];
				if (!ClusterInOutput[SourceVID]) // vertex belongs to a cluster that we need to reprocess
				{
					// update the max distance for the cluster
					float& MaxFoundClusterDist = MaxSourceDistInCluster[VIDtoNeedReclusterIdx[SourceVID]];
					MaxFoundClusterDist = FMath::Max(MaxFoundClusterDist, SourceDist[VID]);
					ReProcessBuckets[(int32)VertexTags[VID]].Add(VID);
				}
			}
			// For all re-process vertices, set down-scaled target lengths, and clear source/sourcedist
			for (int32 BucketIdx = 0; BucketIdx < ReProcessBuckets.Num(); ++BucketIdx)
			{
				for (int32 Idx = 0; Idx < ReProcessBuckets[BucketIdx].Num(); ++Idx)
				{
					int32 UpdateVID = ReProcessBuckets[BucketIdx][Idx];
					int32 SourceVID = Source[UpdateVID];
					float ClusterMaxDist = MaxSourceDistInCluster[VIDtoNeedReclusterIdx[SourceVID]];
					TargetLengths[UpdateVID] = ClusterMaxDist * TargetLengthAsMaxClusterLengthFactor;
					Source[UpdateVID] = INDEX_NONE;
					SourceDist[UpdateVID] = FMathf::MaxReal;
				}
			}
			// Re-run tagging
			ClusterVerticesByRegionGrowth(InMesh, InMeshConnectivity, Source, SourceDist, SimplifyOptions, EdgeTags, VertexTags, TargetLengths, MinTargetLength, ReProcessBuckets);
			// Update tracking of clusters in output
			FindClustersInOutput(InMesh, Source, ClusterInOutput);
		}
	}

	template<typename InMeshType, typename InMeshConnType, typename ResultMeshType>
	void BuildSimplifiedMeshFromClusters(ResultMeshType& ResultMesh, TArray<int32>& FromResVID, TArray<int32>& ResultToSourceTri,
		TArray<bool>& ClusterInOutput,
		bool& bOutResultHasDuplicateVertices,
		const InMeshType& InMesh, const InMeshConnType& InMeshConnectivity, TArray<int32>& Source, TArray<float>& SourceDist, const FSimplifyOptions& SimplifyOptions,
		const TArray<EElemTag>& EdgeTags, TArray<EElemTag>& VertexTags, const TArray<float>& TargetLengths, float MinTargetLength,
		FVertexBuckets& ProcessBuckets)
	{
		TArray<int32> ToResVID;

		// If simplification introduces non-manifold edges, we can often recover by fixing more vertices and re-attempting the build.
		// After MeshBuildAttempts tries, if still failing, we stop adding vertices and just duplicate vertices to add the non-manifold triangles.
		// TODO: We could potentially analyze the cluster connectivity more carefully handle more degenerate cluster connectivity, more robustly.
		//		(if so -- it may be better to do so by analyzing the graph before building the ResultMesh, rather than this rebuilding approach!)
		int32 MeshBuildAttempts = 2;

		while (MeshBuildAttempts-- > 0)
		{
			// clear mesh outputs
			ToResVID.Reset();
			FromResVID.Reset();
			ResultToSourceTri.Reset();
			ResultMesh.Clear();

			bool bAllowDegenerate = MeshBuildAttempts <= 0;
			// Array of vertex IDs to set to 'fixed' on a rebuild attempt
			TArray<int32> SourceVIDToFix;

			ToResVID.Init(INDEX_NONE, InMesh.MaxVertexID());
			for (int32 VID = 0; VID < Source.Num(); ++VID)
			{
				if (Source[VID] == VID && ClusterInOutput[VID])
				{
					ToResVID[VID] = ResultMesh.AppendVertex(InMesh.GetVertex(VID));
					// we need the reverse mapping if we're transferring seams
					if (SimplifyOptions.bTransferAttributes)
					{
						FromResVID.Add(VID);
					}
				}
			}

			for (int32 TID = 0, MaxTID = InMesh.MaxTriangleID(); TID < MaxTID; ++TID)
			{
				if (!InMesh.IsTriangle(TID))
				{
					continue;
				}
				FIndex3i Tri = InMesh.GetTriangle(TID);
				FIndex3i SourceTri(Source[Tri.A], Source[Tri.B], Source[Tri.C]);
				if (SourceTri.A != SourceTri.B && SourceTri.A != SourceTri.C && SourceTri.B != SourceTri.C)
				{
					FIndex3i ResTri(ToResVID[SourceTri.A], ToResVID[SourceTri.B], ToResVID[SourceTri.C]);
					int32 ResultTID = ResultMesh.AppendTriangle(ResTri);
					if (ResultTID == FDynamicMesh3::NonManifoldID)
					{
						if (bAllowDegenerate)
						{
							// TODO: only duplicate vertices on the non-manifold edge(s)
							FIndex3i ExtraTri;
							ExtraTri.A = ResultMesh.AppendVertex(ResultMesh.GetVertex(ResTri.A));
							FromResVID.Add(SourceTri.A);
							ExtraTri.B = ResultMesh.AppendVertex(ResultMesh.GetVertex(ResTri.B));
							FromResVID.Add(SourceTri.B);
							ExtraTri.C = ResultMesh.AppendVertex(ResultMesh.GetVertex(ResTri.C));
							FromResVID.Add(SourceTri.C);
							ResultTID = ResultMesh.AppendTriangle(ExtraTri);
							bOutResultHasDuplicateVertices = true;
						}
						else
						{
							// Non-manifold edges can often be resolved by adding an extra vertex --
							// mark the vertex with largest SourceDist for inclusion in the result mesh
							int32 BestSubIdx = INDEX_NONE;
							float BestDist = 0;
							for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
							{
								if (SourceDist[Tri[SubIdx]] > BestDist)
								{
									BestDist = SourceDist[Tri[SubIdx]];
									BestSubIdx = SubIdx;
								}
							}
							if (BestSubIdx != INDEX_NONE)
							{
								SourceVIDToFix.Add(Tri[BestSubIdx]);
							}
						}
					}
					if ((SimplifyOptions.bTransferAttributes || SimplifyOptions.bTransferGroups) && ResultTID >= 0)
					{
						checkSlow(ResultTID == ResultToSourceTri.Num()); // ResultMesh starts empty and should be compact
						ResultToSourceTri.Add(TID);
					}
				}
			}

			// We marked some new vertices for inclusion in the result; tag them and re-try
			if (!bAllowDegenerate && SourceVIDToFix.Num() > 0)
			{
				for (int32 VID : SourceVIDToFix)
				{
					VertexTags[VID] = EElemTag::Fixed;
				}
				ProcessBuckets[0] = MoveTemp(SourceVIDToFix);
				ClusterVerticesByRegionGrowth(InMesh, InMeshConnectivity, Source, SourceDist, SimplifyOptions, EdgeTags, VertexTags, TargetLengths, MinTargetLength, ProcessBuckets);
				FindClustersInOutput(InMesh, Source, ClusterInOutput);
				continue;
			}

			// Accept the result mesh triangulation
			break;
		}
	}
}

namespace UE::Geometry::MeshClusterSimplify
{

bool Simplify(const FDynamicMesh3& InMesh, FDynamicMesh3& ResultMesh, const FSimplifyOptions& SimplifyOptions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshClusterSimplify::Simplify);

	using namespace UE::MeshClusterSimplifyLocals;

	// we build the result mesh by incrementally copying from the input mesh, so they shouldn't be the same mesh
	if (!ensure(&ResultMesh != &InMesh))
	{
		return false;
	}

	ResultMesh.Clear();
	
	const FDynamicMeshAttributeSet* InAttribs = InMesh.Attributes();

	// We tag edges and vertices w/ the constraint level, abbreviated to EElemTag for convenience
	using EElemTag = FSimplifyOptions::EConstraintLevel;

	// TODO: optionally also compute some vertex curvature feature & sort by it, to favor capturing less flat parts of the input shape?

	///
	/// Step 1, Data Prep: Translate all mesh constraint options to simple per-edge and per-vertex tags, so we know what to try to especially preserve in the result
	///
	
	auto GetEdgeConstraintLevelFromAttributes =
		[InAttribs, &InMesh, &SimplifyOptions]
		(int32 EID, EElemTag UseTag) -> EElemTag
		{
			if (IsMoreConstrained(SimplifyOptions.PreserveEdges.PolyGroup, UseTag) && InMesh.IsGroupBoundaryEdge(EID))
			{
				UseTag = SimplifyOptions.PreserveEdges.PolyGroup;
			}

			if (InAttribs)
			{
				if (IsMoreConstrained(SimplifyOptions.PreserveEdges.Material, UseTag) && InAttribs->IsMaterialBoundaryEdge(EID))
				{
					UseTag = SimplifyOptions.PreserveEdges.Material;
				}

				if (IsMoreConstrained(SimplifyOptions.PreserveEdges.UVSeam, UseTag))
				{
					for (int32 UVLayer = 0; UVLayer < InAttribs->NumUVLayers(); ++UVLayer)
					{
						if (InAttribs->GetUVLayer(UVLayer)->IsSeamEdge(EID))
						{
							UseTag = SimplifyOptions.PreserveEdges.UVSeam;
							break;
						}
					}
				}
				if (IsMoreConstrained(SimplifyOptions.PreserveEdges.TangentSeam, UseTag))
				{
					for (int32 NormalLayer = 1; NormalLayer < InAttribs->NumNormalLayers(); ++NormalLayer)
					{
						if (InAttribs->GetNormalLayer(NormalLayer)->IsSeamEdge(EID))
						{
							UseTag = SimplifyOptions.PreserveEdges.TangentSeam;
							break;
						}
					}
				}
				if (IsMoreConstrained(SimplifyOptions.PreserveEdges.NormalSeam, UseTag))
				{
					if (const FDynamicMeshNormalOverlay* Normals = InAttribs->PrimaryNormals())
					{
						if (Normals->IsSeamEdge(EID))
						{
							UseTag = SimplifyOptions.PreserveEdges.NormalSeam;
						}
					}
				}
				if (IsMoreConstrained(SimplifyOptions.PreserveEdges.ColorSeam, UseTag))
				{
					if (const FDynamicMeshColorOverlay* Colors = InAttribs->PrimaryColors())
					{
						if (Colors->IsSeamEdge(EID))
						{
							UseTag = SimplifyOptions.PreserveEdges.ColorSeam;
						}
					}
				}
			}

			return UseTag;
		};
	auto IsSeamIntersectionVertex =
		[&InAttribs, &SimplifyOptions]
		(int32 VID) -> bool
		{
			if (!InAttribs)
			{
				return false;
			}

			if (SimplifyOptions.PreserveEdges.UVSeam == EElemTag::Constrained)
			{
				for (int32 Layer = 0; Layer < InAttribs->NumUVLayers(); ++Layer)
				{
					if (InAttribs->GetUVLayer(Layer)->IsSeamIntersectionVertex(VID))
					{
						return true;
					}
				}
			}
			if (SimplifyOptions.PreserveEdges.NormalSeam == EElemTag::Constrained)
			{
				if (const FDynamicMeshNormalOverlay* Normals = InAttribs->PrimaryNormals())
				{
					if (Normals->IsSeamIntersectionVertex(VID))
					{
						return true;
					}
				}
			}
			if (SimplifyOptions.PreserveEdges.TangentSeam == EElemTag::Constrained)
			{
				for (int32 Layer = 1; Layer < InAttribs->NumNormalLayers(); ++Layer)
				{
					if (InAttribs->GetNormalLayer(Layer)->IsSeamIntersectionVertex(VID))
					{
						return true;
					}
				}
			}
			if (SimplifyOptions.PreserveEdges.ColorSeam == EElemTag::Constrained)
			{
				if (const FDynamicMeshColorOverlay* Colors = InAttribs->PrimaryColors())
				{
					if (Colors->IsSeamIntersectionVertex(VID))
					{
						return true;
					}
				}
			}

			return false;
		};
	
	TArray<EElemTag> EdgeTags, VertexTags;
	TagVerticesAndEdges(InMesh, InMesh, 
		GetEdgeConstraintLevelFromAttributes, IsSeamIntersectionVertex,
		SimplifyOptions, EdgeTags, VertexTags);

	float MinTargetLength;
	TArray<float> TargetLengths = InitTargetLengths(InMesh, MinTargetLength, SimplifyOptions);

	///
	/// Step 2. Clustering: Grow vertex clusters out to the target edge length size
	///
	
	// Buckets of vertices to process -- vertices that are processed sooner are more likely to be directly included in the output
	FVertexBuckets ProcessBuckets;
	for (int32 VID : InMesh.VertexIndicesItr())
	{
		ProcessBuckets[(int32)VertexTags[VID]].Add(VID);
	}

	TArray<float> SourceDist;
	TArray<int32> Source;
	Source.Init(INDEX_NONE, InMesh.MaxVertexID());
	SourceDist.Init(FMathf::MaxReal, InMesh.MaxVertexID());

	ClusterVerticesByRegionGrowth(InMesh, InMesh, Source, SourceDist, SimplifyOptions, EdgeTags, VertexTags, TargetLengths, MinTargetLength, ProcessBuckets);
	for (int32 Idx = 0; Idx < ProcessBuckets.Num(); ++Idx)
	{
		ProcessBuckets[Idx].Empty();
	}
	
	/// Step 2.5: Determine which clusters are in the output, and optionally try reduce target lengths to preserve missing clusters
	TArray<bool> ClusterInOutput;
	FindClustersInOutput(InMesh, Source, ClusterInOutput);
	PreserveCollapsedClusters(InMesh, InMesh, ClusterInOutput, Source, SourceDist, SimplifyOptions, EdgeTags, VertexTags, TargetLengths, MinTargetLength, ProcessBuckets);

	///
	/// Step 3: Copy the cluster connectivity out to our ResultMesh
	///

	TArray<int32> FromResVID, ResultToSourceTri;
	bool bResultHasDuplicateVertices = false;
	BuildSimplifiedMeshFromClusters<FDynamicMesh3, FDynamicMesh3, FDynamicMesh3>(ResultMesh, FromResVID, ResultToSourceTri, ClusterInOutput, bResultHasDuplicateVertices,
		InMesh, InMesh, Source, SourceDist, SimplifyOptions, EdgeTags, VertexTags, TargetLengths, MinTargetLength,
		ProcessBuckets);

	///
	/// Step 4: After accepting the final ResultMesh triangulation, copy the input mesh's attributes (UVs, materials, etc) over as well
	/// 

	if (SimplifyOptions.bTransferAttributes)
	{
		ResultMesh.EnableMatchingAttributes(InMesh);

		if (InMesh.HasAttributes())
		{
			FDynamicMeshAttributeSet* ResultAttribs = ResultMesh.Attributes();

			const bool bPreserveAnySeams =
				SimplifyOptions.PreserveEdges.UVSeam != EElemTag::Free ||
				SimplifyOptions.PreserveEdges.NormalSeam != EElemTag::Free ||
				SimplifyOptions.PreserveEdges.TangentSeam != EElemTag::Free ||
				SimplifyOptions.PreserveEdges.ColorSeam != EElemTag::Free;

			// Seam mapping for overlays
			{
				// Compute a general wedge mapping that all the overlays can build from

				// Map from ResultTID -> a source triangle per tri-vertex [aka wedge]
				TArray<FIndex3i> ResultWedgeSourceTris;
				// sub-indices per wedge
				TArray<int8> SourceTriWedgeSubIndices;
				ResultWedgeSourceTris.SetNumUninitialized(ResultMesh.MaxTriangleID());
				SourceTriWedgeSubIndices.SetNumUninitialized(ResultMesh.MaxTriangleID() * 3);
				ParallelFor(ResultMesh.MaxTriangleID(), 
					[&ResultMesh, &ResultWedgeSourceTris, &SourceTriWedgeSubIndices,
					bPreserveAnySeams, &FromResVID, &ResultToSourceTri, &Source,
					&VertexTags, &EdgeTags, &InMesh]
					(int32 ResultTID)
					{
						TArray<int32> TriQ;
						TSet<int32> LocalSeenTris;
						FIndex3i ResultVIDs = ResultMesh.GetTriangle(ResultTID);
						for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
						{
							int32 ResultVID = ResultVIDs[SubIdx];
							int32 SourceVID = FromResVID[ResultVID];
							bool bFound = false;

							// we're on a seam vertex, do a local search (w/out crossing seam edges) to from the init triangle to the source vertex
							// to try to find the best tri to use as a wedge reference
							if (VertexTags[SourceVID] != EElemTag::Free && bPreserveAnySeams)
							{
								TriQ.Reset();
								LocalSeenTris.Reset();
								int32 SourceTID = ResultToSourceTri[ResultTID];
								TriQ.Add(SourceTID);
								while (!TriQ.IsEmpty())
								{
									int32 SearchTID = TriQ.Pop(EAllowShrinking::No);

									if (LocalSeenTris.Contains(SearchTID))
									{
										continue;
									}
									LocalSeenTris.Add(SearchTID);

									FIndex3i Tri = InMesh.GetTriangle(SearchTID);
									int32 FoundSubIdx = Tri.IndexOf(SourceVID);
									if (FoundSubIdx != INDEX_NONE)
									{
										bFound = true;
										ResultWedgeSourceTris[ResultTID][SubIdx] = SearchTID;
										SourceTriWedgeSubIndices[ResultTID * 3 + SubIdx] = (int8)FoundSubIdx;
										break;
									}

									// check we're still on a valid triangle that has a vert tagged w/ our source VID
									FIndex3i SourceTri(Source[Tri.A], Source[Tri.B], Source[Tri.C]);
									if (!SourceTri.Contains(SourceVID))
									{
										continue;
									}
									FIndex3i TriEdges = InMesh.GetTriEdges(SearchTID);
									for (int32 EdgeSubIdx = 0; EdgeSubIdx < 3; ++EdgeSubIdx)
									{
										int32 WalkSourceEID = TriEdges[EdgeSubIdx];
										if (EdgeTags[WalkSourceEID] == EElemTag::Free)
										{
											FIndex2i EdgeT = InMesh.GetEdgeT(WalkSourceEID);
											int32 WalkTID = EdgeT.A == SearchTID ? EdgeT.B : EdgeT.A;
											if (WalkTID != INDEX_NONE)
											{
												TriQ.Add(WalkTID);
											}
										}
									}
								}
							}

							if (!bFound)
							{
								// no seams, or search failed; just grab any triangle
								int32 NbrTID = *InMesh.VtxTrianglesItr(SourceVID).begin();
								checkSlow(NbrTID != INDEX_NONE); // should not be possible for a vert w/ no neighbors to end up as a source VID
								ResultWedgeSourceTris[ResultTID][SubIdx] = NbrTID;
								SourceTriWedgeSubIndices[ResultTID * 3 + SubIdx] = (int8)InMesh.GetTriangle(NbrTID).IndexOf(SourceVID);
							}
						}
					}
				);

				// Helper to use the general wedge mapping to copy elements for a given overlay
				auto OverlayTransfer = 
					[&ResultMesh, &ResultWedgeSourceTris, &SourceTriWedgeSubIndices,
					bResultHasDuplicateVertices]
					<typename OverlayType>
					(OverlayType* ResultOverlay, const OverlayType* SourceOverlay)
				{
					TArray<int32> SourceToResElID;
					SourceToResElID.Init(INDEX_NONE, SourceOverlay->MaxElementID());

					// Note: Unfortunately can't parallelize this part easily; the overlay append and set both are not thread safe (due to ref counts)
					for (int32 ResultTID : ResultMesh.TriangleIndicesItr())
					{
						FIndex3i ResultElemTri;
						bool bHasUnsetSources = false;
						for (int32 ResultSubIdx = 0; ResultSubIdx < 3; ++ResultSubIdx)
						{
							int32 SourceTID = ResultWedgeSourceTris[ResultTID][ResultSubIdx];
							int8 SourceSubIdx = SourceTriWedgeSubIndices[ResultTID * 3 + ResultSubIdx];
							int32 SourceElemID = SourceOverlay->GetTriangle(SourceTID)[SourceSubIdx];
							if (SourceElemID == INDEX_NONE)
							{
								// if we mapped to an unset triangle in the source overlay, there is no element to copy
								// we do not support partially-set triangles, so the whole result triangle will also be unset in this case
								bHasUnsetSources = true;
								break;
							}
							int32 UseElemID;
							if (SourceToResElID[SourceElemID] == INDEX_NONE)
							{
								SourceToResElID[SourceElemID] = ResultOverlay->AppendElement(SourceOverlay->GetElement(SourceElemID));
								UseElemID = SourceToResElID[SourceElemID];
							}
							else
							{
								UseElemID = SourceToResElID[SourceElemID];
								// if we have duplicate vertices, may need to also duplicate the element
								if (bResultHasDuplicateVertices)
								{
									if (ResultOverlay->GetParentVertex(UseElemID) != ResultMesh.GetTriangle(ResultTID)[ResultSubIdx])
									{
										UseElemID = ResultOverlay->AppendElement(SourceOverlay->GetElement(SourceElemID));
									}
								}
							}
							ResultElemTri[ResultSubIdx] = UseElemID;
						}

						if (!bHasUnsetSources)
						{
							ResultOverlay->SetTriangle(ResultTID, ResultElemTri);
						}
					}
				};

				for (int32 LayerIdx = 0; LayerIdx < InAttribs->NumUVLayers(); ++LayerIdx)
				{
					FDynamicMeshUVOverlay* ResultUVs = ResultAttribs->GetUVLayer(LayerIdx);
					const FDynamicMeshUVOverlay* SourceUVs = InAttribs->GetUVLayer(LayerIdx);
					OverlayTransfer(ResultUVs, SourceUVs);
				}

				for (int32 LayerIdx = 0; LayerIdx < InAttribs->NumNormalLayers(); ++LayerIdx)
				{
					OverlayTransfer(ResultAttribs->GetNormalLayer(LayerIdx), InAttribs->GetNormalLayer(LayerIdx));
				}

				if (InAttribs->HasPrimaryColors())
				{
					OverlayTransfer(ResultAttribs->PrimaryColors(), InAttribs->PrimaryColors());
				}
			}

			for (int32 WeightLayerIdx = 0; WeightLayerIdx < InAttribs->NumWeightLayers(); ++WeightLayerIdx)
			{
				UE::MeshClusterSimplifyLocals::CopyAttribs<float, 1>(
					ResultAttribs->GetWeightLayer(WeightLayerIdx),
					InAttribs->GetWeightLayer(WeightLayerIdx),
					FromResVID, ResultMesh.MaxVertexID()
				);
			}

			for (int32 SculptLayerIdx = 0; SculptLayerIdx < InAttribs->NumSculptLayers(); ++SculptLayerIdx)
			{
				UE::MeshClusterSimplifyLocals::CopyAttribs<double, 3>(
					ResultAttribs->GetSculptLayers()->GetLayer(SculptLayerIdx),
					InAttribs->GetSculptLayers()->GetLayer(SculptLayerIdx),
					FromResVID, ResultMesh.MaxVertexID()
				);
			}

			for (int32 GroupLayerIdx = 0; GroupLayerIdx < InAttribs->NumPolygroupLayers(); ++GroupLayerIdx)
			{
				UE::MeshClusterSimplifyLocals::CopyAttribs<int32, 1>(
					ResultAttribs->GetPolygroupLayer(GroupLayerIdx),
					InAttribs->GetPolygroupLayer(GroupLayerIdx),
					ResultToSourceTri, ResultMesh.MaxTriangleID()
				);
			}

			if (const FDynamicMeshMaterialAttribute* InMats = InAttribs->GetMaterialID())
			{
				UE::MeshClusterSimplifyLocals::CopyAttribs<int32, 1>(
					ResultAttribs->GetMaterialID(),
					InMats,
					ResultToSourceTri, ResultMesh.MaxTriangleID()
				);
			}
		}
	}

	if (SimplifyOptions.bTransferGroups && InMesh.HasTriangleGroups())
	{
		ResultMesh.EnableTriangleGroups();
		ParallelFor(ResultMesh.MaxTriangleID(),
			[&ResultMesh, &InMesh, &ResultToSourceTri]
			(int32 ResultTID)
			{
				checkSlow(ResultMesh.IsTriangle(ResultTID)); // ResultMesh is compact so all tris should be valid
				int32 SourceTID = ResultToSourceTri[ResultTID];
				ResultMesh.SetTriangleGroup(ResultTID, InMesh.GetTriangleGroup(SourceTID));
			}
		);
	}

	return true;
}

bool Simplify(const FTriangleMeshAdapterd& InMesh, FResultMeshAdapter& OutSimplifiedMesh, const FSimplifyOptions& SimplifyOptions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshClusterSimplify::Simplify);

	using namespace UE::MeshClusterSimplifyLocals;

	// We tag edges and vertices w/ the constraint level, abbreviated to EElemTag for convenience
	using EElemTag = FSimplifyOptions::EConstraintLevel;



	///
	/// Step 1, Data Prep: Translate all mesh constraint options to simple per-edge and per-vertex tags, so we know what to try to especially preserve in the result
	///
	
	using FMeshConnectivity = TTriangleMeshAdapterEdgeConnectivity<TTriangleMeshAdapter<double>>;
	FMeshConnectivity MeshConnectivity(&InMesh);

	auto TagEdgeFromAttributes = [](int32 EID, EElemTag InitialTag)->EElemTag
		{
			return InitialTag;
		};
	auto IsSeamIntersectionVertex = [](int32 VID)->bool
		{
			return false;
		};

	TArray<EElemTag> EdgeTags, VertexTags;
	TagVerticesAndEdges(InMesh, MeshConnectivity,
		TagEdgeFromAttributes, IsSeamIntersectionVertex,
		SimplifyOptions, EdgeTags, VertexTags);

	float MinTargetLength;
	TArray<float> TargetLengths = InitTargetLengths(InMesh, MinTargetLength, SimplifyOptions);

	///
	/// Step 2. Clustering: Grow vertex clusters out to the target edge length size
	///

	// Buckets of vertices to process -- vertices that are processed sooner are more likely to be directly included in the output
	FVertexBuckets ProcessBuckets;
	for (int32 VID = 0, MaxVID = InMesh.MaxVertexID(); VID < MaxVID; ++VID)
	{
		if (InMesh.IsVertex(VID))
		{
			ProcessBuckets[(int32)VertexTags[VID]].Add(VID);
		}
	}

	TArray<float> SourceDist;
	TArray<int32> Source;
	Source.Init(INDEX_NONE, InMesh.MaxVertexID());
	SourceDist.Init(FMathf::MaxReal, InMesh.MaxVertexID());

	ClusterVerticesByRegionGrowth(InMesh, MeshConnectivity, Source, SourceDist, SimplifyOptions, EdgeTags, VertexTags, TargetLengths, MinTargetLength, ProcessBuckets);
	for (int32 Idx = 0; Idx < ProcessBuckets.Num(); ++Idx)
	{
		ProcessBuckets[Idx].Empty();
	}


	/// Step 2.5: Optionally look for clusters that didn't make it into triangles
	TArray<bool> ClusterInOutput;
	FindClustersInOutput(InMesh, Source, ClusterInOutput);
	PreserveCollapsedClusters(InMesh, MeshConnectivity, ClusterInOutput, Source, SourceDist, SimplifyOptions, EdgeTags, VertexTags, TargetLengths, MinTargetLength, ProcessBuckets);

	///
	/// Step 3: Copy the cluster connectivity out to our ResultMesh
	///

	TArray<int32> FromResVID, ResultToSourceTri;
	bool bResultHasDuplicateVertices = false;
	BuildSimplifiedMeshFromClusters<TTriangleMeshAdapter<double>, FMeshConnectivity, FResultMeshAdapter>(OutSimplifiedMesh, FromResVID, ResultToSourceTri, ClusterInOutput,
		bResultHasDuplicateVertices,
		InMesh, MeshConnectivity, Source, SourceDist, SimplifyOptions, EdgeTags, VertexTags, TargetLengths, MinTargetLength,
		ProcessBuckets);

	//
	// Step 4: Transfer attributes out to the result mesh
	//

	if (SimplifyOptions.bTransferAttributes)
	{
		if (OutSimplifiedMesh.TransferPerVertexAttributes)
		{
			OutSimplifiedMesh.TransferPerVertexAttributes(FromResVID);
		}
		if (OutSimplifiedMesh.TransferPerTriangleAttributes)
		{
			OutSimplifiedMesh.TransferPerTriangleAttributes(ResultToSourceTri);
		}
	}

	return true;
}

} // namespace UE::Geometry
