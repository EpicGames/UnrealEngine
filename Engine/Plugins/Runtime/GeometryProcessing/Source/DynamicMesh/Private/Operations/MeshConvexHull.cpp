// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MeshConvexHull.h"
#include "Solvers/MeshLinearization.h"
#include "MeshSimplification.h"
#include "ProjectionTargets.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "CompGeom/ConvexHull3.h"
#include "Util/GridIndexing3.h"

using namespace UE::Geometry;

bool FMeshConvexHull::Compute(FProgressCancel* Progress)
{
	bool bOK = false;
	if (VertexSet.Num() > 0)
	{
		bOK = Compute_VertexSubset(Progress);
	}
	else
	{
		bOK = Compute_FullMesh(Progress);
	}
	if (!bOK)
	{
		return false;
	}


	if (bPostSimplify)
	{
		bOK = SimplifyHull(ConvexHull, SimplifyOptions, Progress);
	}

	return bOK;
}


bool FMeshConvexHull::SimplifyHull(FDynamicMesh3& HullMesh, FMeshConvexHull::FSimplifyOptions SimplifyOptions, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	if (!ensure(SimplifyOptions.MaxTargetFaceCount > 0))
	{
		SimplifyOptions.MaxTargetFaceCount = 1;
	}
	bool bSimplified = false;

	
	bool bUseGeometricError = SimplifyOptions.GeometricToleranceAfterTarget > 0;
	FDynamicMeshAABBTree3 Spatial;
	TUniquePtr<FDynamicMesh3> RefMesh;
	if (bUseGeometricError)
	{
		RefMesh = MakeUnique<FDynamicMesh3>(HullMesh);
		Spatial.SetMesh(RefMesh.Get(), true);
	}

	if (HullMesh.TriangleCount() > SimplifyOptions.MaxTargetFaceCount)
	{
		FVolPresMeshSimplification Simplifier(&HullMesh);
		Simplifier.CollapseMode = FVolPresMeshSimplification::ESimplificationCollapseModes::MinimalExistingVertexError;
		Simplifier.SimplifyToTriangleCount(SimplifyOptions.MaxTargetFaceCount);
		if (bUseGeometricError)
		{
			Simplifier.GeometricErrorConstraint = FVolPresMeshSimplification::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
			Simplifier.GeometricErrorTolerance = SimplifyOptions.GeometricToleranceAfterTarget;
			UE::Geometry::FMeshProjectionTarget ProjTarget(RefMesh.Get(), &Spatial);
			Simplifier.SetProjectionTarget(&ProjTarget);
			Simplifier.SimplifyToTriangleCount(1);
		}
		bSimplified = true;
	}

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	if (bSimplified)
	{
		// recalculate convex hull
		// TODO: test if simplified mesh is convex first, can just re-use in that case!!
		FMeshConvexHull SimplifiedHull(&HullMesh);
		if (SimplifiedHull.Compute(Progress))
		{
			HullMesh = MoveTemp(SimplifiedHull.ConvexHull);
		}
	}

	return true;
}



bool FMeshConvexHull::Compute_FullMesh(FProgressCancel* Progress)
{
	return Compute_Helper(Progress, Mesh->MaxVertexID(), [this](int32 Index) { return Mesh->GetVertex(Index); }, [this](int32 Index) { return Mesh->IsVertex(Index); });
}

bool FMeshConvexHull::Compute_VertexSubset(FProgressCancel* Progress)
{
	return Compute_Helper(Progress, VertexSet.Num(), [this](int32 Index) { return Mesh->GetVertex(VertexSet[Index]); }, [](int32 Index) { return true; });
}

bool FMeshConvexHull::Compute_Helper(FProgressCancel* Progress, int32 MaxVertexIndex, TFunctionRef<FVector3d(int32)> GetVertex, TFunctionRef<bool(int32)> IsVertex, bool bTestMinDimension)
{
	if (bTestMinDimension && MinDimension > 0)
	{
		// Use extreme points to quickly get an aligned box, which should align to very thin shapes
		TExtremePoints3<double> ExtremePoints(MaxVertexIndex, GetVertex, IsVertex);
		// Fill out any missing vectors in the basis
		switch (ExtremePoints.Dimension)
		{
		case 0:
			ExtremePoints.Basis[0] = FVector(1, 0, 0);
			ExtremePoints.Basis[1] = FVector(0, 1, 0);
			ExtremePoints.Basis[2] = FVector(0, 0, 1);
			break;
		case 1:
			VectorUtil::MakePerpVectors(ExtremePoints.Basis[0], ExtremePoints.Basis[1], ExtremePoints.Basis[2]);
			break;
		case 2:
			ExtremePoints.Basis[2] = ExtremePoints.Basis[0].Cross(ExtremePoints.Basis[1]);
			break;
		}
		FAxisAlignedBox3d BasisBox;
		for (int32 Index = 0; Index < MaxVertexIndex; ++Index)
		{
			if (!IsVertex(Index))
			{
				continue;
			}
			FVector3d Vertex = GetVertex(Index);
			FVector3d InBasis(
				ExtremePoints.Basis[0].Dot(Vertex),
				ExtremePoints.Basis[1].Dot(Vertex),
				ExtremePoints.Basis[2].Dot(Vertex)
			);
			BasisBox.Contain(InBasis);
		}
		int32 NumSmallDims = 0;
		int32 SmallDims[3]{ -1,-1,-1 };
		FVector3d ThickenVecs[3];
		FVector3d BasisDims = BasisBox.Diagonal();
		for (int32 DimIdx = 0; DimIdx < 3; ++DimIdx)
		{
			if (BasisDims[DimIdx] < MinDimension)
			{
				int32 SmallIdx = NumSmallDims++;
				SmallDims[SmallIdx] = DimIdx;
				ThickenVecs[SmallIdx] = .5 * (MinDimension - BasisDims[DimIdx]) * ExtremePoints.Basis[DimIdx];
			}
		}
		if (NumSmallDims > 0)
		{
			TArray<FVector3d> ThickenedHullVertices;
			ThickenedHullVertices.Reserve(MaxVertexIndex * (NumSmallDims * 2));
			for (int32 Index = 0; Index < MaxVertexIndex; ++Index)
			{
				if (!IsVertex(Index))
				{
					continue;
				}
				FVector3d Vertex = GetVertex(Index);
				for (int32 SmallDimIdx = 0; SmallDimIdx < NumSmallDims; ++SmallDimIdx)
				{
					ThickenedHullVertices.Add(Vertex + ThickenVecs[SmallDimIdx]);
					ThickenedHullVertices.Add(Vertex - ThickenVecs[SmallDimIdx]);
				}
			}
			return Compute_Helper(Progress, ThickenedHullVertices.Num(), 
				[&ThickenedHullVertices](int32 Index) { return ThickenedHullVertices[Index]; },
				[](int32) { return true; }, false);
		}
	}

	FConvexHull3d HullCompute;
	HullCompute.Progress = Progress;
	bool bOK = HullCompute.Solve(MaxVertexIndex, GetVertex, IsVertex);
	if (!bOK)
	{
		return false;
	}

	TMap<int32, int32> HullVertMap;

	ConvexHull = FDynamicMesh3(EMeshComponents::None);
	HullCompute.GetTriangles([&](FIndex3i Triangle)
	{
		for (int32 j = 0; j < 3; ++j)
		{
			int32 Index = Triangle[j];
			if (HullVertMap.Contains(Index) == false)
			{
				FVector3d OrigPos = GetVertex(Index);
				int32 NewVID = ConvexHull.AppendVertex(OrigPos);
				HullVertMap.Add(Index, NewVID);
				Triangle[j] = NewVID;
			}
			else
			{
				Triangle[j] = HullVertMap[Index];
			}
		}

		ConvexHull.AppendTriangle(Triangle);
	});

	return true;
}


FVector3i FMeshConvexHull::DebugGetCellIndex(const FDynamicMesh3& Mesh,
											 int GridResolutionMaxAxis,
											 int VertexIndex)
{
	FAxisAlignedBox3d Bounds = Mesh.GetBounds();
	Bounds.Min = Bounds.Min - 1e-4;			// Pad to avoid problems with vertices lying exactly on bounding box
	Bounds.Max = Bounds.Max + 1e-4;

	const double GridCellSize = Bounds.MaxDim() / (double)GridResolutionMaxAxis;

	FBoundsGridIndexer3d Indexer(Bounds, GridCellSize);

	return Indexer.ToGrid(Mesh.GetVertex(VertexIndex));
}


void FMeshConvexHull::GridSample(const FDynamicMesh3& Mesh, 
								 int GridResolutionMaxAxis, 
								 TArray<int32>& OutSamples)
{
	// Simple spatial hash to find a representative vertex for each occupied grid cell

	FAxisAlignedBox3d Bounds = Mesh.GetBounds();
	
	// Empty bounds, nothing to sample
	if (Bounds.IsEmpty())
	{
		return;
	}
	Bounds.Min = Bounds.Min - 1e-4;			// Pad to avoid problems with vertices lying exactly on bounding box
	Bounds.Max = Bounds.Max + 1e-4;
	// Clamp grid resolution to prevent overflowing int
	const double GridCellSize = Bounds.MaxDim() / (double)FMath::Clamp(GridResolutionMaxAxis, 1, 1290);

	FBoundsGridIndexer3d Indexer(Bounds, GridCellSize);
	const FVector3i GridResolution = Indexer.GridResolution();

	// TODO: If the grid resolution is too high, use a TMap from grid cell index to vertex index instead of an array.
	// For smallish grids the array is more efficient.
	int TotalNumberGridCells = GridResolution.X * GridResolution.Y * GridResolution.Z;
	TArray<int32> GridCellVertex;
	GridCellVertex.Init(-1, TotalNumberGridCells);

	for (int VertexIndex : Mesh.VertexIndicesItr())
	{
		FVector3i CellIndex = Indexer.ToGrid(Mesh.GetVertex(VertexIndex));
		check(CellIndex.X >= 0 && CellIndex.X < GridResolution.X);
		check(CellIndex.Y >= 0 && CellIndex.Y < GridResolution.Y);
		check(CellIndex.Z >= 0 && CellIndex.Z < GridResolution.Z);

		int Key = CellIndex.X + CellIndex.Y * GridResolution.X + CellIndex.Z * GridResolution.X * GridResolution.Y;

		GridCellVertex[Key] = VertexIndex;
	}

	for (const int32 VertexIndex : GridCellVertex)
	{
		if (VertexIndex >= 0)
		{
			OutSamples.Add(VertexIndex);
		}
	}

}
