// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MeshGeodesicSurfaceTracer.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "Intersection/IntrLine2Line2.h"
#include "Intersection/IntrLine2Triangle2.h"
#include "IndexTypes.h"
#include "MathUtil.h" // for ::TwoPi
#include "LineTypes.h"
#include "Util/IndexUtil.h"

#include "BoxTypes.h"

using namespace UE::Geometry;


/**
*  Local Utils for manipulating FIndex{2,3}-based mesh information
*/ 
namespace  
{
	// assumes VID0 and VID1 are contained in Edge
	bool IsOrderReversed(int VID0, int32 VID1, const FIndex2i& Edge)
	{
		return (VID0 == Edge.B && VID1 == Edge.A);
	}

	/**
	* Test if the ordering of the pair {VID0, VID1} is consistent with the vertex ordering in TriVIDs.
	*
	* returns 'false' if TriVIDs = {VID0, VID1, X} or {X, VID0, VID1} or { VID1, X, VID0}
	* where 'X' is some other index in TriVIDs.  Any thing else will return 'true'.
	* Note: Assumes VID0 and VID1 are contained in TriVIDs and it will  
	*       return 'true' in the case that either or both VIDs are not contained in TriVIDs
	*/
	bool IsOrderReversed(int VID0, int32 VID1, const FIndex3i& TriVIDs)
	{
		return (IndexUtil::FindTriOrderedEdge(VID0, VID1, TriVIDs) == IndexConstants::InvalidID);
	}
	
	// Permute the input so SrcIndex3[PivotIndex] becomes ResultIndex3[0] while preserving wrap around order
	FIndex3i PermuteIndex3(const FIndex3i& SrcIndex3, int32 PivotIndex)
	{
		static const int32 AddOneModThree[3] = { 1, 2, 0 };
		static const int32 AddTwoModThree[3] = { 2, 0, 1 };
		return FIndex3i(SrcIndex3[PivotIndex], SrcIndex3[AddOneModThree[PivotIndex]], SrcIndex3[AddTwoModThree[PivotIndex]]);
	}

	// Helper to return the edges ordered as if VID is the start vertex.  
	// note VID must be one of the Verts for the Tri, or something unexpected will happen
	FIndex3i GetPermutedEdges(const FDynamicMesh3& Mesh, int32 TriID, int32 VID)
	{
		const FIndex3i TriVIDs = Mesh.GetTriangle(TriID);
		const FIndex3i TriEIDs = Mesh.GetTriEdges(TriID);
		const int32 TriVertIndex = IndexUtil::FindTriIndex(VID, TriVIDs);
		return PermuteIndex3(TriEIDs, TriVertIndex);
	}

	int AbsMinIndex(const FVector3d& Values)
	{
		FVector3d AbsValues(FMath::Abs(Values[0]), FMath::Abs(Values[1]), FMath::Abs(Values[2]));

		int MinIndex = 0;
		for (int i = 1; i < 3; ++i)
		{
			if (AbsValues[i] < AbsValues[MinIndex])
			{
				MinIndex = i;
			}
		}
		return MinIndex;
	}

	// Find TriID of the next counter clockwise Triangle when traveling around vertex VID.
	// Note, this assumes that VID is a vertex in the current triangle.
	int NextCCWTriangle(const FDynamicMesh3& Mesh, int32 CurTriID, int32 VID)
	{
		// Find next triangle traveling counter clockwise.
		//  - do this by finding the edge shared with the current triangle the next triangle
		const FIndex3i PermutedTriEIDs = GetPermutedEdges(Mesh, CurTriID, VID);
		int NextEdgeID = PermutedTriEIDs.C;


		FIndex2i NextEdgeTris = Mesh.GetEdgeT(NextEdgeID);
		int NextTriID = (NextEdgeTris.A == CurTriID) ? NextEdgeTris.B : NextEdgeTris.A;
		return NextTriID;
	}
};

/**
 *  Misc Triangle Utils
 */
namespace 
{
	/**
	* Construct triangle in R2 with same vertex order and edge length as a triangle in R3
	* where Pos0, Pos1, Pos2 are the vertices of the R3 triangle.
	*
	* in particular: Pos0 maps to V[0] = (0,0)
	*                Pos1 maps to V[1] = (|Pos1 - Pos0|, 0)
	*                Pos2 maps to V[2] = ( X, Y)  where Y > = 0  and X and Y are such that the edge lengths are preserved.
	*/
	template <typename ScalarType>
	TTriangle2<ScalarType>  MakeTriangle2(const UE::Math::TVector<ScalarType>& Pos0, const UE::Math::TVector<ScalarType>& Pos1, const UE::Math::TVector<ScalarType>& Pos2)
	{
		typedef  UE::Math::TVector2<ScalarType>     Vector2Type;
		typedef UE::Math::TVector<ScalarType>       Vector3Type;
		typedef TTriangle2<ScalarType>              TriangleType;

		const ScalarType Zero(0);

		// spanning vectors 
		const Vector3Type TriEdgeVectors[2] = { Pos1 - Pos0,
												Pos2 - Pos0 };

		const ScalarType BaseEdgeLength = TriEdgeVectors[0].Length();
		const Vector3Type BaseEdgeDir = TriEdgeVectors[0] / BaseEdgeLength;


		// BaseEdgeDir is to be aligned with x-axis.
		// compute x,y coord of the TriEdgeVector[2] relative to BaseEdgeDir
		const ScalarType XCoord = BaseEdgeDir.Dot(TriEdgeVectors[1]);
		Vector3Type cp = BaseEdgeDir.Cross(TriEdgeVectors[1]);
		const ScalarType YCoord = cp.Length();


		return TriangleType(Vector2Type(Zero, Zero),
			Vector2Type(BaseEdgeLength, Zero),
			Vector2Type(XCoord, YCoord));
	}


	// Return a valid Barycentric position interior to defining triangle by clamping values and then rescaling to ensure alpha + beta + gamma = 1
	FVector3d ClampBarycentric(const FVector3d& BCin)
	{
		FVector3d BClamped(FMath::Clamp(BCin.X, 0., 1.), FMath::Clamp(BCin.Y, 0., 1.), FMath::Clamp(BCin.Z, 0., 1.));
		double Sum = BClamped.X + BClamped.Y + BClamped.Z;
		if (Sum > TMathUtilConstants<double>::ZeroTolerance)
		{
			double InvSum = 1. / Sum;
			BClamped *= InvSum;
		}
		else
		{
			// default to center.
			BClamped = FVector3d(1. / 3., 1. / 3., 1. / 3.);
		}
		return BClamped;
	}

	// Test if the given BarycentricPoint describes a valid point with a defining triangle
	bool IsBarycentricPointInTriangle(const FVector3d& BarycentricPoint, double Tol = TMathUtilConstants<double>::ZeroTolerance)
	{
		const double MinVal = FMath::Min3(BarycentricPoint.X, BarycentricPoint.Y, BarycentricPoint.Z);
		const double MaxVal = FMath::Max3(BarycentricPoint.X, BarycentricPoint.Y, BarycentricPoint.Z);
		const double Error = FMath::Abs(1. - ( BarycentricPoint.X + BarycentricPoint.Y + BarycentricPoint.Z));
		return (MinVal > -Tol && MaxVal < 1. + Tol && Error < 3. * Tol);
	}
};


// -- FTangentTri2 Implementations

GeodesicSingleTriangleUtils::FTangentTri2::FTangentTri2(const FDynamicMesh3& Mesh, int32 TriID, int32 PrimaryEdgeID)
{
	Tri3ID = TriID;
	const FIndex3i TriVIDs = Mesh.GetTriangle(TriID);
	const FIndex3i TriEIDs = Mesh.GetTriEdges(TriID);
	const FIndex2i EdgeVertIDs = Mesh.GetEdgeV(PrimaryEdgeID);

	// current edge
	// e0 = edge(v0, v1),  e1 = edge(v1, v2), e2 = edge(v2, v0)
	const int32 EdgeIndexInTri = IndexUtil::FindEdgeIndexInTri(EdgeVertIDs.A, EdgeVertIDs.B, TriVIDs);

	PermutedIndices = PermuteIndex3(FIndex3i(0, 1, 2), EdgeIndexInTri);

	PermutedTriVIDs = PermuteIndex3(TriVIDs, EdgeIndexInTri);
	PermutedTriEIDs = PermuteIndex3(TriEIDs, EdgeIndexInTri);

	for (int i = 0; i < 3; ++i)
	{
		FIndex2i EdgeV = Mesh.GetEdgeV(PermutedTriEIDs[i]);
		EdgeOrientationSign[i] = IsOrderReversed(EdgeV.A, EdgeV.B, PermutedTriVIDs) ? -1 : 1;
	}

	// The 3d vertices in the Permuted Order.
	FVector3d Tri3dVerts[3] = { Mesh.GetVertex(PermutedTriVIDs[0]),
								Mesh.GetVertex(PermutedTriVIDs[1]),
								Mesh.GetVertex(PermutedTriVIDs[2]) };

	Tri2d = MakeTriangle2(Tri3dVerts[0], Tri3dVerts[1], Tri3dVerts[2]);
	

	Tri2dEdges[0] = Tri2d.V[1] - Tri2d.V[0];
	Tri2dEdges[1] = Tri2d.V[2] - Tri2d.V[1];
	Tri2dEdges[2] = Tri2d.V[0] - Tri2d.V[2];

	Tri2dLengths[0] = Tri2dEdges[0].Length();
	Tri2dLengths[1] = Tri2dEdges[1].Length();
	Tri2dLengths[2] = Tri2dEdges[2].Length();

}

void GeodesicSingleTriangleUtils::FTangentTri2::Reset()
{
	Tri3ID = IndexConstants::InvalidID;
}

double  GeodesicSingleTriangleUtils::FTangentTri2::DistanceToEdge(int EdgeIndex, const FVector2d& Point) const
{
	return DotPerp(Tri2dEdges[EdgeIndex], Point - Tri2d.V[EdgeIndex]) / Tri2dLengths[EdgeIndex];
}

// Closest Point on the edge, given in terms of 0,1 parameter measured along the edge (EdgeIndex = 0, 1, 2)
double  GeodesicSingleTriangleUtils::FTangentTri2::ProjectOnEdge(int EdgeIndex, const FVector2d& Point) const
{
	return Tri2dEdges[EdgeIndex].Dot(Point - Tri2d.V[EdgeIndex]) / (Tri2dLengths[EdgeIndex] * Tri2dLengths[EdgeIndex]);
}

// Express the vector in terms of a basis that is aligned with the EdgeIndex edge of the triangle
FVector2d GeodesicSingleTriangleUtils::FTangentTri2::ChangeBasis(const FVector2d& Vec, int32 EdgeIndex) const
{
	// rotate the Ray relative to the side of the triangle we hit.
	const FVector2d FrameEdge = Tri2dEdges[EdgeIndex] / Tri2dLengths[EdgeIndex];
	// Rotation matrix                                   (Cos = FrameEdge.X    Sin =  FrameEdge.Y)
	// aligns FrameEdge with x-axis                      (-Sin = -FrameEdge.Y  Cos =  FrameEdge.X) 
	FVector2d RotatedVector(FrameEdge.X * Vec.X + FrameEdge.Y * Vec.Y, -FrameEdge.Y * Vec.X + FrameEdge.X * Vec.Y);
	RotatedVector = Normalized(RotatedVector);

	return RotatedVector;
}

// Barycentric coords for this point in the tri2d
FVector3d  GeodesicSingleTriangleUtils::FTangentTri2::GetBarycentricCoords(const FVector2d& PoinInTri2) const
{
	return Tri2d.GetBarycentricCoords(PoinInTri2);
}

// Convert a point on this tri, to Barycentric coords in the un-rotated source triangle.
FVector3d  GeodesicSingleTriangleUtils::FTangentTri2::GetBarycentricCoordsInSrcTri(const FVector2d& PoinInTri2) const
{
	return ReorderVertexDataForSrcTri3d(GetBarycentricCoords(PoinInTri2));
}

// ----------------------------------------------------------------------------------------------------//
// Methods for tracing across a single triangle in a mesh                                              //
//-----------------------------------------------------------------------------------------------------//


GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceTangentTriangle(const FTangentTri2& TangentTri2, const FVector2d& RayOrigin, const FVector2d& RayDir, double MaxDistance)
{
	bool bHasIntersection = false;
	double RayDistance = TMathUtilConstants<double>::MaxReal;
	FVector2d HitPoint = FVector2d::ZeroVector;

	// special case of ray starting on edge and traveling parallel to the edge.  Say the vertex at the end of the edge is the hit
	if (FMath::Abs(RayOrigin.Y) < TMathUtilConstants<double>::ZeroTolerance && FMath::Abs(RayDir.Y) < TMathUtilConstants<double>::ZeroTolerance)
	{
		bHasIntersection = true;
		if (RayDir.X > 0)
		{
			HitPoint = TangentTri2.Tri2d.V[1];
			RayDistance = (RayOrigin - HitPoint).Length();
		}
		else
		{
			HitPoint = TangentTri2.Tri2d.V[0];
			RayDistance = (RayOrigin - HitPoint).Length();
		}
	}
	else // test ray against triangle.
	{ 

		const FLine2d RayPath(RayOrigin, RayDir);

		// Find Intr with Tri2
		FIntrLine2Triangle2d LineTriIntr(RayPath, TangentTri2.Tri2d);
		bHasIntersection = LineTriIntr.Find();

		// classify the intersection(s)
		const bool bIsSegmentIntersection = (LineTriIntr.Type == EIntersectionType::Segment);
		const bool bIsPointIntersection = (LineTriIntr.Type == EIntersectionType::Point);


		if (bIsSegmentIntersection)
		{
			// RayPath.Origin is located on the base of the triangle in the case of tracing from edge
			// or inside the triangle when tracing from barycentric, 
			// so there will be an intersection at Param <=0 at or behind the ray origin.
			// we want the other intersection.
			if (LineTriIntr.Param0 > LineTriIntr.Param1)
			{
				RayDistance = LineTriIntr.Param0;
				HitPoint = LineTriIntr.Point0;
			}
			else
			{
				RayDistance = LineTriIntr.Param1;
				HitPoint = LineTriIntr.Point1;
			}
		}
		else if (bIsPointIntersection) // this is the grazing case where the line clips a single vertex of the triangle
		{
			RayDistance = LineTriIntr.Param0;
			HitPoint = LineTriIntr.Point0;
		}

	}
	


	// populate the result.
	FVector2d EndPoint;
	FTraceResult  TraceResult;
	if (bHasIntersection == false)
	{
		TraceResult.Classification = ETraceClassification::Failed;
		return TraceResult;
	}

	bool bEdgeOrderReversed;
	if (RayDistance <= MaxDistance)  // result on edge of triangle
	{
		EndPoint = HitPoint;

		// Expect hit will be on either e1 = v2-v1 
		//                          or  e2 = v0-v2
		// unless the trace direction was actually out of the triangle
		// compute the distance to each edge.  one should be zero, although all could be small.
		const FVector3d DistHitToEdge(TangentTri2.DistanceToEdge(0, HitPoint), TangentTri2.DistanceToEdge(1, HitPoint), TangentTri2.DistanceToEdge(2, HitPoint));

		// Pick the edge we hit
		const int32 HitEdgeIndex = AbsMinIndex(DistHitToEdge);

		bEdgeOrderReversed = (TangentTri2.EdgeOrientationSign[HitEdgeIndex] == -1);
		// EdgeID for the edge we hit. 
		const int32 HitEID = TangentTri2.PermutedTriEIDs[HitEdgeIndex];

		// --- encode the hit relative to the edge we hit.
		// 
		// Distance measured from V[HitIndex] (along the edge) to the hit point as fraction of edge length.
		const double HitEdgeAlpha = TangentTri2.ProjectOnEdge(HitEdgeIndex, HitPoint);

		// rotate the Ray relative to the side of the triangle we hit. i.e. the rotation that aligns the hit edge with the x-axis
		const FVector2d RotatedRayDir = TangentTri2.ChangeBasis(RayDir, HitEdgeIndex);

		// populate trace result
		TraceResult.Classification = (FMath::Abs(RayDistance - MaxDistance) < TMathUtilConstants<double>::ZeroTolerance) ? ETraceClassification::DistanceTerminated : ETraceClassification::Continuing;
		TraceResult.TraceDist = RayDistance;
		TraceResult.TriID = TangentTri2.Tri3ID;

		TraceResult.SurfaceDirection = FMeshSurfaceDirection(HitEID, RotatedRayDir);

		TraceResult.bIsEdgePoint = true;
		TraceResult.EdgeID = HitEID;
		TraceResult.EdgeAlpha = HitEdgeAlpha;

	}
	else // result internal to tri face
	{
		bEdgeOrderReversed = (TangentTri2.EdgeOrientationSign[0] == -1);
		// Note: EndPoint isn't the HitPoint in this case
		EndPoint = MaxDistance * RayDir + RayOrigin;

		TraceResult.Classification = ETraceClassification::DistanceTerminated;
		TraceResult.TraceDist = MaxDistance;
		TraceResult.TriID = TangentTri2.Tri3ID;

		TraceResult.bIsEdgePoint = false;
		TraceResult.EdgeID = TangentTri2.PermutedTriEIDs[0]; //EID;
		TraceResult.EdgeAlpha = 0.; // not used in this case

		TraceResult.SurfaceDirection = FMeshSurfaceDirection(TraceResult.EdgeID, RayDir);

	}
	TraceResult.Barycentric = TangentTri2.GetBarycentricCoordsInSrcTri(EndPoint);

	// may need to update this encoding to match the vertex order in Mesh.GetEdgeV 
	if (bEdgeOrderReversed)
	{
		TraceResult.SurfaceDirection.Dir = -TraceResult.SurfaceDirection.Dir;
		TraceResult.EdgeAlpha = 1. - TraceResult.EdgeAlpha;
	}
	TraceResult.EdgeAlpha = FMath::Clamp(TraceResult.EdgeAlpha, 0., 1.);

	return TraceResult;
}

GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceTriangleFromBaryPoint(const FDynamicMesh3& Mesh, const int32 TriID, const FVector3d& BaryPoint, const FMeshSurfaceDirection& Direction, double MaxDistance)
{
	FTangentTri2 ScratchTangentTri2;
	return TraceTriangleFromBaryPoint(Mesh, TriID, BaryPoint, Direction, ScratchTangentTri2, MaxDistance );
}

GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceTriangleFromBaryPoint(const FDynamicMesh3& Mesh, const int32 TriID, const FVector3d& BaryPointIn, const FMeshSurfaceDirection& Direction, FTangentTri2& TangentTri2, double MaxDistance)
{
	FIndex3i TriVIDs = Mesh.GetTriangle(TriID);

	
	FVector3d BaryPoint(BaryPointIn);
	constexpr double CornerEpsilon = 1.e-4; // trace from vertex tolerance (in barycentric space )

	// Don't trace from a corner of the triangle, slightly offsetset towards the interior and start there.
	if (FMath::Abs(BaryPoint.X - 1.) < CornerEpsilon)
	{
		BaryPoint = FVector3d( 1. -CornerEpsilon, CornerEpsilon/2., CornerEpsilon/2.);
	}
	else if (FMath::Abs(BaryPoint.Y - 1.) < CornerEpsilon)
	{
		BaryPoint = FVector3d( CornerEpsilon / 2., 1. - CornerEpsilon, CornerEpsilon / 2.);
	}
	else if (FMath::Abs(BaryPoint.Z - 1.) < CornerEpsilon)
	{
		BaryPoint = FVector3d(CornerEpsilon / 2., CornerEpsilon / 2., 1. - CornerEpsilon);
	}

	FVector2d RayDir = Direction.Dir;
	int EID = Direction.EdgeID;
	// The 2d Tangent Triangle is in the first quadrant with EID aligned with the x-axis
	TangentTri2 = FTangentTri2(Mesh, TriID, EID);
	if (TangentTri2.EdgeOrientationSign[0] == -1)
	{
		// the frame that encoded the direction vector is rotated 180 degrees from the first edge of this triangle (our local reference )
		// update the direction.
		RayDir = -RayDir;
	}

	// Barycentric coord relative to the rotated 2d triangle.
	const FVector3d Bary2d = { BaryPoint[TangentTri2.PermutedIndices[0]], BaryPoint[TangentTri2.PermutedIndices[1]], BaryPoint[TangentTri2.PermutedIndices[2]] };
	const FVector2d RayOrigin = TangentTri2.Tri2d.BarycentricPoint(Bary2d);

	// Path Entering the 2d triangle, relative to this 2d triangle  
	FLine2d PathRay(RayOrigin, RayDir);

	FTraceResult TraceResult = TraceTangentTriangle(TangentTri2, RayOrigin, RayDir, MaxDistance);
	if (!IsTerminated(TraceResult))
	{
		FIndex2i EdgeT = Mesh.GetEdgeT(TraceResult.EdgeID);
		if (EdgeT.A == FDynamicMesh3::InvalidID || EdgeT.B == FDynamicMesh3::InvalidID)
		{
			TraceResult.Classification = ETraceClassification::BoundaryTerminated;
		}
	}

	checkSlow(TraceResult.bIsEdgePoint || TraceResult.EdgeID == EID);

	return TraceResult;

}

GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceTriangleFromEdge(const FDynamicMesh3& Mesh, const int32 TriID, double EdgeAlpha, const FMeshSurfaceDirection& Direction, double MaxDistance)
{
	FTangentTri2 ScratchTangentTri2;
	return TraceTriangleFromEdge(Mesh, TriID, EdgeAlpha,  Direction, ScratchTangentTri2, MaxDistance);
}

GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceTriangleFromEdge(const FDynamicMesh3& Mesh, const int32 TriID, double EdgeAlpha, const FMeshSurfaceDirection& Direction, FTangentTri2& TangentTri2, double MaxDistance)
{

	FVector2d RayDir = Direction.Dir;
	const int32 EID = Direction.EdgeID;

	// The 2d Tangent Triangle is in the first quadrant with EID aligned with the x-axis
	TangentTri2 = FTangentTri2(Mesh, TriID, EID);
	if (TangentTri2.EdgeOrientationSign[0] == -1)
	{
		// the crossing data was encoded with the vector pointing in the direction opposite the local triangle order, update it
		EdgeAlpha = 1. - EdgeAlpha;
		RayDir = -RayDir;
	}
	EdgeAlpha = FMath::Clamp(EdgeAlpha, 0., 1.);

	FVector2d RayOrigin(TangentTri2.Tri2dEdges[0] * EdgeAlpha);

	FTraceResult TraceResult = TraceTangentTriangle(TangentTri2, RayOrigin, RayDir, MaxDistance);
	if (!IsTerminated(TraceResult))
	{
		FIndex2i EdgeT = Mesh.GetEdgeT(TraceResult.EdgeID);
		if (EdgeT.A == FDynamicMesh3::InvalidID || EdgeT.B == FDynamicMesh3::InvalidID)
		{
			TraceResult.Classification = ETraceClassification::BoundaryTerminated;
		}
	}

	checkSlow(TraceResult.bIsEdgePoint || TraceResult.EdgeID == EID);

	return TraceResult;
}



// Note: Assumes RayDir is defined in a triangle adjacent to the edge, not on the tangent plane!
GeodesicSingleTriangleUtils::FTraceResult GeodesicSingleTriangleUtils::TraceFromVertex(const FDynamicMesh3& Mesh, int32 VID, const FMeshSurfaceDirection& Direction, double MaxDistance)
{

	FTangentTri2 ScratchTangentTri2;
	return TraceFromVertex(Mesh, VID, Direction, ScratchTangentTri2, MaxDistance);
}
GeodesicSingleTriangleUtils::FTraceResult GeodesicSingleTriangleUtils::TraceFromVertex(const FDynamicMesh3& Mesh, int32 VID, const FMeshSurfaceDirection& Direction, FTangentTri2& ScratchTangentTri2, double MaxDistance)
{
	// This doesn't work at a bowtie
	// [todo] make this work form a bowtie, potentially returning an immediately terminated trace
	check(!Mesh.IsBowtieVertex(VID));

	// Find total angle around the vertex.
	double TotalAngle = 0;
	const bool bIsBoundaryVertex = Mesh.IsBoundaryVertex(VID);
	const int32 RefEID = Direction.EdgeID;
	{
		int32 FirstEID = RefEID;
		if (bIsBoundaryVertex)
		{
			// vertex is on the mesh boundary, 
			// choose the boundary edge such that traveling CCW (about the vertex) moves into the mesh

			for (int32 EID : Mesh.VtxEdgesItr(VID))
			{
				if (Mesh.IsBoundaryEdge(EID))
				{
					const int32 TID = Mesh.GetEdgeT(EID).A;
					const int32 IndexOf = Mesh.GetTriEdges(TID).IndexOf(EID);
					if (Mesh.GetTriangle(TID)[IndexOf] == VID)
					{
						FirstEID = EID;
						break;
					}
				}
			}
		}
		const FIndex2i FirstLastTri = Mesh.GetEdgeT(FirstEID);

		// Travel counter-clockwise around the vertex, visiting each triangle in order.
		// Start with the first triangle we visit when traveling counter clockwise around VID, starting with edge EID.
		int CurTriID = FirstLastTri.A;
		if (!bIsBoundaryVertex && GetPermutedEdges(Mesh, FirstLastTri.B, VID).A == RefEID)
		{
			CurTriID = FirstLastTri.B;
		}
		int FirstTriID = CurTriID;
		do
		{
			const FIndex3i TriVIDS = Mesh.GetTriangle(CurTriID);
			const int32 TriIndex = IndexUtil::FindTriIndex(VID, TriVIDS);
			const double AngleR = Mesh.GetTriInternalAngleR(CurTriID, TriIndex);

			CurTriID = NextCCWTriangle(Mesh, CurTriID, VID);
			TotalAngle += AngleR;

		} while (CurTriID != FirstTriID && CurTriID != FDynamicMesh3::InvalidID);
	}

	//convert input from a vector defined relative to a surface tri to a vector on the tangent plane.
	const double ToRadians = TMathUtilConstants<double>::TwoPi / FMath::Max(TotalAngle, TMathUtilConstants<double>::ZeroTolerance);
	const double PolarAngleR = AsZeroToTwoPi(ToRadians * FMath::Atan2(Direction.Dir[1], Direction.Dir[0]));
	FMeshTangentDirection TangentDirAtVertex = { VID, RefEID, PolarAngleR };
	return TraceFromVertex(Mesh, TangentDirAtVertex, ScratchTangentTri2, MaxDistance);
}


GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceFromVertex(const FDynamicMesh3& Mesh, const FMeshTangentDirection& TangentDirection, double MaxDistance)
{

	FTangentTri2 ScratchTangentTri2;
	return TraceFromVertex(Mesh, TangentDirection, ScratchTangentTri2, MaxDistance);
}
GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceFromVertex(const FDynamicMesh3& Mesh, const FMeshTangentDirection& TangentDirection, FTangentTri2& ScratchTangentTri2, double MaxDistance)
{
	const int32 RefEID = TangentDirection.EdgeID;
	const int32 VID    = TangentDirection.VID;

	FIndex2i RefEdgeV = Mesh.GetEdgeV(RefEID);
	check(RefEdgeV.A == VID || RefEdgeV.B == VID);

	// This doesn't work at a bowtie
	// [todo] make this work form a bowtie, potentially returning an imediately terminated trace
	check(!Mesh.IsBowtieVertex(VID));
	const bool bIsBoundaryVertex = Mesh.IsBoundaryVertex(VID);


	// Compute local tangent plane information. 
	struct FInternalAngle
	{
		int32 TriID;
		double Angle; // internal angle
		double OffsetAngle; // sum of prior internal angles.
	};

	int32 FirstEID = RefEID;
	if (bIsBoundaryVertex)
	{
		// vertex is on the mesh boundary, 
		// choose the boundary edge such that traveling CCW (about the vertex) moves into the mesh

		for (int32 EID : Mesh.VtxEdgesItr(VID))
		{
			if (Mesh.IsBoundaryEdge(EID))
			{
				const int32 TID = Mesh.GetEdgeT(EID).A;
				const int32 IndexOf = Mesh.GetTriEdges(TID).IndexOf(EID);
				if (Mesh.GetTriangle(TID)[IndexOf] == VID)
				{
					FirstEID = EID;
					break;
				}
			}
		}
	}
	const FIndex2i FirstLastTri = Mesh.GetEdgeT(FirstEID);
	double TotalAngle = 0;
	TArray<FInternalAngle> InternalAngles;
	// Travel counter-clockwise around the vertex, visiting each triangle in order.
	// Start with the first triangle we visit when traveling counter clockwise around VID, starting with edge EID.
	int CurTriID = FirstLastTri.A;
	if (!bIsBoundaryVertex && GetPermutedEdges(Mesh, FirstLastTri.B, VID).A == RefEID)
	{
		CurTriID = FirstLastTri.B;
	}
	int FirstTriID = CurTriID;
	double AngleOfRefEID = 0.; // to be computed. the offset from the first edge to the reference edge.
	do
	{
		FIndex3i TriVIDS = Mesh.GetTriangle(CurTriID);

		const int32 TriIndex = IndexUtil::FindTriIndex(VID, TriVIDS);
		const int32 CurEID = Mesh.GetTriEdges(CurTriID)[TriIndex];
		const double AngleR = Mesh.GetTriInternalAngleR(CurTriID, TriIndex);


		FInternalAngle& InternalAngle = InternalAngles.AddDefaulted_GetRef();
		InternalAngle.TriID = CurTriID;
		InternalAngle.Angle = AngleR;
		InternalAngle.OffsetAngle = TotalAngle; // counting from FirstEID;
		if (CurEID == RefEID)
		{
			AngleOfRefEID = TotalAngle;
		}

		CurTriID = NextCCWTriangle(Mesh, CurTriID, VID);
		TotalAngle += AngleR;

	} while (CurTriID != FirstTriID && CurTriID != FDynamicMesh3::InvalidID);

	for (FInternalAngle& InternalAngle : InternalAngles)
	{
		InternalAngle.OffsetAngle -= AngleOfRefEID;
	}
	const double ToSurfaceAngle = TotalAngle / TMathUtilConstants<double>::TwoPi;
	// if the edge "ends" at VID then it aligns with Pi in the tangent space (i.e. the vector A->B aligns with 0 radians, and the vector B->A aligns with pi )
	double PolarAngle = (RefEdgeV.B == VID) ? TangentDirection.PolarAngle + TMathUtilConstants<double>::Pi : TangentDirection.PolarAngle;
	const double SurfaceAngle = AsZeroToTwoPi(PolarAngle) * ToSurfaceAngle;


	// find the triangle that the ray enters, and angle from reference to the first side of that triangle
	int DstTriID = FDynamicMesh3::InvalidID;
	double DstOffsetAngle = 0;
	for (const FInternalAngle& AngleData : InternalAngles)
	{
		if (AngleData.OffsetAngle <= SurfaceAngle && (AngleData.OffsetAngle + AngleData.Angle) >= SurfaceAngle)
		{
			DstTriID = AngleData.TriID;
			DstOffsetAngle = AngleData.OffsetAngle;
			break;
		}
	}

	if (DstTriID == FDynamicMesh3::InvalidID)
	{
		// this case should only happen if we are exiting the mesh
		check(bIsBoundaryVertex);

		GeodesicSingleTriangleUtils::FTraceResult TraceResult;
		TraceResult.Classification = ETraceClassification::BoundaryTerminated;
		TraceResult.TraceDist = 0.;
		TraceResult.bIsEdgePoint = true;
		TraceResult.EdgeID = RefEID;
		TraceResult.EdgeAlpha = (RefEdgeV.A == VID) ? 0. : 1.;

		return TraceResult;
	}

	// What is the first edge of the Dst Tri?
	const FIndex3i PermutedDstEdges = GetPermutedEdges(Mesh, DstTriID, VID);
	int DstEdgeID = PermutedDstEdges.A; // First Edge.

	double AngleEdgeToRay = SurfaceAngle - DstOffsetAngle;

	// ray relative to edge exiting VID
	double DstAlpha = 0.;
	FVector2d DstRayDir(FMath::Cos(AngleEdgeToRay), FMath::Sin(AngleEdgeToRay));
	FIndex2i DstEdge = Mesh.GetEdgeV(DstEdgeID);
	if (DstEdge.B == VID)
	{
		DstAlpha = 1. - DstAlpha;
		DstRayDir = -DstRayDir;
	}
	FMeshSurfaceDirection OutGoingDirection(DstEdgeID, DstRayDir);
	return TraceTriangleFromEdge(Mesh, DstTriID, DstAlpha, OutGoingDirection, ScratchTangentTri2, MaxDistance);
}


GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceNextTriangle(const FDynamicMesh3& Mesh, const  FMeshGeodesicSurfaceTracer::FTraceResult& StartTrace, double MaxDistance)
{
	FTangentTri2 ScratchTangentTri2;
	return TraceNextTriangle(Mesh, StartTrace, ScratchTangentTri2, MaxDistance);
}
GeodesicSingleTriangleUtils::FTraceResult  GeodesicSingleTriangleUtils::TraceNextTriangle(const FDynamicMesh3& Mesh, const FMeshGeodesicSurfaceTracer::FTraceResult& LastTrace, FTangentTri2& ScratchTangentTri2, double MaxDistance)
{

	constexpr double CornerEpsilon = 1.e-4;
	if (LastTrace.bIsEdgePoint)
	{
		double EdgeAlpha = LastTrace.EdgeAlpha;
		// don't allow starting on a vertex: displace slightly from the vertex and trace from edge.
		if (FMath::Abs(EdgeAlpha - 1.) < CornerEpsilon)
		{
			EdgeAlpha = 1. - CornerEpsilon;
		}
		else if (FMath::Abs(EdgeAlpha - 0.) < CornerEpsilon)
		{
			EdgeAlpha = CornerEpsilon;
		}

		const int32 EdgeID = LastTrace.SurfaceDirection.EdgeID;
		int TriID; // The triangle we are entering.
		{
			FIndex2i AdjTriIDs = Mesh.GetEdgeT(EdgeID);
			TriID = (AdjTriIDs.A == LastTrace.TriID) ? AdjTriIDs.B : AdjTriIDs.A;
		}

		if (TriID == FDynamicMesh3::InvalidID)
		{
			// we hit a boundary, copy the trace surface point and direction, but update the classification and distance
			FTraceResult Result = LastTrace;

			Result.Classification = FMeshGeodesicSurfaceTracer::ETraceClassification::BoundaryTerminated;
			Result.TraceDist = 0.;

			return Result;
		}

		//start from edge
		return TraceTriangleFromEdge(Mesh, TriID, EdgeAlpha, LastTrace.SurfaceDirection, ScratchTangentTri2, MaxDistance);
	}
	else
	{
		return TraceTriangleFromBaryPoint(Mesh, LastTrace.TriID, LastTrace.Barycentric, LastTrace.SurfaceDirection, ScratchTangentTri2, MaxDistance);
	}

}


double FMeshGeodesicSurfaceTracer::TraceMeshFromBaryPoint(const int32 TriID, const FVector3d& BaryPoint, const FVector3d& RayDir3, double MaxDistance)
{
	using namespace GeodesicSingleTriangleUtils;

	check(Mesh);

	checkSlow(IsBarycentricPointInTriangle(BaryPoint));

	// sanitize.. 
	FVector3d BaryPointClamped = ClampBarycentric(BaryPoint);

	// project the input direction onto the plane of the specified triangle.

	// create a frame for the face with the origin at V0	
	const FIndex3i TriVIDs = Mesh->GetTriangle(TriID);
	const FVector3d ZDir = Mesh->GetTriNormal(TriID);
	const FVector3d XDir = Normalized(Mesh->GetVertex(TriVIDs[1]) - Mesh->GetVertex(TriVIDs[0]));
	// Unreal has Left-Hand Coordinate System so we need to reverse this cross-product. (would be Y = Z x X  normally)
	const FVector3d YDir = Normalized(FVector3d::CrossProduct(XDir, ZDir));

	// project the ray onto the X,Y plane ( i.e. the face of the triangle )
	FVector2d ProjectedRay(FVector3d::DotProduct(XDir, RayDir3), FVector3d::DotProduct(YDir, RayDir3));
	if (!ProjectedRay.Normalize())
	{
		ProjectedRay = FVector2d::UnitX();
	}

	// lower level code expects the 2d ray to be encoded relative to the "direction" of the first edge, edge(v0,v1)
	const int32 EID = Mesh->GetTriEdge(TriID, 0);
	const FIndex2i Edge0 = Mesh->GetEdgeV(EID);

	if (Edge0.B == TriVIDs.A && Edge0.A == TriVIDs.B)
	{
		ProjectedRay = -ProjectedRay;
	}

	FMeshSurfaceDirection Direction(EID, ProjectedRay);
	return TraceMeshFromBaryPoint(TriID, BaryPointClamped, Direction, MaxDistance);
}

double FMeshGeodesicSurfaceTracer::TraceMeshFromBaryPoint(const int32 TriID, const FVector3d& BaryPoint, const FMeshSurfaceDirection& Direction, double MaxDistance)
{
	using namespace GeodesicSingleTriangleUtils;

	check(Mesh);

	checkSlow(IsBarycentricPointInTriangle(BaryPoint));

	// sanitize.. 
	FVector3d BaryPointClamped = ClampBarycentric(BaryPoint);

	FVector2d RayDir = Normalized(Direction.Dir);
	const int32 EID = Direction.EdgeID;
	// early out if the edge isn't part of the tri
	FIndex3i TriEdges = Mesh->GetTriEdges(TriID);
	if (TriEdges.A != EID && TriEdges.B != EID && TriEdges.C != EID)
	{
		return 0;
	}

	double TotalDist = 0.;
	FMeshSurfaceDirection StartDirection(EID, RayDir);
	{
		FTraceResult TraceStart;
		TraceStart.Classification = FMeshGeodesicSurfaceTracer::ETraceClassification::Start;
		TraceStart.TraceDist = 0.;
		TraceStart.TriID = TriID;
		TraceStart.SurfaceDirection = StartDirection;
		TraceStart.Barycentric = BaryPointClamped;
		TraceStart.bIsEdgePoint = false;
		TraceStart.EdgeID = EID;
		TraceStart.EdgeAlpha = -1.;

		SurfaceTrace.Add(TraceStart);
	}

	FTraceResult TraceResult = TraceTriangleFromBaryPoint(*Mesh, TriID, BaryPointClamped, StartDirection, ScratchTri2, MaxDistance);
	bool bTerminated = IsTerminated(TraceResult);
	TotalDist += TraceResult.TraceDist;
	SurfaceTrace.Add(TraceResult);

	MaxDistance -= TraceResult.TraceDist;
	if (bTerminated)
	{
		return TotalDist;
	}

	do
	{
		TraceResult = TraceNextTriangle(*Mesh, TraceResult, ScratchTri2, MaxDistance);
		bTerminated = IsTerminated(TraceResult);
		TotalDist += TraceResult.TraceDist;
		MaxDistance -= TraceResult.TraceDist;
		SurfaceTrace.Add(TraceResult);

	} while (bTerminated == false);

	return TotalDist;
}


double FMeshGeodesicSurfaceTracer::TraceMeshFromVertex(const FMeshTangentDirection& Direction, double MaxDistance)
{
	using namespace GeodesicSingleTriangleUtils;

	check(Mesh);

	const int32 VID = Direction.VID;
	const int32 EID = Direction.EdgeID;
	// early out if we try to trace from a bowtie
	bool bIsBowTie = Mesh->IsBowtieVertex(VID);
	if (bIsBowTie)
	{
		return 0;
	}

	double TotalDist = 0.;
	FMeshSurfaceDirection StartDirection(EID, FVector2d(FMath::Cos(Direction.PolarAngle), FMath::Sin(Direction.PolarAngle)));
	{
		int TriID = Mesh->GetEdgeT(EID).A;
		FIndex3i TriVIDs = Mesh->GetTriangle(TriID);
		int IndexOfVID = TriVIDs.IndexOf(VID);
		FVector3d BaryPoint(0);
		BaryPoint[IndexOfVID] = 1.;

		FTraceResult TraceStart;
		TraceStart.Classification = ETraceClassification::Start;
		TraceStart.TraceDist = 0.;
		TraceStart.TriID = TriID;
		TraceStart.SurfaceDirection = StartDirection;
		TraceStart.Barycentric = BaryPoint;
		TraceStart.bIsEdgePoint = false;
		TraceStart.EdgeID = EID;
		TraceStart.EdgeAlpha = -1.;

		SurfaceTrace.Add(TraceStart);
	}

	FTraceResult TraceResult = TraceFromVertex(*Mesh, Direction, ScratchTri2, MaxDistance);
	bool bTerminated = IsTerminated(TraceResult);
	TotalDist += TraceResult.TraceDist;
	SurfaceTrace.Add(TraceResult);

	MaxDistance -= TraceResult.TraceDist;
	if (bTerminated)
	{
		return TotalDist;
	}

	do
	{
		TraceResult = TraceNextTriangle(*Mesh, TraceResult, ScratchTri2, MaxDistance);
		bTerminated = IsTerminated(TraceResult);
		TotalDist += TraceResult.TraceDist;
		MaxDistance -= TraceResult.TraceDist;
		SurfaceTrace.Add(TraceResult);

	} while (bTerminated == false);

	return TotalDist;
}
