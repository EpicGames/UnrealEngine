// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MeshIsoCurves.h"

#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Operations/LocalPlanarSimplify.h"

using namespace UE::Geometry;

namespace UE::Geometry::MeshIsoCurve
{
	// Given a triangle's vertices and its highest-weight labels, find the triangle's full set of weights for each of those labels
	static TStaticArray<FVector3f, 3> GetFullTriLabelWeights(FIndex3i TriV, FIndex3i TriLabels, FVector3f BestWeights, const TFunction<float(int32 VID, int32 Label)>& InGetVertexLabelWeight)
	{
		TStaticArray<FVector3f, 3> LabelWeights;
		for (int32 LabelIdx = 0; LabelIdx < 3; ++LabelIdx)
		{
			FVector3f& Wts = LabelWeights[LabelIdx];
			int32 Label = TriLabels[LabelIdx];
			for (int32 VertIdx = 0; VertIdx < 3; ++VertIdx)
			{
				if (VertIdx == LabelIdx)
				{
					Wts[VertIdx] = BestWeights[VertIdx];
				}
				else
				{
					Wts[VertIdx] = InGetVertexLabelWeight(TriV[VertIdx], Label);
				}
			}
		}
		return LabelWeights;
	}

	void FMultiLabelIsoCurveAdapter::SetDefaultFindCutFunctions(const TFunction<float(int32 VID, int32 Label)>& InGetVertexLabelWeight)
	{
		FindEdgeCutParam = [InGetVertexLabelWeight](FIndex2i EdgeV, FIndex2i EdgeLabels, FVector2f EdgeWeights) -> double
			{
				if (!ensure(EdgeLabels.A != EdgeLabels.B))
				{
					return .5;
				}
				// Get the edge weights that for the other (non-assigned, so lower) label on each vertex
				FVector2f AltEdgeWeights;
				for (int32 SubIdx = 0; SubIdx < 2; ++SubIdx)
				{
					AltEdgeWeights[SubIdx] = InGetVertexLabelWeight(EdgeV[SubIdx], EdgeLabels[1 - SubIdx]);
					checkSlow(AltEdgeWeights[SubIdx] <= EdgeWeights[SubIdx]);
				}
				// Solve for the point on the edge where the label weights cross (are equal)
				double Diff0 = EdgeWeights[0] - AltEdgeWeights[0];
				double Denom = Diff0 + (EdgeWeights[1] - AltEdgeWeights[1]);
				if (FMath::Abs(Denom) < UE_DOUBLE_SMALL_NUMBER)
				{
					return .5;
				}
				return FMath::Clamp(Diff0 / Denom, 0., 1.);
			};
		FindTriCutBaryCoords = [InGetVertexLabelWeight](FIndex3i TriV, FIndex3i TriLabels, FVector3f TriWeights) ->FVector3d
			{
				TStaticArray<FVector3f, 3> LabelWeights = GetFullTriLabelWeights(TriV, TriLabels, TriWeights, InGetVertexLabelWeight);
				// Given separate weights per vertex for each label, LabelWeights[NumLabels] 
				// we solve for the barycentric coordinates where the linearly-interpolated weights intersect at the same weight value
				// 
				// i.e., for TriWeights A,B,C and Barycentric coords W, we solve:
				//  W.Dot(A) == W.Dot(B) -> W.Dot(A-B) == 0
				//  W.Dot(B) == W.Dot(C) -> W.Dot(B-C) == 0
				//  W.X + W.Y + W.Z == 1
				// Expanding out w/ AmB = A-B and BmC = B-C:
				//  W.X * AmB.X + W.Y * AmB.Y + W.Z * AmB.Z == 0, (and same eqn for BmC)
				// Sub W.Z == 1 - W.X - W.Y, and re-distribute to get:
				//  W.X * (AmB.X - AmB.Z) + W.Y * (AmB.Y - AmB.Z) = -AmB.Z
				//  W.X * (BmC.X - BmC.Z) + W.Y * (BmC.Y - BmC.Z) = -BmC.Z
				// We solve this 2x2 system w/ Cramer's rule. For small determinant, we fall back to just taking the centroid.

				FVector3d AmB = FVector3d(LabelWeights[0] - LabelWeights[1]);
				FVector3d BmC = FVector3d(LabelWeights[1] - LabelWeights[2]);
				FMatrix2d M(
					AmB.X - AmB.Z, AmB.Y - AmB.Z,
					BmC.X - BmC.Z, BmC.Y - BmC.Z
				);
				double Det = M.Determinant();
				
				if (FMath::Abs(Det) < UE_DOUBLE_SMALL_NUMBER)
				{
					return FVector3d(1. / 3.);
				}
				FVector3d Res;
				Res.X = (M.Row0.Y * BmC.Z - AmB.Z * M.Row1.Y) / Det;
				Res.Y = (AmB.Z * M.Row1.X - M.Row0.X * BmC.Z) / Det;
				Res.Z = 1 - Res.X - Res.Y;
				return Res;
			};
	}

	void MultiLabelCut(const FTriangleMeshAdapterd& Mesh, const FMultiLabelIsoCurveAdapter& IsoCurveAdapter, const FMeshUpdateAdapter& UpdateAdapter, const FMultiLabelCutSettings& Settings)
	{
		const double SnapThreshSq = Settings.SnapToExistingTolerance * Settings.SnapToExistingTolerance;

		const int MaxVID = Mesh.MaxVertexID();
		TArray<int32> VertexLabels;
		TArray<float> VertexValues;
		VertexLabels.SetNumUninitialized(MaxVID);
		VertexValues.SetNumUninitialized(MaxVID);
		// Pass 1: the best label and corresponding weight for each vertex
		ParallelFor(MaxVID, [&Mesh, &IsoCurveAdapter, &Settings, &VertexLabels, &VertexValues](int32 VID)
		{
			if (Mesh.IsVertex(VID))
			{
				VertexLabels[VID] = IsoCurveAdapter.LabelVertex(VID, VertexValues[VID]);
			}
		});
		
		TMap<FIndex2i, TPair<int32,double>> EdgeCrossings;
		struct FTriCutInfo
		{
			int32 TID;
			FIndex3i EdgeCutVIDs;
			FVector3d EdgeCutParams;
			int32 PokeVID = INDEX_NONE;
			FVector3d BaryCoords;
			bool bSnapped = false;
		};
		TArray<FTriCutInfo> Cuts;

		// Pass 2: Decide which tris to cut, and add new cut vertices as needed (but no triangles yet)
		for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
		{
			if (!Mesh.IsTriangle(TID))
			{
				continue;
			}

			FIndex3i Tri = Mesh.GetTriangle(TID);
			FIndex3i TriLabels{ VertexLabels[Tri.A], VertexLabels[Tri.B], VertexLabels[Tri.C] };
			int32 NumCut = 0;
			FIndex3i EdgeCutVIDs = FIndex3i::Invalid();
			FVector3d EdgeCutParams(.5);
			bool bHasSnapped = false;
			if (Settings.bNeverCut)
			{
				bHasSnapped = TriLabels.A != TriLabels.B || TriLabels.B != TriLabels.C;
			}
			else
			{
				for (int32 PrevIdx = 2, SubIdx = 0; SubIdx < 3; PrevIdx = SubIdx++)
				{

					if (TriLabels[PrevIdx] != TriLabels[SubIdx] && (!IsoCurveAdapter.ShouldCutEdge || IsoCurveAdapter.ShouldCutEdge(TriLabels[PrevIdx], TriLabels[SubIdx])))
					{
						FIndex2i EdgeV(Tri[PrevIdx], Tri[SubIdx]);
						FIndex2i EdgeL(TriLabels[PrevIdx], TriLabels[SubIdx]);
						FVector2f EdgeW(VertexValues[EdgeV.A], VertexValues[EdgeV.B]);
						bool bSwapped = false;
						if (EdgeV.A > EdgeV.B)
						{
							bSwapped = true;
							EdgeV.Swap();
							EdgeL.Swap();
							Swap(EdgeW.X, EdgeW.Y);
						}
						TPair<int32, double>* FoundCut = EdgeCrossings.Find(EdgeV);
						int32 UseCutVID = INDEX_NONE;
						double EdgeParam = .5f;
						if (!FoundCut)
						{
							EdgeParam = IsoCurveAdapter.FindEdgeCutParam(EdgeV, EdgeL, EdgeW);
							bool bSnap = false;
							if (EdgeParam <= 0 || EdgeParam >= 1.)
							{
								bSnap = true;
							}
							else if (Settings.SnapToExistingTolerance > 0.)
							{
								FVector3d V[2];
								V[0] = Mesh.GetVertex(EdgeV.A);
								V[1] = Mesh.GetVertex(EdgeV.B);
								FVector3d InterpV = FMath::LerpStable(V[0], V[1], EdgeParam);
								bSnap = (FVector3d::DistSquared(V[0], InterpV) < SnapThreshSq)
									|| (FVector3d::DistSquared(V[1], InterpV) < SnapThreshSq);
							}
							if (!bSnap)
							{
								UseCutVID = UpdateAdapter.AddInterpolatedEdgeVertex(EdgeV.A, EdgeV.B, EdgeParam);
							}
							else
							{
								bHasSnapped = true;
							}
							// We add the crossing even if we didn't add a vertex due to snapping, 
							// so the same snapping will be picked up by any triangles sharing the edge
							EdgeCrossings.Add(EdgeV, TPair<int32, double>(UseCutVID, EdgeParam));
						}
						else
						{
							if (FoundCut->Key == INDEX_NONE)
							{
								bHasSnapped = true;
							}
							else
							{
								UseCutVID = FoundCut->Key;
								EdgeParam = FoundCut->Value;
							}
						}
						if (UseCutVID != INDEX_NONE)
						{
							++NumCut;
							EdgeCutVIDs[SubIdx] = UseCutVID;
							EdgeCutParams[SubIdx] = bSwapped ? (1 - EdgeParam) : EdgeParam;
						}
					}
				}
			}
			if (NumCut > 0)
			{
				FTriCutInfo& CutInfo = Cuts.AddDefaulted_GetRef();
				if (NumCut == 3)
				{
					FVector3f TriWeights(VertexValues[Tri.A], VertexValues[Tri.B], VertexValues[Tri.C]);
					CutInfo.BaryCoords = IsoCurveAdapter.FindTriCutBaryCoords(Tri, TriLabels, TriWeights);
					// Do not poke if the barycentric coordinates are outside or very close to the boundary of the triangle
					bool bSnapPoke = false;
					if (CutInfo.BaryCoords.GetMin() < UE_DOUBLE_SMALL_NUMBER)
					{
						bSnapPoke = true;
					}
					else if (Settings.SnapToExistingTolerance > 0.)
					{
						FTriangle3d TriVerts(Mesh.GetVertex(Tri.A), Mesh.GetVertex(Tri.B), Mesh.GetVertex(Tri.C));
						FVector3d InterpV = TriVerts.BarycentricPoint(CutInfo.BaryCoords);
						for (int32 Idx = 0, Prev = 2; Idx < 3; Prev = Idx++)
						{
							if ((double)FMath::PointDistToSegmentSquared(InterpV, TriVerts.V[Prev], TriVerts.V[Idx]) < SnapThreshSq)
							{
								bSnapPoke = true;
								break;
							}
						} 
					}
					if (!bSnapPoke)
					{
						CutInfo.PokeVID = UpdateAdapter.AddInterpolatedTriangleVertex(Tri, CutInfo.BaryCoords);
					}
					
				}
				CutInfo.TID = TID;
				CutInfo.EdgeCutVIDs = EdgeCutVIDs;
				CutInfo.EdgeCutParams = EdgeCutParams;
				CutInfo.bSnapped = bHasSnapped;
			}
			else
			{
				if (bHasSnapped) // tri verts were multi-label, but all cuts were snapped away
				{
					FVector3f BestTriWeights(VertexValues[Tri.A], VertexValues[Tri.B], VertexValues[Tri.C]);
					TStaticArray<FVector3f, 3> FullTriWeights = GetFullTriLabelWeights(Tri, TriLabels, BestTriWeights, IsoCurveAdapter.GetVertexLabelWeight);
					const FVector3f MidBary(1.f / 3.f);
					int32 LabelIdx = FMath::Max3Index(FullTriWeights[0].Dot(MidBary), FullTriWeights[1].Dot(MidBary), FullTriWeights[2].Dot(MidBary));
					UpdateAdapter.LabelTriangle(TID, TriLabels[LabelIdx]);
				}
				else // tri is cleanly a single label
				{
					UpdateAdapter.LabelTriangle(TID, TriLabels[0]);
				}
			}
		}


		// Pass 3: Add triangles as needed for cut triangles
		TArray<FIndex3i, TInlineAllocator<6>> NewTris;
		TArray<int32, TInlineAllocator<6>> NewTriLabels;
		for (FTriCutInfo& CutInfo : Cuts)
		{
			int32 NumCut = 0;
			int32 FirstCut = INDEX_NONE, FirstNoCut = INDEX_NONE;
			NewTris.Reset();
			NewTriLabels.Reset();
			FIndex3i Tri = Mesh.GetTriangle(CutInfo.TID);
			
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				if (CutInfo.EdgeCutVIDs[SubIdx] != INDEX_NONE)
				{
					if (FirstCut == INDEX_NONE)
					{
						FirstCut = SubIdx;
					}
					++NumCut;
				}
				else if (FirstNoCut == INDEX_NONE)
				{
					FirstNoCut = SubIdx;
				}
			}
			if (!ensure(NumCut > 0)) // should not have added to CutInfo if nothing to cut
			{
				continue;
			}

			TStaticArray<FVector3f, 3> FullTriWeights;
			if (CutInfo.bSnapped || (CutInfo.PokeVID == INDEX_NONE && NumCut == 3))
			{
				FVector3f BestTriWeights(VertexValues[Tri.A], VertexValues[Tri.B], VertexValues[Tri.C]);
				FIndex3i TriLabels(VertexLabels[Tri.A], VertexLabels[Tri.B], VertexLabels[Tri.C]);
				FullTriWeights = GetFullTriLabelWeights(Tri, TriLabels, BestTriWeights, IsoCurveAdapter.GetVertexLabelWeight);
			}

			// mapping from local indexing to actual mesh vertex IDs
			// Original triangle vertices: 0,1,2; Triangle edge cut vertices: 3,4,5; Poke vertex: 6
			const int32 LocalToGlobal[7]{ Tri[0], Tri[1], Tri[2], CutInfo.EdgeCutVIDs[0], CutInfo.EdgeCutVIDs[1], CutInfo.EdgeCutVIDs[2], CutInfo.PokeVID };
			// Offsets for local indexing -- EOff for edge cuts, TOff for the triangle 'poke' vertex
			constexpr int32 EOff = 3;
			constexpr int32 TOff = 6;

			auto AddCutTri = [&VertexLabels, &NewTris, &NewTriLabels, &CutInfo, &FullTriWeights, &LocalToGlobal, &Tri, EOff, TOff]
				(FIndex3i LocalTriIndices, int32 DefaultLabel, bool bComputeLabel) -> void
			{
				FIndex3i NewTri(
					LocalToGlobal[LocalTriIndices[0]],
					LocalToGlobal[LocalTriIndices[1]],
					LocalToGlobal[LocalTriIndices[2]]);
				NewTris.Add(NewTri);
				// for cases where some snapping was applied, there are effectively multiple labels on some vertices
				// so rather than copy vertex labels directly, we compute a best label from the triangle centroid
				if (bComputeLabel)
				{
					// Get the centroid of the new triangle, expressed as barycentric coordinates of the original triangle
					FVector3d BarySum(0.);
					for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
					{
						int32 LocalIdx = LocalTriIndices[SubIdx];
						FVector3d Bary(0.);
						if (LocalIdx < EOff)
						{
							Bary[LocalIdx] = 1.;
						}
						else if (LocalIdx < TOff)
						{
							double Param = CutInfo.EdgeCutParams[LocalIdx - 3];
							Bary[LocalIdx - 3] = Param;
							Bary[(LocalIdx + 2) % 3] = 1. - Param;
						}
						else
						{
							Bary = CutInfo.BaryCoords;
						}
						BarySum += Bary;
					}
					// Choose the label w/ the highest interpolated weight on the original triangle, at the new triangle's centroid
					FVector3f MidBary = FVector3f(BarySum / 3.);
					int32 LabelIdx = FMath::Max3Index(FullTriWeights[0].Dot(MidBary), FullTriWeights[1].Dot(MidBary), FullTriWeights[2].Dot(MidBary));
					NewTriLabels.Add(VertexLabels[Tri[LabelIdx]]);
				}
				// for cases w/out snapping, we just copy an appropriate vertex label
				else
				{
					NewTriLabels.Add(DefaultLabel);
				}
			};

			if (NumCut == 1)
			{
				int32 Shift[3]{ FirstCut, (FirstCut + 1) % 3, (FirstCut + 2) % 3 };
				int32 Label = VertexLabels[Tri[Shift[1]]];
				int32 CutV = CutInfo.EdgeCutVIDs[FirstCut];
				AddCutTri(FIndex3i(EOff + FirstCut, Shift[0], Shift[1]), Label, CutInfo.bSnapped);
				AddCutTri(FIndex3i(EOff + FirstCut, Shift[1], Shift[2]), Label, CutInfo.bSnapped);
			}
			else if (NumCut == 2)
			{
				int32 ShiftN[3]{ FirstNoCut, (FirstNoCut + 1) % 3, (FirstNoCut + 2) % 3 };
				int32 Label0 = VertexLabels[Tri[ShiftN[0]]];
				int32 Label1 = VertexLabels[Tri[ShiftN[1]]];
				AddCutTri(FIndex3i(ShiftN[0], EOff + ShiftN[1], EOff + ShiftN[2]), Label0, CutInfo.bSnapped);
				AddCutTri(FIndex3i(EOff + ShiftN[2], ShiftN[2], ShiftN[0]), Label0, CutInfo.bSnapped);
				AddCutTri(FIndex3i(EOff + ShiftN[1], ShiftN[1], EOff + ShiftN[2]), Label1, CutInfo.bSnapped);
			}
			else // NumCut == 3
			{
				// We didn't add a new 'poke' vertex inside the triangle, so use a 1->4 tri split
				if (CutInfo.PokeVID == INDEX_NONE)
				{
					AddCutTri(FIndex3i(EOff + 0, EOff + 1, EOff + 2), 0, true);
					for (int32 Idx = 0, Next = 1, Prev = 2; Idx < 3; Next = Prev, Prev = Idx++)
					{
						int32 Label = VertexLabels[Tri[Idx]];
						AddCutTri(FIndex3i(EOff + Idx, Idx, EOff + Next), Label, CutInfo.bSnapped);
					}
				}
				else
				{
					// 1->6 tri split, making a fan from the new 'poke' vertex
					for (int32 Idx = 0, Next = 1, Prev = 2; Idx < 3; Next = Prev, Prev = Idx++)
					{
						int32 Label = VertexLabels[Tri[Idx]];
						AddCutTri(FIndex3i(EOff + Idx, Idx, TOff), Label, CutInfo.bSnapped);
						AddCutTri(FIndex3i(Idx, EOff + Next, TOff), Label, CutInfo.bSnapped);
					}
				}
			}

			check(NewTris.Num() == NewTriLabels.Num());
			UpdateAdapter.ReplaceTriangle(CutInfo.TID, NewTris, NewTriLabels);
		}
	}
}

void FMeshIsoCurves::Cut(FDynamicMesh3& Mesh, TFunctionRef<float(int32)> VertexFn, TFunctionRef<float(int32 VIDMin, int32 VIDMax, float ValMin, float ValMax)> EdgeCutFn, float IsoValue)
{
	int MaxVID = Mesh.MaxVertexID();
	TArray<float> VertexValues;
	VertexValues.SetNumUninitialized(MaxVID);

	constexpr bool bNoParallel = false;
	ParallelFor(MaxVID, [this, &VertexValues, &Mesh, &VertexFn, IsoValue](int32 VID)
	{
		if (Mesh.IsVertex(VID))
		{
			VertexValues[VID] = VertexFn(VID);
		}
		else
		{
			// set IsoValue on invalid vertices; any vertex that is later inserted and uses this ID will be on the curve, so should have this value
			VertexValues[VID] = IsoValue;
		}
	}, bNoParallel);

	TSet<int> OnCutEdges;
	SplitCrossingEdges(Mesh, VertexValues, OnCutEdges, EdgeCutFn, IsoValue);

	// collapse degenerate edges
	if (Settings.bCollapseDegenerateEdgesOnCut)
	{
		FLocalPlanarSimplify::CollapseDegenerateEdges(Mesh, OnCutEdges, false, Settings.DegenerateEdgeTol);
	}

}

void FMeshIsoCurves::SplitCrossingEdges(FDynamicMesh3& Mesh, const TArray<float>& VertexValues,
	TSet<int32>& OnCutEdges,
	TFunctionRef<float(int32 VIDMin, int32 VIDMax, float ValMin, float ValMax)> EdgeCutFn,
	float IsoValue)
{
	OnCutEdges.Reset();

	// have to skip processing of new edges. If edge id
	// is > max at start, is new. Otherwise if in NewEdges list, also new.
	int MaxEID = Mesh.MaxEdgeID();
	TSet<int> NewEdgesBeforeMaxID;
	auto AddNewEdge = [&NewEdgesBeforeMaxID, MaxEID](int32 NewEID)
	{
		if (NewEID < MaxEID)
		{
			NewEdgesBeforeMaxID.Add(NewEID);
		}
	};

	double UseSnapVertTol = FMath::Max(0, Settings.SnapToExistingVertexTol);
	const double SnapExistingTolSq = UseSnapVertTol * UseSnapVertTol;

	// split existing edges where the value crosses isovalue
	for (int32 EID = 0; EID < MaxEID; ++EID)
	{
		if (!Mesh.IsEdge(EID) || NewEdgesBeforeMaxID.Contains(EID))
		{
			continue;
		}

		FIndex2i EdgeV = Mesh.GetEdgeV(EID);
		const float ValueA = VertexValues[EdgeV.A];
		const float ValueB = VertexValues[EdgeV.B];
		const float DistA = ValueA - IsoValue;
		const float DistB = ValueB - IsoValue;

		// If both Signs are 0, this edge is on-contour
		// If one sign is 0, that vertex is on-contour
		int AOnCurve = (FMathd::Abs(DistA) <= Settings.CurveIsoValueSnapTolerance) ? 1 : 0;
		int BOnCurve = (FMathd::Abs(DistB) <= Settings.CurveIsoValueSnapTolerance) ? 1 : 0;
		if (AOnCurve || BOnCurve)
		{
			continue;
		}

		// no crossing
		if (DistA * DistB >= 0)
		{
			continue;
		}

		double Param = EdgeCutFn(EdgeV.A, EdgeV.B, ValueA, ValueB);
		// Cut must be within edge
		if (Param <= 0 || Param >= 1)
		{
			continue;
		}
		// Skip the edge split if we're within tolerance of an existing vertex
		if (SnapExistingTolSq > 0)
		{
			FVector3d PosA = Mesh.GetVertex(EdgeV.A);
			FVector3d PosB = Mesh.GetVertex(EdgeV.B);
			FVector3d EdgeVec = PosB - PosA;
			double MinSepSq = (EdgeVec * (Param > .5 ? 1 - Param : Param)).SquaredLength();
			if (MinSepSq <= SnapExistingTolSq)
			{
				continue;
			}
		}

		FDynamicMesh3::FEdgeSplitInfo SplitInfo;
		EMeshResult SplitResult = Mesh.SplitEdge(EID, SplitInfo, Param);
		if (!ensureMsgf(SplitResult == EMeshResult::Ok, TEXT("FMeshIsoCurves::SplitCrossingEdges: failed to SplitEdge")))
		{
			continue; // edge split really shouldn't fail; skip the edge if it somehow does
		}

		AddNewEdge(SplitInfo.NewEdges.A);
		AddNewEdge(SplitInfo.NewEdges.B);

		// We need to check whether the other vertices are on curve to decide if the connected edges are on the curve or not
		int32 OtherVIDA = SplitInfo.OtherVertices.A;
		// Other vertex is on curve if it's newly created or within curve isovalue tolerance
		// (Note a newly-created vertex w/ ID < VertexValues.Num() will also have a VertexValue of IsoValue, since we use this as the default value)
		if (OtherVIDA >= VertexValues.Num() || FMath::Abs(VertexValues[OtherVIDA] - IsoValue) <= Settings.CurveIsoValueSnapTolerance)
		{
			OnCutEdges.Add(SplitInfo.NewEdges.B);
		}
		
		if (SplitInfo.NewEdges.C != FDynamicMesh3::InvalidID)
		{
			AddNewEdge(SplitInfo.NewEdges.C);
			int32 OtherVIDB = SplitInfo.OtherVertices.B;
			if (OtherVIDB >= VertexValues.Num() || FMath::Abs(VertexValues[OtherVIDB] - IsoValue) <= Settings.CurveIsoValueSnapTolerance)
			{
				OnCutEdges.Add(SplitInfo.NewEdges.C);
			}
		}
	}
}
