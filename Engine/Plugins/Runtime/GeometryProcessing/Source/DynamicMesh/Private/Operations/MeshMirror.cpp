// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MeshMirror.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMeshEditor.h"

using namespace UE::Geometry;

// Forward declarations of local functions
void MirrorAndAppendVertices(FDynamicMesh3& Mesh, const FVector3d& PlaneNormal, const FVector3d& PlaneOrigin,
	TArray<int32>& OutVidToMirrorMap);
void MirrorAndAppendVerticesWithWelding(FDynamicMesh3& Mesh, const FVector3d& PlaneNormal, const FVector3d& PlaneOrigin,
	TArray<int32>& OutVidToMirrorMap, bool bAllowBowtieVertexCreation, double PlaneTolerance, EMeshMirrorNormalMode WeldNormalMode);
FVertexInfo CreateMirrorVertex(const FVertexInfo& OriginalVertexInfo, const FVector3d& PlaneNormal, double SignedDistance);
FVertexInfo CreateMirrorVertex(const FVertexInfo& OriginalVertexInfo, const FVector3d& PlaneNormal, const FVector3d& PlaneOrigin);

void FMeshMirror::Mirror(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// We assume the normal is unit length everywhere, so it's worth forcing it to be just in case.
	Normalize(PlaneNormal);

	// Change the existing vertex positions and normals to be mirrored.
	for (int32 Vid : Mesh->VertexIndicesItr())
	{
		FVector3d OldPosition = Mesh->GetVertex(Vid);
		FVector3d NewPosition = OldPosition - 2 * PlaneNormal.Dot(OldPosition - PlaneOrigin) * PlaneNormal;
		Mesh->SetVertex(Vid, NewPosition);

		if (Mesh->HasVertexNormals())
		{
			FVector3f OldNormal = Mesh->GetVertexNormal(Vid);
			FVector3f NewNormal = OldNormal - (FVector3f)(2 * PlaneNormal.Dot((FVector3d)OldNormal) * PlaneNormal);
			Mesh->SetVertexNormal(Vid, NewNormal);
		}

		// Color and UV don't need adjustment.
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// The triangle orientations need to be reversed
	for (int32 Tid : Mesh->TriangleIndicesItr())
	{
		Mesh->ReverseTriOrientation(Tid);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (Mesh->HasAttributes())
	{
		// Any normal layers need flipping
		FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
		for (int LayerIndex = 0; LayerIndex < Attributes->NumNormalLayers(); ++LayerIndex)
		{
			FDynamicMeshNormalOverlay* NormalOverlay = Attributes->GetNormalLayer(LayerIndex);
			for (int32 i : NormalOverlay->ElementIndicesItr())
			{
				FVector3f OldNormal = NormalOverlay->GetElement(i);
				FVector3f NewNormal = OldNormal - (FVector3f)(2 * PlaneNormal.Dot((FVector3d)OldNormal) * PlaneNormal);
				NormalOverlay->SetElement(i, NewNormal);
			}
		}
		// Currently, these are the only attributes that need mirroring. If that ever changes, or if this turns out
		// to be common across multiple tools, we should consider adding a function inside DynamicMeshEditor.
	}
}

void FMeshMirror::MirrorAndAppend(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// We assume the normal is unit length everywhere, so it's worth forcing it to be just in case.
	Normalize(PlaneNormal);

	// Start by creating and appending mirrored vertices. VidToMirrorMap will map each original
	// vertex to the id of its mirror (which may be itself if it is not duplicated due to welding).
	TArray<int32> VidToMirrorMap;
	if (bWeldAlongPlane)
	{
		MirrorAndAppendVerticesWithWelding(*Mesh, PlaneNormal, PlaneOrigin, VidToMirrorMap,
			bAllowBowtieVertexCreation, PlaneTolerance, WeldNormalMode);
	}
	else
	{
		MirrorAndAppendVertices(*Mesh, PlaneNormal, PlaneOrigin, VidToMirrorMap);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Now we'll need to fill in the new triangles. An array of original indices lets us iterate
	// properly even as we add new triangles to the mesh.
	TArray<int32> OriginalTriangleIndices;
	OriginalTriangleIndices.Reserve(Mesh->TriangleCount());
	for (int32 Tid : Mesh->TriangleIndicesItr())
	{
		OriginalTriangleIndices.Add(Tid);
	}

	// We'll also want new groups for all the new triangles, and a mapping from new to old.
	TMap<int, int> GroupMapping;

	// These are used for copying attributes over as the triangles are mirrored
	FDynamicMeshEditor MeshEditor(Mesh);
	FMeshIndexMappings IndexMappings;
	IndexMappings.Initialize(Mesh);
	FDynamicMeshEditResult ResultAccumulator;

	for (int32 Tid : OriginalTriangleIndices)
	{
		FIndex3i OriginalTriangle = Mesh->GetTriangle(Tid);

		// Create the new triangle. We're going to reverse it in a second, but that is better to
		// do after copying the attributes so that they get reversed properly.
		int32 NewTid;
		if (!Mesh->HasTriangleGroups())
		{
			NewTid = Mesh->AppendTriangle(
				VidToMirrorMap[OriginalTriangle.A],
				VidToMirrorMap[OriginalTriangle.B],
				VidToMirrorMap[OriginalTriangle.C]
			);
		}
		else
		{
			int Gid = Mesh->GetTriangleGroup(Tid);
			if (!GroupMapping.Contains(Gid))
			{
				GroupMapping.Add(Gid, Mesh->AllocateTriangleGroup());
			}
			NewTid = Mesh->AppendTriangle(
				VidToMirrorMap[OriginalTriangle.A],
				VidToMirrorMap[OriginalTriangle.B],
				VidToMirrorMap[OriginalTriangle.C],
				GroupMapping[Gid]);
		}
		check(NewTid >= 0);

		// Copy attributes before reversing the triangle. We'll adjust them in a separate pass, to allow for a cancel in between.
		if (Mesh->HasAttributes())
		{
			MeshEditor.CopyAttributes(Tid, NewTid, IndexMappings, ResultAccumulator);
		}

		// Reverse the triangle.
		Mesh->ReverseTriOrientation(NewTid);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Though we copied attributes over, some of them need mirroring
	if (Mesh->HasAttributes())
	{
		FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();

		// Specifically, we need to mirror any new normals in the normal layers
		for (int LayerIndex = 0; LayerIndex < ResultAccumulator.NewNormalOverlayElements.Num(); ++LayerIndex)
		{
			FDynamicMeshNormalOverlay* NormalOverlay = Attributes->GetNormalLayer(LayerIndex);

			for (int32 ElementIndex : ResultAccumulator.NewNormalOverlayElements[LayerIndex])
			{
				FVector3f OldNormal = NormalOverlay->GetElement(ElementIndex);
				FVector3f NewNormal = OldNormal - (FVector3f)(2 * PlaneNormal.Dot((FVector3d)OldNormal) * PlaneNormal);
				NormalOverlay->SetElement(ElementIndex, NewNormal);
			}
		}
		
		// Currently, these are the only attributes that need mirroring. If that ever changes, or if this turns out
		// to be common across multiple tools, we should consider adding a function inside DynamicMeshEditor.
	}
}

/**
 * Creates mirror copies of vertices in the mesh and appends them to the source mesh. Does not do anything to triangles.
 *
 * @param Mesh Mesh to mirror and append to.
 * @param PlaneNormal Mirror plane normal, in mesh coordinate space.
 * @param PlaneOrigin Mirror plane origin, in mesh coordinate space.
 * @param OutVidToMirrorMap Output map that maps original vertex id's to their mirror counterpart.
 */
void MirrorAndAppendVertices(FDynamicMesh3& Mesh, const FVector3d& PlaneNormal, const FVector3d& PlaneOrigin,
	TArray<int32>& OutVidToMirrorMap)
{
	OutVidToMirrorMap.Empty();
	OutVidToMirrorMap.AddUninitialized(Mesh.MaxVertexID());

	TArray<int32> OriginalVertexIds;
	OriginalVertexIds.Reserve(Mesh.VertexCount());
	for (int32 Vid : Mesh.VertexIndicesItr())
	{
		OriginalVertexIds.Add(Vid);
	}

	for (int32 Vid : OriginalVertexIds)
	{
		OutVidToMirrorMap[Vid] = Mesh.AppendVertex(CreateMirrorVertex(Mesh.GetVertexInfo(Vid), PlaneNormal, PlaneOrigin));
	}
}

/**
 * Creates mirror copies of vertices in the mesh while welding vertices on the mirror plane and appends them to the source
 * mesh. Does not do anything to triangles. Vertices will not be welded if doing so would create an edge with more than two
 * faces, or if they are part of a face in the mirror plane.
 *
 * @param Mesh Mesh to mirror and append to.
 * @param PlaneNormal Mirror plane normal, in mesh coordinate space.
 * @param PlaneOrigin Mirror plane origin, in mesh coordinate space.
 * @param OutVidToMirrorMap Output map that maps original vertex id's to their mirror counterpart. 
 *   This is the same vid for vertices on the mirror plane.
 * @param bAllowBowtieVertexCreation Whether bowtie vertices should be created when a point lies on the mirror plane
 *   without an edge in the plane.
 * @param PlaneTolerance Tolerance to use when determining whether a vertex is in the mirror plane.
 */
void MirrorAndAppendVerticesWithWelding(FDynamicMesh3& Mesh, const FVector3d& PlaneNormal, const FVector3d& PlaneOrigin,
	TArray<int32>& OutVidToMirrorMap, bool bAllowBowtieVertexCreation, double PlaneTolerance, EMeshMirrorNormalMode WeldNormalMode)
{
	// This will map an original vertex id to the id of its mirror. For vertices on the mirror plane,
	// this will map the vertex id to the same vertex id, (unless we duplicate the vertex), though we
	// also keep a separate set that marks vertices as being on the plane, as this allows us to have
	// welding/duplication behavior that doesn't depend on order of iteration through vertices.
	OutVidToMirrorMap.Empty();
	OutVidToMirrorMap.AddUninitialized(Mesh.MaxVertexID());

	TSet<int32> VerticesOnPlane;

	// Since we'll be adding to the same mesh as we go, we need to limit the vertices we iterate over
	TArray<int32> OriginalVertexIds;
	OriginalVertexIds.Reserve(Mesh.VertexCount());
	for (int32 Vid : Mesh.VertexIndicesItr())
	{
		OriginalVertexIds.Add(Vid);
	}

	// Sort out the weldable vertices and mirror the rest.
	for (int32 Vid : OriginalVertexIds)
	{
		double SignedDistanceFromPlane = (Mesh.GetVertex(Vid) - PlaneOrigin).Dot(PlaneNormal); // As long as normal is normalized
		if (FMathd::Abs(SignedDistanceFromPlane) < PlaneTolerance)
		{
			// On the mirror plane, so potentially a weldable vertex
			OutVidToMirrorMap[Vid] = Vid;
			VerticesOnPlane.Add(Vid);
		}
		else
		{
			// Create the mirror vertex
			OutVidToMirrorMap[Vid] = Mesh.AppendVertex(CreateMirrorVertex(Mesh.GetVertexInfo(Vid), PlaneNormal, SignedDistanceFromPlane));
		}
	}

	// Sort the weldable vertices into ones that will and will not actually be duplicated (vertices may
	// need duplication if they will create non-manifold geometry after mirroring. Also, depending on settings,
	// we may allow or disallow bowtie vertex creation.
	const bool bHasVertexNormals = Mesh.HasVertexNormals();
	const bool bHasAttributeNormals = Mesh.Attributes() && Mesh.Attributes()->PrimaryNormals();
	for (int32 Vid : VerticesOnPlane)
	{
		// The rules we use are as follows:
		// 1. Vertices that are not part of any edges in the plane would become bowtie vertices, whose duplication
		//    will be dependent on a parameter.
		// 2. Vertices that are a part of a 2-triangle (i.e., non-boundary) edge in the plane must be duplicated,
		//    as we won't be able to attach more triangles to them.
		// 3. Vertices that are part of only boundary edges in the plane should still get duplicated if they
		//    are part of a triangle in the plane, because not doing so would make duplication behavior dependent
		//    on absence of the other, non-plane triangle, which goes against the principle of least astonishment.
		// 4. Otherwise, the vertex can avoid duplication (which amounts to welding).

		bool bForcedToDuplicate = false;
		bool bAtLeastOneEdgeInPlane = false;

		TArray<int32> EdgesInPlane;

		// Check the incident edges
		for (int32 EdgeId : Mesh.VtxEdgesItr(Vid))
		{
			FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeId);
			int32 NeighborId = (Edge.Vert.A == Vid) ? Edge.Vert.B : Edge.Vert.A;

			// Is the edge in the plane?
			if (VerticesOnPlane.Contains(NeighborId))
			{
				bAtLeastOneEdgeInPlane = true;

				// Is this a non-boundary edge?
				if (Edge.Tri.B != FDynamicMesh3::InvalidID)
				{
					bForcedToDuplicate = true;
					break;
				}
				else
				{
					// If a boundary edge, does its triangle lie in the plane? To figure out, find opposite vertex.
					FIndex3i VertIndices = Mesh.GetTriangle(Edge.Tri.A);
					int32 OppositeVertexId = VertIndices.A;
					if (OppositeVertexId == Vid)
					{
						OppositeVertexId = (VertIndices.B == NeighborId) ? VertIndices.C : VertIndices.B;
					}
					else if (OppositeVertexId == NeighborId)
					{
						OppositeVertexId = (VertIndices.B == Vid) ? VertIndices.C : VertIndices.B;
					}

					// Now see if opposite vertex lies in the plane
					if (VerticesOnPlane.Contains(OppositeVertexId))
					{
						bForcedToDuplicate = true;
						break;
					}
				}
			}//end if edge is in plane

			// If we got here, the edge didn't force duplication, so go to the next edge
		}//end looking through incident edges

		if (bForcedToDuplicate || (!bAtLeastOneEdgeInPlane && !bAllowBowtieVertexCreation))
		{
			// Duplicate the vertex.
			OutVidToMirrorMap[Vid] = Mesh.AppendVertex(CreateMirrorVertex(Mesh.GetVertexInfo(Vid), PlaneNormal, PlaneOrigin));
		}
		else
		{
			// We don't need to duplicate vertex, but we do need to adjust it. Specifically, let's move it
			// exactly onto the plane, and make the normal an average with its mirror (i.e., project it
			// onto the plane).
			FVector3d OldPosition = Mesh.GetVertex(Vid);
			FVector3d NewPosition = OldPosition - PlaneNormal.Dot(OldPosition - PlaneOrigin) * PlaneNormal;
			Mesh.SetVertex(Vid, NewPosition);

			if (WeldNormalMode == EMeshMirrorNormalMode::AverageMirrorNormals)
			{
				auto ComputeVertexNormal = [&PlaneNormal](const FVector3f& OldNormal, FVector3f& NewNormal)
				{
					NewNormal = (OldNormal - OldNormal.Dot((FVector3f)PlaneNormal) * (FVector3f)PlaneNormal); // Not yet normalized
				
					// For bowtie vertices, the projection of the normal onto the plane might be 0. The normal here is nonsensical anyway,
					// so it doesn't matter what we do. Let's just leave it as is if we didn't get a projection.
					return Normalize(NewNormal) != 0;
				};

				if (bHasVertexNormals)
				{
					FVector3f OldNormal = Mesh.GetVertexNormal(Vid);
					FVector3f NewNormal;
					if (ComputeVertexNormal(OldNormal, NewNormal))
					{
						Mesh.SetVertexNormal(Vid, NewNormal);
					}
				}

				if (bHasAttributeNormals)
				{
					FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
					NormalOverlay->EnumerateVertexElements(Vid,[NormalOverlay, &ComputeVertexNormal](int, int ElementId, const FVector3f& OldNormal)
					{
						FVector3f NewNormal;
						if (ComputeVertexNormal(OldNormal, NewNormal))
						{
							NormalOverlay->SetElement(ElementId, NewNormal);
						}
						return true;
					});
				}
			}
		}//end if not duplicating vertex
	}//end dealing with vertices on plane
}

FVertexInfo CreateMirrorVertex(const FVertexInfo& OriginalVertexInfo, const FVector3d& PlaneNormal, double SignedDistance)
{
	FVertexInfo MirrorVertexInfo(OriginalVertexInfo);
	MirrorVertexInfo.Position = OriginalVertexInfo.Position - 2 * SignedDistance * PlaneNormal;

	// Normal has to get mirrored.
	if (MirrorVertexInfo.bHaveN)
	{
		MirrorVertexInfo.Normal = MirrorVertexInfo.Normal - 2 * MirrorVertexInfo.Normal.Dot((FVector3f)PlaneNormal) * (FVector3f)PlaneNormal;
	}
	// Color and UV don't need adjustment.

	return MirrorVertexInfo;
}

FVertexInfo CreateMirrorVertex(const FVertexInfo& OriginalVertexInfo, const FVector3d& PlaneNormal, const FVector3d& PlaneOrigin)
{
	return CreateMirrorVertex(OriginalVertexInfo, PlaneNormal, (OriginalVertexInfo.Position - PlaneOrigin).Dot(PlaneNormal));
}