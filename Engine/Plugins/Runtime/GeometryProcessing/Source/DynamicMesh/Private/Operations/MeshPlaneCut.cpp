// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MeshPlaneCut.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshTriangleAttribute.h"

#include "Operations/SimpleHoleFiller.h"
#include "Operations/PlanarHoleFiller.h"
#include "Operations/MinimalHoleFiller.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "MathUtil.h"
#include "Selections/MeshConnectedComponents.h"

#include "Async/ParallelFor.h"

using namespace UE::Geometry;

void FMeshPlaneCut::ComputeVertexSignedDistances(TArray<double>& Signs, double InvalidDist)
{
	int MaxVID = Mesh->MaxVertexID();
	Signs.SetNum(MaxVID);
	constexpr bool bNoParallel = false;
	ParallelFor(MaxVID, [&](int32 VID)
	{
		if (Mesh->IsVertex(VID))
		{
			Signs[VID] = (Mesh->GetVertex(VID) - PlaneOrigin).Dot(PlaneNormal);
		}
		else
		{
			Signs[VID] = InvalidDist;
		}
	}, bNoParallel);
}

void FMeshPlaneCut::SplitCrossingEdges(bool bDeleteTrisOnPlane, TArray<double>& Signs, TSet<int>& AlreadyOnPlaneEdges, TSet<int32>& CutPlaneEdges, TSet<int>* OnSplitEdges, TSet<int>* OnPlaneVertices, TSet<int32>* TriangleSelection)
{
	AlreadyOnPlaneEdges.Reset();
	CutPlaneEdges.Reset();
	if (OnSplitEdges)
	{
		OnSplitEdges->Reset();
	}
	if (OnPlaneVertices)
	{
		OnPlaneVertices->Reset();
	}

	// Compute signed distances at all vertices. Set invalid dists to zero, because any vertex we add will be on the plane.
	ComputeVertexSignedDistances(Signs, 0.0);

	if (bDeleteTrisOnPlane)
	{
		for (int TID = 0; TID < Mesh->MaxTriangleID(); TID++)
		{
			if (!Mesh->IsTriangle(TID))
			{
				continue;
			}

			FIndex3i Tri = Mesh->GetTriangle(TID);
			FIndex3i TriEdges = Mesh->GetTriEdges(TID);
			if (FMathd::Abs(Signs[Tri.A]) < PlaneTolerance &&
				FMathd::Abs(Signs[Tri.B]) < PlaneTolerance &&
				FMathd::Abs(Signs[Tri.C]) < PlaneTolerance)
			{
				Mesh->RemoveTriangle(TID, true, false);
				if (TriangleSelection)
				{
					// Remove triangle from the selection as well (does nothing if it was not in the selection already)
					TriangleSelection->Remove(TID);
				}
			}
		}
	}

	// have to skip processing of new edges. If edge id
	// is > max at start, is new. Otherwise if in NewEdges list, also new.
	int MaxEID = Mesh->MaxEdgeID();
	TSet<int> NewEdgesBeforeMaxID;
	auto AddNewEdge = [&NewEdgesBeforeMaxID, MaxEID](int32 NewEID)
	{
		if (NewEID < MaxEID)
		{
			NewEdgesBeforeMaxID.Add(NewEID);
		}
	};

	// cut existing edges with plane, using edge split
	for (int32 EID = 0; EID < MaxEID; ++EID)
	{
		if (!Mesh->IsEdge(EID) || NewEdgesBeforeMaxID.Contains(EID))
		{
			continue;
		}
		if (EdgeFilterFunc && EdgeFilterFunc(EID) == false)
		{
			continue;
		}

		FIndex2i EdgeV = Mesh->GetEdgeV(EID);
		const double DistA = Signs[EdgeV.A];
		const double DistB = Signs[EdgeV.B];

		// If both Signs are 0, this edge is on-contour
		// If one sign is 0, that vertex is on-contour
		int AOnPlane = (FMathd::Abs(DistA) < PlaneTolerance) ? 1 : 0;
		int BOnPlane = (FMathd::Abs(DistB) < PlaneTolerance) ? 1 : 0;
		if (AOnPlane || BOnPlane)
		{
			if (AOnPlane && BOnPlane)
			{
				AlreadyOnPlaneEdges.Add(EID);
				if (OnPlaneVertices)
				{
					OnPlaneVertices->Add(EdgeV.A);
					OnPlaneVertices->Add(EdgeV.B);
				}
			}
			else if (OnPlaneVertices)
			{
				// Note: if (!BOnPlane), then this will pick EdgeV.A, otherwise it will pick EdgeV.B
				OnPlaneVertices->Add(EdgeV[BOnPlane]);
			}
			continue;
		}

		// no crossing
		if (DistA * DistB > 0)
		{
			continue;
		}

		FDynamicMesh3::FEdgeSplitInfo SplitInfo;
		double Param = DistA / (DistA - DistB);
		EMeshResult SplitResult = Mesh->SplitEdge(EID, SplitInfo, Param);
		if (!ensureMsgf(SplitResult == EMeshResult::Ok, TEXT("FMeshPlaneCut::Cut: failed to SplitEdge")))
		{
			continue; // edge split really shouldn't fail; skip the edge if it somehow does
		}

		if (TriangleSelection && TriangleSelection->Contains(SplitInfo.OriginalTriangles.A))
		{
			TriangleSelection->Add(SplitInfo.NewTriangles.A);
		}

		AddNewEdge(SplitInfo.NewEdges.A);
		AddNewEdge(SplitInfo.NewEdges.B);

		// If requested, track the edges we've split
		if (OnSplitEdges)
		{
			OnSplitEdges->Add(EID);
			OnSplitEdges->Add(SplitInfo.NewEdges.A);
		}

		// We need to check whether the other vertices are on plane to decide if the connected edges are on the plane or not
		int32 OtherVIDA = SplitInfo.OtherVertices.A;
		// Other vertex is on plane if it's newly created  or within plane tolerance
		if (OtherVIDA >= Signs.Num() || FMath::Abs(Signs[OtherVIDA]) < PlaneTolerance)
		{
			CutPlaneEdges.Add(SplitInfo.NewEdges.B);
		}
		
		if (SplitInfo.NewEdges.C != FDynamicMesh3::InvalidID)
		{
			if (TriangleSelection && TriangleSelection->Contains(SplitInfo.OriginalTriangles.B))
			{
				TriangleSelection->Add(SplitInfo.NewTriangles.B);
			}
			AddNewEdge(SplitInfo.NewEdges.C);
			int32 OtherVIDB = SplitInfo.OtherVertices.B;
			if (OtherVIDB >= Signs.Num() || FMath::Abs(Signs[OtherVIDB]) < PlaneTolerance)
			{
				CutPlaneEdges.Add(SplitInfo.NewEdges.C);
			}
		}

		if (OnPlaneVertices)
		{
			OnPlaneVertices->Add(SplitInfo.NewVertex);
		}
	}
}

void FMeshPlaneCut::SplitCrossingEdges(TArray<double>& Signs, TSet<int>& ZeroEdges, TSet<int>& OnCutEdges, TSet<int>& OnSplitEdges, bool bDeleteTrisOnPlane)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SplitCrossingEdges(bDeleteTrisOnPlane, Signs, ZeroEdges, OnCutEdges, &OnSplitEdges, &OnCutVertices, nullptr);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FMeshPlaneCut::SplitCrossingEdges(TArray<double>& Signs, TSet<int>& ZeroEdges, TSet<int>& OnCutEdges, bool bDeleteTrisOnPlane)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SplitCrossingEdges(bDeleteTrisOnPlane, Signs, ZeroEdges, OnCutEdges, nullptr, &OnCutVertices, nullptr);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


bool FMeshPlaneCut::CutWithoutDelete(bool bSplitVerticesAtPlane, float OffsetVertices, TDynamicMeshScalarTriangleAttribute<int>* TriLabels, int NewLabelStartID, bool bAddBoundariesFirstHalf, bool bAddBoundariesSecondHalf)
{
	TArray<double> Signs;
	TSet<int> AlreadyOnPlaneEdges, OnCutEdges;
	SplitCrossingEdges(bSplitVerticesAtPlane, Signs, AlreadyOnPlaneEdges, OnCutEdges, nullptr, nullptr);

	if (!bSplitVerticesAtPlane)
	{
		ensure(OffsetVertices == 0.0); // it would be weird to not split vertices and still request any offset of the 'other side' vertices; please don't do that
	}

	// collapse degenerate edges
	if (bCollapseDegenerateEdgesOnCut)
	{
		CollapseDegenerateEdges(OnCutEdges, false);
	}

	if (bSimplifyAlongNewEdges)
	{
		SimplifySettings.SimplifyAlongEdges(*Mesh, OnCutEdges);
	}

	if (!bSplitVerticesAtPlane)
	{
		return true;
	}

	if (!ensure(TriLabels))
	{
		return false; // need labels to split verts currently
	}

	// Update label IDs for triangles on the positive side of the plane, and shift vertices by the offset vector
	TMap<int, int> OldLabelToNew;
	int AvailableID = NewLabelStartID;
	FVector3d VertexOffsetVec = (double)OffsetVertices * PlaneNormal;
	for (int VID : Mesh->VertexIndicesItr())
	{
		if (VID < Signs.Num() && Signs[VID] > PlaneTolerance)
		{
			Mesh->SetVertex(VID, Mesh->GetVertex(VID) + VertexOffsetVec);
			for (int TID : Mesh->VtxTrianglesItr(VID))
			{
				int LabelID = TriLabels->GetValue(TID);
				if (LabelID >= NewLabelStartID)
				{
					continue;
				}
				if (!OldLabelToNew.Contains(LabelID))
				{
					OldLabelToNew.Add(LabelID, AvailableID++);
				}
				TriLabels->SetValue(TID, OldLabelToNew[LabelID]);
			}
		}
	}

	if (!bSplitVerticesAtPlane)
	{
		return true;
	}

	// split the mesh apart and add open boundary info
	TMap<int, int> SplitVertices;
	TSet<int> BoundaryVertices;
	const TSet<int>* Sets[2] { &AlreadyOnPlaneEdges, &OnCutEdges };
	for (int SetIdx = 0; SetIdx < 2; SetIdx++)
	{
		const TSet<int>& Set = *(Sets[SetIdx]);
		for (int EID : Set)
		{
			if (!Mesh->IsEdge(EID))
			{
				continue;
			}
			const FDynamicMesh3::FEdge Edge = Mesh->GetEdge(EID);
			BoundaryVertices.Add(Edge.Vert[0]);
			BoundaryVertices.Add(Edge.Vert[1]);
		}
	}
	TArray<int> Triangles;
	DynamicMeshInfo::FVertexSplitInfo SplitInfo;
	for (int VID : BoundaryVertices)
	{
		if (!ensure(Mesh->IsVertex(VID))) // should not have invalid vertices in BoundaryVertices
		{
			continue;
		}
		Triangles.Reset();
		int NonSplitTriCount = 0;
		for (int TID : Mesh->VtxTrianglesItr(VID))
		{
			if (TriLabels->GetValue(TID) >= NewLabelStartID)
			{
				Triangles.Add(TID);
			}
			else
			{
				NonSplitTriCount++;
			}
		}
		if (NonSplitTriCount > 0) // connected to old labels
		{
			// if also connected to new labels, needs split
			if (Triangles.Num() > 0 && EMeshResult::Ok == Mesh->SplitVertex(VID, Triangles, SplitInfo))
			{
				SplitVertices.Add(VID, SplitInfo.NewVertex);
				Mesh->SetVertex(SplitInfo.NewVertex, Mesh->GetVertex(SplitInfo.NewVertex) + VertexOffsetVec);
			}
		}
		else if (Signs[VID] <= PlaneTolerance) // wasn't already offset and has no connections to 'old' labels -- needs offset
		{
			Mesh->SetVertex(VID, Mesh->GetVertex(VID) + VertexOffsetVec);
		}
	}
	
	// if boundary loops are requested for either or both sides of the cut, extract + label them
	bool AllExtractionsOk = true;
	if (bAddBoundariesFirstHalf || bAddBoundariesSecondHalf)
	{
		// organize edges by label and transfer ZeroEdges and OnCutEdges to newly split edges
		TMap<int, TSet<int>> LabelToCutEdges;
		for (int SetIdx = 0; SetIdx < 2; SetIdx++)
		{
			const TSet<int>& Set = *(Sets[SetIdx]);
			for (int EID : Set)
			{
				if (!Mesh->IsEdge(EID))
				{
					continue;
				}
				const FDynamicMesh3::FEdge Edge = Mesh->GetEdge(EID);
				if (Edge.Tri[1] >= 0) // only care about boundary edges
				{
					continue;
				}
				{
					int LabelID = TriLabels->GetValue(Edge.Tri[0]);
					TSet<int>& LabelCutEdges = LabelToCutEdges.FindOrAdd(LabelID);
					LabelCutEdges.Add(EID);
				}

				if (bAddBoundariesSecondHalf)
				{
					// try to find and add the corresponding edge
					const int* SplitA = SplitVertices.Find(Edge.Vert[0]);
					const int* SplitB = SplitVertices.Find(Edge.Vert[1]);
					if (SplitA && SplitB)
					{
						int CorrEID = Mesh->FindEdge(*SplitA, *SplitB);
						if (CorrEID >= 0) // corresponding edge exists
						{
							FDynamicMesh3::FEdge CorrEdge = Mesh->GetEdge(CorrEID);
							if (CorrEdge.Tri[1] < 0) // we only care if it's a boundary edge
							{
								int LabelID = TriLabels->GetValue(CorrEdge.Tri[0]);
								TSet<int>& LabelCutEdges = LabelToCutEdges.FindOrAdd(LabelID);
								LabelCutEdges.Add(CorrEID);
							}
						}
					}
				}
				BoundaryVertices.Add(Edge.Vert[0]);
				BoundaryVertices.Add(Edge.Vert[1]);
			}
		}
	
		TSet<int> UnusedZeroEdgesSet; // This set is unused except to pass to ExtractBoundaryLoops, which expects two sets of edges
		for (TPair<int, TSet<int>>& LabelIDEdges : LabelToCutEdges)
		{
			int LabelID = LabelIDEdges.Key;
			if (!bAddBoundariesFirstHalf && LabelID < NewLabelStartID)
			{
				continue;
			}
			if (!bAddBoundariesSecondHalf && LabelID >= NewLabelStartID)
			{
				continue;
			}
			TSet<int>& Edges = LabelIDEdges.Value;
			FMeshPlaneCut::FOpenBoundary& Boundary = OpenBoundaries.Emplace_GetRef();
			Boundary.Label = LabelID;
			if (LabelID >= NewLabelStartID)
			{
				Boundary.NormalSign = -1;
			}

			bool ExtractOk = ExtractBoundaryLoops(Edges, UnusedZeroEdgesSet, Boundary);
			AllExtractionsOk = ExtractOk && AllExtractionsOk;
		}
	}

	return AllExtractionsOk;
}

bool FMeshPlaneCut::Cut()
{
	TArray<double> Signs;
	TSet<int> ZeroEdges, OnCutEdges;
	SplitCrossingEdges(true, Signs, ZeroEdges, OnCutEdges, nullptr, nullptr);
	
	// remove one-rings of all positive-side vertices. 
	for (int VID = 0; VID < Signs.Num(); ++VID)
	{
		if (Signs[VID] > PlaneTolerance)
		{
			constexpr bool bPreserveManifold = false;
			Mesh->RemoveVertex(VID, bPreserveManifold);
		}
	}

	// collapse degenerate edges if we got em
	if (bCollapseDegenerateEdgesOnCut)
	{
		CollapseDegenerateEdges(OnCutEdges, false);
	}

	if (bSimplifyAlongNewEdges)
	{
		SimplifySettings.SimplifyAlongEdges(*Mesh, OnCutEdges);
	}

	FMeshPlaneCut::FOpenBoundary& Boundary = OpenBoundaries.Emplace_GetRef();
	return ExtractBoundaryLoops(OnCutEdges, ZeroEdges, Boundary);

}

bool FMeshPlaneCut::SplitEdgesOnlyHelper(bool bAssignNewGroups, TSet<int32>* TriangleSelection, bool bAddDeprecatedResultSeedTriangles)
{
	// split edges with current plane
	TArray<double> Signs;
	TSet<int32> ZeroEdges, OnCutEdges, OnSplitEdges;
	SplitCrossingEdges(false, Signs, ZeroEdges, OnCutEdges, &OnSplitEdges, nullptr, TriangleSelection);

	if (bAssignNewGroups == false)
	{
		return true;
	}

	if (bSimplifyAlongNewEdges)
	{
		if (ensureMsgf(!bAddDeprecatedResultSeedTriangles, TEXT("Deprecated SplitEdgesOnly without TriangleSelection parameter does not support simplification option")))
		{
			// TODO: Consider making the simplification also preserve the TriangleSelection boundary
			CollapseDegenerateEdges(OnCutEdges, false, TriangleSelection);
			if (TriangleSelection)
			{
				SimplifySettings.SimplifyAlongEdges(*Mesh, OnCutEdges, [&TriangleSelection](const DynamicMeshInfo::FEdgeCollapseInfo& CollapseInfo)
				{
					TriangleSelection->Remove(CollapseInfo.RemovedTris.A);
					if (CollapseInfo.RemovedTris.B != FDynamicMesh3::InvalidID)
					{
						TriangleSelection->Remove(CollapseInfo.RemovedTris.B);
					}
				});
			}
			else
			{
				SimplifySettings.SimplifyAlongEdges(*Mesh, OnCutEdges);
			}
		}
	}

	TSet<int32> OnPlaneEdges = ZeroEdges;
	OnPlaneEdges.Append(OnCutEdges);

	// Use the edges along the cut (both pre-existing and newly added) to assign group IDs across both sides of the triangulation
	TArray<int32> CutEdgeTriangles;		// triangles connected to those edges
	TSet<int32> CutEdgeGroups;			// group IDs of those triangles (ie groups touching cut)
	for (int32 EID : OnPlaneEdges)
	{
		FIndex2i EdgeTris = Mesh->GetEdgeT(EID);
		for (int32 j = 0; j < 2; ++j)
		{
			if (EdgeTris[j] != FDynamicMesh3::InvalidID)
			{
				CutEdgeTriangles.Add(EdgeTris[j]);
				int32 Group = Mesh->GetTriangleGroup(EdgeTris[j]);
				CutEdgeGroups.Add(Group);
			}
		}
	}
	
	// find group-connected-components touching cut, but split each group on either side of the cut into a separate component
	FMeshConnectedComponents GroupRegions(Mesh);
	GroupRegions.FindTrianglesConnectedToSeeds( CutEdgeTriangles, [&](int32 t0, int32 t1) {
		int32 Group0 = Mesh->GetTriangleGroup(t0);
		int32 Group1 = Mesh->GetTriangleGroup(t1);
		if (Group0 == Group1)
		{
			int32 SharedEdge = Mesh->FindEdgeFromTriPair(t0, t1);
			if (OnPlaneEdges.Contains(SharedEdge) == false)
			{
				return true;
			}
		}
		return false;
	});

	// Assign a new group ID for each component
	// Do we want to keep existing groups? possibly cleaner to assign new ones because one input group may
	// be split into multiple child groups on each side of the cut. 
	// But perhaps should track group-mapping?
	ResultRegions.Reset();
	ResultRegions.Reserve(GroupRegions.Num());
	for (FMeshConnectedComponents::FComponent& Component : GroupRegions)
	{
		int32 NewGroup = Mesh->AllocateTriangleGroup();
		for (int tid : Component.Indices)
		{
			Mesh->SetTriangleGroup(tid, NewGroup);
		}

		FCutResultRegion& Result = ResultRegions.Emplace_GetRef();
		Result.GroupID = NewGroup;
		Result.Triangles = MoveTemp(Component.Indices);
	}

	if (!bAddDeprecatedResultSeedTriangles)
	{
		return true;
	}

	// Note: the below logic will be removed when ResultSeedTriangles is removed
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Compute the set of triangle IDs in the cut mesh that represent
	// the original/seed triangle selection along the cut. This assumes
	// that the edge filter function contains edges that originate
	// from source triangles.
	ResultSeedTriangles.Reset();
	for (int tid : CutEdgeTriangles)
	{
		// TODO: We currently assume that all cut edges are on the interior of
		// our seed triangles, thus all tris adjacent to the cut edge are seed
		// triangles. This assumption fails in the following edge case.
		//
		//    o---o---o
		//    |xx/ \xx|
		//    |x/   \x| <---> Cut plane   
		//    |/     \|
		//    o-------o    x = Seed tris
		//
		// CutEdges filters the OnCutEdges list by checking if both ends of the edge
		// are OnCutVertices. In this scenario, the edge introduced across the non
		// seed triangle on the bottom is included.
		ResultSeedTriangles.Add(tid);

		// Walk the edges of the CutEdgeTriangles, skipping seed, split & cut edges,
		// to identify extra interior edges. Both triangles along that interior
		// edge are also seed triangles.
		FIndex3i TriEdges = Mesh->GetTriEdges(tid);
		for (int j = 0; j < 3; j++)
		{
			int eid = TriEdges[j];
			FIndex2i ev = Mesh->GetEdgeV(eid);
			bool bIsSeedEdge = (EdgeFilterFunc && EdgeFilterFunc(eid));
			if (bIsSeedEdge || OnCutEdges.Contains(eid) || OnSplitEdges.Contains(eid))
			{
				continue;
			}
			FIndex2i EdgeTris = Mesh->GetEdgeT(eid);
			ResultSeedTriangles.Add(EdgeTris.A);
			ResultSeedTriangles.Add(EdgeTris.B);
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return true;
}


bool FMeshPlaneCut::ExtractBoundaryLoops(const TSet<int>& OnCutEdges, const TSet<int>& ZeroEdges, FMeshPlaneCut::FOpenBoundary& Boundary)
{
	// ok now we extract boundary loops, but restricted
	// to either the zero-edges we found, or the edges we created! bang!!

	FMeshBoundaryLoops Loops(Mesh, false);
	Loops.EdgeFilterFunc = [&OnCutEdges, &ZeroEdges](int EID)
	{
		return OnCutEdges.Contains(EID) || ZeroEdges.Contains(EID);
	};
	bool bFoundLoops = Loops.Compute();

	if (bFoundLoops)
	{
		Boundary.CutLoops = Loops.Loops;
		Boundary.CutSpans = Loops.Spans;
		Boundary.CutLoopsFailed = false;
		Boundary.FoundOpenSpans = Boundary.CutSpans.Num() > 0;
	}
	else
	{
		Boundary.CutLoops.Empty();
		Boundary.CutLoopsFailed = true;
	}

	return !Boundary.CutLoopsFailed;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FMeshPlaneCut::CollapseDegenerateEdges(const TSet<int>& OnCutEdges, const TSet<int>& ZeroEdges)
{
	const TSet<int>* Sets[2] { &OnCutEdges, &ZeroEdges };

	double Tol2 = DegenerateEdgeTol * DegenerateEdgeTol;
	FVector3d A, B;
	int Collapsed = 0;
	do
	{
		Collapsed = 0;
		for (int SetIdx = 0; SetIdx < 2; SetIdx++)
		{
			const TSet<int>& Set = *(Sets[SetIdx]);
			for (int EID : Set)
			{
				if (!Mesh->IsEdge(EID))
				{
					continue;
				}
				Mesh->GetEdgeV(EID, A, B);
				if (DistanceSquared(A, B) > Tol2)
				{
					continue;
				}

				FIndex2i EV = Mesh->GetEdgeV(EID);
				// if the vertex we'd remove is on a seam, try removing the other one instead
				if (Mesh->HasAttributes() && Mesh->Attributes()->IsSeamVertex(EV.B, false))
				{
					Swap(EV.A, EV.B);
					// if they were both on seams, then collapse should not happen?  (& would break OnCollapseEdge assumptions in overlay)
					if (Mesh->HasAttributes() && Mesh->Attributes()->IsSeamVertex(EV.B, false))
					{
						continue;
					}
				}
				FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
				EMeshResult Result = Mesh->CollapseEdge(EV.A, EV.B, CollapseInfo);
				if (Result == EMeshResult::Ok)
				{
					Collapsed++;
				}
			}
		}
	} while (Collapsed != 0);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FMeshPlaneCut::CollapseDegenerateEdges(TSet<int32>& Edges, bool bRemoveAllDegenerateFromInputSet, TSet<int>* TriangleSelection)
{
	FLocalPlanarSimplify::CollapseDegenerateEdges(*Mesh, Edges, bRemoveAllDegenerateFromInputSet, DegenerateEdgeTol, TriangleSelection);
}

bool FMeshPlaneCut::SimpleHoleFill(int ConstantGroupID)
{
	bool bAllOk = true;

	HoleFillTriangles.Empty();
	for (FOpenBoundary& Boundary : OpenBoundaries)
	{
		TArray<int>& BoundaryFillTriangles = HoleFillTriangles.Emplace_GetRef();
		FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);

		for (const FEdgeLoop& Loop : Boundary.CutLoops)
		{
			FSimpleHoleFiller Filler(Mesh, Loop);
			int GID = ConstantGroupID >= 0 ? ConstantGroupID : Mesh->AllocateTriangleGroup();
			bAllOk = Filler.Fill(GID) && bAllOk;

			BoundaryFillTriangles.Append(Filler.NewTriangles);

			if (Mesh->HasAttributes())
			{
				FDynamicMeshEditor Editor(Mesh);
				Editor.SetTriangleNormals(Filler.NewTriangles, (FVector3f)PlaneNormal * Boundary.NormalSign);
				Editor.SetTriangleUVsFromProjection(Filler.NewTriangles, ProjectionFrame, UVScaleFactor);
			}
		}
	}

	return bAllOk;
}



bool FMeshPlaneCut::MinimalHoleFill(int ConstantGroupID)
{
	bool bAllOk = true;

	HoleFillTriangles.Empty();
	for (FOpenBoundary& Boundary : OpenBoundaries)
	{
		TArray<int>& BoundaryFillTriangles = HoleFillTriangles.Emplace_GetRef();
		FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);

		for (const FEdgeLoop& Loop : Boundary.CutLoops)
		{
			FMinimalHoleFiller Filler(Mesh, Loop);
			int GID = ConstantGroupID >= 0 ? ConstantGroupID : Mesh->AllocateTriangleGroup();
			bAllOk = Filler.Fill(GID) && bAllOk;

			BoundaryFillTriangles.Append(Filler.NewTriangles);

			if (Mesh->HasAttributes())
			{
				FDynamicMeshEditor Editor(Mesh);
				Editor.SetTriangleNormals(Filler.NewTriangles, (FVector3f)PlaneNormal * Boundary.NormalSign);
				Editor.SetTriangleUVsFromProjection(Filler.NewTriangles, ProjectionFrame, UVScaleFactor);
			}
		}
	}

	return bAllOk;
}



bool FMeshPlaneCut::HoleFill(TFunction<TArray<FIndex3i>(const FGeneralPolygon2d&)> PlanarTriangulationFunc, bool bFillSpans, int ConstantGroupID, int MaterialID)
{
	bool bAllOk = true;

	HoleFillTriangles.Empty();
	for (FMeshPlaneCut::FOpenBoundary& Boundary : OpenBoundaries)
	{
		TArray<TArray<int>> LoopVertices;
		for (const FEdgeLoop& Loop : Boundary.CutLoops)
		{
			LoopVertices.Add(Loop.Vertices);
		}
		if (bFillSpans)
		{
			for (const FEdgeSpan& Span : Boundary.CutSpans)
			{
				LoopVertices.Add(Span.Vertices);
			}
		}
		FVector3d SignedPlaneNormal = PlaneNormal*(double)Boundary.NormalSign;
		FPlanarHoleFiller Filler(Mesh, &LoopVertices, PlanarTriangulationFunc, PlaneOrigin, SignedPlaneNormal);

		int GID = ConstantGroupID >= 0 ? ConstantGroupID : Mesh->AllocateTriangleGroup();
		bool bFullyFilledHole = Filler.Fill(GID);

		if (Mesh->HasAttributes())
		{
			FDynamicMeshEditor Editor(Mesh);
			Editor.SetTriangleNormals(Filler.NewTriangles, (FVector3f)(SignedPlaneNormal));

			FFrame3d ProjectionFrame(PlaneOrigin, SignedPlaneNormal);
			for (int UVLayerIdx = 0, NumLayers = Mesh->Attributes()->NumUVLayers(); UVLayerIdx < NumLayers; UVLayerIdx++)
			{
				Editor.SetTriangleUVsFromProjection(Filler.NewTriangles, ProjectionFrame, UVScaleFactor, FVector2f::Zero(), true, UVLayerIdx);
			}

			if (MaterialID > -1 && Mesh->Attributes()->HasMaterialID())
			{
				FDynamicMeshMaterialAttribute* MaterialAttrib = Mesh->Attributes()->GetMaterialID();
				for (int32 TID : Filler.NewTriangles)
				{
					MaterialAttrib->SetValue(TID, MaterialID);
				}
			}
		}

		HoleFillTriangles.Add(MoveTemp(Filler.NewTriangles));

		bAllOk = bAllOk && bFullyFilledHole;
	}

	return bAllOk;
}

void FMeshPlaneCut::TransferTriangleLabelsToHoleFillTriangles(TDynamicMeshScalarTriangleAttribute<int>* TriLabels)
{
	if (!ensure(OpenBoundaries.Num() == HoleFillTriangles.Num()))
	{
		return;
	}
	for (int BoundaryIdx = 0; BoundaryIdx < OpenBoundaries.Num(); BoundaryIdx++)
	{
		const TArray<int>& Triangles = HoleFillTriangles[BoundaryIdx];
		const FOpenBoundary& Boundary = OpenBoundaries[BoundaryIdx];
		for (int TID : Triangles)
		{
			TriLabels->SetValue(TID, Boundary.Label);
		}
	}
}
