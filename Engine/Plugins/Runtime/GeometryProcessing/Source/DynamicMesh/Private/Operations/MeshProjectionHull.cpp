// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MeshProjectionHull.h"
#include "Solvers/MeshLinearization.h"
#include "MeshSimplification.h"
#include "DynamicMesh/MeshNormals.h"
#include "CompGeom/ConvexHull2.h"
#include "CompGeom/ConvexHull3.h"
#include "Generators/SweepGenerator.h"

using namespace UE::Geometry;

bool FMeshProjectionHull::Compute()
{
	// compute set of projected 2D vertices, and find the 1D bounding interval along the projection axis
	FVector3d ProjAxis = ProjectionFrame.Z();
	FInterval1d ProjInterval = FInterval1d::Empty();
	TArray<FVector2d> Vertices;
	Vertices.Reserve(Mesh->VertexCount());
	for (int32 vid : Mesh->VertexIndicesItr())
	{
		FVector3d Position = Mesh->GetVertex(vid);
		FVector2d ProjPos = ProjectionFrame.ToPlaneUV(Position, 2);
		Vertices.Add(ProjPos);

		ProjInterval.Contain( (Position-ProjectionFrame.Origin).Dot(ProjAxis) );
	}

	// compute the 2D convex hull
	FConvexHull2d HullCompute;
	bool bOK = HullCompute.Solve(Vertices);
	if (!bOK)
	{
		return false;
	}

	// extract hull polygon
	ConvexHull2D = FPolygon2d(Vertices, HullCompute.GetPolygonIndices());
	if (ConvexHull2D.Area() < FMathf::ZeroTolerance)
	{
		return false;
	}

	// simplify if requested
	if (bSimplifyPolygon)
	{
		SimplifiedHull2D = ConvexHull2D;
		SimplifiedHull2D.Simplify(MinEdgeLength, DeviationTolerance);
	}

	// shift the projection frame to the min position along the axis
	FFrame3d CenterFrame = ProjectionFrame;
	CenterFrame.Origin += ProjInterval.Min * ProjAxis;

	// apply min-thickness if necessary
	double ExtrudeLength = ProjInterval.Length();
	if (ExtrudeLength < MinThickness)
	{
		ExtrudeLength = MinThickness;
		CenterFrame.Origin -= (ExtrudeLength * 0.5) * ProjAxis;
	}

	if (Keep3DHullSide != EKeep3DHullSide::None)
	{
		return ComputeWith3DHullSide(CenterFrame, ExtrudeLength);
	}

	// generate the swept-polygon mesh
	FGeneralizedCylinderGenerator MeshGen;
	MeshGen.CrossSection = (bSimplifyPolygon) ? SimplifiedHull2D : ConvexHull2D;
	MeshGen.Path.Add(CenterFrame.Origin);
	CenterFrame.Origin += ExtrudeLength * ProjAxis;
	MeshGen.Path.Add(CenterFrame.Origin);
	MeshGen.bCapped = true;
	MeshGen.Generate();
	ConvexHull3D.Copy(&MeshGen);

	return true;
}


bool FMeshProjectionHull::ComputeWith3DHullSide(FFrame3d CenterFrame, double ExtrudeLength)
{
	FVector3d ProjAxis = ProjectionFrame.Z();

	TArray<FVector3d> Verts3D;
	const FPolygon2d* UsePolygon = bSimplifyPolygon ? &SimplifiedHull2D : &ConvexHull2D;
	Verts3D.Reserve(Mesh->VertexCount() + UsePolygon->VertexCount());

	if (bSimplifyPolygon)
	{
		// if the polygon was simplified, project all mesh vertices inside the polygon
		for (FVector3d Position : Mesh->VerticesItr())
		{
			FVector2d FramePos = ProjectionFrame.ToPlaneUV(Position, 2);
			FVector2d InPolyPos = FramePos;
			bool bProjected = CurveUtil::ProjectPointInsideConvexPolygon<double>(SimplifiedHull2D.GetVertices(), InPolyPos, false);
			if (bProjected)
			{
				FVector3d UnprojectedOriginal = ProjectionFrame.FromPlaneUV(FramePos, 2);
				FVector3d UnprojectedInside = ProjectionFrame.FromPlaneUV(InPolyPos, 2);
				Position += UnprojectedInside - UnprojectedOriginal;
			}
			Verts3D.Add(Position);
		}
	}
	else
	{
		// otherwise, can just take the vertices as-is
		for (FVector3d Position : Mesh->VerticesItr())
		{
			Verts3D.Add(Position);
		}
	}

	// Add the polygon vertices, aligned to the flat side
	if (Keep3DHullSide == EKeep3DHullSide::Back)
	{
		CenterFrame.Origin += ExtrudeLength * ProjAxis;
	}
	const TArray<FVector2d>& Verts = UsePolygon->GetVertices();
	for (const FVector2d& Vert : Verts)
	{
		Verts3D.Add(CenterFrame.FromPlaneUV(Vert, 2));
	}

	// Compute a 3D hull of the flat polygon face vertices and the 3D vertices
	FConvexHull3d HullCompute3d;
	bool bOK = HullCompute3d.Solve<FVector3d>(Verts3D);
	if (!bOK)
	{
		return false;
	}

	// Convert the hull triangles to a compact dynamic mesh with no inner/unused vertices
	TArray<int32> SourceVertToHullVert;
	SourceVertToHullVert.Init(-1, Verts3D.Num());
	ConvexHull3D = FDynamicMesh3(EMeshComponents::None);
	HullCompute3d.GetTriangles([&](FIndex3i Triangle)
		{
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				int32 SourceIndex = Triangle[SubIdx];
				int32 HullIndex = SourceVertToHullVert[SourceIndex];
				if (HullIndex == -1)
				{
					FVector3d OrigPos = Verts3D[SourceIndex];
					int32 NewVID = ConvexHull3D.AppendVertex(OrigPos);
					SourceVertToHullVert[SourceIndex] = NewVID;
					Triangle[SubIdx] = NewVID;
				}
				else
				{
					Triangle[SubIdx] = HullIndex;
				}
			}

			ConvexHull3D.AppendTriangle(Triangle);
		});

	return true;
}