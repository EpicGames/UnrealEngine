// Copyright Epic Games, Inc. All Rights Reserved.


#include "Operations/MeshResolveTJunctions.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Spatial/PointSetHashTable.h"
#include "SegmentTypes.h"
#include "TriangleTypes.h"

using namespace UE::Geometry;

const double FMeshResolveTJunctions::DEFAULT_TOLERANCE = FMathf::ZeroTolerance;

bool FMeshResolveTJunctions::Apply()
{
	NumSplitEdges = 0;

	// make clean boundary edge set, either for the whole mesh, or based on input set
	if (BoundaryEdges.Num() == 0)
	{
		for (int32 eid : Mesh->BoundaryEdgeIndicesItr())
		{
			BoundaryEdges.Add(eid);
		}
	}
	else
	{
		TArray<int32> ToRemove;
		for (int32 eid : Mesh->BoundaryEdgeIndicesItr())
		{
			if (Mesh->IsBoundaryEdge(eid) == false)
			{
				ToRemove.Add(eid);
			}
		}
		for (int32 eid : ToRemove)
		{
			BoundaryEdges.Remove(eid);
		}
	}


	TSet<int32> BoundaryVertices;
	for (int32 eid : BoundaryEdges)
	{
		FIndex2i EdgeV = Mesh->GetEdgeV(eid);
		BoundaryVertices.Add(EdgeV.A);
		BoundaryVertices.Add(EdgeV.B);
	}

	// TODO: this should probably use a hash table like MeshMergeCoincidentEdges.
	// However in this case adding the midpoints won't work, the entire edge boxes
	// needed to be added. And when edges are split, the hash will need to
	// be updated. So for now this is just O(N*M).
	// (idea: a heuristic to avoid removing from the hash might be to just not
	//  bother updating the bounds of the pre-split edges. It will mean some extra
	//  checking but may be cheaper than a hash-remove)

	for (int32 vid : BoundaryVertices)
	{
		FVector3d Position = Mesh->GetVertex(vid);
		int e0, e1;
		Mesh->GetVtxBoundaryEdges(vid, e0, e1);

		int32 OnEdgeID = -1;
		double MinDistSqr = TNumericLimits<double>::Max();
		FSegment3d OnEdgeSegment;

		// find the edge in the boundary edge set that is closest to this vertex, 
		// and not connected to this vertex
		for (int32 eid : BoundaryEdges)
		{
			if (eid == e0 || eid == e1) continue;

			FVector3d A, B;
			Mesh->GetEdgeV(eid, A, B);
			FSegment3d EdgeSegment(A, B);
			double DistSqr = EdgeSegment.DistanceSquared(Position);
			if (DistSqr < MinDistSqr)
			{
				MinDistSqr = DistSqr;
				OnEdgeID = eid;
				OnEdgeSegment = EdgeSegment;
			}
		}

		// if we did not find an edge, or we are too far from any edge, give up
		if (OnEdgeID == -1 || MinDistSqr > DistanceTolerance*DistanceTolerance)
		{
			continue;
		}
		// if we are within tolerance of either edge endpoint, we do not need to split
		if (Distance(OnEdgeSegment.StartPoint(), Position) < DistanceTolerance
			|| Distance(OnEdgeSegment.EndPoint(), Position) < DistanceTolerance)
		{
			continue;
		}

		// check that the position is within the span of the edge
		double SegmentT = OnEdgeSegment.Project(Position);
		if (FMathd::Abs(SegmentT) > FMathd::Max(OnEdgeSegment.Extent - DistanceTolerance, 0.0) )
		{
			continue;		// at/on endpoint, should be able to weld, can skip
		}

		// now get edge parameter
		double SplitParameter = OnEdgeSegment.ProjectUnitRange(Position);

		// split boundary edge and add new boundary edge to the active edge set
		FDynamicMesh3::FEdgeSplitInfo SplitInfo;
		if (Mesh->SplitEdge(OnEdgeID, SplitInfo, SplitParameter) == EMeshResult::Ok)
		{
			BoundaryEdges.Add(SplitInfo.NewEdges.A);
			NumSplitEdges++;
		}
	}

	// re-normalize normal/tangent layers
	if (Mesh->HasAttributes() && Mesh->Attributes()->NumNormalLayers() > 0)
	{
		int32 NumNormalLayers = Mesh->Attributes()->NumNormalLayers();
		for (int32 Index = 0; Index < NumNormalLayers; ++Index)
		{
			FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->GetNormalLayer(Index);
			for (int32 ElemIdx : NormalOverlay->ElementIndicesItr())
			{
				FVector3f Normal = NormalOverlay->GetElement(ElemIdx);
				Normalize(Normal);
				NormalOverlay->SetElement(ElemIdx, Normal);
			}
		}
	}

	return true;
}





const double FMeshSnapOpenBoundaries::DEFAULT_TOLERANCE = FMathf::ZeroTolerance;

bool FMeshSnapOpenBoundaries::Apply()
{
	NumVertexSnaps = 0;

	// make clean boundary edge set, either for the whole mesh, or based on input set
	if (BoundaryEdges.Num() == 0)
	{
		for (int32 EID : Mesh->BoundaryEdgeIndicesItr())
		{
			BoundaryEdges.Add(EID);
		}
	}
	else
	{
		// filter out any invalid / non-boundary edges from the input set
		TArray<int32> ToRemove;
		for (int32 EID : BoundaryEdges)
		{
			if (!Mesh->IsEdge(EID) || !Mesh->IsBoundaryEdge(EID))
			{
				ToRemove.Add(EID);
			}
		}
		for (int32 EID : ToRemove)
		{
			BoundaryEdges.Remove(EID);
		}
	}


	TSet<int32> BoundaryVertices;
	for (int32 EID : BoundaryEdges)
	{
		FIndex2i EdgeV = Mesh->GetEdgeV(EID);
		BoundaryVertices.Add(EdgeV.A);
		BoundaryVertices.Add(EdgeV.B);
	}

	// TODO: this should probably use a hash table like MeshMergeCoincidentEdges.
	// However in this case adding the midpoints won't work, the entire edge boxes
	// needed to be added. And when edges are moved via snapping, the hash will need to
	// be updated. So for now this is just O(N*M).

	// TODO: Write a 'fast path' for the bSnapToEdge==false case?
	
	// TODO: Optimize subsequent iterations w/ correspondences from previous iteration 
	// and perhaps optionally report the correspondences back to caller?

	double DistanceToleranceSq = DistanceTolerance * DistanceTolerance;
	double VertexDistanceToleranceSq = DistanceToleranceSq * VertexSnapToleranceFactor * VertexSnapToleranceFactor;

	auto TestIfFlips = [this](int32 TID, int32 VID, FVector3d NewPos, double FlipThreshold = 0) -> bool
	{
		FIndex3i TriVIDs = Mesh->GetTriangle(TID);
		int32 SubIdx = TriVIDs.IndexOf(VID);
		FTriangle3d Tri(Mesh->GetVertex(TriVIDs.A), Mesh->GetVertex(TriVIDs.B), Mesh->GetVertex(TriVIDs.C));
		FVector3d InitialNormal = Tri.Normal();
		Tri.V[SubIdx] = NewPos;
		FVector3d NewNormal = Tri.Normal();
		return InitialNormal.Dot(NewNormal) < FlipThreshold;
	};

	bool bSnappedLastIteration = true;
	int32 LastNumSnapped = NumVertexSnaps;
	for (int32 Iters = 0; Iters < MaxIterations && bSnappedLastIteration; ++Iters)
	{
		for (int32 VID : BoundaryVertices)
		{
			FVector3d Position = Mesh->GetVertex(VID);
			int E0, E1;
			Mesh->GetVtxBoundaryEdges(VID, E0, E1);

			int32 OnEdgeID = -1;
			double MinDistSqr = TNumericLimits<double>::Max();
			FSegment3d OnEdgeSegment;
			FVector3d SegmentA = FVector3d::ZeroVector, SegmentB = FVector3d::ZeroVector;

			// find the edge in the boundary edge set that is closest to this vertex, 
			// and not connected to this vertex
			for (int32 EID : BoundaryEdges)
			{
				if (EID == E0 || EID == E1) continue;

				FVector3d A, B;
				Mesh->GetEdgeV(EID, A, B);
				FSegment3d EdgeSegment(A, B);
				double DistSqr = EdgeSegment.DistanceSquared(Position);
				if (DistSqr < MinDistSqr)
				{
					MinDistSqr = DistSqr;
					OnEdgeID = EID;
					OnEdgeSegment = EdgeSegment;
					SegmentA = A;
					SegmentB = B;
				}
			}

			// if we did not find an edge, or we are too far from any edge, do not snap
			if (OnEdgeID == -1 || MinDistSqr > DistanceToleranceSq)
			{
				continue;
			}

			// if we are within vertex tolerance of either edge endpoint, snap to the closest edge endpoint
			double ToStartSq = DistanceSquared(OnEdgeSegment.StartPoint(), Position);
			double ToEndSq = DistanceSquared(OnEdgeSegment.EndPoint(), Position);
			FVector3d SnapToPt = Position;
			if (ToStartSq < VertexDistanceToleranceSq || ToEndSq < VertexDistanceToleranceSq)
			{
				if (ToStartSq < ToEndSq)
				{
					SnapToPt = SegmentA;
				}
				else
				{
					SnapToPt = SegmentB;
				}
			}
			else if (bSnapToEdges) // otherwise snap to the edge
			{
				// check that the position is within the span of the edge
				double SegmentT = OnEdgeSegment.Project(Position);
				if (SegmentT <= OnEdgeSegment.Extent && SegmentT >= -OnEdgeSegment.Extent)
				{
					SnapToPt = OnEdgeSegment.NearestPoint(Position);
				}
				else
				{
					// outside of span, would be a vertex snap so should use the (possibly lower) vertex snap tolerance
					continue;
				}
			}

			// if vertex was moved farther than a very small tolerance, test for triangle flips and then count it as a snap
			constexpr double CountsAsSnapDistance = FMathd::ZeroTolerance;
			constexpr double CountsAsSnapDistanceSq = CountsAsSnapDistance * CountsAsSnapDistance;
			if (DistanceSquared(Position, SnapToPt) > CountsAsSnapDistanceSq)
			{
				if (bPreventFlips)
				{
					bool bFlippedTri = false;
					Mesh->EnumerateVertexTriangles(VID, [&bFlippedTri, &TestIfFlips, VID, SnapToPt](int32 TID)
					{
						bFlippedTri = bFlippedTri || TestIfFlips(TID, VID, SnapToPt);
					});
					if (bFlippedTri)
					{
						continue;
					}
				}
				NumVertexSnaps++;
			}

			// snap the vertex
			Mesh->SetVertex(VID, SnapToPt);
		}
		bSnappedLastIteration = NumVertexSnaps > LastNumSnapped;
		LastNumSnapped = NumVertexSnaps;
	}

	return true;
}