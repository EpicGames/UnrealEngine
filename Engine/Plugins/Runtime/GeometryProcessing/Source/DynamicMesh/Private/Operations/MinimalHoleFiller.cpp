// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/MinimalHoleFiller.h"
#include "Operations/SimpleHoleFiller.h"
#include "Operations/HoleFillUtil.h"
#include "MeshQueries.h"

using namespace UE::Geometry;

namespace
{

	TSet<int> GetBoundaryEdgeVertices(const FDynamicMesh3& Mesh)
	{
		TSet<int> BoundaryVertices;
		for (int EdgeID : Mesh.BoundaryEdgeIndicesItr())
		{
			const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeID);
			BoundaryVertices.Add(Edge.Vert[0]);
			BoundaryVertices.Add(Edge.Vert[1]);
		}
		return BoundaryVertices;
	}

	/// Check if collapsing edge EdgeID to point NewVertexPosition will flip normal of any attached face
	bool CheckIfCollapseCreatesFlip(const FDynamicMesh3& Mesh, int EdgeID, const FVector3d& NewVertexPosition, double EdgeFlipTolerance = 0.0)
	{
		const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeID);
		int TriangleC = Edge.Tri[0];
		int TriangleD = Edge.Tri[1];

		for (int j = 0; j < 2; ++j)
		{
			int VertexID = Edge.Vert[j];
			int OtherVertexID = Edge.Vert[(j + 1) % 2];

			for (int TriangleID : Mesh.VtxTrianglesItr(VertexID))
			{
				if (TriangleID == TriangleC || TriangleID == TriangleD)
				{
					continue;
				}

				FIndex3i CurrentTriangle = Mesh.GetTriangle(TriangleID);
				if (CurrentTriangle.A == OtherVertexID || CurrentTriangle.B == OtherVertexID || CurrentTriangle.C == OtherVertexID)
				{
					return true;        // invalid nbrhood for collapse
				}

				FVector3d VertexA = Mesh.GetVertex(CurrentTriangle.A);
				FVector3d VertexB = Mesh.GetVertex(CurrentTriangle.B);
				FVector3d VertexC = Mesh.GetVertex(CurrentTriangle.C);
				FVector3d CurrentTriangleNormal = (VertexB - VertexA).Cross(VertexC - VertexA);
				double Sign = 0;
				if (CurrentTriangle.A == VertexID) 
				{
					FVector3d NewTriangleNormal = (VertexB - NewVertexPosition).Cross(VertexC - NewVertexPosition);
					Sign = CurrentTriangleNormal.Dot(NewTriangleNormal);
				}
				else if (CurrentTriangle.B == VertexID) 
				{
					FVector3d NewTriangleNormal = (NewVertexPosition - VertexA).Cross(VertexC - VertexA);
					Sign = CurrentTriangleNormal.Dot(NewTriangleNormal);
				}
				else if (CurrentTriangle.C == VertexID) 
				{
					FVector3d NewTriangleNormal = (VertexB - VertexA).Cross(NewVertexPosition - VertexA);
					Sign = CurrentTriangleNormal.Dot(NewTriangleNormal);
				}
				else
				{
					check(!"Should not get here");
					return false;
				}

				if (Sign <= 0.0)
				{
					return true;
				}
			}
		}

		return false;
	}


	double EdgeFlipMetric(FVector3d Normal0, FVector3d Normal1, double FlipDotTol)
	{
		return (FlipDotTol == 0) ? Normal0.Dot(Normal1) : Normalized(Normal0).Dot(Normalized(Normal1));
	}

	void GetEdgeFlipTris(const FDynamicMesh3& Mesh, int EdgeID,
		FIndex3i& IncidentTriangle0, FIndex3i& IncidentTriangle1,
		FIndex3i& NewTriangle0, FIndex3i& NewTriangle1)
	{
		const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeID);
		FIndex2i OpposingVertices = Mesh.GetEdgeOpposingV(EdgeID);
		int a = Edge.Vert[0];
		int b = Edge.Vert[1];
		int c = OpposingVertices.A;
		int d = OpposingVertices.B;

		FIndex3i IncidentTriangle = Mesh.GetTriangle(Edge.Tri[0]);
		int oa = a;
		int ob = b;
		IndexUtil::OrientTriEdge(oa, ob, IncidentTriangle);

		IncidentTriangle0 = { oa, ob, c };
		IncidentTriangle1 = { ob, oa, d };
		NewTriangle0 = { c, d, ob };
		NewTriangle1 = { d, c, oa };
	}

	/// For given edge, return normals of its two triangles, and normals
	/// of the triangles created if edge is flipped (used in edge-flip optimizers)
	void GetEdgeFlipNormals(const FDynamicMesh3& Mesh, int EdgeID,
		FVector3d& IncidentTriangleNormal0, FVector3d& IncidentTriangleNormal1,
		FVector3d& NewTriangleNormal0, FVector3d& NewTriangleNormal1)
	{
		const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeID);
		FIndex2i OpposingVertices = Mesh.GetEdgeOpposingV(EdgeID);
		FVector3d vC = Mesh.GetVertex(OpposingVertices.A), vD = Mesh.GetVertex(OpposingVertices.B);
		int IncidentTriangleIndex = Edge.Tri[0];
		FIndex3i IncidentTriangle = Mesh.GetTriangle(IncidentTriangleIndex);
		int OrientedA = Edge.Vert[0], OrientedB = Edge.Vert[1];
		IndexUtil::OrientTriEdge(OrientedA, OrientedB, IncidentTriangle);

		FVector3d OrientedAPosition = Mesh.GetVertex(OrientedA), OrientedBPosition = Mesh.GetVertex(OrientedB);

		IncidentTriangleNormal0 = VectorUtil::Normal(OrientedAPosition, OrientedBPosition, vC);
		IncidentTriangleNormal1 = VectorUtil::Normal(OrientedBPosition, OrientedAPosition, vD);
		NewTriangleNormal0 = VectorUtil::Normal(vC, vD, OrientedBPosition);
		NewTriangleNormal1 = VectorUtil::Normal(vD, vC, OrientedAPosition);
	}


	/// If before a flip we have normals (n1,n2) and after we have (m1,m2), check if
	/// the dot between any of the 4 pairs changes sign after the flip, or is
	/// less than the dot-product tolerance (i.e. angle tolerance)
	bool CheckIfEdgeFlipCreatesFlip(const FDynamicMesh3& Mesh, int EdgeID, double FlipDotTol = 0.0)
	{
		check(Mesh.IsBoundaryEdge(EdgeID) == false);
		const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeID);
		FIndex2i OpposingVertex = Mesh.GetEdgeOpposingV(EdgeID);

		FIndex3i TriangleVertices = Mesh.GetTriangle(Edge.Tri[0]);
		int OrientedA = Edge.Vert[0];
		int OrientedB = Edge.Vert[1];
		IndexUtil::OrientTriEdge(OrientedA, OrientedB, TriangleVertices);
		FVector3d OrientedAPosition = Mesh.GetVertex(OrientedA), OrientedBPosition = Mesh.GetVertex(OrientedB);
		FVector3d CPosition = Mesh.GetVertex(OpposingVertex.A), DPosition = Mesh.GetVertex(OpposingVertex.B);

		FVector3d Normal0 = VectorUtil::NormalDirection(OrientedAPosition, OrientedBPosition, CPosition);
		FVector3d Normal1 = VectorUtil::NormalDirection(OrientedBPosition, OrientedAPosition, DPosition);
		FVector3d NewNormal0 = VectorUtil::NormalDirection(CPosition, DPosition, OrientedBPosition);
		if (EdgeFlipMetric(Normal0, NewNormal0, FlipDotTol) <= FlipDotTol || EdgeFlipMetric(Normal1, NewNormal0, FlipDotTol) <= FlipDotTol)
		{
			return true;
		}

		FVector3d NewNormal1 = VectorUtil::NormalDirection(DPosition, CPosition, OrientedAPosition);
		if (EdgeFlipMetric(Normal0, NewNormal1, FlipDotTol) <= FlipDotTol || EdgeFlipMetric(Normal1, NewNormal1, FlipDotTol) <= FlipDotTol)
		{
			return true;
		}

		return false;
	}

	// TODO: Not sure why, but std::swap doesn't compile with TSet
	void SwapSets(TSet<int>& A, TSet<int>& B)
	{
		TSet<int> tmp = MoveTemp(A);
		A = MoveTemp(B);
		B = MoveTemp(tmp);
	}

}	// namespace


double FMinimalHoleFiller::AspectMetric(int EdgeID)
{
	FIndex3i IncidentTriangle0, IncidentTriangle1, NewTriangle0, NewTriangle1;
	GetEdgeFlipTris(*FillMesh, EdgeID, IncidentTriangle0, IncidentTriangle1, NewTriangle0, NewTriangle1);
	double AspectTriangle0 = GetTriAspect(*FillMesh, IncidentTriangle0);
	double AspectTriangle1 = GetTriAspect(*FillMesh, IncidentTriangle1);
	double AspectNewTriangle0 = GetTriAspect(*FillMesh, NewTriangle0);
	double AspectNewTriangle1 = GetTriAspect(*FillMesh, NewTriangle1);
	double MetricExistingTriangles = FMath::Abs(AspectTriangle0 - 1.0) + FMath::Abs(AspectTriangle1 - 1.0);
	double MetricNewTriangles = FMath::Abs(AspectNewTriangle0 - 1.0) + FMath::Abs(AspectNewTriangle1 - 1.0);
	return MetricNewTriangles / MetricExistingTriangles;
}

void FMinimalHoleFiller::RemoveRemainingInteriorVerts()
{
	TSet<int> InteriorVertices;
	for (int VertexIndex : FillMesh->VertexIndicesItr())
	{
		if (!FillMesh->IsBoundaryVertex(VertexIndex))
		{
			InteriorVertices.Add(VertexIndex);
		}
	}

	int PreviousCount = 0;
	while (InteriorVertices.Num() > 0 && InteriorVertices.Num() != PreviousCount)
	{
		PreviousCount = InteriorVertices.Num();
		TArray<int> CurrentVertices = InteriorVertices.Array();

		for (int VertexID : CurrentVertices)
		{
			for (int EdgeID : FillMesh->VtxEdgesItr(VertexID))
			{
				FIndex2i EdgeVertices = FillMesh->GetEdgeV(EdgeID);
				int OtherVertexID = (EdgeVertices.A == VertexID) ? EdgeVertices.B : EdgeVertices.A;
				FDynamicMesh3::FEdgeCollapseInfo Info;
				EMeshResult Result = FillMesh->CollapseEdge(OtherVertexID, VertexID, Info);
				if (Result == EMeshResult::Ok)
				{
					break;
				}
			}

			if (FillMesh->IsVertex(VertexID) == false)
			{
				InteriorVertices.Remove(VertexID);
			}
		}
	}
}


void FMinimalHoleFiller::UpdateCurvature(int VertexID)
{
	double AngleSum = 0;
	if (ExteriorAngleSums.Contains(VertexID))
	{
		AngleSum = ExteriorAngleSums[VertexID];
	}

	for (int TriangleID : FillMesh->VtxTrianglesItr(VertexID))
	{
		FIndex3i Triangle = FillMesh->GetTriangle(TriangleID);
		int IndexInTriangle = IndexUtil::FindTriIndex(VertexID, Triangle);
		AngleSum += FillMesh->GetTriInternalAngleR(TriangleID, IndexInTriangle);
	}

	Curvatures[VertexID] = AngleSum - 2.0 * PI;
}

double FMinimalHoleFiller::CurvatureMetricCached(int A, int B, int C, int D)
{
	double DefectA = Curvatures[A];
	double DefectB = Curvatures[B];
	double DefectC = Curvatures[C];
	double DefectD = Curvatures[D];
	return FMath::Abs(DefectA) + FMath::Abs(DefectB) + FMath::Abs(DefectC) + FMath::Abs(DefectD);
}


double FMinimalHoleFiller::CurvatureMetricEval(int A, int B, int C, int D)
{
	double DefectA = ComputeGaussCurvature(A);
	double DefectB = ComputeGaussCurvature(B);
	double DefectC = ComputeGaussCurvature(C);
	double DefectD = ComputeGaussCurvature(D);
	return FMath::Abs(DefectA) + FMath::Abs(DefectB) + FMath::Abs(DefectC) + FMath::Abs(DefectD);
}

double FMinimalHoleFiller::ComputeGaussCurvature(int VertexID)
{
	double AngleSum = 0;
	if (ExteriorAngleSums.Contains(VertexID))
	{
		AngleSum = ExteriorAngleSums[VertexID];
	}

	for (int TriangleID : FillMesh->VtxTrianglesItr(VertexID))
	{
		FIndex3i Triangle = FillMesh->GetTriangle(TriangleID);
		int IndexInTriangle = IndexUtil::FindTriIndex(VertexID, Triangle);
		AngleSum += FillMesh->GetTriInternalAngleR(TriangleID, IndexInTriangle);
	}

	return AngleSum - 2.0 * PI;
}

double FMinimalHoleFiller::GetTriAspect(const FDynamicMesh3& Mesh, FIndex3i& Triangle)
{
	return VectorUtil::AspectRatio(Mesh.GetVertex(Triangle.A), Mesh.GetVertex(Triangle.B), Mesh.GetVertex(Triangle.C));
}

namespace
{
	bool ShouldFlip_CollapseToMinimal(const FDynamicMesh3* FillMesh, int EdgeID)
	{
		if (FillMesh->IsEdge(EdgeID) == false || FillMesh->IsBoundaryEdge(EdgeID))
		{
			return false;
		}

		bool bDoFlip = false;

		FIndex2i EdgeVertices = FillMesh->GetEdgeV(EdgeID);
		FVector3d IncidentTriangleNormal0, IncidentTriangleNormal1, NewTriangleNormal0, NewTriangleNormal1;
		GetEdgeFlipNormals(*FillMesh, EdgeID, IncidentTriangleNormal0, IncidentTriangleNormal1, NewTriangleNormal0, NewTriangleNormal1);

		double CurrentTrianglesDot = IncidentTriangleNormal0.Dot(IncidentTriangleNormal1);
		double FlipTrianglesDot = NewTriangleNormal0.Dot(NewTriangleNormal1);

		if (CurrentTrianglesDot < 0.1 || FlipTrianglesDot > CurrentTrianglesDot + FMathd::ZeroTolerance)
		{
			bDoFlip = true;
		}

		if (bDoFlip == false)
		{
			FIndex2i OpposingVertices = FillMesh->GetEdgeOpposingV(EdgeID);
			double CurrentEdgeLength = Distance(FillMesh->GetVertex(EdgeVertices.A), FillMesh->GetVertex(EdgeVertices.B));
			double FlipEdgeLength = Distance(FillMesh->GetVertex(OpposingVertices.A), FillMesh->GetVertex(OpposingVertices.B));
			if (FlipEdgeLength < CurrentEdgeLength)
			{
				if (CheckIfEdgeFlipCreatesFlip(*FillMesh, EdgeID) == false)
				{
					bDoFlip = true;
				}
			}
		}

		return bDoFlip;
	}

	bool ShouldFlip_FlipToFlatter(const FDynamicMesh3* FillMesh, int EdgeID, int FlatterPasses)
	{
		if (FillMesh->IsEdge(EdgeID) == false || FillMesh->IsBoundaryEdge(EdgeID))
		{
			return false;
		}

		bool bDoFlip = false;

		FIndex2i EdgeVertices = FillMesh->GetEdgeV(EdgeID);
		FVector3d IncidentTriangleNormal0, IncidentTriangleNormal1, NewTriangleNormal0, NewTriangleNormal1;
		GetEdgeFlipNormals(*FillMesh, EdgeID, IncidentTriangleNormal0, IncidentTriangleNormal1, NewTriangleNormal0, NewTriangleNormal1);

		double CurrentTrianglesDot = IncidentTriangleNormal0.Dot(IncidentTriangleNormal1);
		double FlipTrianglesDot = NewTriangleNormal0.Dot(NewTriangleNormal1);

		if (FlatterPasses < 20 && CurrentTrianglesDot < 0.1)   // this check causes oscillatory behavior
		{
			bDoFlip = true;
		}

		if (FlipTrianglesDot > CurrentTrianglesDot + FMathd::ZeroTolerance)
		{
			bDoFlip = true;
		}

		return bDoFlip;
	}
}


void FMinimalHoleFiller::CollapseToMinimal()
{
	FRemesher Remesher(FillMesh);
	FMeshConstraints Constraints;
	FMeshConstraintsUtil::FullyConstrainEdges(Constraints, *FillMesh, FillMesh->BoundaryEdgeIndicesItr());
	if (Constraints.HasConstraints())
	{
		Remesher.SetExternalConstraints(MoveTemp(Constraints));
	}

	// Try to collapse every edge except boundaries
	Remesher.bEnableCollapses = true;
	Remesher.bPreventNormalFlips = true;
	Remesher.SetTargetEdgeLength(BIG_NUMBER);


	Remesher.bEnableFlips = false;
	Remesher.bEnableSplits = false;
	Remesher.bEnableSmoothing = false;
	Remesher.ProjectionMode = FMeshRefinerBase::ETargetProjectionMode::NoProjection;

	int ZeroCollapsePasses = 0;
	int CollapsePasses = 0;
	while (CollapsePasses++ < 20 && ZeroCollapsePasses < 2)
	{
		// collapse pass
		Remesher.Precompute();
		Remesher.BasicRemeshPass();

		// flip pass. we flip in these cases:
		//  1) if angle between current triangles is too small (slightly more than 90 degrees, currently)
		//  2) if angle between flipped triangles is smaller than between current triangles
		//  3) if flipped edge length is shorter *and* such a flip won't flip the normal
		int NumEdges = FillMesh->MaxEdgeID();
		for (int EdgeID = 0; EdgeID < NumEdges; ++EdgeID)
		{
			bool bDoFlip = ShouldFlip_CollapseToMinimal(FillMesh, EdgeID);

			if (bDoFlip)
			{
				FDynamicMesh3::FEdgeFlipInfo Info;
				EMeshResult Result = FillMesh->FlipEdge(EdgeID, Info);
			}
		}
	}
}

void FMinimalHoleFiller::AddAllEdges(int EdgeID, TSet<int>& EdgeSet)
{
	FIndex2i EdgeTriangles = FillMesh->GetEdgeT(EdgeID);
	FIndex3i TriangleEdges = FillMesh->GetTriEdges(EdgeTriangles.A);
	EdgeSet.Add(TriangleEdges.A);
	EdgeSet.Add(TriangleEdges.B);
	EdgeSet.Add(TriangleEdges.C);
	TriangleEdges = FillMesh->GetTriEdges(EdgeTriangles.B);
	EdgeSet.Add(TriangleEdges.A);
	EdgeSet.Add(TriangleEdges.B);
	EdgeSet.Add(TriangleEdges.C);
}

void FMinimalHoleFiller::FlipToFlatter()
{
	int FlatterPasses = 0;

	TSet<int> RemainingEdges;
	for (auto EdgeID : FillMesh->EdgeIndicesItr())
	{
		RemainingEdges.Add(EdgeID);
	}

	TSet<int> UpdatedEdges;

	int ZeroFlipsPasses = 0;
	while (FlatterPasses++ < 40 && ZeroFlipsPasses < 2 && RemainingEdges.Num() > 0)
	{
		++ZeroFlipsPasses;
		for (int EdgeID : RemainingEdges)
		{
			bool bDoFlip = ShouldFlip_FlipToFlatter(FillMesh, EdgeID, FlatterPasses);

			if (bDoFlip)
			{
				FDynamicMesh3::FEdgeFlipInfo Info;
				EMeshResult Result = FillMesh->FlipEdge(EdgeID, Info);
				if (Result == EMeshResult::Ok)
				{
					ZeroFlipsPasses = 0;
					AddAllEdges(EdgeID, UpdatedEdges);
				}
			}
		}

		SwapSets(RemainingEdges, UpdatedEdges);
		UpdatedEdges.Reset();
	}
}

void FMinimalHoleFiller::FlipToMinimizeCurvature()
{
	int CurvaturePasses = 0;
	Curvatures.SetNum(FillMesh->MaxVertexID());

	for (int VertexID : FillMesh->VertexIndicesItr())
	{
		UpdateCurvature(VertexID);
	}

	TSet<int> RemainingEdges;
	for (auto EdgeID : FillMesh->EdgeIndicesItr())
	{
		RemainingEdges.Add(EdgeID);
	}

	TSet<int> UpdatedEdges;
	while (CurvaturePasses++ < 40 && RemainingEdges.Num() > 0)
	{
		for (int EdgeID : RemainingEdges)
		{
			if (FillMesh->IsBoundaryEdge(EdgeID))
				continue;

			FIndex2i EdgeVertices = FillMesh->GetEdgeV(EdgeID);
			FIndex2i OpposingVertices = FillMesh->GetEdgeOpposingV(EdgeID);

			int FoundOtherEdge = FillMesh->FindEdge(OpposingVertices.A, OpposingVertices.B);
			if (FoundOtherEdge != FDynamicMesh3::InvalidID)
			{
				continue;
			}

			double CurrentTotalCurvature = CurvatureMetricCached(EdgeVertices.A, EdgeVertices.B, OpposingVertices.A, OpposingVertices.B);

			if (CurrentTotalCurvature < FMathd::ZeroTolerance)
			{
				continue;
			}

			FDynamicMesh3::FEdgeFlipInfo Info;
			EMeshResult Result = FillMesh->FlipEdge(EdgeID, Info);
			if (Result != EMeshResult::Ok)
			{
				continue;
			}

			double FlipTotalCurvature = CurvatureMetricEval(EdgeVertices.A, EdgeVertices.B, OpposingVertices.A, OpposingVertices.B);
			bool bKeepFlip = FlipTotalCurvature < CurrentTotalCurvature - FMathd::ZeroTolerance;
			if (bKeepFlip == false)
			{
				Result = FillMesh->FlipEdge(EdgeID, Info);
			}
			else
			{
				UpdateCurvature(EdgeVertices.A);
				UpdateCurvature(EdgeVertices.B);
				UpdateCurvature(OpposingVertices.A);
				UpdateCurvature(OpposingVertices.B);
				AddAllEdges(EdgeID, UpdatedEdges);
			}
		}

		SwapSets(RemainingEdges, UpdatedEdges);
	}
}


void FMinimalHoleFiller::FlipToImproveAspectRatios()
{
	TSet<int> RemainingEdges;
	for (auto EdgeID : FillMesh->EdgeIndicesItr())
	{
		RemainingEdges.Add(EdgeID);
	}

	TSet<int> UpdatedEdges;

	int AreaPasses = 0;
	while (RemainingEdges.Num() > 0 && AreaPasses++ < 20)
	{
		for (int EdgeID : RemainingEdges)
		{
			if (!FillMesh->IsEdge(EdgeID) || FillMesh->IsBoundaryEdge(EdgeID))
			{
				continue;
			}

			FIndex2i EdgeVertices = FillMesh->GetEdgeV(EdgeID);
			FIndex2i OpposingVertices = FillMesh->GetEdgeOpposingV(EdgeID);

			int FoundOtherEdge = FillMesh->FindEdge(OpposingVertices.A, OpposingVertices.B);
			if (FoundOtherEdge != FDynamicMesh3::InvalidID)
			{
				continue;
			}

			double Aspect = AspectMetric(EdgeID);
			if (Aspect > 1.0)
			{
				continue;
			}

			double CurrentTotalCurvature = CurvatureMetricCached(EdgeVertices.A, EdgeVertices.B, OpposingVertices.A, OpposingVertices.B);

			FDynamicMesh3::FEdgeFlipInfo Info;
			EMeshResult Result = FillMesh->FlipEdge(EdgeID, Info);
			if (Result != EMeshResult::Ok)
			{
				continue;
			}

			double NewTotalCurvature = CurvatureMetricEval(EdgeVertices.A, EdgeVertices.B, OpposingVertices.A, OpposingVertices.B);

			bool bKeepFlip = FMath::Abs(CurrentTotalCurvature - NewTotalCurvature) < DevelopabilityTolerance;
			if (bKeepFlip == false)
			{
				Result = FillMesh->FlipEdge(EdgeID, Info);
			}
			else
			{
				UpdateCurvature(EdgeVertices.A);
				UpdateCurvature(EdgeVertices.B);
				UpdateCurvature(OpposingVertices.A);
				UpdateCurvature(OpposingVertices.B);
				AddAllEdges(EdgeID, UpdatedEdges);
			}
		}

		SwapSets(RemainingEdges, UpdatedEdges);
	}
}


bool FMinimalHoleFiller::Fill(int32 GroupID)
{
	if (GroupID < 0 && Mesh->HasTriangleGroups())
	{
		GroupID = Mesh->AllocateTriangleGroup();
	}

	// do a simple fill
	FSimpleHoleFiller Simplefill(Mesh, FillLoop);
	Simplefill.bAutoFillPrimaryColors = false;

	bool bOK = Simplefill.Fill(GroupID);
	if (bOK == false)
	{
		return false;
	}

	auto FillColors = [this]()
	{
		if (bAutoFillPrimaryColors)
		{
			HoleFillUtil::FillColorOverlay(*Mesh, FillLoop.Edges, NewTriangles);
		}
	};

	if (FillLoop.Vertices.Num() <= 3) 
	{
		NewTriangles = MoveTemp(Simplefill.NewTriangles);
		FillColors();
		return true;
	}

	// if the loop is tiny just use the simple fill
	double LoopTotalLength = TMeshQueries<FDynamicMesh3>::TotalEdgeLength(*Mesh, FillLoop.Edges);
	if (LoopTotalLength < 100.0f*FMathf::ZeroTolerance)
	{
		NewTriangles = MoveTemp(Simplefill.NewTriangles);
		FillColors();
		return true;
	}

	// extract the simple fill mesh as a submesh, via RegionOperator, so we can backsub later
	TSet<int> IntialFillTris(Simplefill.NewTriangles);

	RegionOp = MakeUnique<FMeshRegionOperator>(Mesh, Simplefill.NewTriangles,
		[](FDynamicSubmesh3& SubMesh) { SubMesh.bComputeTriMaps = true; });

	FillMesh = &(RegionOp->Region.GetSubmesh());

	// for each boundary vertex, compute the exterior angle sum
	// we will use this to compute gaussian curvature later
	BoundaryVertices = TSet<int>(GetBoundaryEdgeVertices(*FillMesh));

	if (bIgnoreBoundaryTriangles == false)
	{
		RegionOp->BackPropropagate(true);

		for (int SubmeshVertexID : BoundaryVertices)
		{
			double AngleSum = 0;
			int BaseMeshVertexID = RegionOp->ReinsertSubToBaseMapV[SubmeshVertexID];

			for (int TriangleID : RegionOp->BaseMesh->VtxTrianglesItr(BaseMeshVertexID))
			{
				if (IntialFillTris.Contains(TriangleID) == false)
				{
					FIndex3i Triangle = RegionOp->BaseMesh->GetTriangle(TriangleID);
					int IndexInTriangle = IndexUtil::FindTriIndex(BaseMeshVertexID, Triangle);
					AngleSum += RegionOp->BaseMesh->GetTriInternalAngleR(TriangleID, IndexInTriangle);
				}
			}

			ExteriorAngleSums.Emplace(SubmeshVertexID, AngleSum);
		}
	}

	// try to guess a reasonable edge length that will give us enough geometry to work with in simplify pass
	double LoopMinEdgeLength, LoopMaxEdgeLength, LoopAverageEdgeLength;
	TMeshQueries<FDynamicMesh3>::EdgeLengthStatsFromEdges(*Mesh, FillLoop.Edges, LoopMinEdgeLength, LoopMaxEdgeLength, LoopAverageEdgeLength);

	double FillMinEdgeLength, FillMaxEdgeLength, FillAverageEdgeLength;
	TMeshQueries<FDynamicMesh3>::EdgeLengthStats(*FillMesh, FillMinEdgeLength, FillMaxEdgeLength, FillAverageEdgeLength);

	double RemeshTargetEdgeLength = LoopAverageEdgeLength;
	if (FillMaxEdgeLength / RemeshTargetEdgeLength > 10)
		RemeshTargetEdgeLength = FillMaxEdgeLength / 10;

	// remesh up to target edge length, ideally gives us some triangles to work with
	FQueueRemesher Remesher(FillMesh);
	Remesher.SmoothSpeedT = 1.0;

	// Constrain all boundary edges
	FMeshConstraints Constraints;
	FMeshConstraintsUtil::FullyConstrainEdges(Constraints, *FillMesh, FillMesh->BoundaryEdgeIndicesItr());
	if (Constraints.HasConstraints())
	{
		Remesher.SetExternalConstraints(MoveTemp(Constraints));
	}

	// TODO: we should estimate/valide and/or try to cap the number of triangles being created here.
	// If RemeshTargetEdgeLength value is extremely tiny this could be trying to create infinity triangles.
	Remesher.SetTargetEdgeLength(RemeshTargetEdgeLength);
	Remesher.FastestRemesh();

	/*
	 * first round: collapse to minimal mesh, while flipping to try to
	 * get to ballpark minimal mesh. We stop these passes as soon as
	 * we have done two rounds where we couldn't do another collapse
	 *
	 * This is the most unstable part of the algorithm because there
	 * are strong ordering effects. maybe we could sort the edges somehow??
	 */

	CollapseToMinimal();

	// Sometimes, for some reason, we have a remaining interior vertex (have only ever seen one?) 
	// Try to force removal of such vertices, even if it makes ugly mesh
	RemoveRemainingInteriorVerts();

	// enable/disable passes. 
	const bool DO_FLATTER_PASS = true;
	const bool DO_CURVATURE_PASS = bOptimizeDevelopability && true;
	const bool DO_AREA_PASS = bOptimizeDevelopability && bOptimizeTriangles && true;

	/*
	* In this pass we repeat the flipping iterations from the previous pass.
	*
	* Note that because of the always-flip-if-dot-is-small case (commented),
	* this pass will frequently not converge, as some number of edges will
	* be able to flip back and forth (because neither has large enough dot).
	* This is not ideal, but also, if we remove this behavior, then we
	* generally get worse fills. This case basically introduces a sort of
	* randomization factor that lets us escape local minima...
	*
	*/

	if (DO_FLATTER_PASS)
	{
		FlipToFlatter();
	}

	/*
	*  In this pass we try to minimize gaussian curvature at all the vertices.
	*  This will recover sharp edges, etc, and do lots of good stuff.
	*  However, this pass will not make much progress if we are not already
	*  relatively close to a minimal mesh, so it really relies on the previous
	*  passes getting us in the ballpark.
	*/                            

	if (DO_CURVATURE_PASS)
	{
		FlipToMinimizeCurvature();
	}

	/*
	* In this final pass, we try to improve triangle quality. We flip if
	* the flipped triangles have better total aspect ratio, and the
	* curvature doesn't change **too** much. The DevelopabilityTolerance
	* parameter determines what is "too much" curvature change.
	*/

	if (DO_AREA_PASS)
	{
		FlipToImproveAspectRatios();
	}


	RegionOp->BackPropropagate(true);
	NewTriangles = RegionOp->CurrentBaseTriangles();

	FillColors();
	return true;
}
