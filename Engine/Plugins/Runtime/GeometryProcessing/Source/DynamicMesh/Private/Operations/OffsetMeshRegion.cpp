// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/OffsetMeshRegion.h"

#include "Algo/Reverse.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Selections/MeshVertexSelection.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Selections/MeshConnectedComponents.h"
#include "Operations/ExtrudeMesh.h"
#include "EdgeLoop.h"
#include "DynamicSubmesh3.h"
#include "Selections/QuadGridPatch.h"
#include "Operations/PolyEditingUVUtil.h"
#include "Operations/PolyEditingEdgeUtil.h"
#include "Operations/QuadGridPatchUtil.h"
#include "Operations/PolyModeling/PolyModelingMaterialUtil.h" 

using namespace UE::Geometry;


FOffsetMeshRegion::FOffsetMeshRegion(FDynamicMesh3* mesh) : Mesh(mesh)
{
}

bool FOffsetMeshRegion::Apply()
{
	FMeshConnectedComponents RegionComponents(Mesh);
	RegionComponents.FindConnectedTriangles(Triangles);

	bool bAllOK = true;
	OffsetRegions.SetNum(RegionComponents.Num());
	for (int k = 0; k < RegionComponents.Num(); ++k)
	{
		FOffsetInfo& Region = OffsetRegions[k];
		Region.OffsetTids = MoveTemp(RegionComponents.Components[k].Indices);

		if (bOffsetFullComponentsAsSolids)
		{
			TArray<int32> AllTriangles;
			FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, Region.OffsetTids, AllTriangles);
			Region.bIsSolid = AllTriangles.Num() == Region.OffsetTids.Num();
		}

		bool bRegionOK = ApplyOffset(Region);

		bAllOK = bAllOK && bRegionOK;
	}

	return bAllOK;
}


namespace OffsetMeshRegionLocals
{

typedef TPair<int32, TPair<int8, int8>> TriVertPair;

/**
 * @return true if mesh edges are parallel or either is degenerate (the degenerate tolerance is quite large though, 0.0001).
 */
bool EdgesAreParallel(FDynamicMesh3* Mesh, int32 Eid1, int32 Eid2)
{
	FIndex2i Vids1 = Mesh->GetEdgeV(Eid1);
	FIndex2i Vids2 = Mesh->GetEdgeV(Eid2);

	FVector3d Vec1 = Mesh->GetVertex(Vids1.A) - Mesh->GetVertex(Vids1.B);
	FVector3d Vec2 = Mesh->GetVertex(Vids2.A) - Mesh->GetVertex(Vids2.B);
	if (!Vec1.Normalize(KINDA_SMALL_NUMBER) || !Vec2.Normalize(KINDA_SMALL_NUMBER))
	{
		// A degenerate edge is parallel enough for our purposes
		return true;
	}
	return FMath::Abs(Vec1.Dot(Vec2)) >= 1 - KINDA_SMALL_NUMBER;
}

/**
 * Assuming GroupIDs is a list of integer IDs for a contiguous "loop", ie last
 * element is adjacent to first element, this function returns the index of the first
 * value in the first non-broken contiguous span of duplicate values. IE if the array 
 * is [0,0,0,1,1,2,2] it will return "0" but if it were [0,0,1,1,2,2,0] it would return "2", 
 * ie the inde of the first "1". The purpose is to allow iterating through the array
 * in the order [1,1,2,2,0,0,0], ie the "0" values are not split across the N-1/0 transition
 */
static int32 FindLoopShiftFromGroupIDs(const TArray<int32>& GroupIDs)
{
	int32 N = GroupIDs.Num();
	if (GroupIDs[0] != GroupIDs[N - 1])
	{
		return 0;
	}
	for (int32 k = 0; k < N-1; ++k)
	{
		if (GroupIDs[k] != GroupIDs[k + 1])
		{
			return k+1;
		}
	}
	return 0;	// all values are the same, no shift
}

/**
 * Cycle/Shift the values in the Values array to the left ShiftNum times.
 * Uses a temporary array internally.
 */
template<typename ValueType>
static void LeftShiftArray(TArray<ValueType>& Values, int ShiftNum)
{
	if (ShiftNum == 0 ) return;

	int32 N = Values.Num();

	// there is probably some clever way to do this w/o a temporary array...
	TArray<ValueType> Tmp;
	Tmp.SetNum(N);
	for (int32 k = 0; k < N; ++k)
	{
		Tmp[k] = Values[(ShiftNum+k)%N];
	}
	Values = MoveTemp(Tmp);
}


struct FStitchConfigOptions
{
	int NumSubdivisions = 0;
	double UVScaleFactor = 1.0;
	bool bGroupPerSubdivision = true;
	bool bUVIslandPerGroup = true;
	double CreaseAngleThreshold = 180.0;

	int SetMaterialID = 0;
	bool bInferMaterialID = false;
};



/**
 * Stitch pairs of border loops of Mesh defined by LoopPairs.
 * 
 */
static bool StitchRegionBorderLoopPairs_Version1(
	FDynamicMesh3& Mesh,
	const TArray<FDynamicMeshEditor::FLoopPairSet>& LoopPairs,
	const TArray<TArray<int32>>& LoopsPerEdgeNewGroupIDs,
	const TArray<TArray<FMeshTriOrderedEdgeID>>& InnerTriOrderedEdgeLoops,
	FStitchConfigOptions StitchConfig,
	TArray<bool>& bLoopOK,
	TArray<TArray<int32>>& PerLoopTrianglesOut,
	TArray<TArray<int32>>& PerLoopGroupsOut )
{
	FDynamicMeshEditor Editor(&Mesh);

	bool bAllSuccess = true;
	int NumInitialLoops = LoopPairs.Num();

	PerLoopTrianglesOut.SetNum(NumInitialLoops);
	PerLoopGroupsOut.SetNum(NumInitialLoops);
	bLoopOK.Init(false, NumInitialLoops);

	for (int32 LoopIndex = 0; LoopIndex < LoopPairs.Num(); ++LoopIndex)
	{
		// note we are assuming here that the loops are aligned w/ a group transition

		const TArray<int32>& OuterLoopV = LoopPairs[LoopIndex].OuterVertices;
		const TArray<int32>& InnerLoopV = LoopPairs[LoopIndex].InnerVertices;
		const TArray<FMeshTriOrderedEdgeID>& InnerTriOrderedEdgeLoop = InnerTriOrderedEdgeLoops[LoopIndex];
		int32 NV = OuterLoopV.Num();

		// populate list of Material IDs attached to inner loop
		TArray<int32> MaterialIDs;
		FDynamicMeshMaterialAttribute* MaterialIDAttrib = (Mesh.HasAttributes() && Mesh.Attributes()->HasMaterialID()) ? 
				Mesh.Attributes()->GetMaterialID() : nullptr;
		if ( MaterialIDAttrib && StitchConfig.bInferMaterialID )
		{
			UE::Geometry::ComputeMaterialIDsForVertexPath(Mesh, InnerLoopV, true, MaterialIDs, StitchConfig.SetMaterialID);
		}
		else if ( MaterialIDAttrib )
		{
			MaterialIDs.Init(StitchConfig.SetMaterialID, NV);
		}

		// LoopStack is the set of NumSubdivisions+1 loops that will be stitched,
		// where the "last" loop is the Outer loop. The "first" Inner loop is not 
		// included in LoopStack, it has special handling below
		TArray<TArray<int32>> LoopStack;
		LoopStack.SetNum(StitchConfig.NumSubdivisions+1);
		LoopStack[StitchConfig.NumSubdivisions] = OuterLoopV;
		int LastTemp = LoopStack[StitchConfig.NumSubdivisions][0];

		// build interior loops if there are any
		for ( int32 StackIdx = 0; StackIdx < StitchConfig.NumSubdivisions; ++StackIdx )
		{
			LoopStack[StackIdx].Reserve(NV);
			double t = (double)(StackIdx+1) / (double)(StitchConfig.NumSubdivisions+1);
			for ( int32 k = 0; k < NV; ++k )
			{
				FVector3d A = Mesh.GetVertex(InnerLoopV[k]);
				FVector3d B = Mesh.GetVertex(OuterLoopV[k]);
				FVector3d StackPos = Lerp(A, B, t);
				LoopStack[StackIdx].Add( Mesh.AppendVertex(StackPos) );
			}
			LastTemp = LoopStack[StackIdx][0];
		}

		// build the first strip from the Inner loop to the next loop
		FDynamicMeshEditResult StitchResult;
		TArray<int32> InnerVertexLoop, TempEdgeLoop;
		UE::Geometry::ConvertTriOrderedEdgeLoopToLoop(Mesh, InnerTriOrderedEdgeLoop, InnerVertexLoop, &TempEdgeLoop);
		bLoopOK[LoopIndex] = Editor.StitchVertexLoopsMinimal(InnerVertexLoop, LoopStack[0], StitchResult);
		if (!bLoopOK[LoopIndex])
		{
			bAllSuccess = false;
			continue;
		}

		FQuadGridPatch QuadPatch;
		TArray<int32> TempSpan = LoopStack[0];		// annoying to have to make a copy like this...need to perserve for LoopStack[k-1] below though
		TempSpan.Add(LoopStack[0][0]);
		QuadPatch.InitializeFromQuadStrip(Mesh, StitchResult.NewQuads, TempSpan);
		QuadPatch.ReverseRows();	// need LoopStack[0] to be the second span but it's currently the first one...

		// incrementally stitch intermediate strips and accumulate into the Quad Patch
		for ( int32 k = 1; k <= StitchConfig.NumSubdivisions; ++k )
		{
			FDynamicMeshEditResult StripResult;
			bLoopOK[LoopIndex] = Editor.StitchVertexLoopsMinimal(LoopStack[k-1], LoopStack[k], StripResult);

			FQuadGridPatch NextQuadStrip;
			TempSpan = LoopStack[k-1];		// must preserve LoopStack arrays currently
			TempSpan.Add(LoopStack[k-1][0]);
			NextQuadStrip.InitializeFromQuadStrip(Mesh, StripResult.NewQuads, TempSpan);

			QuadPatch.AppendQuadPatchRows(MoveTemp(NextQuadStrip), true);
		}

		QuadPatch.ReverseRows();	// for an extrude/offset this makes sense, quads will "start" at the transition from base loop

		// set group IDs on patch, propagating up the subdivisions columns
		const TArray<int32>& PerEdgeNewGroupIDs = LoopsPerEdgeNewGroupIDs[LoopIndex];		// is it always guaranteed to be the same length??
		QuadPatch.ForEachQuad( [&](int32 QuadRow, int32 QuadCol, FIndex2i QuadTris)
		{
			int32 GroupID = PerEdgeNewGroupIDs[QuadCol];
			Mesh.SetTriangleGroup(QuadTris.A, GroupID);
			Mesh.SetTriangleGroup(QuadTris.B, GroupID);
			if (MaterialIDAttrib)
			{
				MaterialIDAttrib->SetValue(QuadTris.A, MaterialIDs[QuadCol]);
				MaterialIDAttrib->SetValue(QuadTris.B, MaterialIDs[QuadCol]);
			}
		});

		// split quad patch by group ID into a set of quad patches
		bool bSplitDueToCrease = false;
		TArray<FQuadGridPatch> GroupStrips;
		QuadPatch.SplitColumnsByPredicate(
			[&](int32 ColumnIdx, int32 NextColumnIdx) { 
				if (PerEdgeNewGroupIDs[ColumnIdx] != PerEdgeNewGroupIDs[NextColumnIdx])
				{
					return true;
				}
				else if (QuadPatch.GetQuadOpeningAngleDeg(Mesh, NextColumnIdx, NextColumnIdx+1, 0) > StitchConfig.CreaseAngleThreshold)
				{
					bSplitDueToCrease = true;
					return true;
				}
				return false;
			},
			GroupStrips);

		// compute normals and UVs for each group-quad-patch
		if (Mesh.HasAttributes())
		{
			for (FQuadGridPatch& GroupPatch : GroupStrips)
			{
				UE::Geometry::ComputeNormalsForQuadPatch(Mesh, GroupPatch);
				if ( StitchConfig.bUVIslandPerGroup )
				{
					UE::Geometry::ComputeUVIslandForQuadPatch(Mesh, GroupPatch, StitchConfig.UVScaleFactor);
				}
			}
		}
		// if not per-group UV islands, do it for the entire strip)
		if (StitchConfig.bUVIslandPerGroup == false)
		{
			UE::Geometry::ComputeUVIslandForQuadPatch(Mesh, QuadPatch, StitchConfig.UVScaleFactor);
		}

		// if we split groups due to a crease, we need to reassign groups in row 0
		if (bSplitDueToCrease)
		{
			for (FQuadGridPatch& GroupPatch : GroupStrips)
			{
				int32 NewPatchGroup = Mesh.AllocateTriangleGroup();
				GroupPatch.ForEachQuad([&](int32 Row, int32 Col, FIndex2i QuadTris)
				{
					Mesh.SetTriangleGroup(QuadTris.A, NewPatchGroup);
					Mesh.SetTriangleGroup(QuadTris.B, NewPatchGroup);
				});
			}
		}

		// Assign a new group to each row of group-quad-patch
		if ( StitchConfig.bGroupPerSubdivision )
		{
			for (FQuadGridPatch& GroupPatch : GroupStrips)
			{
				int32 NumQuadRows = GroupPatch.NumVertexRowsV-1;
				TArray<int32> RowGroups;
				RowGroups.SetNum(NumQuadRows);
				// if we split at a crease we need to reassign first column too...
				for ( int32 k =1; k < NumQuadRows; ++k )
				{
					RowGroups[k] = Mesh.AllocateTriangleGroup();
				}
				GroupPatch.ForEachQuad([&](int32 Row, int32 Col, FIndex2i QuadTris)
				{
					if ( Row > 0 )
					{
						Mesh.SetTriangleGroup(QuadTris.A, RowGroups[Row]);
						Mesh.SetTriangleGroup(QuadTris.B, RowGroups[Row]);
					}
				});
			}
		}

		// save the stitch triangles set and associated group IDs
		QuadPatch.GetAllTriangles(PerLoopTrianglesOut[LoopIndex]);
		for (int32 tid : PerLoopTrianglesOut[LoopIndex])
		{
			PerLoopGroupsOut[LoopIndex].AddUnique( Mesh.GetTriangleGroup(tid) );
		}
	}

	return bAllSuccess;
}




static void SetStitchStripNormalsUVs_Legacy(
	FDynamicMesh3& Mesh,
	FDynamicMeshEditor& Editor,
	FDynamicMeshEditResult& StitchResult,
	const TArray<int32>& OuterLoopV,
	double UVScaleFactor )
{
	int NumNewQuads = StitchResult.NewQuads.Num();

	double AccumUVTranslation = 0;
	FFrame3d FirstProjectFrame;
	FVector3d FrameUp;

	for (int k = 0; k < NumNewQuads; k++)
	{
		FVector3f Normal = Editor.ComputeAndSetQuadNormal(StitchResult.NewQuads[k], true);

		// align axis 0 of projection frame to first edge, then for further edges,
		// rotate around 'up' axis to keep normal aligned and frame horizontal
		FFrame3d ProjectFrame;
		if (k == 0)
		{
			FVector3d FirstEdge = Mesh.GetVertex(OuterLoopV[1]) - Mesh.GetVertex(OuterLoopV[0]);
			Normalize(FirstEdge);
			FirstProjectFrame = FFrame3d(FVector3d::Zero(), (FVector3d)Normal);
			FirstProjectFrame.ConstrainedAlignAxis(0, FirstEdge, (FVector3d)Normal);
			FrameUp = FirstProjectFrame.GetAxis(1);
			ProjectFrame = FirstProjectFrame;
		}
		else
		{
			ProjectFrame = FirstProjectFrame;
			ProjectFrame.ConstrainedAlignAxis(2, (FVector3d)Normal, FrameUp);
		}

		if (k > 0)
		{
			AccumUVTranslation += Distance(Mesh.GetVertex(OuterLoopV[k]), Mesh.GetVertex(OuterLoopV[k-1]));
		}

		// translate horizontally such that vertical spans are adjacent in UV space (so textures tile/wrap properly)
		double TranslateU = UVScaleFactor * AccumUVTranslation;
		Editor.SetQuadUVsFromProjection(StitchResult.NewQuads[k], ProjectFrame, (float)UVScaleFactor, FVector2f((float)TranslateU, 0));
	}
}


static bool StitchRegionBorderLoopPairs_Legacy(
	FDynamicMesh3& Mesh,
	const TArray<FDynamicMeshEditor::FLoopPairSet>& LoopPairs,
	const TArray<TArray<int32>>& LoopsPerEdgeNewGroupIDs,
	const TArray<TArray<TriVertPair>> OffsetStitchSides,
	double UVScaleFactor,
	TArray<bool>& bLoopOK,
	TArray<TArray<int32>>& PerLoopTrianglesOut,
	TArray<TArray<int32>>& PerLoopGroupsOut )
{
	FDynamicMeshEditor Editor(&Mesh);

	bool bAllSuccess = true;
	int NumInitialLoops = LoopPairs.Num();

	PerLoopTrianglesOut.SetNum(NumInitialLoops);
	PerLoopGroupsOut.SetNum(NumInitialLoops);
	bLoopOK.Init(false, NumInitialLoops);

	for (int32 LoopIndex = 0; LoopIndex < LoopPairs.Num(); ++LoopIndex)
	{
		const FDynamicMeshEditor::FLoopPairSet& LoopPair = LoopPairs[LoopIndex];
		const TArray<int32>& EdgeGroups = LoopsPerEdgeNewGroupIDs[LoopIndex];

		const TArray<int32>& BaseLoopV = LoopPair.OuterVertices;
		const TArray<int32>& OffsetLoopV = LoopPair.InnerVertices;

		const TArray<TriVertPair>& OffsetLoopTriVertPairs = OffsetStitchSides[LoopIndex];

		// stitch the loops
		FDynamicMeshEditResult StitchResult;
		bLoopOK[LoopIndex] = Editor.StitchVertexLoopToTriVidPairSequence(OffsetLoopTriVertPairs, LoopPair.OuterVertices, StitchResult);
		if (!bLoopOK[LoopIndex])
		{
			bAllSuccess = false;
			continue;
		}

		// set the groups of the new quad strip created by the stitching
		int NumNewQuads = StitchResult.NewQuads.Num();
		TArray<int32> NewGroupsInLoop;
		const TArray<int32>& PerEdgeNewGroupIDs = LoopsPerEdgeNewGroupIDs[LoopIndex];		// is it always guaranteed to be the same length??
		for (int32 k = 0; k < NumNewQuads; k++)
		{
			Mesh.SetTriangleGroup(StitchResult.NewQuads[k].A, PerEdgeNewGroupIDs[k]);
			Mesh.SetTriangleGroup(StitchResult.NewQuads[k].B, PerEdgeNewGroupIDs[k]);
			NewGroupsInLoop.AddUnique(PerEdgeNewGroupIDs[k]);
		}

		// save the stitch triangles set and associated group IDs
		StitchResult.GetAllTriangles(PerLoopTrianglesOut[LoopIndex]);
		PerLoopGroupsOut[LoopIndex] = MoveTemp(NewGroupsInLoop);

		// for each polygon we created in stitch, set UVs and normals
		if (Mesh.HasAttributes())
		{
			SetStitchStripNormalsUVs_Legacy(Mesh, Editor, StitchResult, BaseLoopV, UVScaleFactor);
		}
	}

	return bAllSuccess;
}


template<typename ListType>
FVector3d GetAngleWeightedAverageNormal(const FDynamicMesh3& Mesh, int32 VertexID, const ListType& TriangleList)
{
	FVector3d ExtrusionVector = FVector3d::Zero();

	// Get angle-weighted normalized average vector
	for (int32 TriangleID : Mesh.VtxTrianglesItr(VertexID))
	{
		if (TriangleList.Contains(TriangleID))
		{
			FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
			double Angle = Mesh.GetTriInternalAngleR(TriangleID, Triangle.IndexOf(VertexID));
			ExtrusionVector += Angle * Mesh.GetTriNormal(TriangleID);
		}
	}
	ExtrusionVector.Normalize();
	return ExtrusionVector;
}


template<typename ListType>
FVector3d GetAngleWeightedAdjustedNormal(const FDynamicMesh3& Mesh, int32 VertexID, const ListType& TriangleList,
	double MaxAdjustmentScale)
{
	FVector3d InitialExtrusionVector = GetAngleWeightedAverageNormal<ListType>(Mesh, VertexID, TriangleList);

	// Perform an angle-weighted adjustment of the vector length. For each triangle normal, the
	// length needs to be multiplied by 1/cos(theta) to place the vertex in the plane that it
	// would be in if the face was moved a unit along triangle normal (where theta is angle of
	// triangle normal to the current extrusion vector).
	double AngleSum = 0;
	double Adjustment = 0;
	double InvertedMaxScale = FMath::Max(FMathd::ZeroTolerance, 1.0 / MaxAdjustmentScale);
	for (int32 TriangleID : Mesh.VtxTrianglesItr(VertexID))
	{
		if (TriangleList.Contains(TriangleID))
		{
			FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
			double Angle = Mesh.GetTriInternalAngleR(TriangleID, Triangle.IndexOf(VertexID));
			double CosTheta = Mesh.GetTriNormal(TriangleID).Dot(InitialExtrusionVector);

			if (CosTheta <= InvertedMaxScale)
			{
				CosTheta = InvertedMaxScale;
			}
			Adjustment += Angle / CosTheta;

			// For the average at the end
			AngleSum += Angle;
		}
	}
	Adjustment /= AngleSum;

	return InitialExtrusionVector * Adjustment;
}


} // end namespace OffsetMeshRegionLocals


bool FOffsetMeshRegion::ApplyOffset(FOffsetInfo& Region)
{
	if (UseVersion == EVersion::Legacy)
	{
		return ApplyOffset_Legacy(Region);
	}
	else  // Version1, Current
	{
		return ApplyOffset_Version1(Region);
	}
}


bool FOffsetMeshRegion::ApplyOffset_Version1(FOffsetInfo& Region)
{
	const TArray<int32>& RegionTriangles = Region.OffsetTids;

	// Split any bowties in the offset region. An extrusion of a bowtie creates a nonmanifold edge which is not permitted
	FDynamicMeshEditor TmpEditor(Mesh);
	FDynamicMeshEditResult IgnoreResult;
	TmpEditor.SplitBowtiesAtTriangles(RegionTriangles, IgnoreResult);

	TMap<int32, int32> OffsetGroupMap;

	// Remap GroupIDs in offset region
	if (Mesh->HasTriangleGroups())
	{
		for (int32 TriangleID : RegionTriangles)
		{
			int32 CurGroupID = Mesh->GetTriangleGroup(TriangleID);
			int32 NewGroupID = CurGroupID;
			int32* FoundNewGroupID = OffsetGroupMap.Find(CurGroupID);
			if (FoundNewGroupID == nullptr)
			{
				NewGroupID = Mesh->AllocateTriangleGroup();
				OffsetGroupMap.Add(CurGroupID, NewGroupID);
				Region.OffsetGroups.Add(NewGroupID);
			}
			else
			{
				NewGroupID = *FoundNewGroupID;
			}
			Mesh->SetTriangleGroup(TriangleID, NewGroupID);
		}
	}

	FMeshRegionBoundaryLoops InitialLoops(Mesh, RegionTriangles, false);
	bool bOK = InitialLoops.Compute();
	if (bOK == false)
	{
		return false;
	}

	AllModifiedAndNewTriangles.Append(RegionTriangles);
	TSet<int32> TriangleSet(RegionTriangles);

	// Before we start changing triangles, prepare by allocating group IDs that we'll use
	// for the stitched sides (doing it before changes to the mesh allows user-provided
	// LoopEdgesShouldHaveSameGroup functions to operate on the original mesh).
	TArray<TArray<int32>> LoopsEdgeGroups;
	TArray<int32> NewGroupIDs;
	LoopsEdgeGroups.SetNum(InitialLoops.Loops.Num());
	for (int32 i = 0; i < InitialLoops.Loops.Num(); ++i)
	{
		TArray<int32>& LoopEids = InitialLoops.Loops[i].Edges;
		TArray<int32>& CurrentEdgeGroups = LoopsEdgeGroups[i];
		UE::Geometry::ComputeNewGroupIDsAlongEdgeLoop(*Mesh, LoopEids, CurrentEdgeGroups, NewGroupIDs,
			LoopEdgesShouldHaveSameGroup);

		// if a group is split over the start/end of the loop, shift the loops to so that index 0 lies on a group transition,
		// this will simplify later processing
		int GroupShift = OffsetMeshRegionLocals::FindLoopShiftFromGroupIDs(CurrentEdgeGroups);
		if (GroupShift != 0)
		{
			OffsetMeshRegionLocals::LeftShiftArray(InitialLoops.Loops[i].Vertices, GroupShift);
			OffsetMeshRegionLocals::LeftShiftArray(InitialLoops.Loops[i].Edges, GroupShift);
			OffsetMeshRegionLocals::LeftShiftArray(CurrentEdgeGroups, GroupShift);
		}
	}

	FDynamicMeshEditor Editor(Mesh);
	TArray<FDynamicMeshEditor::FLoopPairSet> LoopPairs;

	FDynamicMeshEditResult DuplicateResult;
	if (Region.bIsSolid)
	{
		// In the solid case, we want to duplicate the region so we can cap it.
		FMeshIndexMappings IndexMap;
		Editor.DuplicateTriangles(RegionTriangles, IndexMap, DuplicateResult);

		AllModifiedAndNewTriangles.Append(DuplicateResult.NewTriangles);

		// Populate LoopPairs
		LoopPairs.SetNum(InitialLoops.Loops.Num());
		for (int LoopIndex = 0; LoopIndex < InitialLoops.Loops.Num(); ++LoopIndex)
		{
			FEdgeLoop& BaseLoop = InitialLoops.Loops[LoopIndex];
			FDynamicMeshEditor::FLoopPairSet& LoopPair = LoopPairs[LoopIndex];

			// The original RegionTriangles are the ones that are offset, so InnerVertices/Edges
			// should be the boundaries of those.
			LoopPair.InnerVertices = BaseLoop.Vertices;
			LoopPair.InnerEdges = BaseLoop.Edges;

			// However depending on whether we extruded down or up, we may need to reverse
			// the loops to get them to be stitched right side out.
			if (!bIsPositiveOffset)
			{
				Algo::Reverse(LoopPair.InnerVertices);

				// Reversing the edges is slightly different because the last edge is between the first
				// and last vertex, and that needs to stay in the same place when vertices are reversed.
				int32 LastEid = LoopPair.InnerEdges.Pop();
				Algo::Reverse(LoopPair.InnerEdges);
				LoopPair.InnerEdges.Add(LastEid);

				int32 LastEdgeGroupID = LoopsEdgeGroups[LoopIndex].Pop();
				Algo::Reverse(LoopsEdgeGroups[LoopIndex]);
				LoopsEdgeGroups[LoopIndex].Add(LastEdgeGroupID);
			}

			// Now assemble the paired loop
			for (int32 Vid : LoopPair.InnerVertices)
			{
				LoopPair.OuterVertices.Add(IndexMap.GetNewVertex(Vid));
			}
			FEdgeLoop::VertexLoopToEdgeLoop(Mesh, LoopPair.OuterVertices, LoopPair.OuterEdges);
		}
	}
	else
	{
		bOK = Editor.DisconnectTriangles(TriangleSet, InitialLoops.Loops, LoopPairs, true /*bHandleBoundaryVertices*/);
	}

	if (bOK == false)
	{
		return false;
	}

	// Store the vid-independent offset loop before we break bowties
	// NO LONGER NECESSARY
	TArray<TArray<FMeshTriOrderedEdgeID>> OffsetTriOrderedEdgeLoops;
	OffsetTriOrderedEdgeLoops.SetNum(LoopPairs.Num());
	for (int32 i = 0; i < LoopPairs.Num(); ++i)
	{
		bOK = bOK && UE::Geometry::ConvertLoopToTriOrderedEdgeLoop(
			*Mesh, LoopPairs[i].InnerVertices, LoopPairs[i].InnerEdges, OffsetTriOrderedEdgeLoops[i]);
	}
	if (bOK == false)
	{
		return false;
	}

	FMeshVertexSelection SelectionV(Mesh);
	SelectionV.SelectTriangleVertices(RegionTriangles);
	TArray<int32> SelectedVids = SelectionV.AsArray();

	// If we need to, assemble the vertex vectors for us to use (before we actually start moving things)
	TArray<FVector3d> VertexExtrudeVectors;
	if (ExtrusionVectorType == EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAverage
		|| ExtrusionVectorType == EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAdjusted)
	{
		VertexExtrudeVectors.SetNumUninitialized(SelectedVids.Num());

		// Used to test which triangles are in selection
		for (int32 i = 0; i < SelectedVids.Num(); ++i)
		{
			VertexExtrudeVectors[i] = ( ExtrusionVectorType == EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAdjusted ) ?
				OffsetMeshRegionLocals::GetAngleWeightedAdjustedNormal(*Mesh, SelectedVids[i], TriangleSet, MaxScaleForAdjustingTriNormalsOffset)
				 : OffsetMeshRegionLocals::GetAngleWeightedAverageNormal(*Mesh, SelectedVids[i], TriangleSet);
		}
	}
	else if (ExtrusionVectorType == EVertexExtrusionVectorType::VertexNormal)
	{
		VertexExtrudeVectors.SetNumUninitialized(SelectedVids.Num());
		for (int32 i = 0; i < SelectedVids.Num(); ++i)
		{
			VertexExtrudeVectors[i] = Mesh->HasVertexNormals() ? 
				(FVector3d)Mesh->GetVertexNormal(SelectedVids[i]) : FMeshNormals::ComputeVertexNormal(*Mesh, SelectedVids[i]);
		}
	}

	// Perform the actual vertex displacement.
	for (int32 i = 0; i < SelectedVids.Num(); ++i)
	{
		int32 VertexID = SelectedVids[i];
		FVector3d OldPosition = Mesh->GetVertex(VertexID);
		FVector3d ExtrusionVector = (ExtrusionVectorType == EVertexExtrusionVectorType::Zero) ? FVector3d::Zero() : VertexExtrudeVectors[i];

		FVector3d NewPosition = OffsetPositionFunc(OldPosition, ExtrusionVector, VertexID);
		Mesh->SetVertex(VertexID, NewPosition);
	}

	// Stitch the loops
	OffsetMeshRegionLocals::FStitchConfigOptions StitchConfig;
	StitchConfig.UVScaleFactor = this->UVScaleFactor;
	StitchConfig.NumSubdivisions = this->NumSubdivisions;
	StitchConfig.bGroupPerSubdivision = this->bGroupPerSubdivision;
	StitchConfig.bUVIslandPerGroup = this->bUVIslandPerGroup;
	StitchConfig.CreaseAngleThreshold = this->CreaseAngleThresholdDeg;
	StitchConfig.SetMaterialID = this->SetMaterialID;
	StitchConfig.bInferMaterialID = this->bInferMaterialID;
	TArray<bool> bLoopSuccess;
	bool bSuccess = OffsetMeshRegionLocals::StitchRegionBorderLoopPairs_Version1( *Mesh, 
			LoopPairs, LoopsEdgeGroups, OffsetTriOrderedEdgeLoops,
			StitchConfig,
			bLoopSuccess, Region.StitchTriangles, Region.StitchPolygonIDs);

	int NumInitialLoops = LoopPairs.Num();
	Region.BaseLoops.SetNum(NumInitialLoops);
	Region.OffsetLoops.SetNum(NumInitialLoops);
	for (int32 LoopIndex = 0; LoopIndex < NumInitialLoops; ++LoopIndex)
	{
		if (bLoopSuccess[LoopIndex])
		{
			Region.BaseLoops[LoopIndex].InitializeFromVertices(Mesh, LoopPairs[LoopIndex].OuterVertices);
			Region.OffsetLoops[LoopIndex].InitializeFromVertices(Mesh, LoopPairs[LoopIndex].InnerVertices);
		}
	}

	if (Region.bIsSolid)
	{
		if (bIsPositiveOffset)
		{
			// Flip the "bottom" of the region to face outwards
			Editor.ReverseTriangleOrientations(DuplicateResult.NewTriangles, true);
		}
		else
		{
			Editor.ReverseTriangleOrientations(RegionTriangles, true);
		}
	}

	if (bSingleGroupPerArea && Mesh->HasTriangleGroups() && Region.OffsetGroups.Num() > 1)
	{
		int32 NewGroupID = Region.OffsetGroups[0];
		for (int32 TriangleID : RegionTriangles)
		{
			Mesh->SetTriangleGroup(TriangleID, NewGroupID);
		}
		Region.OffsetGroups.SetNum(1);
	}

	return bSuccess;
}



bool FOffsetMeshRegion::ApplyOffset_Legacy(FOffsetInfo& Region)
{
	// Store offset groups
	if (Mesh->HasTriangleGroups())
	{
		for (int32 Tid : Region.OffsetTids)
		{
			Region.OffsetGroups.AddUnique(Mesh->GetTriangleGroup(Tid));
		}
	}

	FMeshRegionBoundaryLoops InitialLoops(Mesh, Region.OffsetTids, false);
	bool bOK = InitialLoops.Compute();
	if (bOK == false)
	{
		return false;
	}

	AllModifiedAndNewTriangles.Append(Region.OffsetTids);

	// Before we start changing triangles, prepare by allocating group IDs that we'll use
	// for the stitched sides (doing it before changes to the mesh allows user-provided
	// LoopEdgesShouldHaveSameGroup functions to operate on the original mesh).
	TArray<TArray<int32>> LoopsEdgeGroups;
	TArray<int32> NewGroupIDs;
	LoopsEdgeGroups.SetNum(InitialLoops.Loops.Num());
	for (int32 i = 0; i < InitialLoops.Loops.Num(); ++i)
	{
		TArray<int32>& LoopEids = InitialLoops.Loops[i].Edges;
		int32 NumEids = LoopEids.Num();

		if (!ensure(NumEids > 2))
		{
			// Shouldn't actually happen because we're extruding triangles
			continue;
		}

		TArray<int32>& CurrentEdgeGroups = LoopsEdgeGroups[i];
		CurrentEdgeGroups.SetNumUninitialized(NumEids);
		CurrentEdgeGroups[0] = Mesh->AllocateTriangleGroup();
		NewGroupIDs.Add(CurrentEdgeGroups[0]);

		// Propagate the group backwards first so we don't allocate an unnecessary group
		// at the end and then have to fix it.
		int32 LastDifferentGroupIndex = NumEids - 1;
		while (LastDifferentGroupIndex > 0
			&& LoopEdgesShouldHaveSameGroup(LoopEids[0], LoopEids[LastDifferentGroupIndex]))
		{
			CurrentEdgeGroups[LastDifferentGroupIndex] = CurrentEdgeGroups[0];
			--LastDifferentGroupIndex;
		}

		// Now add new groups forward
		for (int32 j = 1; j <= LastDifferentGroupIndex; ++j)
		{
			if (!LoopEdgesShouldHaveSameGroup(LoopEids[j], LoopEids[j - 1]))
			{
				CurrentEdgeGroups[j] = Mesh->AllocateTriangleGroup();
				NewGroupIDs.Add(CurrentEdgeGroups[j]);
			}
			else
			{
				CurrentEdgeGroups[j] = CurrentEdgeGroups[j-1];
			}
		}
	}

	FDynamicMeshEditor Editor(Mesh);
	TArray<FDynamicMeshEditor::FLoopPairSet> LoopPairs;

	FDynamicMeshEditResult DuplicateResult;
	if (Region.bIsSolid)
	{
		// In the solid case, we want to duplicate the region so we can cap it.
		FMeshIndexMappings IndexMap;
		Editor.DuplicateTriangles(Region.OffsetTids, IndexMap, DuplicateResult);

		AllModifiedAndNewTriangles.Append(DuplicateResult.NewTriangles);

		// Populate LoopPairs
		LoopPairs.SetNum(InitialLoops.Loops.Num());
		for (int LoopIndex = 0; LoopIndex < InitialLoops.Loops.Num(); ++LoopIndex)
		{
			FEdgeLoop& BaseLoop = InitialLoops.Loops[LoopIndex];
			FDynamicMeshEditor::FLoopPairSet& LoopPair = LoopPairs[LoopIndex];

			// The original OffsetTids are the ones that are offset, so InnerVertices/Edges
			// should be the boundaries of those.
			LoopPair.InnerVertices = BaseLoop.Vertices;
			LoopPair.InnerEdges = BaseLoop.Edges;

			// However depending on whether we extruded down or up, we may need to reverse
			// the loops to get them to be stitched right side out.
			if (!bIsPositiveOffset)
			{
				Algo::Reverse(LoopPair.InnerVertices);

				// Reversing the edges is slightly different because the last edge is between the first
				// and last vertex, and that needs to stay in the same place when vertices are reversed.
				int32 LastEid = LoopPair.InnerEdges.Pop();
				Algo::Reverse(LoopPair.InnerEdges);
				LoopPair.InnerEdges.Add(LastEid);

				int32 LastEdgeGroupID = LoopsEdgeGroups[LoopIndex].Pop();
				Algo::Reverse(LoopsEdgeGroups[LoopIndex]);
				LoopsEdgeGroups[LoopIndex].Add(LastEdgeGroupID);
			}

			// Now assemble the paired loop
			for (int32 Vid : LoopPair.InnerVertices)
			{
				LoopPair.OuterVertices.Add(IndexMap.GetNewVertex(Vid));
			}
			FEdgeLoop::VertexLoopToEdgeLoop(Mesh, LoopPair.OuterVertices, LoopPair.OuterEdges);
		}
	}
	else
	{
		// Disconnect the triangles. Note that this will recompute FMeshRegionBoundaryLoops internally,
		// and so the output LoopPairs will generally correspond to InitialLoops for that reason,
		// but there isn't a guarantee otherwise
		bOK = Editor.DisconnectTriangles(Region.OffsetTids, LoopPairs, true /*bHandleBoundaryVertices*/);
	}

	if (bOK == false)
	{
		return false;
	}

	// Store the vid-independent offset loop before we break bowties
	TArray<TArray<OffsetMeshRegionLocals::TriVertPair>> OffsetStitchSides;
	OffsetStitchSides.SetNum(LoopPairs.Num());
	for (int32 i = 0; i < LoopPairs.Num(); ++i)
	{
		bOK = bOK && FDynamicMeshEditor::ConvertLoopToTriVidPairSequence(*Mesh, LoopPairs[i].InnerVertices, LoopPairs[i].InnerEdges, OffsetStitchSides[i]);
	}

	if (bOK == false)
	{
		return false;
	}
	
	// Split bowties in the chosen region
	FDynamicMeshEditResult Result;
	Editor.SplitBowtiesAtTriangles(Region.OffsetTids, Result);
	bool bSomeLoopsBroken = Result.NewVertices.Num() > 0;

	// BUG: once we split bowties, there some of the existing LoopPairs may actually no longer be boundary loops.
	// This occurs with an "interior" bowtie, where say a figure-8 shape that was two separate loops, becomes one loop.
	// In this case things go downhill as OffsetTriOrderedEdgeLoops can no longer be converted back to a valid
	// boundary loop (since it's not a loop anymore). The FDynamicMeshEditor::StitchVertexLoopsMinimal function called below
	// does not actually check that the loop it is stitching is a boundary loop, it simply stitches edge pairs with quads, 
	// which means that the function will usually appear to succeed even though it may have created broken topology.

	// If we broke bowties, the loops in the offset region have changed, and our OffsetLoops no longer
	// match BaseLoops.
	if (bSomeLoopsBroken)
	{
		// NOTE: Region.OffsetLoops is only used for output here, this does not affect the loops that
		// will be stitched, it just needs to be computed before stitching
		FMeshRegionBoundaryLoops UpdatedOffsetLoops(Mesh, Region.OffsetTids, false);
		bOK = UpdatedOffsetLoops.Compute();
		if (!bOK)
		{
			return false;
		}
		Region.OffsetLoops = UpdatedOffsetLoops.Loops;
	}

	FMeshVertexSelection SelectionV(Mesh);
	SelectionV.SelectTriangleVertices(Region.OffsetTids);
	TArray<int32> SelectedVids = SelectionV.AsArray();

	// If we need to, assemble the vertex vectors for us to use (before we actually start moving things)
	TArray<FVector3d> VertexExtrudeVectors;
	if (ExtrusionVectorType == EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAverage
		|| ExtrusionVectorType == EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAdjusted)
	{
		VertexExtrudeVectors.SetNumUninitialized(SelectedVids.Num());

		// Used to test which triangles are in selection
		TSet<int32> TriangleSet(Region.OffsetTids);

		// Used to test which triangles are in selection
		for (int32 i = 0; i < SelectedVids.Num(); ++i)
		{
			VertexExtrudeVectors[i] = ( ExtrusionVectorType == EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAdjusted ) ?
				OffsetMeshRegionLocals::GetAngleWeightedAdjustedNormal(*Mesh, SelectedVids[i], TriangleSet, MaxScaleForAdjustingTriNormalsOffset)
				 : OffsetMeshRegionLocals::GetAngleWeightedAverageNormal(*Mesh, SelectedVids[i], TriangleSet);
		}
	}
	else if (ExtrusionVectorType == EVertexExtrusionVectorType::VertexNormal)
	{
		VertexExtrudeVectors.SetNumUninitialized(SelectedVids.Num());
		for (int32 i = 0; i < SelectedVids.Num(); ++i)
		{
			int32 Vid = SelectedVids[i];
			VertexExtrudeVectors[i] = Mesh->HasVertexNormals() ? (FVector3d)Mesh->GetVertexNormal(Vid) : FMeshNormals::ComputeVertexNormal(*Mesh, Vid);
		}
	}

	// Perform the actual vertex displacement.
	for (int32 i = 0; i < SelectedVids.Num(); ++i)
	{
		int32 Vid = SelectedVids[i];
		FVector3d OldPosition = Mesh->GetVertex(Vid);
		FVector3d ExtrusionVector = (ExtrusionVectorType == EVertexExtrusionVectorType::Zero) ? FVector3d::Zero() : VertexExtrudeVectors[i];

		FVector3d NewPosition = OffsetPositionFunc(OldPosition, ExtrusionVector, Vid);
		Mesh->SetVertex(Vid, NewPosition);
	}

	// Stitch the loops

	bool bSuccess = true;
	int NumInitialLoops = LoopPairs.Num();
	Region.BaseLoops.SetNum(NumInitialLoops);
	if (!bSomeLoopsBroken)
	{
		Region.OffsetLoops.SetNum(NumInitialLoops);
	}
	Region.StitchTriangles.SetNum(NumInitialLoops);
	Region.StitchPolygonIDs.SetNum(NumInitialLoops);

	for (int32 LoopIndex = 0; LoopIndex < LoopPairs.Num(); ++LoopIndex)
	{
		FDynamicMeshEditor::FLoopPairSet& LoopPair = LoopPairs[LoopIndex];
		const TArray<int32>& EdgeGroups = LoopsEdgeGroups[LoopIndex];

		TArray<int32>& BaseLoopV = LoopPair.OuterVertices;
		TArray<int32>& OffsetLoopV = LoopPair.InnerVertices;

		TArray<OffsetMeshRegionLocals::TriVertPair>& OffsetLoopTriVertPairs = OffsetStitchSides[LoopIndex];

		// stitch the loops
		FDynamicMeshEditResult StitchResult;
		bool bStitchSuccess = Editor.StitchVertexLoopToTriVidPairSequence(OffsetLoopTriVertPairs, LoopPair.OuterVertices, StitchResult);
		if (!bStitchSuccess)
		{
			bSuccess = false;
			continue;
		}

		// set the groups of the new quads along the stitch
		int NumNewQuads = StitchResult.NewQuads.Num();
		for (int32 k = 0; k < NumNewQuads; k++)
		{
			Mesh->SetTriangleGroup(StitchResult.NewQuads[k].A, EdgeGroups[k]);
			Mesh->SetTriangleGroup(StitchResult.NewQuads[k].B, EdgeGroups[k]);
		}

		// save the stitch triangles set and associated group IDs
		StitchResult.GetAllTriangles(Region.StitchTriangles[LoopIndex]);
		Region.StitchPolygonIDs[LoopIndex] = NewGroupIDs;

		AllModifiedAndNewTriangles.Append(Region.StitchTriangles[LoopIndex]);

		// for each polygon we created in stitch, set UVs and normals
		if (Mesh->HasAttributes())
		{
			float AccumUVTranslation = 0;
			FFrame3d FirstProjectFrame;
			FVector3d FrameUp;

			for (int k = 0; k < NumNewQuads; k++)
			{
				FVector3f Normal = Editor.ComputeAndSetQuadNormal(StitchResult.NewQuads[k], true);

				// align axis 0 of projection frame to first edge, then for further edges,
				// rotate around 'up' axis to keep normal aligned and frame horizontal
				FFrame3d ProjectFrame;
				if (k == 0)
				{
					FVector3d FirstEdge = Mesh->GetVertex(BaseLoopV[1]) - Mesh->GetVertex(BaseLoopV[0]);
					Normalize(FirstEdge);
					FirstProjectFrame = FFrame3d(FVector3d::Zero(), (FVector3d)Normal);
					FirstProjectFrame.ConstrainedAlignAxis(0, FirstEdge, (FVector3d)Normal);
					FrameUp = FirstProjectFrame.GetAxis(1);
					ProjectFrame = FirstProjectFrame;
				}
				else
				{
					ProjectFrame = FirstProjectFrame;
					ProjectFrame.ConstrainedAlignAxis(2, (FVector3d)Normal, FrameUp);
				}

				if (k > 0)
				{
					AccumUVTranslation += (float)Distance(Mesh->GetVertex(BaseLoopV[k]), Mesh->GetVertex(BaseLoopV[k - 1]));
				}

				// translate horizontally such that vertical spans are adjacent in UV space (so textures tile/wrap properly)
				float TranslateU = UVScaleFactor * AccumUVTranslation;
				Editor.SetQuadUVsFromProjection(StitchResult.NewQuads[k], ProjectFrame, UVScaleFactor, FVector2f(TranslateU, 0));
			}
		}

		Region.BaseLoops[LoopIndex].InitializeFromVertices(Mesh, BaseLoopV);
		if (!bSomeLoopsBroken)
		{
			Region.OffsetLoops[LoopIndex].InitializeFromVertices(Mesh, OffsetLoopV);
		}
	}

	if (Region.bIsSolid)
	{
		if (bIsPositiveOffset)
		{
			// Flip the "bottom" of the region to face outwards
			Editor.ReverseTriangleOrientations(DuplicateResult.NewTriangles, true);
		}
		else
		{
			Editor.ReverseTriangleOrientations(Region.OffsetTids, true);
		}
	}

	return bSuccess;
}

bool FOffsetMeshRegion::EdgesSeparateSameGroupsAndAreColinearAtBorder(FDynamicMesh3* Mesh, 
	int32 Eid1, int32 Eid2, bool bCheckColinearityAtBorder)
{
	if (!Mesh->IsEdge(Eid1) || !Mesh->IsEdge(Eid2))
	{
		return ensure(false);
	}

	FIndex2i Tris1 = Mesh->GetEdgeT(Eid1);
	FIndex2i Groups1(Mesh->GetTriangleGroup(Tris1.A),
		Tris1.B == IndexConstants::InvalidID ? IndexConstants::InvalidID : Mesh->GetTriangleGroup(Tris1.B));

	FIndex2i Tris2 = Mesh->GetEdgeT(Eid2);
	FIndex2i Groups2(Mesh->GetTriangleGroup(Tris2.A), 
		Tris2.B == IndexConstants::InvalidID ? IndexConstants::InvalidID : Mesh->GetTriangleGroup(Tris2.B));

	if (bCheckColinearityAtBorder
		&& Groups1.A == Groups2.A
		&& Groups1.B == IndexConstants::InvalidID
		&& Groups2.B == IndexConstants::InvalidID)
	{
		return OffsetMeshRegionLocals::EdgesAreParallel(Mesh, Eid1, Eid2);
	}
	else return (Groups1.A == Groups2.A && Groups1.B == Groups2.B)
		|| (Groups1.A == Groups2.B && Groups1.B == Groups2.A);
}

