// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/PNTriangles.h"
#include "IndexTypes.h"
#include "VectorTypes.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Async/ParallelFor.h"
#include "Util/ProgressCancel.h"
#include "Operations/UniformTessellate.h"

using namespace UE::Geometry;

namespace FPNTrianglesLocals
{ 
	/**
	 * PN Triangle is represented by a control points cage. Each edge has 2 control points and 
	 * 1 control point is inside the triangle. Additionally, each edge contains one control point 
	 * of the normal component. We separate control points into 2 categories to avoid repeating 
	 * computations since control points on edges are shared.
	 */
		
	using FTriangleControlPoint = FVector3d;

	struct FEdgeControlPoints
	{	
		FVector3d Point1;
		FVector3d Point2;
		FVector3d NormalTriA;
		FVector3d NormalTriB;
	}; 
	
	struct FControlPoints 
	{
		TArray<FEdgeControlPoints> OnEdges; // Map edge ID to the edge control points
		TArray<FTriangleControlPoint> OnTriangles; // Map triangle ID to the triangle center control point
	};

	/**
	 *  Compute PN Triangle control points and optionally their normal component.
	 * 
	 *  @param Mesh The mesh used to compute the per-triangle control points.
	 *  @param UseNormals If normals are disabled for the Mesh then pass manually computed normals.
	 *  @param bComputePNNormals Compute control points for the normal component for calculating quadratically varying normals.
	 *  @param ProgressCancel Set this to be able to cancel running operation.
	 *  @param OutControlPoints Contains all the control points for the Mesh.
	 * 
	 *  @return true if the operation succeeded, false if it failed or was canceled by the user.
	 */
	bool ComputeControlPoints(FControlPoints& OutControlPoints,
							  const FDynamicMesh3& Mesh, 
							  const FMeshNormals* UseNormals, 
							  const bool bComputePNNormals, 
							  FProgressCancel* ProgressCancel) 
	{
		if (!Mesh.HasAttributes() && !Mesh.HasVertexNormals() && UseNormals == nullptr)
		{
			return false; // Mesh must have normals
		}

		// We know ahead of time the total number of control points:
		// 2 control points per edge and one in the middle of each triangle.
		OutControlPoints.OnEdges.SetNum(Mesh.MaxEdgeID());
		OutControlPoints.OnTriangles.SetNum(Mesh.MaxTriangleID());

		auto ComputeControlPoint = [](const FVector3d& Vertex1, const FVector3d& Vertex2, const FVector3f& Normal)
		{ 
			FVector3d Edge12 = Vertex2 - Vertex1;
			double Weight12 = Edge12.Dot(FVector3d(Normal));
			FVector3d Point = (2.0*Vertex1 + Vertex2 - Weight12*FVector3d(Normal))/3.0;  
			return Point;
		 };

		auto ComputeControlNormal = [](const FVector3d& Vertex1, const FVector3d& Vertex2, const FVector3f& Normal1, const FVector3f& Normal2)
		{ 	
			FVector3d Edge12 = Vertex2 - Vertex1;
			FVector3f Normal21 = Normal2 + Normal1;

			double Divisor = Edge12.Dot(Edge12);
			
			FVector3f Normal;
			if (FMath::IsNearlyZero(Divisor)) 
			{
				Normal = 0.5f * Normal21; // If edge is degenerate then simply interpolate between the normals
			} 
			else
			{ 
				double NWeight12 = 2.0*Edge12.Dot(FVector3d(Normal21))/Divisor;
				Normal = Normalized(Normal21 - NWeight12*FVector3f(Edge12));
			}
			
			return Normal;
		 };

		// First compute only the control points on the edges
		ParallelFor(Mesh.MaxEdgeID(), [&](int32 EID)
		{
			if (ProgressCancel && ProgressCancel->Cancelled()) 
			{
				return;  
			}

			if (Mesh.IsEdge(EID)) 
			{
				FIndex2i EdgeV = Mesh.GetEdgeV(EID);
				
				FVector3d Vertex1 = Mesh.GetVertex(EdgeV.A);
				FVector3d Vertex2 = Mesh.GetVertex(EdgeV.B);

				FIndex2i EdgeTri = Mesh.GetEdgeT(EID);

				FVector3f Normal1, Normal2;
				if (Mesh.HasAttributes()) 
				{
					const FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
					if (NormalOverlay->IsSetTriangle(EdgeTri.A)) 
					{
						NormalOverlay->GetElementAtVertex(EdgeTri.A, EdgeV.A, Normal1);
						NormalOverlay->GetElementAtVertex(EdgeTri.A, EdgeV.B, Normal2);
					}
					else
					{
						Normal1 = static_cast<FVector3f>(Mesh.GetTriNormal(EdgeTri.A));
						Normal2 = Normal1;
					}
				}
				else 
				{
					Normal1 = (UseNormals != nullptr) ? (FVector3f)(*UseNormals)[EdgeV.A] : Mesh.GetVertexNormal(EdgeV.A);
					Normal2 = (UseNormals != nullptr) ? (FVector3f)(*UseNormals)[EdgeV.B] : Mesh.GetVertexNormal(EdgeV.B);
				}

				FEdgeControlPoints& ControlPnts = OutControlPoints.OnEdges[EID];

				if (bComputePNNormals)
				{
					ControlPnts.NormalTriA = FVector3d(ComputeControlNormal(Vertex1, Vertex2, Normal1, Normal2));
				}

				FVector3d Point1ForTriA = ComputeControlPoint(Vertex1, Vertex2, Normal1);
				FVector3d Point2ForTriA = ComputeControlPoint(Vertex2, Vertex1, Normal2);

				if (Mesh.HasAttributes() && Mesh.Attributes()->PrimaryNormals()) 
				{
					const FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
					if (EdgeTri.B != FDynamicMesh3::InvalidID && NormalOverlay->IsSeamEdge(EID)) 
					{	
						if (NormalOverlay->IsSetTriangle(EdgeTri.B)) 
						{
							NormalOverlay->GetElementAtVertex(EdgeTri.B, EdgeV.A, Normal1);
							NormalOverlay->GetElementAtVertex(EdgeTri.B, EdgeV.B, Normal2);
						}
						else 
						{
							Normal1 = static_cast<FVector3f>(Mesh.GetTriNormal(EdgeTri.B));
							Normal2 = Normal1;
						}

						FVector3d Point1ForTriB = ComputeControlPoint(Vertex1, Vertex2, Normal1);
						FVector3d Point2ForTriB = ComputeControlPoint(Vertex2, Vertex1, Normal2);

						// Following the core idea of Crack-free PN Triangles we average the control point we'd like 
						// with the control point our neighbor triangle would like. 
						Point1ForTriA = (Point1ForTriA + Point1ForTriB) / 2.0;
						Point2ForTriA = (Point2ForTriA + Point2ForTriB) / 2.0;

						if (bComputePNNormals)
						{
							ControlPnts.NormalTriB = FVector3d(ComputeControlNormal(Vertex1, Vertex2, Normal1, Normal2));
						}
					}
				}

				ControlPnts.Point1 = Point1ForTriA;
				ControlPnts.Point2 = Point2ForTriA;
			}
		});

		if (ProgressCancel && ProgressCancel->Cancelled())         
		{
			return false; 
		}

		// Compute the center control point for each triangle
		ParallelFor(Mesh.MaxTriangleID(), [&](int32 TID)
		{
			if (ProgressCancel && ProgressCancel->Cancelled()) 
			{
				return;  
			}

			if (Mesh.IsTriangle(TID)) 
			{
				const FIndex3i& TriEdgeID = Mesh.GetTriEdgesRef(TID);
				FVector3d ControlMidpoint = FVector3d::Zero();
				for (int EIdx = 0; EIdx < 3; ++EIdx) 
				{
					const FEdgeControlPoints& ControlPnts = OutControlPoints.OnEdges[TriEdgeID[EIdx]];
					ControlMidpoint += ControlPnts.Point1 + ControlPnts.Point2;
				}
				ControlMidpoint /= 6.0;

				const FIndex3i& TriVertID = Mesh.GetTriangleRef(TID);
				FVector3d VertexMidpoint = (Mesh.GetVertex(TriVertID.A) + 
											Mesh.GetVertex(TriVertID.B) + 
											Mesh.GetVertex(TriVertID.C))/3.0;

				OutControlPoints.OnTriangles[TID] = ControlMidpoint + (ControlMidpoint - VertexMidpoint)/2.0;	
			}
		});

		if (ProgressCancel && ProgressCancel->Cancelled())         
		{
			return false; 
		}

		return true;
	}

	/** 
	 * Tessellate the mesh by recursively subdividing it using loop-style subdivision. Keep track of the new vertices 
	 * added and which original triangles (before tessellation) they belong to. New vertices on original non-boundary 
	 * edges can belong to either of the original triangles that share the edge.
	 * 
	 * @param Mesh The mesh we are tessellating.
	 * @param Level How many times we are recursively subdividing the Mesh.
	 * @param ProgressCancel Set this to be able to cancel running operation.
	 * @param OutNewVertices Array of tuples of the new vertex ID and the original triangle ID the vertex belongs to.
	 * @param OutMesh Result of the tessellation.
	 * 
	 * @return true if the operation succeeded, false if it failed or was canceled by the user.
	 */
	bool TessellateMesh(const FDynamicMesh3& Mesh, 
					   const int32 Level, 
					   FProgressCancel* ProgressCancel,
					   TArray<FIndex2i>& OutNewVertices,
					   FDynamicMesh3& OutMesh) 
	{
		checkSlow(Level >= 0);

		if (Level < 0) 
		{
			return false;
		}
		else if (Level == 0) 
		{
			return true; // nothing to do
		}

		FUniformTessellate Tessellator(&Mesh, &OutMesh);
		Tessellator.TessellationNum = Level;
		Tessellator.bComputeMappings = true;
		if (Tessellator.Validate() != EOperationValidationResult::Ok)
		{
			return false;
		}

		if (Tessellator.Compute() == false) 
		{
			return false;
		} 

		const int32 NewVertCount = Tessellator.VertexEdgeMap.Num() + Tessellator.VertexTriangleMap.Num();
		OutNewVertices.Reserve(NewVertCount);

		for (auto It = Tessellator.VertexEdgeMap.CreateConstIterator(); It; ++It)
		{
			FIndex2i EdgeTri = It.Value();
			OutNewVertices.Add(FIndex2i(It.Key(), EdgeTri.A));
		}

		for (auto It = Tessellator.VertexTriangleMap.CreateConstIterator(); It; ++It)
		{
			OutNewVertices.Add(FIndex2i(It.Key(), It.Value()));
		}

		checkSlow(OutNewVertices.Num() == NewVertCount);

		return true;
	}

	/** 
	 * Displace the vertices created from the tessellation using the cubic patch formula based on their barycentric 
	 * coordinates.
	 * 
	 * @param OriginalMesh The original mesh (before tessellation).
	 * @param FControlPoints PN Triangle control points.
	 * @param VerticesToDisplace Array of vertices into the Mesh that we are displacing.
	 * @param bComputePNNormals Should we be computing quadratically varying normals using control points.
	 * @param ProgressCancel Set this to be able to cancel running operation.
	 * @param Mesh The tessellated mesh whose vertices we are displacing.
	 * 
	 * @return true if the operation succeeded, false if it failed or was canceled by the user.
	 */

	bool DisplaceAndSetQuadraticNormals(const FDynamicMesh3& OriginalMesh,
					           			const FControlPoints& FControlPoints,
					           			const TArray<FIndex2i>& VerticesToDisplace,
					           			const bool bComputePNNormals,
							   			FProgressCancel* ProgressCancel,
							   			FDynamicMesh3& Mesh)
	{
		bool bHasVertexNormals = Mesh.HasVertexNormals();
		bool bHasAttributes = Mesh.HasAttributes();
		
		// Iterate over every new vertex and compute its displacement and optionally a normal
		ParallelFor(VerticesToDisplace.Num(), [&](int32 IDX)
		{	
			if (ProgressCancel && ProgressCancel->Cancelled()) 
			{
				return; 
			}
			
			FIndex2i NewVtx = VerticesToDisplace[IDX];
			int VertexID = NewVtx[0]; // ID of the new vertex added with tessellation
			int OriginalTriangleID = NewVtx[1]; // ID of the original triangle new vertex belongs to
			
			// Get the topology information of the original triangle
			FIndex3i TriVertex = OriginalMesh.GetTriangle(OriginalTriangleID);
			FIndex3i TriEdges = OriginalMesh.GetTriEdges(OriginalTriangleID);

			FVector3d Bary = VectorUtil::BarycentricCoords(Mesh.GetVertexRef(VertexID), OriginalMesh.GetVertexRef(TriVertex.A),
														   OriginalMesh.GetVertexRef(TriVertex.B), OriginalMesh.GetVertexRef(TriVertex.C));
														   
			FVector3d BarySquared = Bary*Bary;

			// Displaced vertex. First compute contribution of the original control points at the original vertices
			FVector3d NewPos = Bary[0]*BarySquared[0]*OriginalMesh.GetVertexRef(TriVertex.A) + 
							   Bary[1]*BarySquared[1]*OriginalMesh.GetVertexRef(TriVertex.B) + 
							   Bary[2]*BarySquared[2]*OriginalMesh.GetVertexRef(TriVertex.C);
			
			// Compute contribution of the control points at the edges
			for (int EIDX = 0; EIDX < 3; ++EIDX) 
			{   
				int EID = TriEdges[EIDX];
				FIndex2i EdgeV = OriginalMesh.GetEdgeRef(EID).Vert;
				const FEdgeControlPoints& ControlPnts = FControlPoints.OnEdges[EID];

				// Check that the orientation of the edge is consistent so we use the correct barycentric values
				int EdgeVtx1 = EdgeV[0];
				int EdgeVtx2 = EdgeV[1];
				IndexUtil::OrientTriEdgeAndFindOtherVtx(EdgeVtx1, EdgeVtx2, TriVertex);
				
				int BaryIdx1 = EdgeV[0] == EdgeVtx1 ? EIDX : (EIDX + 1) % 3;
				int BaryIdx2 = EdgeV[0] == EdgeVtx1 ? (EIDX + 1) % 3 : EIDX;
				
				NewPos += (3.0*BarySquared[BaryIdx1]*Bary[BaryIdx2])*ControlPnts.Point1 + 
						  (3.0*BarySquared[BaryIdx2]*Bary[BaryIdx1])*ControlPnts.Point2;
			}

			// Finally compute contribution of the single control point inside the triangle
			NewPos += (6.0*Bary[0]*Bary[1]*Bary[2])*FControlPoints.OnTriangles[OriginalTriangleID];
			
			Mesh.SetVertex(VertexID, NewPos); 

			if (bComputePNNormals) 
			{
				if (bHasVertexNormals) 
				{
					// Compute contribution of the normal control points at the original vertices
					FVector3d NewNormal(BarySquared[0]*OriginalMesh.GetVertexNormal(TriVertex.A) + 
										BarySquared[1]*OriginalMesh.GetVertexNormal(TriVertex.B) +
										BarySquared[2]*OriginalMesh.GetVertexNormal(TriVertex.C));
					
					// Compute contribution of the normal control points at the edges
					for (int EIDX = 0; EIDX < 3; ++EIDX) 
					{   
						int EID = TriEdges[EIDX];
						const FEdgeControlPoints& ControlPnts = FControlPoints.OnEdges[EID];
						NewNormal += Bary[EIDX]*Bary[(EIDX + 1) % 3]*ControlPnts.NormalTriA;
					}

					Normalize(NewNormal);

					Mesh.SetVertexNormal(VertexID, FVector3f(NewNormal)); 
				} 
				
				if (bHasAttributes) 
				{
					//TODO: Compute per-element quadractic PN normals
				}
			}
		});

		if (ProgressCancel && ProgressCancel->Cancelled()) 
		{
			return false; 
		}
		
		return true;
	}
}

bool FPNTriangles::Compute()
{
	using namespace FPNTrianglesLocals;

	check(TessellationLevel >= 0);
	check(Mesh != nullptr);

	if (TessellationLevel < 0 || Mesh == nullptr) 
	{
		return false;
	}

	if (TessellationLevel == 0) 
	{
		return true; // nothing to do
	}

	// Compute per-vertex normals if no normals exist
	FMeshNormals Normals;
	bool bHasNormals = Mesh->HasVertexNormals() || (Mesh->HasAttributes() && Mesh->Attributes()->PrimaryNormals() != nullptr);
	if (bHasNormals == false)
	{
		Normals = FMeshNormals(Mesh);
		Normals.ComputeVertexNormals();
	}
	FMeshNormals* UseNormals = (bHasNormals) ? nullptr : &Normals;
	
	// Compute PN triangle control points for each flat triangle of the original mesh
	FControlPoints FControlPoints;

	// TODO: We disable quadratically varying normal computation until we fully support seam normal overlay edges.
	// We might remove them completely since using quadratically varying normals would imply that we should be using 
	// the same approach when computing normals inside the shaders for lighting computations 
	// (see Tessellation on Any Budget, GDC, 2011). 
	const bool bComputePNNormals = false;
	
	bool bOk = ComputeControlPoints(FControlPoints, *Mesh, UseNormals, bComputePNNormals, Progress);
	if (bOk == false) 
	{
		return false;
	}
	
	// Tessellate the original mesh
	TArray<FIndex2i> NewVertices;
	FDynamicMesh3 ResultMesh;
	bOk = TessellateMesh(*Mesh, TessellationLevel, Progress, NewVertices, ResultMesh);
	if (bOk == false) 
	{
		return false;
	}

	// Compute displacement and optionally quadratically varying normals
	bOk = DisplaceAndSetQuadraticNormals(*Mesh, FControlPoints, NewVertices, bComputePNNormals, Progress, ResultMesh);
	if (bOk == false) 
	{
		return false;
	}
				
	if (bRecalculateNormals && bComputePNNormals == false)
	{
		if (ResultMesh.HasAttributes())
		{
			FMeshNormals NewNormals(&ResultMesh);
			FDynamicMeshNormalOverlay* NormalOverlay = ResultMesh.Attributes()->PrimaryNormals();
			NewNormals.RecomputeOverlayNormals(NormalOverlay);
			NewNormals.CopyToOverlay(NormalOverlay);
		}
		else if (ResultMesh.HasVertexNormals())
		{
			FMeshNormals::QuickComputeVertexNormals(ResultMesh);
		}
	}

	Mesh->Copy(ResultMesh);

	return true;
}