// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/PlanarFlipsOptimization.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "MathUtil.h"
#include "VectorTypes.h"

using namespace UE::Geometry;

namespace UE {
namespace PlanarFlipsOptimizationLocals{

/** @return triangle aspect ratio transformed to be in [0,1] range */
double UnitAspectRatio(const FVector3d& A, const FVector3d& B, const FVector3d& C)
{
	double AspectRatio = VectorUtil::AspectRatio(A, B, C);
	return (AspectRatio > 1.0) ? FMathd::Clamp(1.0 / AspectRatio, 0.0, 1.0) : AspectRatio;
}
/** @return triangle aspect ratio transformed to be in [0,1] range */
double UnitAspectRatio(const FDynamicMesh3& Mesh, int32 TriangleID)
{
	FVector3d A, B, C;
	Mesh.GetTriVertices(TriangleID, A, B, C);
	return UnitAspectRatio(A, B, C);
}
}}//end namespace UE::PlanarFlipsOptimizationLocals

void FPlanarFlipsOptimization::Apply()
{
	check(Mesh);
	for (int32 i = 0; i < NumPasses; ++i)
	{
		ApplySinglePass();
	}
}

void FPlanarFlipsOptimization::ApplySinglePass()
{
	using namespace UE::PlanarFlipsOptimizationLocals;

	//Note: could be more efficient to do multiple passes directly in Apply(), would save on the initial computation

	struct FFlatEdge
	{
		int32 eid;
		double MinAspect;
	};

	TArray<double> AspectRatios;
	TArray<FVector3d> Normals;
	AspectRatios.SetNum(Mesh->MaxTriangleID());
	Normals.SetNum(Mesh->MaxTriangleID());
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		FVector3d A, B, C;
		Mesh->GetTriVertices(tid, A, B, C);
		AspectRatios[tid] = UnitAspectRatio(A, B, C);
		Normals[tid] = VectorUtil::Normal(A, B, C);
	}

	TArray<FFlatEdge> Flips;
	for (int32 eid : Mesh->EdgeIndicesItr())
	{
		if (Mesh->IsBoundaryEdge(eid) == false)
		{
			FIndex2i EdgeT = Mesh->GetEdgeT(eid);
			if (AspectRatios[EdgeT.A] < 0.01 && AspectRatios[EdgeT.B] < 0.01)
			{
				continue;		// if both are degenerate we can't fix by flipping edge between them
			}
			if (bRespectGroupBoundaries && Mesh->IsGroupBoundaryEdge(eid))
			{
				continue;
			}
			double MinAspect = FMathd::Min(AspectRatios[EdgeT.A], AspectRatios[EdgeT.B]);
			
			// The triangles need to be coplanar, but we also check for zero normals, which are a sign
			// of a degenerate triangle. We want to try to flip those.
			// TODO: Perhaps should also look for edges where an opposite vert is almost in the plane
			// of the other triangle even if the normals disagree (i.e., triangle is practically degenerate).
			double NormDot = Normals[EdgeT.A].Dot(Normals[EdgeT.B]);
			if (NormDot > PlanarDotThresh 
				|| Normals[EdgeT.A] == FVector3d::Zero() 
				|| Normals[EdgeT.B] == FVector3d::Zero())
			{
				Flips.Add({ eid, MinAspect });
			}
		}
	}

	Flips.Sort([&](const FFlatEdge& A, const FFlatEdge& B) { return A.MinAspect < B.MinAspect; });

	for (int32 k = 0; k < Flips.Num(); ++k)
	{
		int32 eid = Flips[k].eid;
		FIndex2i EdgeV = Mesh->GetEdgeV(eid);
		int32 a = EdgeV.A, b = EdgeV.B;
		FIndex2i EdgeT = Mesh->GetEdgeT(eid);
		FIndex3i Tri0 = Mesh->GetTriangle(EdgeT.A), Tri1 = Mesh->GetTriangle(EdgeT.B);
		int32 c = IndexUtil::OrientTriEdgeAndFindOtherVtx(a, b, Tri0);
		int32 d = IndexUtil::FindTriOtherVtx(a, b, Tri1);

		double AspectA = AspectRatios[EdgeT.A], AspectB = AspectRatios[EdgeT.B];
		double Metric = FMathd::Min(AspectA, AspectB);
		FVector3d Normal = (AspectA > AspectB) ? Normals[EdgeT.A] : Normals[EdgeT.B];

		FVector3d A = Mesh->GetVertex(a), B = Mesh->GetVertex(b);
		FVector3d C = Mesh->GetVertex(c), D = Mesh->GetVertex(d);

		double FlipAspect1 = UnitAspectRatio(C, D, B);
		double FlipAspect2 = UnitAspectRatio(D, C, A);
		FVector3d FlipNormal1 = VectorUtil::Normal(C, D, B);
		FVector3d FlipNormal2 = VectorUtil::Normal(D, C, A);
		if (FlipNormal1.Dot(Normal) < PlanarDotThresh || FlipNormal2.Dot(Normal) < PlanarDotThresh)
		{
			// This should only happen if flipping would result in a degenerate tri (which
			// will have a zero normal). We don't want the flip in that case.
			continue;
		}

		if (FMathd::Min(FlipAspect1, FlipAspect2) > Metric)
		{
			FDynamicMesh3::FEdgeFlipInfo FlipInfo;
			if (Mesh->FlipEdge(eid, FlipInfo) == EMeshResult::Ok)
			{
				AspectRatios[EdgeT.A] = UnitAspectRatio(*Mesh, EdgeT.A);
				AspectRatios[EdgeT.B] = UnitAspectRatio(*Mesh, EdgeT.B);

				// safety check - if somehow we flipped the normal, flip it back
				bool bInvertedNormal = (Mesh->GetTriNormal(EdgeT.A).Dot(Normal) < PlanarDotThresh) ||
					(Mesh->GetTriNormal(EdgeT.B).Dot(Normal) < PlanarDotThresh);
				if (bInvertedNormal)
				{
					UE_LOGF(LogTemp, Warning, "FPlanarFlipsOptimization: Invalid Flip!");
					Mesh->FlipEdge(eid, FlipInfo);
					AspectRatios[EdgeT.A] = UnitAspectRatio(*Mesh, EdgeT.A);
					AspectRatios[EdgeT.B] = UnitAspectRatio(*Mesh, EdgeT.B);
				}
			}
		}
	}
}