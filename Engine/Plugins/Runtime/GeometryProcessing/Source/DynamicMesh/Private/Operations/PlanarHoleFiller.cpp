// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/PlanarHoleFiller.h"
#include "Curve/PlanarComplex.h"
#include "Operations/HoleFillUtil.h"

using namespace UE::Geometry;

bool FPlanarHoleFiller::Fill(int GroupID)
{
	if (GroupID < 0 && Mesh->HasTriangleGroups())
	{
		GroupID = Mesh->AllocateTriangleGroup();
	}

	FVector3d PlaneX, PlaneY;
	VectorUtil::MakePerpVectors(PlaneNormal, PlaneX, PlaneY);

	
	FPlanarComplexd Complex;
	for (const TArray<int>& Loop : *VertexLoops)
	{
		FPolygon2d& Polygon = Complex.Polygons.Emplace_GetRef();
		for (int VID : Loop)
		{
			FVector3d VertMinusOrigin = Mesh->GetVertex(VID) - PlaneOrigin;
			Polygon.AppendVertex(FVector2d(PlaneX.Dot(VertMinusOrigin), PlaneY.Dot(VertMinusOrigin)));
		}
	}

	Complex.bTrustOrientations = true;
	Complex.bAllowOverlappingHoles = false;

	Complex.FindSolidRegions();

	bool bAddedAll = true;
	for (const FPlanarComplexd::FPolygonNesting& Nest : Complex.Solids)
	{
		FGeneralPolygon2d GPoly = Complex.ConvertNestToGeneralPolygon(Nest);
		TArray<FIndex3i> Triangles = PlanarTriangulationFunc(GPoly);
		// create mapping from local triangulation problem to larger mesh vertex indices
		TArray<int> VertexMapping = (*VertexLoops)[Nest.OuterIndex]; // init to copy of outer loop vertices
		for (int HoleIdx : Nest.HoleIndices) // append hole loop indices
		{
			VertexMapping.Append((*VertexLoops)[HoleIdx]);
		}
		for (const FIndex3i& SourceTri : Triangles)
		{
			FIndex3i OutTri(VertexMapping[SourceTri.A], VertexMapping[SourceTri.B], VertexMapping[SourceTri.C]);
			int NewTID = Mesh->AppendTriangle(OutTri, GroupID);
			if (NewTID < 0) // e.g. if triangle would create non-manifold geometry in mesh, it will be skipped
			{
				bAddedAll = false;
			}
			else
			{
				NewTriangles.Add(NewTID);
			}
		}
	}

	if (bAutoFillPrimaryColors)
	{
		HoleFillUtil::FillColorOverlay(*Mesh, *VertexLoops, NewTriangles);
	}
	
	return bAddedAll;
}
