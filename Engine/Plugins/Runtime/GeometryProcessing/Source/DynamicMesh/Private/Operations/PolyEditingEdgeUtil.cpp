// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/PolyEditingEdgeUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Distance/DistLine3Line3.h"


using namespace UE::Geometry;


void UE::Geometry::ComputeInsetLineSegmentsFromEdges(
	const FDynamicMesh3& Mesh,
	const TArray<int32>& EdgeList,
	double InsetDistance,
	TArray<FLine3d>& InsetLinesOut)
{
	int32 NumEdges = EdgeList.Num();
	InsetLinesOut.SetNum(NumEdges);
	for (int32 k = 0; k < NumEdges; ++k)
	{
		if (Mesh.IsEdge(EdgeList[k]))
		{
			const FDynamicMesh3::FEdge EdgeVT = Mesh.GetEdge(EdgeList[k]);
			FVector3d A = Mesh.GetVertex(EdgeVT.Vert[0]);
			FVector3d B = Mesh.GetVertex(EdgeVT.Vert[1]);
			FVector3d EdgeDir = Normalized(A - B);
			FVector3d Midpoint = (A + B) * 0.5;
			int32 EdgeTri = EdgeVT.Tri[0];
			FVector3d Normal, Centroid; double Area;
			Mesh.GetTriInfo(EdgeTri, Normal, Area, Centroid);

			FVector3d InsetDir = Normal.Cross(EdgeDir);
			if ((Centroid - Midpoint).Dot(InsetDir) < 0)
			{
				InsetDir = -InsetDir;
			}

			InsetLinesOut[k] = FLine3d(Midpoint + InsetDistance * InsetDir, EdgeDir);
		}
		else
		{
			// This may produce nonsense in the calling code...but unclear what else to do here.
			// Have observed this case coming from Bevel but so far have not determined the source.
			InsetLinesOut[k] = FLine3d();		
		}
	}
}



FVector3d UE::Geometry::SolveInsetVertexPositionFromLinePair(
	const FVector3d& Position,
	const FLine3d& InsetEdgeLine1,
	const FLine3d& InsetEdgeLine2)
{
	FVector3d NewPos = Position;
	if (FMathd::Abs(InsetEdgeLine1.Direction.Dot(InsetEdgeLine2.Direction)) > 0.999)
	{
		// in this case lines are parallel so intersection point is not useful, just use nearest point on either line
		NewPos = InsetEdgeLine1.NearestPoint(Position);
	}
	else
	{
		// inset point to intersection point of the two lines
		FDistLine3Line3d Distance(InsetEdgeLine1, InsetEdgeLine2);
		double DistSqr = Distance.GetSquared();
		NewPos = 0.5 * (Distance.Line1ClosestPoint + Distance.Line2ClosestPoint);
	}
	return NewPos;
}


void UE::Geometry::SolveInsetVertexPositionsFromInsetLines(
	const FDynamicMesh3& Mesh,
	const TArray<FLine3d>& InsetEdgeLines,
	const TArray<int32>& VertexIDs,
	TArray<FVector3d>& VertexPositionsOut,
	bool bIsLoop)
{
	int32 NumVertices = VertexIDs.Num();
	VertexPositionsOut.SetNum(NumVertices);

	int32 StartIndex = 0, EndIndex= NumVertices;

	// if it's an open vertex span, we don't have two lines to intersect at the start/end, so 
	// just use the nearest points there (possibly something higher up will make a better decision)
	if (bIsLoop == false)
	{
		StartIndex = 1;
		EndIndex = NumVertices - 1;

		FVector3d V0 = Mesh.GetVertex(VertexIDs[0]);
		VertexPositionsOut[0] = InsetEdgeLines[0].NearestPoint(V0);				// should just be start point...
		FVector3d VN = Mesh.GetVertex(VertexIDs.Last());
		VertexPositionsOut.Last() = InsetEdgeLines.Last().NearestPoint(VN);		// should just be end point...
	}

	for (int32 vi = StartIndex; vi < EndIndex; ++vi)
	{
		const FLine3d& PrevLine = (vi == 0) ? InsetEdgeLines.Last() : InsetEdgeLines[vi-1];
		const FLine3d& NextLine = InsetEdgeLines[vi];
		FVector3d CurPos = Mesh.GetVertex(VertexIDs[vi]);
		VertexPositionsOut[vi] = SolveInsetVertexPositionFromLinePair(CurPos, PrevLine, NextLine);
	}
}


void UE::Geometry::ComputeNewGroupIDsAlongEdgeLoop(
	FDynamicMesh3& Mesh,
	const TArray<int32>& LoopEdgeIDs,
	TArray<int32>& NewLoopEdgeGroupIDs,
	TArray<int32>& NewGroupIDsOut,
	TFunctionRef<bool(int32 Eid1, int32 Eid2)> EdgesShouldHaveSameGroupFunc)
{
	int32 NumEdgeIDs = LoopEdgeIDs.Num();
	NewLoopEdgeGroupIDs.SetNumUninitialized(NumEdgeIDs);

	if (!ensure(NumEdgeIDs > 2))
	{
		if (NumEdgeIDs > 0)
		{
			int32 OneGroupID = Mesh.AllocateTriangleGroup();
			NewLoopEdgeGroupIDs.Init(OneGroupID, NumEdgeIDs);
			NewGroupIDsOut.Add(OneGroupID);
		}
		return;
	}

	NewLoopEdgeGroupIDs[0] = Mesh.AllocateTriangleGroup();
	NewGroupIDsOut.Add(NewLoopEdgeGroupIDs[0]);

	// Propagate the group backwards first so we don't allocate an unnecessary group
	// at the end and then have to fix it.
	int32 LastDifferentGroupIndex = NumEdgeIDs - 1;
	while (LastDifferentGroupIndex > 0
		&& EdgesShouldHaveSameGroupFunc(LoopEdgeIDs[0], LoopEdgeIDs[LastDifferentGroupIndex]) )
	{
		NewLoopEdgeGroupIDs[LastDifferentGroupIndex] = NewLoopEdgeGroupIDs[0];
		--LastDifferentGroupIndex;
	}

	// Now add new groups forward
	for (int32 j = 1; j <= LastDifferentGroupIndex; ++j)
	{
		if ( ! EdgesShouldHaveSameGroupFunc(LoopEdgeIDs[j], LoopEdgeIDs[j-1]) )
		{
			NewLoopEdgeGroupIDs[j] = Mesh.AllocateTriangleGroup();
			NewGroupIDsOut.Add(NewLoopEdgeGroupIDs[j]);
		}
		else
		{
			NewLoopEdgeGroupIDs[j] = NewLoopEdgeGroupIDs[j-1];
		}
	}
}
