// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/PolyEditingUVUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Parameterization/DynamicMeshUVEditor.h"


using namespace UE::Geometry;


void UE::Geometry::ComputeArbitraryTrianglePatchUVs(
	FDynamicMesh3& Mesh, 
	FDynamicMeshUVOverlay& UVOverlay,
	const TArray<int32>& TriangleSet)
{
	TArray<int32> NbrTriSet;
	double NbrUVAreaSum = 0.0;
	double Nbr3DAreaSum = 0.0;
	for (int32 tid : TriangleSet)
	{
		FIndex3i NbrTris = Mesh.GetTriNeighbourTris(tid);
		for (int32 j = 0; j < 3; ++j)
		{
			if (NbrTris[j] != IndexConstants::InvalidID && NbrTriSet.Contains(NbrTris[j]) == false)
			{
				NbrTriSet.Add(NbrTris[j]);
				if (UVOverlay.IsSetTriangle(NbrTris[j]))
				{
					FVector3d A, B, C;
					Mesh.GetTriVertices(NbrTris[j], A, B, C);
					Nbr3DAreaSum += VectorUtil::Area(A,B,C);
					FVector2f U, V, W;
					UVOverlay.GetTriElements(NbrTris[j], U, V, W);
					NbrUVAreaSum += (double)VectorUtil::Area(U, V, W);
				}
			}
		}
	}

	double UseUVScale = FMathd::Max(FMathd::Sqrt(NbrUVAreaSum),0.0001) / FMathd::Max(FMathd::Sqrt(Nbr3DAreaSum),0.0001);

	FDynamicMeshUVEditor UVEditor(&Mesh, &UVOverlay);
	FUVEditResult UVEditResult;
	UVEditor.SetTriangleUVsFromExpMap(TriangleSet, FDynamicMeshUVEditor::FExpMapOptions(), &UVEditResult);

	UVEditor.TransformUVElements(UVEditResult.NewUVElements, [&](const FVector2f& UV)
	{
		return (float)UseUVScale * UV;
	});
}




double UE::Geometry::ComputeAverageUVScaleRatioAlongVertexPath(
	const FDynamicMesh3& Mesh,
	const FDynamicMeshUVOverlay& UVOverlay,
	const TArray<int32>& VertexPath,
	double* PathLengthOut,
	double* UVPathLengthOut )
{
	double MeshLength = 0;
	double UVLength = 0;
	int32 N = VertexPath.Num();
	for ( int32 k = 0; k < N-1; ++k )
	{
		int32 EdgeID = Mesh.FindEdge(VertexPath[k], VertexPath[k+1]);
		if (EdgeID != IndexConstants::InvalidID)
		{
			double EdgeLength = Distance(Mesh.GetVertex(VertexPath[k]), Mesh.GetVertex(VertexPath[k+1]));
			FIndex2i EdgeTris = Mesh.GetEdgeT(EdgeID);

			double EdgeUVLength = 0;
			int EdgeUVLengthCount = 0;
			for ( int32 j = 0; j < 2; ++j )
			{
				if ( EdgeTris[j] != IndexConstants::InvalidID && UVOverlay.IsSetTriangle(EdgeTris[j]))
				{
					int32 EdgeIdx = Mesh.GetTriEdges(EdgeTris[j]).IndexOf(EdgeID);
					if ( EdgeIdx != IndexConstants::InvalidID )
					{
						FVector2f UVs[3];
						UVOverlay.GetTriElements(EdgeTris[j], UVs[0], UVs[1], UVs[2]);
						EdgeUVLength += (double)Distance( UVs[EdgeIdx], UVs[(EdgeIdx+1)%3] );
						EdgeUVLengthCount++;
					}
				}
			}
			if ( EdgeUVLengthCount > 0 )
			{
				EdgeUVLength /= (double)EdgeUVLengthCount;
				MeshLength += EdgeLength;
				UVLength += EdgeUVLength;
			}
		}
	}
	if ( PathLengthOut )
	{
		*PathLengthOut = MeshLength;
	}
	if ( UVPathLengthOut )
	{
		*UVPathLengthOut = UVLength;
	}
	return ( MeshLength > FMathf::ZeroTolerance ) ? (UVLength / MeshLength) : 1.0;
}