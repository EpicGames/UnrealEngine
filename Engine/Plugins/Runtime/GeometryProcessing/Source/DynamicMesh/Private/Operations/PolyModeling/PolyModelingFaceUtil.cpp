// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/PolyModeling/PolyModelingFaceUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Selections/MeshConnectedComponents.h"


using namespace UE::Geometry;

UE::Geometry::FFrame3d UE::Geometry::ComputeFaceSelectionFrame(
	const FDynamicMesh3& Mesh, 
	const TArray<int32>& Triangles,
	bool bIsDefinitelySingleComponent)
{
	if (Triangles.Num() == 0)
	{
		return FFrame3d();
	}

	// special case for 1 triangle
	if (Triangles.Num() == 1)
	{
		FVector3d TriNormal, TriCentroid; double TriArea;
		Mesh.GetTriInfo(Triangles[0], TriNormal, TriArea, TriCentroid);
		return FFrame3d(TriCentroid, TriNormal);
	}

	// If there is more than one component we want to put the frame on the largest component
	const TArray<int32>* UseTriangles = &Triangles;
	FMeshConnectedComponents Components(&Mesh);
	if (bIsDefinitelySingleComponent == false)
	{
		Components.FindConnectedTriangles(Triangles);
		UseTriangles = &Components.GetComponent(Components.GetLargestIndexByCount()).Indices;		// area would be better here?
	}

	// special case for 1 triangle again
	if (UseTriangles->Num() == 1)
	{
		FVector3d TriNormal, TriCentroid; double TriArea;
		Mesh.GetTriInfo(Triangles[0], TriNormal, TriArea, TriCentroid);
		return FFrame3d(TriCentroid, TriNormal);
	}

	// compute area-weighted average triangle normal and centroid
	FVector3d LargestNormal = FVector3d::UnitZ();
	double LargestArea = 0;
	FVector3d AverageNormal = FVector3d::Zero();
	FVector3d Centroid = FVector3d::Zero();
	double WeightSum = 0;
	for ( int32 tid : *UseTriangles )
	{
		FVector3d TriNormal, TriCentroid; double TriArea;
		Mesh.GetTriInfo(tid, TriNormal, TriArea, TriCentroid);
		Centroid += TriArea * TriCentroid;
		AverageNormal += TriArea * TriNormal;
		WeightSum += TriArea;
		if (TriArea > LargestArea)
		{
			LargestArea = TriArea;
			LargestNormal = TriNormal;
		}
	}
	Centroid /= (WeightSum > FMathd::ZeroTolerance) ? WeightSum : 1.0;
	Normalize(AverageNormal);
	FVector3d UseNormal = ( AverageNormal.SquaredLength() < 0.98 ) ? LargestNormal : AverageNormal;

	// if selection is planar (very common case) then centroid is on mesh and we can stop searching
	FInterval1d ProjectionRange = FInterval1d::Empty();
	for ( int32 tid : *UseTriangles )
	{
		FVector3d A,B,C;
		Mesh.GetTriVertices(tid, A, B, C);
		ProjectionRange.Contain( (A-Centroid).Dot(UseNormal) );
		ProjectionRange.Contain( (B-Centroid).Dot(UseNormal) );
		ProjectionRange.Contain( (C-Centroid).Dot(UseNormal) );
		if (ProjectionRange.Length() > FMathf::ZeroTolerance)
		{
			break;
		}
	}
	if (ProjectionRange.Length() < FMathf::ZeroTolerance)
	{
		return FFrame3d(Centroid, UseNormal);
	}

	// Try to convert the centroid to a point on the selection surface via raycats.
	// probably would be good to have some heuristics here, ie if < 100 triangles just
	// do a linear iteration instead of building a BVTree...
	// (could also only build BVTree for the subset of triangles??)
	FDynamicMeshAABBTree3 BVTree(&Mesh, true);
	TArray<FVector3d, TInlineAllocator<3>> HitPoints;
	// cast rays forward and backward
	const double Signs[2] = {1.0, -1.0};
	for ( int32 j = 0; j < 2; ++j )
	{
		double RayParam; int32 HitTriangleID;
		FRay3d Ray(Centroid, Signs[j]*UseNormal);
		if (BVTree.FindNearestHitTriangle(Ray, RayParam, HitTriangleID))
		{
			if ( UseTriangles->Contains(HitTriangleID) )
			{
				HitPoints.Add(Ray.PointAt(RayParam));
			}
		}
	}
	// this actually seems a bit like a bad idea...if both rays miss we probably prefer the centroid
	//HitPoints.Add(BVTree.FindNearestPoint(Centroid));

	// find the hit point that is closest. Could also consider alignment w/ average normal?
	double Nearest = TNumericLimits<double>::Max();
	FVector3d UsePoint = Centroid;
	for ( FVector3d Pos : HitPoints )
	{
		double Dist = Distance(Centroid, Pos);
		if ( Dist < Nearest )
		{
			Nearest = Dist;
			UsePoint = Pos;
		}
	}

	return FFrame3d(UsePoint, UseNormal);
}