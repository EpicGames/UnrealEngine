// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/PolyModeling/PolyModelingMaterialUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;


bool UE::Geometry::ComputeMaterialIDRange(
	const FDynamicMesh3& Mesh,
	FInterval1i& MaterialIDRange)
{
	MaterialIDRange = FInterval1i(0,0);
	const FDynamicMeshMaterialAttribute* MaterialIDAttrib = (Mesh.HasAttributes() && Mesh.Attributes()->HasMaterialID()) ? 
			Mesh.Attributes()->GetMaterialID() : nullptr;
	if (MaterialIDAttrib)
	{
		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			MaterialIDRange.Contain(MaterialIDAttrib->GetValue(tid));
		}
		return true;
	}
	return false;
}



bool UE::Geometry::ComputeMaterialIDsForVertexPath(
	const FDynamicMesh3& Mesh,
	const TArray<int32>& VertexPath,
	bool bIsLoop,
	TArray<int32>& EdgeMaterialIDs,
	int32 FallbackMaterialID)
{
	int32 NumV = VertexPath.Num();
	EdgeMaterialIDs.Init(FallbackMaterialID, bIsLoop ? NumV : NumV-1);

	const FDynamicMeshMaterialAttribute* MaterialIDAttrib = (Mesh.HasAttributes() && Mesh.Attributes()->HasMaterialID()) ? 
			Mesh.Attributes()->GetMaterialID() : nullptr;
	if (MaterialIDAttrib == nullptr)
	{
		return false;
	}

	int32 NumEdges = EdgeMaterialIDs.Num();
	for (int32 k = 0; k < NumEdges; ++k)
	{
		int32 EdgeID = Mesh.FindEdge(VertexPath[k], VertexPath[(k+1)%NumV]);
		if (EdgeID != IndexConstants::InvalidID)
		{
			EdgeMaterialIDs[k] = MaterialIDAttrib->GetValue( Mesh.GetEdgeT(EdgeID).A );
		}
	}

	return true;
}