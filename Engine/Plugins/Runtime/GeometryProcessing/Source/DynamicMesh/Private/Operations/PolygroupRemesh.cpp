// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/PolygroupRemesh.h"

#include "Algo/ForEach.h"

#include "DynamicMeshEditor.h"
#include "Operations/PlanarHoleFiller.h"
#include "MeshRegionBoundaryLoops.h"

using namespace UE::Geometry;


bool FPolygroupRemesh::Compute()
{
	bool bSuccess = true;

	FMeshNormals OriginalTriNormals(Mesh);
	OriginalTriNormals.ComputeTriangleNormals();

	// This tracks the vertices we *won't* skip over when retriangulating (topological or geometric corners)
	TArray<bool> KeptVertices; KeptVertices.Init(false, Mesh->MaxVertexID());

	// Keep all group corner vertices
	for (const FGroupTopology::FCorner& Corner : Topology->Corners)
	{
		KeptVertices[Corner.VertexID] = true;
	}

	// Helper to keep us from adding edges that already exist or that we already added
	// (to avoid attempts to create non-manifold edges)
	TSet<FIndex2i> NewEdges;
	auto AddNewEdge = [this, &NewEdges](int32 VIDA, int32 VIDB) -> bool
	{
		int32 EID = Mesh->FindEdge(VIDA, VIDB);
		if (EID != FDynamicMesh3::InvalidID)
		{
			return false;
		}
		FIndex2i OrderedEdge(VIDA, VIDB);
		if (VIDA > VIDB)
		{
			OrderedEdge.Swap();
		}
		bool bIsAlreadyInSet = false;
		NewEdges.Add(OrderedEdge, &bIsAlreadyInSet);
		return !bIsAlreadyInSet;
	};

	// Keep all vertices at non-straight edges
	double DotTolerance = FMathd::Cos(SimplificationAngleTolerance * FMathd::DegToRad);
	for (const FGroupTopology::FGroupEdge& GroupEdge : Topology->Edges)
	{
		const TArray<int32>& SpanV = GroupEdge.Span.Vertices;
		auto KeepRangeInclusive = [&KeptVertices, &SpanV](int32 Start, int32 End) -> int32
		{
			if (Start > End)
			{
				for (int32 Idx = Start; Idx < SpanV.Num(); Idx++)
				{
					KeptVertices[SpanV[Idx]] = true;
				}
				for (int32 Idx = 0; Idx <= End; Idx++)
				{
					KeptVertices[SpanV[Idx]] = true;
				}
				return (SpanV.Num() - Start) + End + 1;
			}
			else
			{
				for (int32 Idx = Start; Idx <= End; Idx++)
				{
					KeptVertices[SpanV[Idx]] = true;
				}
				return End + 1 - Start;
			}
		};
		if (SpanV.Num() < 3)
		{
			KeepRangeInclusive(0, SpanV.Num() - 1);
			continue;
		}
		FVector3d Center = Mesh->GetVertex(SpanV[1]);
		FVector3d FirstEdge = Center - Mesh->GetVertex(SpanV[0]);
		FirstEdge.Normalize();
		// if the first vertex wasn't a corner, and the span is a loop, then we extra loop-specific considerations
		bool bSpanIsLoop = SpanV[0] == SpanV.Last();
		bool bSpanCouldSkipV0 = !KeptVertices[SpanV[0]] && bSpanIsLoop;
		if (bSpanCouldSkipV0)
		{
			FVector3d LastEdge = Mesh->GetVertex(SpanV.Last()) - Mesh->GetVertex(SpanV.Last(1));
			LastEdge.Normalize();
			if (FirstEdge.Dot(LastEdge) < DotTolerance)
			{
				KeptVertices[SpanV[0]] = true;
				bSpanCouldSkipV0 = false;
			}
		}
		FVector3d PrevEdge = FirstEdge;
		FVector3d LastKeptEdge = FirstEdge;
		int32 FirstKeptIdx = KeptVertices[SpanV[0]] ? 0 : -1;
		int32 LastKeptIdx = FirstKeptIdx;
		int32 NumKept = int32(KeptVertices[SpanV[0]]) + int32(KeptVertices[SpanV.Last()]);
		for (int32 SpanIdx = 1; SpanIdx + 1 < SpanV.Num(); SpanIdx++)
		{
			FVector3d Next = Mesh->GetVertex(SpanV[SpanIdx + 1]);
			FVector3d NextEdge = Next - Center;
			NextEdge.Normalize();
			int32 SpanVID = SpanV[SpanIdx];
			// track deviation from (1) the most recent edge and (2) the last edge from a vertex we kept, so gentle slow turns across many edges are also captured
			if (KeptVertices[SpanVID] || NextEdge.Dot(PrevEdge) < DotTolerance || NextEdge.Dot(LastKeptEdge) < DotTolerance)
			{
				NumKept++;
				KeptVertices[SpanVID] = true;
				if (LastKeptIdx != -1 && SpanIdx > LastKeptIdx + 1)
				{
					if (!AddNewEdge(SpanV[LastKeptIdx], SpanV[SpanIdx]))
					{
						// Don't skip vertices if doing so would create a new edge that matches an existing (or already added) edge
						// This is to avoid creating non-manifold edges or having the retriangulation 'close' open mesh boundaries
						// TODO: Consider if we could still try to skip *some* of the vertices in these cases
						NumKept += KeepRangeInclusive(LastKeptIdx + 1, SpanIdx - 1);
					}
				}

				LastKeptEdge = NextEdge;
				LastKeptIdx = SpanIdx;
				if (FirstKeptIdx == -1)
				{
					FirstKeptIdx = SpanIdx;
				}
			}
			PrevEdge = NextEdge;
			Center = Next;
		}

		// if we kept too few vertices, just keep them all
		if (NumKept < (bSpanIsLoop ? 3 : 2))
		{
			KeepRangeInclusive(0, SpanV.Num() - 1);
		}
		else
		{
			int32 LastSegEndpt = bSpanCouldSkipV0 ? FirstKeptIdx : SpanV.Num() - 1;
			if (bSpanCouldSkipV0 || LastSegEndpt - LastKeptIdx > 1)
			{
				if (!AddNewEdge(SpanV[LastKeptIdx], SpanV[LastSegEndpt]))
				{
					NumKept += KeepRangeInclusive(LastKeptIdx + 1, LastSegEndpt - 1);
				}
			}
		}
	}

	// Per group, re-triangulate the group and replace the old triangles with the new
	for (const FGroupTopology::FGroup& Group : Topology->Groups)
	{
		TArray<TArray<int32>> VertexLoops;
		VertexLoops.Reserve(Group.Boundaries.Num());
		for (const FGroupTopology::FGroupBoundary& Boundary : Group.Boundaries)
		{
			TArray<int32>& Loop = VertexLoops.Emplace_GetRef();
			for (int32 EdgeIdx = Boundary.GroupEdges.Num() - 1; EdgeIdx >= 0; EdgeIdx--) // traverse backwards to get correct winding
			{
				int32 GroupEdgeID = Boundary.GroupEdges[EdgeIdx];
				const FGroupTopology::FGroupEdge& GroupEdge = Topology->Edges[GroupEdgeID];
				if (GroupEdge.Span.Edges.Num() == 0)
				{
					checkSlow(false);
					continue; // can't happen?
				}
				// Choose which direction to traverse the group edge span, based on the first attached triangle
				FDynamicMesh3::FEdge MeshEdge = Mesh->GetEdge(GroupEdge.Span.Edges[0]);
				int32 TriID = Mesh->GetTriangleGroup(MeshEdge.Tri[0]) == Group.GroupID ? MeshEdge.Tri[0] : MeshEdge.Tri[1];
				checkSlow(TriID >= 0);
				FIndex3i Tri = Mesh->GetTriangle(TriID);
				int32 FirstVSubIdx = Tri.IndexOf(GroupEdge.Span.Vertices[0]);
				int32 NextSubIdx = (FirstVSubIdx + 1) % 3;
				// Add the span of the group edge to our loop
				// (skip the last vertex b/c we'll get from the next group edge in the boundary loop, or if the span is a loop itself then it's a duplicate vertex)
				if (Tri[NextSubIdx] != GroupEdge.Span.Vertices[1])
				{
					for (int32 Idx = 0, End = GroupEdge.Span.Vertices.Num(); Idx + 1 < End; Idx++)
					{
						int32 VID = GroupEdge.Span.Vertices[Idx];
						if (KeptVertices[VID])
						{
							Loop.Add(VID);
						}
					}
				}
				else
				{
					for (int32 Idx = GroupEdge.Span.Vertices.Num() - 1; Idx > 0; Idx--)
					{
						int32 VID = GroupEdge.Span.Vertices[Idx];
						if (KeptVertices[VID])
						{
							Loop.Add(VID);
						}
					}
				}
			}
		}

		if (VertexLoops.IsEmpty())
		{
			continue;
		}

		// Per UV layer, keep maps from vertex ID -> (ElementID, UV coordinate) so we can reconstruct UVs on the simplified mesh
		TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>> VidUVMaps;
		using ElIDAndUV = FMeshRegionBoundaryLoops::ElementIDAndValue<FVector2f>;
		// (Note the above map doesn't support internal UV seams; if those exist we will arbitrarily pick just one UV element per vertex)
		if (Mesh->HasAttributes())
		{
			int32 NumUVLayers = Mesh->Attributes()->NumUVLayers();
			VidUVMaps.SetNum(NumUVLayers);
			for (int32 TID : Group.Triangles)
			{
				for (int32 LayerIdx = 0; LayerIdx < NumUVLayers; LayerIdx++)
				{
					FDynamicMeshUVOverlay* Layer = Mesh->Attributes()->GetUVLayer(LayerIdx);
					FIndex3i ElTri;
					if (!Layer->GetTriangleIfValid(TID, ElTri))
					{
						continue;
					}

					for (int32 SubIdx = 0; SubIdx < 3; SubIdx++)
					{
						int32 ParentVID = Layer->GetParentVertex(ElTri[SubIdx]);
						int32 EID = ElTri[SubIdx];
						VidUVMaps[LayerIdx].Add(ParentVID, ElIDAndUV(EID, Layer->GetElement(EID)));
					}
				}
			}
		}

		// Find projection plane for 2D retriangulation of polygroup
		// TODO: consider splitting the group into multiple triangulation regions if normals are too divergent
		FVector3d Normal(0, 0, 0);
		for (int TID : Group.Triangles)
		{
			Normal += OriginalTriNormals[TID];
		}
		Normal.Normalize();
		FVector3d Origin = Mesh->GetVertex(VertexLoops[0][0]);

		// Delete the old triangles
		FDynamicMeshEditor RmTrisEditor(Mesh);
		RmTrisEditor.RemoveTriangles(Group.Triangles, false);

		// Note this hole filler will not report Delaunay triangulation failures (which could arise e.g. for polygons overlapping in the projection plane)
		// TODO: Re-implement this CDT-insertion in a way that lets us recognize failures and have a fall back approach?
		FPlanarHoleFiller Filler(Mesh, &VertexLoops, PlanarTriangulationFunc, Origin, Normal);
		bool bFillSuccess = Filler.Fill(Group.GroupID);
		if (!bFillSuccess)
		{
			bSuccess = false;
		}

		if (Mesh->HasAttributes())
		{
			int32 NumUVLayers = Mesh->Attributes()->NumUVLayers();

			// Set stale UV layer element IDs to InvalidID before adding any new element IDs
			for (int32 LayerIdx = 0; LayerIdx < NumUVLayers; LayerIdx++)
			{
				FDynamicMeshUVOverlay* Overlay = Mesh->Attributes()->GetUVLayer(LayerIdx);
				// TODO: this is from FMeshRegionBoundaryLoops::UpdateLoopOverlayMapValidity; could be pulled out to a more general place
				Algo::ForEachIf(VidUVMaps[LayerIdx],
					[&Overlay](const TPair<int32, ElIDAndUV>& Entry) { return !Overlay->IsElement(Entry.Value.Key); },
					[](TPair<int32, ElIDAndUV>& Entry) { Entry.Value.Key = IndexConstants::InvalidID; });
			}

			// Set UV layer element IDs on the new triangles
			for (int32 TID : Filler.NewTriangles)
			{
				for (int32 LayerIdx = 0; LayerIdx < NumUVLayers; LayerIdx++)
				{
					FDynamicMeshUVOverlay* Layer = Mesh->Attributes()->GetUVLayer(LayerIdx);
					FIndex3i MeshTri = Mesh->GetTriangle(TID);
					FIndex3i NewElTri;
					for (int32 SubIdx = 0; SubIdx < 3; SubIdx++)
					{
						ElIDAndUV* FindElID = VidUVMaps[LayerIdx].Find(MeshTri[SubIdx]);
						if (FindElID)
						{
							int32 ElID = FindElID->Key;
							if (ElID == IndexConstants::InvalidID)
							{
								ElID = Layer->AppendElement(FindElID->Value);
							}
							NewElTri[SubIdx] = ElID;
						}
						else
						{
							NewElTri = FIndex3i::Invalid();
							break;
						}
					}
					Layer->SetTriangle(TID, NewElTri, false);
				}
			}
		}
	}

	FDynamicMeshEditor CleanupEditor(Mesh);
	CleanupEditor.RemoveIsolatedVertices();
	Mesh->CompactInPlace(); // Compact away all of the previous mesh triangles and any now-removed vertices

	// Recompute overlay normals per polygroup
	if (Mesh->HasAttributes())
	{
		Mesh->Attributes()->PrimaryNormals()->CreateFromPredicate([this](int VID, int TA, int TB)
			{
				return Mesh->GetTriangleGroup(TA) == Mesh->GetTriangleGroup(TB);
			}, 0);
		FMeshNormals MeshNormals(Mesh);
		MeshNormals.RecomputeOverlayNormals(Mesh->Attributes()->PrimaryNormals());
		MeshNormals.CopyToOverlay(Mesh->Attributes()->PrimaryNormals(), false);
	}

	return bSuccess;
}


