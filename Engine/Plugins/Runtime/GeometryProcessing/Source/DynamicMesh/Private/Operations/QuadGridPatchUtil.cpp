// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/QuadGridPatchUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Operations/PolyEditingUVUtil.h"


using namespace UE::Geometry;



void UE::Geometry::ComputeNormalsForQuadPatch(
	FDynamicMesh3& Mesh,
	const FQuadGridPatch& QuadPatch )
{
	TArray<int32> PatchTriangles;
	QuadPatch.GetAllTriangles(PatchTriangles);
	FDynamicMeshEditor Editor(&Mesh);
	Editor.SetTriangleNormals(PatchTriangles);
}




bool UE::Geometry::ComputeUVIslandForQuadPatch(
	FDynamicMesh3& Mesh,
	const FQuadGridPatch& QuadPatch,
	double UVScaleFactor,
	int UVOverlayIndex )
{
	FDynamicMeshUVEditor UVEditor(&Mesh, UVOverlayIndex, false);
	FDynamicMeshUVOverlay* UVOverlay = UVEditor.GetOverlay();
	if (!UVOverlay)
	{
		return false;
	}

	TArray<int32> AllTriangles;
	QuadPatch.GetAllTriangles(AllTriangles);
	UVEditor.ResetUVs(AllTriangles);

	// compute UV scale factor that tries to keep UV dimensions roughly consistent with connected area
	double UVLengthScale = 0, UVLengthWeight = 0;
	for (int32 k = 0; k < 2; ++k)
	{
		const TArray<int32>& VertexSpan = (k == 0) ? QuadPatch.VertexSpans[0] : QuadPatch.VertexSpans.Last();
		double PathLength = 0;
		double UVLengthScaleTmp = UE::Geometry::ComputeAverageUVScaleRatioAlongVertexPath(Mesh, *UVOverlay, VertexSpan, &PathLength);
		if (PathLength > 0)
		{
			UVLengthScale += UVLengthScaleTmp;
			UVLengthWeight += 1.0;
		}
	}
	UVLengthScale = (UVLengthWeight == 0 || UVLengthScale == 0) ? 1.0 : (UVLengthScale / UVLengthWeight);

	TArray<TArray<int32>> UVElementSpans;
	UVElementSpans.SetNum( QuadPatch.NumVertexRowsV );
	const TArray<int32>& StartVertexSpan = QuadPatch.VertexSpans[0];
	const TArray<int32>& EndVertexSpan = QuadPatch.VertexSpans.Last();
	TArray<int32>& StartUVSpan = UVElementSpans[0];
	TArray<int32>& EndUVSpan = UVElementSpans.Last();

	// generate new UV elements along the two vertex spans
	// note even in the case of a closed loop, we have a span here, so we will get an 'unwrapped' cylinder etc
	double AccumDistU = 0;
	int32 NumU = QuadPatch.NumVertexColsU;
	int32 NumV = QuadPatch.NumVertexRowsV;
	for ( int32 k = 0; k < NumU; ++k )
	{
		double DistV = Distance( Mesh.GetVertex(StartVertexSpan[k]), Mesh.GetVertex(EndVertexSpan[k]) );
		//double DistV = Distance( Mesh.GetVertex(StartVertexSpan[0]), Mesh.GetVertex(EndVertexSpan[0]) );
		float UseU = (float)(UVLengthScale * AccumDistU * UVScaleFactor);
		float EndV = (float)(UVLengthScale * DistV * UVScaleFactor);
		for ( int32 j = 0; j < NumV; ++j )
		{
			float t = (float)j / (float)(NumV-1);
			int32 NewElemID = UVOverlay->AppendElement( FVector2f(UseU, (float)(t * EndV)) );
			UVElementSpans[j].Add(NewElemID);
		}

		if ( k < NumU-1 )
		{
			AccumDistU += Distance( Mesh.GetVertex(StartVertexSpan[k]), Mesh.GetVertex(StartVertexSpan[k+1]) );
		}
	}

	QuadPatch.ForEachQuad( [&](int32 Row, int32 QuadIndex, FIndex2i QuadTris)
	{
		for (int32 TriIdx = 0; TriIdx < 2; ++TriIdx)
		{
			int32 TriangleID = QuadTris[TriIdx];
			FIndex3i IndicesTri = QuadPatch.GetQuadTriMappedToSpanIndices(Mesh, Row, QuadIndex, TriIdx);
			FIndex3i UVTri;
			for ( int32 j = 0; j < 3; ++j )
			{
				FIndex2i VertexSpanIndex = QuadPatch.DecodeRowColumnIndex(IndicesTri[j]);
				UVTri[j] = UVElementSpans[VertexSpanIndex.A][VertexSpanIndex.B];
			}
			UVOverlay->SetTriangle(TriangleID, UVTri);
		}
	});

	return true;
}
