// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/RedGreenTessellation.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "CompGeom/PolygonTriangulation.h"

namespace UE
{
namespace Geometry
{
namespace RedGreenTessellationPatternHelpers
{
	/*
	 * Given a triangle and additional vertices for each triangle edge, produce a valid triangulation
	 */
	static void TriangulatePoly(const FIndex3i& Triangle, const TStaticArray<TArray<int32>, 3>& EdgePoints, TArray<FIndex3i>& Tris)
	{
		int32 NumSplitEdges = 0;
		int32 MaxEdgeTessellationLevel = 0;
		for (int32 EdgeIndex = 0; EdgeIndex < 3; ++EdgeIndex)
		{
			if (EdgePoints[EdgeIndex].Num() > 0)
			{
				++NumSplitEdges;
				MaxEdgeTessellationLevel = FMath::Max(MaxEdgeTessellationLevel, EdgePoints[EdgeIndex].Num());
			}
		}

		if (NumSplitEdges == 0)
		{
			Tris.Add({ Triangle[0], Triangle[1], Triangle[2] });
		}
		else if (NumSplitEdges == 1 && MaxEdgeTessellationLevel == 1)
		{
			// Green triangle
			for (int32 TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
			{
				if (EdgePoints[TriVertIndex].Num() > 0)
				{
					ensure(EdgePoints[TriVertIndex].Num() == 1);
					const int32 A = Triangle[TriVertIndex];
					const int32 B = Triangle[(TriVertIndex + 1) % 3];
					const int32 C = Triangle[(TriVertIndex + 2) % 3];
					const int32 M = EdgePoints[TriVertIndex][0];
					Tris.Add({ A, M, C });
					Tris.Add({ M, B, C });
					break;
				}
			}
		}
		else if (NumSplitEdges == 2 && MaxEdgeTessellationLevel == 1)
		{
			// "Double green" (?) triangle
			for (int32 TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
			{
				const int32 Next = (TriVertIndex + 1) % 3;
				if (EdgePoints[TriVertIndex].Num() == 1 && EdgePoints[Next].Num() == 1)
				{
					const int32 A = Triangle[TriVertIndex];
					const int32 B = Triangle[Next];
					const int32 C = Triangle[(TriVertIndex + 2) % 3];
					const int32 AB = EdgePoints[TriVertIndex][0];
					const int32 BC = EdgePoints[Next][0];
					Tris.Add({ A, AB, C });
					Tris.Add({ AB, BC, C });
					Tris.Add({ AB, B, BC });
					break;
				}
			}
		}
		else if (NumSplitEdges == 3 && MaxEdgeTessellationLevel == 1)
		{
			// Red triangle
			ensure(EdgePoints[0].Num() == 1 && EdgePoints[1].Num() == 1 && EdgePoints[2].Num() == 1);
			const int32 A = Triangle[0];
			const int32 B = Triangle[1];
			const int32 C = Triangle[2];
			const int32 AB = EdgePoints[0][0];
			const int32 BC = EdgePoints[1][0];
			const int32 CA = EdgePoints[2][0];
			Tris.Add({ A, AB, CA });
			Tris.Add({ AB, B, BC });
			Tris.Add({ CA, BC, C });
			Tris.Add({ AB, BC, CA });
		}
		else
		{
			// Too complicated -- go full poly triangulation mode
			TArray<int32> Polygon;
			TArray<FVector2d> PolygonVertices;

			const TStaticArray<FVector2d, 3> Corners = { FVector2d{0,0}, FVector2d{1,0}, FVector2d{0,1} };

			for (int32 TriEdgeIndex = 0; TriEdgeIndex < 3; ++TriEdgeIndex)
			{
				PolygonVertices.Add(Corners[TriEdgeIndex]);
				Polygon.Add(Triangle[TriEdgeIndex]);

				const TArray<int32>& Edge = EdgePoints[TriEdgeIndex];
				if (Edge.Num() > 0)
				{
					const double Dx = 1.0 / (Edge.Num() + 1);
					const FVector2d& ThisCorner = Corners[TriEdgeIndex];
					const FVector2d& NextCorner = Corners[(TriEdgeIndex + 1) % 3];

					for (int32 EdgeVertexIndex = 0; EdgeVertexIndex < Edge.Num(); ++EdgeVertexIndex)
					{
						Polygon.Add(Edge[EdgeVertexIndex]);
						const double Alpha = static_cast<double>(EdgeVertexIndex + 1) * Dx;
						PolygonVertices.Add(ThisCorner + Alpha * (NextCorner - ThisCorner));
					}
				}
			}

			const int32 ExpectedNumberOfTriangles = PolygonVertices.Num() - 2;

			// The input "polygon" is geometrically triangular by construction, so the resulting triangulation will have no foldovers or topological issues in 3D
			TArray<FIndex3i> LocalTris;
			constexpr bool bOrientAsHoleFill = false;
			PolygonTriangulation::TriangulateSimplePolygon(PolygonVertices, LocalTris, bOrientAsHoleFill);

			ensure(LocalTris.Num() == ExpectedNumberOfTriangles);

			for (const FIndex3i& LocalTri : LocalTris)
			{
				Tris.Add(FIndex3i{ Polygon[LocalTri[0]], Polygon[LocalTri[1]], Polygon[LocalTri[2]] });
			}
		}
	}

}	// namespace RedGreenTessellationPatternHelpers


//
// FRedGreenTessellationPattern
//

/**
A single iteration of Red-Green subdivision is shown here, where the center triangle is "red" subdivided (1 to 4), and the neighboring triangles are "green" subdivided (1 to 2) to
maintain a conforming mesh (i.e. avoid T-junctions or cracks in the mesh.)


                       O                                                O
                      / \                                              /|\
                     /   \                                            / | \
                    /     \                                          /  |  \
                   /       \                                        /   |   \
                  /         \                                      /    |    \
                 /           \                                    /     |     \
                /             \                                  /      |      \
               /               \                                /       |       \
              /                 \                              /        |        \
             O\-----------------/O                            O\--------O--------/O
            /  \               /  \                          /  \      / \      /  \
           /    \             /    \                        /    \    /   \    /    \
          /      \           /      \                      /      \  /     \  /      \
         /        \         /        \                    /        O/_______O         \
        /          \       /          \                  /        / \       /\         \
       /            \     /            \                /      /     \     /    \       \
      /              \   /              \              /    /         \   /        \     \
     /                \ /                \            /  /             \ /            \   \
    /                  \                  \          //                 \                \ \
   O-------------------O-------------------O    =>  O-------------------O--------------------O

To achieve deeper levels of tessellation, we could repeatedly apply red-green over the entire mesh, however it can be more efficient to tessellate
each triangle to its specified level in one shot.

Here is an example of our triangle patch tessellation. The triangle has a specified tessellation level 2, but the neighbouring triangle below it has level 3,
which necessitates additional triangles along the bottom edge.

    O                                                       O
    |\                                                      | \
    │  \                                                    │   \
    │     \                                                 │     \
    │       \                                               │       \
    O         O                                             O---------O
    │           \                                           │\        │ \
    │             \                                         │   \     │   \
    │               \                                       │      \  │     \
    │                 \                                     │        \│       \
    O                   O                                   O---------O---------O
    │                     \                                 │\        │\        │ \
    │                       \                               │   \     │   \     │   \
    │                         \                             │      \  │      \  │     \
    │                           \                           │        \│        \│       \
    O                             O                         O---------O---------O---------O
    │                               \                       │\\       │\\       │\\       │\\
    │                                 \                     │ \ \     │ \ \     │ \ \     │ \ \
    │                                   \                   │  \   \  │  \  \   │  \  \   │  \  \
    │                                     \                 │   \   \ │   \   \ │   \   \ │   \   \
    O----*----O----*----O----*----O----*----O        =>     O----*----O----*----O----*----O----*----O
*/



FRedGreenTessellationPattern::FRedGreenTessellationPattern(const FDynamicMesh3* InMesh, const TArray<int>& InTriangleTessLevels) :
	FTessellationPattern(InMesh),
	TriangleLevels(InTriangleTessLevels)
{
}

EOperationValidationResult FRedGreenTessellationPattern::Validate() const
{
	if (!Mesh)
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (TriangleLevels.Num() < Mesh->MaxTriangleID())
	{
		// Too few triangle levels specified
		return EOperationValidationResult::Failed_UnknownReason;
	}

	return EOperationValidationResult::Ok;
}

int FRedGreenTessellationPattern::GetNumberOfNewVerticesForEdgePatch(const int InEdgeID) const
{
	if (!Mesh->IsEdge(InEdgeID))
	{
		return FTessellationPattern::InvalidIndex;
	}
	const FIndex2i EdgeTris = Mesh->GetEdgeT(InEdgeID);

	int32 MaxLevel = 0;
	if (EdgeTris[0] != FDynamicMesh3::InvalidID)
	{
		MaxLevel = FMath::Max(MaxLevel, TriangleLevels[EdgeTris[0]]);
	}
	if (EdgeTris[1] != FDynamicMesh3::InvalidID)
	{
		MaxLevel = FMath::Max(MaxLevel, TriangleLevels[EdgeTris[1]]);
	}

	return (1 << MaxLevel) - 1;
}

int FRedGreenTessellationPattern::GetNumberOfNewVerticesForTrianglePatch(const int InTriangleID) const
{
	if (!Mesh->IsTriangle(InTriangleID))
	{
		return FTessellationPattern::InvalidIndex;
	}
	const int32 Level = TriangleLevels[InTriangleID];
	return ((1 << Level) - 1) * ((1 << Level) - 2) / 2;
}


int FRedGreenTessellationPattern::GetNumberOfPatchTriangles(const int InTriangleID) const
{
	if (!Mesh->IsTriangle(InTriangleID))
	{
		return FTessellationPattern::InvalidIndex;
	}

	const int32 Level = TriangleLevels[InTriangleID];

	int32 NewTriangles = 1 << (2 * Level);

	const FIndex3i NeighborTris = Mesh->GetTriNeighbourTris(InTriangleID);
	for (int32 NIndex = 0; NIndex < 3; ++NIndex)
	{
		const int NeighborTri = NeighborTris[NIndex];
		if (Mesh->IsTriangle(NeighborTri) && TriangleLevels[NeighborTri] > Level)
		{
			// We need to subdivide new segments along this edge with green triangles
			NewTriangles += (1 << TriangleLevels[NeighborTri]) - (1 << Level);
		}
	}

	return NewTriangles;
}

void FRedGreenTessellationPattern::TessellateEdgePatch(EdgePatch& EdgePatch) const
{
	const int32 NumNewVertices = GetNumberOfNewVerticesForEdgePatch(EdgePatch.EdgeID);
	const float Param = 1.0f / (static_cast<float>(NumNewVertices) + 1.0f);
	for (int NewVertex = 0; NewVertex < NumNewVertices; ++NewVertex)
	{
		EdgePatch.LinearCoord[NewVertex] = static_cast<float>(NewVertex + 1) * Param;
	}
}

void FRedGreenTessellationPattern::TessellateTriPatch(TrianglePatch& TriPatch) const
{
	if (TriPatch.Triangles.Num() == 1)
	{
		TriPatch.Triangles[0] = TriPatch.UVWCorners;
		return;
	}
	
	const int32 Level = TriangleLevels[TriPatch.TriangleID];

	// Create a grid of vertices which will then be triangulated. The grid structure stores vertex indices
	// (Note the grid itself is triangular, only storing values at nodes where i + j < N)

	const int32 GridN = (1 << Level) + 1;

	// TODO: If using a TMap is slow, investigate creating a triangular grid structure which would allow more direct access. Maybe similar to TDenseGrid2 but only storing the lower half?
	TMap<TPair<int32, int32>, int32> Grid;
	Grid.Reserve(GridN * GridN / 2);

	for (int32 I = 0; I < GridN; ++I)
	{
		for (int32 J = 0; J < GridN - I; ++J)
		{
			Grid.Add({I, J}) = -1;
		}
	}

	// generate interior (non-edge) vertices

	const float GridStep = 1.0f / static_cast<float>(GridN - 1);
	int32 InteriorVertexIndex = 0;
	for (int32 I = 1; I < GridN - 1; ++I)
	{
		for (int32 J = 1; J < GridN - I - 1; ++J)
		{
			const float Bary1 = static_cast<float>(I) * GridStep;
			const float Bary2 = static_cast<float>(J) * GridStep;
			const float Bary0 = 1.0f - Bary1 - Bary2;

			TriPatch.BaryCoord[InteriorVertexIndex] = { Bary0, Bary1, Bary2 };
			Grid[{I, J}] = TriPatch.VIDs[InteriorVertexIndex];

			++InteriorVertexIndex;
		}
	}

	// corners

	const int32 U = TriPatch.UVWCorners[0];
	const int32 V = TriPatch.UVWCorners[1];
	const int32 W = TriPatch.UVWCorners[2];

	Grid[{0, 0}] = U;
	Grid[{GridN - 1, 0}] = V;
	Grid[{0, GridN - 1}] = W;

	// edges

	auto Reverse = [](const TArrayView<int>& Edge) -> TArray<int>
	{
		TArray<int> RevEdge;
		for (int EV : Edge)
		{
			RevEdge.Add(EV);
		}
		Algo::Reverse(RevEdge);
		return RevEdge;
	};

	const TArray<int> EdgeUVSortedVIDs = TriPatch.UVEdge.bIsReversed ? Reverse(TriPatch.UVEdge.VIDs) : TArray<int>(TriPatch.UVEdge.VIDs);
	const TArray<int> EdgeVWSortedVIDs = TriPatch.VWEdge.bIsReversed ? Reverse(TriPatch.VWEdge.VIDs) : TArray<int>(TriPatch.VWEdge.VIDs);
	const TArray<int> EdgeUWSortedVIDs = TriPatch.UWEdge.bIsReversed ? Reverse(TriPatch.UWEdge.VIDs) : TArray<int>(TriPatch.UWEdge.VIDs);

	const FIndex3i NeighborTris = Mesh->GetTriNeighbourTris(TriPatch.TriangleID);

	// Edge UV
	const int32 NumUVVertices = TriPatch.UVEdge.VIDs.Num();

	// Whether the edge has extra vertices because the neighboring patch is higher level. If so, then we need to only grab every Nth vertex from the TriPatch Edge
	const bool bUVEdgeHasAdditionalVertices = (NumUVVertices > (1 << Level) - 1);

	const int32 NbrUVLevel = Mesh->IsTriangle(NeighborTris[0]) ? TriangleLevels[NeighborTris[0]] : Level;
	for (int32 I = 0; I < GridN - 2; ++I)
	{
		if (bUVEdgeHasAdditionalVertices)
		{
			checkf(NbrUVLevel >= Level, TEXT("bUVEdgeHasAdditionalVertices is true but Level > NbrUVLevel"));
			const int32 StepSize = (1 << (NbrUVLevel - Level));
			Grid[{I + 1, 0}] = EdgeUVSortedVIDs[StepSize * (I + 1) - 1];
		}
		else
		{
			Grid[{I + 1, 0}] = EdgeUVSortedVIDs[I];
		}
	}

	// Edge UW
	const int32 NumUWVertices = TriPatch.UWEdge.VIDs.Num();
	const bool bUWEdgeHasAdditionalVertices = (NumUWVertices > (1 << Level) - 1);
	const int32 NbrUWLevel = Mesh->IsTriangle(NeighborTris[2]) ? TriangleLevels[NeighborTris[2]] : Level;
	// Vertices stored in UWEdge initially run along the edge from vertex W to vertex U. But our grid places W at the (i=0, j=N-1) grid point. 
	// So to fill in the i = 0 column from j = 0 to N-1 we need to traverse UWEdge in reverse
	for (int32 J = 0; J < GridN - 2; ++J)
	{
		if (bUWEdgeHasAdditionalVertices)
		{
			checkf(NbrUWLevel >= Level, TEXT("bUWEdgeHasAdditionalVertices is true but Level > NbrUWLevel"));
			const int32 StepSize = (1 << (NbrUWLevel - Level));
			Grid[{0, J + 1}] = EdgeUWSortedVIDs[NumUWVertices - StepSize * (J + 1)];
		}
		else
		{
			Grid[{0, J + 1}] = EdgeUWSortedVIDs[GridN - 3 - J];
		}
	}

	// Edge VW
	const int32 NumVWVertices = TriPatch.VWEdge.VIDs.Num();
	const bool bVWEdgeHasAdditionalVertices = (NumVWVertices > (1 << Level) - 1);
	const int32 NbrVWLevel = Mesh->IsTriangle(NeighborTris[1]) ? TriangleLevels[NeighborTris[1]] : Level;
	for (int32 J = 0; J < GridN - 2; ++J)
	{
		const int32 I = GridN - 1 - J;

		const TPair<int32, int32> MapKey{ I - 1, J + 1 };
		ensure(Grid[MapKey] == -1);

		if (bVWEdgeHasAdditionalVertices)
		{
			checkf(NbrVWLevel >= Level, TEXT("bVWEdgeHasAdditionalVertices is true but Level > NbrVWLevel"));
			const int32 StepSize = (1 << (NbrVWLevel - Level));
			Grid[MapKey] = EdgeVWSortedVIDs[StepSize * (J + 1) - 1];
		}
		else
		{
			Grid[MapKey] = EdgeVWSortedVIDs[J];
		}
	}


	// triangulate each grid cell

	ensure(TriPatch.Triangles.Num() > 1);

	int32 TriangleIndex = 0;
	for (int I = 0; I < GridN - 1; ++I)
	{
		for (int J = 0; J < GridN - 1 - I; ++J)
		{
			FIndex3i GridTriangle = { Grid[{I, J}], Grid[{I + 1, J}], Grid[{I, J + 1}] };
			ensure(GridTriangle[0] >= 0);
			ensure(GridTriangle[1] >= 0);
			ensure(GridTriangle[2] >= 0);

			auto FillEdgePoints = [&](const int32 NbrLevel, const int32 I, const TArray<int32>& InFullEdgePoints, TArray<int32>& EdgePoints, bool bGridCountReversed)
			{
				check(NbrLevel >= Level);	// This lambda should only be called when we are adding additional points due to the neighbor having a higher subdivision level
				const int32 StepSize = (1 << (NbrLevel - Level));

				int32 Start, End;
				if (!bGridCountReversed)
				{
					Start = I * StepSize;
					End = (I + 1) * StepSize - 1;
					for (int32 V = Start; V < End; ++V)
					{
						EdgePoints.Add(InFullEdgePoints[V]);
					}
				}
				else
				{
					const int J = GridN - I - 1;
					const int JNext = J - 1;
					
					Start = JNext * StepSize;
					End = J * StepSize - 1;

					for (int32 V = Start; V < End; ++V)
					{
						EdgePoints.Add(InFullEdgePoints[V]);
					}
				}

				ensure(EdgePoints.Num() == StepSize - 1);
			};

			// If we are on an edge where the neighboring triangle has a higher tessellation level, we'll have more edge vertices we need to think about
			TStaticArray<TArray<int32>, 3> EdgePoints;		// EdgePoints are new vertices in the UV, WU, and VW edges respectively. 
			if (J == 0 && bUVEdgeHasAdditionalVertices)
			{
				FillEdgePoints(NbrUVLevel, I, EdgeUVSortedVIDs, EdgePoints[0], false);
			}
			if (I + J == GridN - 2 && bVWEdgeHasAdditionalVertices)
			{
				FillEdgePoints(NbrVWLevel, J, EdgeVWSortedVIDs, EdgePoints[1], false);
			}
			if (I == 0 && bUWEdgeHasAdditionalVertices)
			{
				FillEdgePoints(NbrUWLevel, J, EdgeUWSortedVIDs, EdgePoints[2], true);
			}

			// lower triangle(s) for the grid cell

			if ((EdgePoints[0].Num() == 0) && (EdgePoints[1].Num() == 0) && (EdgePoints[2].Num() == 0))
			{
				// simple case, no extra triangles
				TriPatch.Triangles[TriangleIndex++] = GridTriangle;
			}
			else
			{
				// adjacent to a patch with higher tessellation level, need extra triangles

				TArray<FIndex3i> Tris;
				RedGreenTessellationPatternHelpers::TriangulatePoly(GridTriangle, EdgePoints, Tris);

				for (const FIndex3i& Tri : Tris)
				{
					ensure(Tri[0] >= 0);
					ensure(Tri[1] >= 0);
					ensure(Tri[2] >= 0);

					TriPatch.Triangles[TriangleIndex++] = Tri;
				}
			}

			// upper triangle for the grid cell
			if (I + J < GridN - 2)
			{
				const FIndex3i UpperTri = { Grid[{I + 1, J}], Grid[{I + 1, J + 1}], Grid[{I, J + 1}] };
				ensure(UpperTri[0] >= 0);
				ensure(UpperTri[1] >= 0);
				ensure(UpperTri[2] >= 0);

				TriPatch.Triangles[TriangleIndex++] = UpperTri;
			}
		}
	}

	ensure(TriangleIndex == TriPatch.Triangles.Num());
}


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
