// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Operations/SelectiveTessellate.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
* Adaptive red-green subdivision. Triangles are tessellated to their specified level while also ensuring crack- and T-junction-free triangulation across 
* triangles with different tessellation levels.
* 
* Each "triangle level" is the number of times to repeatedly apply red subdivision (1 to 4 triangle splits). We must simultaneously apply green subdivision on
* neighboring triangles with lower levels to resolve T-junctions.
*/

class FRedGreenTessellationPattern : public FTessellationPattern
{
public:

	FRedGreenTessellationPattern(const FDynamicMesh3* InMesh, const TArray<int>& InTriangleTessLevels);
	virtual ~FRedGreenTessellationPattern() = default;

	TArray<int32> TriangleLevels;

private:

	// FTessellationPattern  overrides
	virtual EOperationValidationResult Validate() const override;
	virtual int GetNumberOfNewVerticesForEdgePatch(const int InEdgeID) const override;
	virtual int GetNumberOfNewVerticesForTrianglePatch(const int InTriangleID) const override;
	virtual int GetNumberOfPatchTriangles(const int InTriangleID) const override;
	virtual void TessellateEdgePatch(EdgePatch& EdgePatch) const override;
	virtual void TessellateTriPatch(TrianglePatch& TriPatch) const override;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
