// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/RemoveOccludedTriangles.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "Selections/MeshConnectedComponents.h"

using namespace UE::Geometry;

namespace UE
{
	namespace MeshAutoRepair
	{
		// simple adapter to directly work with mesh components rather than creating a copy per component
		struct FComponentMesh
		{
			FDynamicMesh3* Mesh;
			FMeshConnectedComponents::FComponent* Component;

			FComponentMesh(FDynamicMesh3* Mesh, FMeshConnectedComponents::FComponent* Component) : Mesh(Mesh), Component(Component)
			{}

			int32 MaxTriangleID() const
			{
				return Component->Indices.Num();
			}

			int32 MaxVertexID() const
			{
				return Mesh->MaxVertexID();
			}

			bool IsTriangle(int32 Index) const
			{
				return Component->Indices.IsValidIndex(Index) && Mesh->IsTriangle(Component->Indices[Index]);
			}

			bool IsVertex(int32 Index) const
			{
				return Mesh->IsVertex(Index);
			}

			int32 TriangleCount() const
			{
				return Component->Indices.Num();
			}

			FORCEINLINE int32 VertexCount() const
			{
				return Mesh->VertexCount();
			}

			FORCEINLINE uint64 GetChangeStamp() const
			{
				return 1;
			}

			FORCEINLINE FIndex3i GetTriangle(int32 Index) const
			{
				return Mesh->GetTriangle(Component->Indices[Index]);
			}

			FORCEINLINE FVector3d GetVertex(int32 Index) const
			{
				return Mesh->GetVertex(Index);
			}

			FORCEINLINE void GetTriVertices(int32 TriIndex, UE::Math::TVector<double>& V0, UE::Math::TVector<double>& V1, UE::Math::TVector<double>& V2) const
			{
				Mesh->GetTriVertices(Component->Indices[TriIndex], V0, V1, V2);
			}
		};

		bool DYNAMICMESH_API RemoveInternalTriangles(FDynamicMesh3& Mesh, bool bTestPerCompoment,
			EOcclusionTriangleSampling SamplingMethod,
			EOcclusionCalculationMode OcclusionMode,
			int RandomSamplesPerTri, double WindingNumberThreshold, bool bTestOccludedByAny)
		{
			if (bTestPerCompoment)
			{
				TRemoveOccludedTriangles<FComponentMesh> Remover(&Mesh);
				Remover.InsideMode = OcclusionMode;
				Remover.TriangleSamplingMethod = SamplingMethod;
				Remover.AddTriangleSamples = RandomSamplesPerTri;
				FMeshConnectedComponents Components(&Mesh);
				Components.FindConnectedTriangles();
				TArray<FComponentMesh> ComponentMeshes; ComponentMeshes.Reserve(Components.Num());
				TIndirectArray<TMeshAABBTree3<FComponentMesh>> Spatials; Spatials.Reserve(Components.Num());
				TIndirectArray<TFastWindingTree<FComponentMesh>> Windings; Windings.Reserve(Components.Num());
				for (int ComponentIdx = 0; ComponentIdx < Components.Num(); ComponentIdx++)
				{
					ComponentMeshes.Emplace(&Mesh, &Components.GetComponent(ComponentIdx));
					Spatials.Add(new TMeshAABBTree3<FComponentMesh>(&ComponentMeshes[ComponentIdx]));
					Windings.Add(new TFastWindingTree<FComponentMesh>(&Spatials[ComponentIdx]));
				}
				FTransformSRT3d Id = FTransformSRT3d::Identity();
				TArrayView<FTransformSRT3d> TransformsView(&Id, 1);
				TArrayView<TMeshAABBTree3<FComponentMesh>*> SpatialsView(Spatials.GetData(), Spatials.Num());
				TArrayView<TFastWindingTree<FComponentMesh>*> WindingsView(Windings.GetData(), Windings.Num());
				return Remover.Apply(TransformsView, SpatialsView, WindingsView, TArrayView<const FTransformSRT3d>(), bTestOccludedByAny);
			}
			else
			{
				TRemoveOccludedTriangles<FDynamicMesh3> Remover(&Mesh);
				Remover.InsideMode = OcclusionMode;
				Remover.TriangleSamplingMethod = SamplingMethod;
				Remover.AddTriangleSamples = RandomSamplesPerTri;
				FDynamicMeshAABBTree3 Spatial(&Mesh, true);
				return Remover.Apply(FTransformSRT3d::Identity(), &Spatial);
			}
		}
	}
}

