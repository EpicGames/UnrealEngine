// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshRepairOrientation

#include "Operations/RepairOrientation.h"

#include "Async/ParallelFor.h"

#include "DynamicMeshEditor.h"

#include "Misc/ScopeLock.h"

using namespace UE::Geometry;

// TODO:
//  - (in merge coincident) don't merge tris with same/opposite normals (option)
//  - after orienting components, try to find adjacent open components and
//    transfer orientation between them
//  - orient via nesting


void FMeshRepairOrientation::OrientComponents()
{
	Components.Reset();
	
	TSet<int> Remaining; Remaining.Reserve(Mesh->MaxTriangleID());
	for (int TID : Mesh->TriangleIndicesItr())
	{
		Remaining.Add(TID);
	}
	// note: this is only called after we've already verified there are elements in Remaining
	auto PopOneTri = [&Remaining]()
	{
		int32 One = *Remaining.CreateConstIterator();
		Remaining.Remove(One);
		return One;
	};
	TArray<int> Stack;
	while (Remaining.Num() > 0)
	{
		Component& C = Components.Emplace_GetRef();
	
		Stack.Empty();
		int Start = PopOneTri();
		C.Triangles.Add(Start);
		Stack.Add(Start);
		while (Stack.Num() > 0) {
			int Cur = Stack.Pop(EAllowShrinking::No);
			FIndex3i tcur = Mesh->GetTriangle(Cur);
	
			FIndex3i nbrs = Mesh->GetTriNeighbourTris(Cur);
			for (int j = 0; j < 3; ++j) {
				int nbr = nbrs[j];
				if (Remaining.Contains(nbr) == false)
				{
					continue;
				}
	
				int a = tcur[j];
				int b = tcur[(j+1)%3];
	
				FIndex3i tnbr = Mesh->GetTriangle(nbr);
				if (IndexUtil::FindTriOrderedEdge(b, a, tnbr) == IndexConstants::InvalidID)
				{
					ensure(Mesh->ReverseTriOrientation(nbr) == EMeshResult::Ok);
				}
				Stack.Add(nbr);
				Remaining.Remove(nbr);
				C.Triangles.Add(nbr);
			}
		}
	}
}

	
void FMeshRepairOrientation::ComputeStatistics(FDynamicMeshAABBTree3* Tree)
{
	for (Component& C : Components)
	{
		ComputeComponentStatistics(Tree, C);
	}
}


void FMeshRepairOrientation::ComputeComponentStatistics(FDynamicMeshAABBTree3* Tree, Component& C)
{
	C.InFacing = C.OutFacing = 0;
	double Dist = 2 * Mesh->GetBounds(true).DiagonalLength();
	
	// only want to raycast triangles in this component
	TSet<int> TrisInComponent; TrisInComponent.Append(C.Triangles);
	IMeshSpatial::FQueryOptions RaycastOptions([&TrisInComponent](int TID)
		{
			return TrisInComponent.Contains(TID);
		});
	
	// We want to try to figure out what is 'outside' relative to the world.
	// Assumption is that faces we can hit from far away should be oriented outwards.
	// So, for each triangle we construct far-away points in positive and negative normal
	// direction, then raycast back towards the triangle. If we hit the triangle from
	// one side and not the other, that is evidence we should keep/reverse that triangle.
	// If it is not hit, or hit from both, that does not provide any evidence.
	// We collect up this keep/reverse evidence and use the larger to decide on the global orientation.
	
	FCriticalSection Mutex;

	// TODO: profile the parallel for and consider whether block size / locking method should be different
	bool bNoParallel = false;
	ParallelFor(C.Triangles.Num(), [this, &Tree, &RaycastOptions, &Mutex, &C, &Dist](int32 Idx)
	{
		int TID = C.Triangles[Idx];
		FVector3d Normal, Centroid;
		double Area;
		Mesh->GetTriInfo(TID, Normal, Area, Centroid);
		if (Area < FMathf::ZeroTolerance)
		{
			return;
		}

		FVector3d PosPt = Centroid + Dist * Normal;
		FVector3d NegPt = Centroid - Dist * Normal;
		int HitPos = Tree->FindNearestHitTriangle(FRay3d(PosPt, -Normal), RaycastOptions);
		int HitNeg = Tree->FindNearestHitTriangle(FRay3d(NegPt, Normal), RaycastOptions);
		if (HitPos != TID && HitNeg != TID)
		{
			return; // no evidence
		}
		if (HitPos == TID && HitNeg == TID)
		{
			return; // no evidence?
		}

		{
			FScopeLock Lock(&Mutex);
			if (HitNeg == TID)
			{
				C.InFacing += Area;
			}
			else if (HitPos == TID)
			{
				C.OutFacing += Area;
			}
		}
	},
		bNoParallel);
}


void FMeshRepairOrientation::SolveGlobalOrientation(FDynamicMeshAABBTree3* Tree)
{
	check(Tree->GetMesh() == Mesh);
	ComputeStatistics(Tree);
	FDynamicMeshEditor Editor(Mesh);
	for (const Component& C : Components)
	{
		if (C.InFacing > C.OutFacing)
		{
			Editor.ReverseTriangleOrientations(C.Triangles, true);
		}
	}
}
