// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/SelectiveTessellate.h"
#include "VectorTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Async/ParallelFor.h"
#include "Util/ProgressCancel.h"
#include "Distance/DistLine3Line3.h"
#include "Misc/AssertionMacros.h"
#include "Util/CompactMaps.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "Operations/RedGreenTessellation.h"

using namespace UE::Geometry;

namespace SelectiveTessellateLocals 
{	
	// Forward declare
	class FTessellationData;

	template<typename RealType, int ElementSize> 
	class FOverlayTessellationData;

	bool ConstructTessellatedMesh(const FDynamicMesh3* Mesh,
								  FProgressCancel* Progress,
								  FTessellationData& TessData,
								  FCompactMaps& CompactInfo,
								  FDynamicMesh3* ResultMesh,
								  FSelectiveTessellate::FTessellationInformation& TessInfo);

	template<typename RealType, int ElementSize> 
	bool ConstructTessellatedOverlay(const FDynamicMesh3* Mesh, 
									 const TDynamicMeshOverlay<RealType, ElementSize>* Overlay,
									 FTessellationData& MeshTessData, 
									 FOverlayTessellationData<RealType, ElementSize>& TessData, 
									 const FCompactMaps& CompactInfo,
									 FProgressCancel* Progress, 
								     TDynamicMeshOverlay<RealType, ElementSize>* ResultOverlay);

	// Utility functions
	template<typename RealType, int ElementSize>
	void LerpElements(const RealType* Element1, const RealType* Element2, RealType* OutElement, double Alpha)
	{
		Alpha = FMath::Clamp(Alpha, 0.0, 1.0);
		double OneMinusAlpha = 1.0 - Alpha;
		for (int Idx = 0; Idx < ElementSize; ++Idx) 
		{
			OutElement[Idx] = RealType(OneMinusAlpha * double(Element1[Idx]) + Alpha * double(Element2[Idx]));
		}
	} 

	template<typename RealType, int ElementSize>
	void BaryElements(const RealType* Element1, const RealType* Element2, const RealType* Element3,
					  RealType* OutElement, 
					  double Alpha, double Beta, double Theta)
	{
		checkSlow(FMath::Abs(Alpha + Beta + Theta - 1.0) < KINDA_SMALL_NUMBER);
		for (int Idx = 0; Idx < ElementSize; ++Idx)
		{
			OutElement[Idx] = RealType(Alpha * double(Element1[Idx]) + Beta * double(Element2[Idx]) + Theta * double(Element3[Idx]));
		}
	};


	/**
     * Manage data containing tessellation information. This includes barycentric coordinates of the new vertices,
     * new vertex/elemnent IDs and triangle indices. This class handles cases where 2 elements are stored per vertex. 
	 * By default we assume we are interpolating geometry data. Can subclass and implement the virtual methods 
	 * to handle other sources of data (see FOverlayTessellationData).
     *
     * Given an Edge or Triangle ID, allows to create a TArrayView for reading and writing to the data block 
	 * containing barycentric coordinates, vertex/element IDs or triangle data.
	 */
	class FTessellationData
	{
	public:

		// Track all of the edges and triangles that we need to tessellate. Allows for fast Contains() queries.
		TSet<int> EdgesToTessellate;
		TSet<int> TrianglesToTessellate;

        // Array of the new IDs for the vertices/elements added along the tessellated edges. Different edges can have a 
		// different number of vertices/elements added. Each vertex can have 1 or 2 IDs added to handle the overlays. 
		// Use the EdgeIDOffsets to figure out the starting point and the length of the data block containing all of 
		// the IDs for the edge/triangle pair. Similar logic applies to EdgeCoord, InnerIDs, InnerCoord, TriangleIDs, 
		// Triangles.
		//
		//            Edge1 (seam edge)    Edge2            Edge50  Edge51
		//            --------------------------------------------------------------------------
		// EdgeIDs   | ID1  ID2 : ID3 ID4 | ID5  ID6 | ... | ID100 | ID101 ID102 |....
		//            --------------------------------------------------------------------------
		//           /          /										
		//          /           EdgeIDOffsets[(Edge1 , Triangle2)][0]
		//         EdgeIDOffsets[(Edge1 , Triangle1)][0] 	

		TArray<int> EdgeIDs;     			    // IDs of the vertices/elements added inside the tessellated edges 
		TMap<FIndex2i, FIndex2i> EdgeIDOffsets; // Maps a tuple (Edge ID, Triangle ID) to tuple of (offset, length) 
											    // numbers used to access a data block in the EdgeIDs array.
			
		TArray<double> EdgeCoord; 			   // Lerp coefficients of the vertices/elements added along the tessellated edges		
		TMap<int, FIndex2i> EdgeCoordOffsets;  // Maps Edge ID to tuple of (offset, length) numbers used to access a data 
											   // block in the EdgeCoord arrays.

		TArray<int> InnerIDs; 	          // IDs of the vertices/elements added inside the tessellated triangles 
		TArray<FVector3d> InnerCoord;     // Barycentric coordinates of the vertices added inside the tessellated triangles 
		TMap<int, FIndex2i> InnerOffsets; // Maps Triangle ID to tuple of (offset, length) numbers used to access 
										  // a data block in the InnerIDs and InnerCoord arrays.

		TArray<int> TriangleIDs; 			  // IDs of the new triangles that the original triangle is split into
		TArray<FIndex3i> Triangles; 		  // Triangle indicies (i.e. vertex id, element id, etc)
		TMap<int, FIndex2i> TrianglesOffsets; // Maps Triangle ID to tuple of (offset, length) numbers used to access 
											  // a data block in the TriangleIDs and Triangles array.
	protected:

		const FDynamicMesh3* Mesh = nullptr;

		// If we are not working with overlays then we don't care which side of the edge we are working with
		constexpr static int AnyTriangleID = -1; 

		// The starting ID for new vertices/elements to be added
		int MaxID = -1;
 
	public:

		FTessellationData(const FDynamicMesh3* InMesh) 
		:
		Mesh(InMesh)
		{
			MaxID = Mesh->MaxVertexID();
		}

		virtual ~FTessellationData()
		{
		}

		void Init(const FTessellationPattern* Pattern) 
		{
			this->InitEdgeVertexBuffers(Pattern);
			this->InitTriVertexBuffers(Pattern); 
			this->InitTrianglesBuffers(Pattern);
		}


		/** 
		 * @return Array view of the data block containing the barycentric coordinates for the new vertices added along 
		 * the edge. 
		 */
		TArrayView<double> MapEdgeCoordBufferBlock(const int EdgeID) 
		{
			TArrayView<double> ArrayView(EdgeCoord);
			const FIndex2i OffsetLength = EdgeCoordOffsets[EdgeID]; 
			return ArrayView.Slice(OffsetLength[0], OffsetLength[1]);
		}

		/** @return Array view of the data block containing the ids for the new vertices/elements added along the edge. */
		TArrayView<int> MapEdgeIDBufferBlock(const int EdgeID) 
		{
			TArrayView<int> ArrayView(EdgeIDs);
			const FIndex2i OffsetLength = EdgeIDOffsets[FIndex2i(EdgeID, AnyTriangleID)];
			return ArrayView.Slice(OffsetLength[0], OffsetLength[1]);
		}


		/** @return Array view of the data block containing the ids for the new elements assosiated with an edge and a triangle. */
		TArrayView<int> MapEdgeIDBufferBlock(const int EdgeID, const int TriangleID) 
		{
			TArrayView<int> ArrayView(EdgeIDs);
			const FIndex2i OffsetLength = EdgeIDOffsets[FIndex2i(EdgeID, TriangleID)]; 
			return ArrayView.Slice(OffsetLength[0], OffsetLength[1]);
		}


		/** @return Array view of the data block containing the barycentric coordinates for new vertices added inside the triangle. */
		TArrayView<FVector3d> MapInnerCoordBufferBlock(const int TriangleID)
		{
			if (InnerOffsets.Contains(TriangleID))
			{
				TArrayView<FVector3d> ArrayView(InnerCoord);
				const FIndex2i OffsetLength = InnerOffsets[TriangleID];
				return ArrayView.Slice(OffsetLength[0], OffsetLength[1]);
			}
			return TArrayView<FVector3d>();
		}

		/** @return Array view of the data block containing the ids for new vertices added inside the triangle. */
		TArrayView<int> MapInnerIDBufferBlock(const int TriangleID)
		{
			if (InnerOffsets.Contains(TriangleID))
			{
				TArrayView<int> ArrayView(InnerIDs);
				const FIndex2i OffsetLength = InnerOffsets[TriangleID];
				return ArrayView.Slice(OffsetLength[0], OffsetLength[1]);
			}
			return TArrayView<int>();
		}


		/** @return Array view of a data block containing the triangle ids for new triangles. */
		TArrayView<int> MapTriangleIDBufferBlock(const int TriangleID)
		{
			if (TrianglesOffsets.Contains(TriangleID))
			{
				TArrayView<int> ArrayView(TriangleIDs);
				FIndex2i OffsetLengthPair = TrianglesOffsets[TriangleID];
				return ArrayView.Slice(OffsetLengthPair[0], OffsetLengthPair[1]);
			}
			return TArrayView<int>();
		}

		/** @return Array view of a data block containing the triangle indices for new triangles. */
		TArrayView<FIndex3i> MapTrianglesBufferBlock(const int TriangleID)
		{
			if (TrianglesOffsets.Contains(TriangleID))
			{
				TArrayView<FIndex3i> ArrayView(Triangles);
				FIndex2i OffsetLengthPair = TrianglesOffsets[TriangleID];
				return ArrayView.Slice(OffsetLengthPair[0], OffsetLengthPair[1]);
			}
			return TArrayView<FIndex3i>();
		}

		//TODO: Add const versions of the Map* methods

	protected:

		virtual void InitEdgeVertexBuffers(const FTessellationPattern* Pattern)
		{
			int BufferNumVertices = 0;
			for (const int EdgeID : Mesh->EdgeIndicesItr())
			{
				const int NumNewVertices = Pattern->GetNumberOfNewVerticesForEdgePatch(EdgeID); 
				if (NumNewVertices > 0)
				{
					EdgesToTessellate.Add(EdgeID);
					EdgeCoordOffsets.Add(EdgeID, FIndex2i(BufferNumVertices, NumNewVertices));
					EdgeIDOffsets.Add(FIndex2i(EdgeID, AnyTriangleID), FIndex2i(BufferNumVertices, NumNewVertices));
					BufferNumVertices += NumNewVertices;
				}
			}

			EdgeIDs.SetNum(BufferNumVertices);
			
			for (int Index = 0; Index < EdgeIDs.Num(); ++Index)
			{
				EdgeIDs[Index] = Index + MaxID;
			}
			
			// Pre-allocate memory for linear coordinates we will be computing
			EdgeCoord.SetNum(BufferNumVertices);
		}


		virtual void InitTriVertexBuffers(const FTessellationPattern* Pattern) 
		{
			int BufferNumVertices = 0;
			for (int TriangleID = 0; TriangleID < Mesh->MaxTriangleID(); ++TriangleID)
			{
				if (this->IsValidTriangle(TriangleID))
				{
					const int NumNewVertices = Pattern->GetNumberOfNewVerticesForTrianglePatch(TriangleID); 
					if (NumNewVertices > 0) 
					{
						InnerOffsets.Add(TriangleID, FIndex2i(BufferNumVertices, NumNewVertices));
						BufferNumVertices += NumNewVertices;
					}
				}
			}

			InnerIDs.SetNum(BufferNumVertices);
			
			// IDs of the new vertices/elements added inside triangles start with the last id added along edges. It is 
			// possible that none of the edges were tessellated in which case we start with the max vertex/element id. 
			const int LastEdgeVIDs = EdgeIDs.Num() > 0 ? EdgeIDs.Last() + 1 : MaxID;
			for (int Index = 0; Index < InnerIDs.Num(); ++Index)
			{
				InnerIDs[Index] = Index + LastEdgeVIDs;
			}

			// Pre-allocate memory for barycentric coordinates we will be computing
			InnerCoord.SetNum(BufferNumVertices);
		}


		virtual void InitTrianglesBuffers(const FTessellationPattern* Pattern) 
		{
			int BufferNumTriangles = 0;
			for (int TriangleID = 0; TriangleID < Mesh->MaxTriangleID(); ++TriangleID)
			{
				if (this->IsValidTriangle(TriangleID))
				{
					const int NumNewTriangles = Pattern->GetNumberOfPatchTriangles(TriangleID); 
					if (NumNewTriangles > 0) 
					{	
						TrianglesToTessellate.Add(TriangleID);
						TrianglesOffsets.Add(TriangleID, FIndex2i(BufferNumTriangles, NumNewTriangles));
						BufferNumTriangles += NumNewTriangles;
					}
				}
			}

			TriangleIDs.SetNum(BufferNumTriangles);

			for (int Index = 0; Index < TriangleIDs.Num(); ++Index) 
			{
				TriangleIDs[Index] = Mesh->MaxTriangleID() + Index;
			}

			// Pre-allocate memory for triangles we will be computing
			Triangles.SetNum(BufferNumTriangles);
		}

		virtual bool IsValidTriangle(int TriangleID) const
		{
			return Mesh->IsTriangle(TriangleID);
		}
	}; 


	/**  
	 * Handle the data for tessellating overlays. The main difference between this class and its parent is that we 
	 * handle 2 elements per vertex along edges. We also need to consider the fact that a triangle could be marked 
	 * for tessellation but not be set in the overlay, in which case we can skip any element computation for it.
	 */
	template<typename RealType, int ElementSize>
	class FOverlayTessellationData : public FTessellationData
	{
	protected:

		const TDynamicMeshOverlay<RealType, ElementSize>* Overlay = nullptr; 

	public:

		FOverlayTessellationData(const FDynamicMesh3* InMesh, const TDynamicMeshOverlay<RealType, ElementSize>* InOverlay)
		:
		FTessellationData(InMesh), Overlay(InOverlay)
		{
			MaxID = Overlay->MaxElementID();
		}

	protected:

		virtual void InitEdgeVertexBuffers(const FTessellationPattern* Pattern) override
		{
			int CoordBufferSize = 0;
			int VIDBufferSize = 0;
			for (const int EdgeID : Mesh->EdgeIndicesItr())
			{
				const int NumNewVertices = Pattern->GetNumberOfNewVerticesForEdgePatch(EdgeID); 
				const FIndex2i EdgeTri = Mesh->GetEdgeT(EdgeID);

				// Edge is invalid in the overlay if both triangles that share it are not set in the overlay 
				const bool bIsNotBndry = Mesh->GetEdgeT(EdgeID).B != FDynamicMesh3::InvalidID;
				const bool bIsValidEdge = Overlay->IsSetTriangle(EdgeTri.A) || (bIsNotBndry && Overlay->IsSetTriangle(EdgeTri.B));

				if (NumNewVertices > 0 && bIsValidEdge)
				{
					EdgesToTessellate.Add(EdgeID);			
					EdgeCoordOffsets.Add(EdgeID, FIndex2i(CoordBufferSize, NumNewVertices));	
					CoordBufferSize += NumNewVertices;

					if (Overlay->IsSetTriangle(EdgeTri.A)) 
					{	
						EdgeIDOffsets.Add(FIndex2i(EdgeID, EdgeTri.A), FIndex2i(VIDBufferSize, NumNewVertices));
						
						if (bIsNotBndry && Overlay->IsSetTriangle(EdgeTri.B)) 
						{
							if (Overlay->IsSeamEdge(EdgeID) == true) 
							{
								EdgeIDOffsets.Add(FIndex2i(EdgeID, EdgeTri.B), FIndex2i(VIDBufferSize + NumNewVertices, NumNewVertices));
								VIDBufferSize += 2*NumNewVertices;
							}
							else 
							{
								// Not a seam edge so simply point to the same data block as FIndex2i(EdgeID, EdgeTri.A) case
								EdgeIDOffsets.Add(FIndex2i(EdgeID, EdgeTri.B), FIndex2i(VIDBufferSize, NumNewVertices));
								VIDBufferSize += NumNewVertices;
							}
						}
						else 
						{
							VIDBufferSize += NumNewVertices;
						}
					}
					else 
					{
						checkSlow(bIsNotBndry && Overlay->IsSetTriangle(EdgeTri.B));
						EdgeIDOffsets.Add(FIndex2i(EdgeID, EdgeTri.B), FIndex2i(VIDBufferSize, NumNewVertices));
						VIDBufferSize += NumNewVertices;
					}
				}
			}

			EdgeIDs.SetNum(VIDBufferSize);
			
			for (int Index = 0; Index < EdgeIDs.Num(); ++Index)
			{
				EdgeIDs[Index] = Index + MaxID;
			}
			
			// Pre-allocate memory for linear coordinates we will be computing
			EdgeCoord.SetNum(CoordBufferSize);
		}

		virtual bool IsValidTriangle(int TriangleID) const override
		{
			return Mesh->IsTriangle(TriangleID) && Overlay->IsSetTriangle(TriangleID);
		}
	}; 


	/**
     * Pattern where the inner area is tessellated using the style of the OpenGL tessellation shader. The inner area 
	 * consists of multiple inner rings. The original triangle we are tessellating is called the outer ring. Areas 
	 * between rings are tessellated with the new triangles.
	 * 
     *                 O
     *                / \
     *               O. .O
     *              /  O  \
     *             O. / \ .O
     *            /  O. .O  \
     *           /  /  O  \  \
     *          /  /  / \  \  \
     *         O. /  /   \  \ .O
     *        /  O. /     \ .O  \
     *       /  /  O-------O  \  \
     *      O. / inner ring 2  \ .O
     *     /  O----O-------O----O  \
     *    /   .  inner ring 1   .   \
     *   O----O----O-------O----O----O 
     *			 outer ring
	 *
     */
	class FConcentricRingsTessellationPattern : public FTessellationPattern
	{
	public:

		/**
		 * FRing edge is a collection of vertices that includes two corner vertices (marked + below) plus all the 
		 * vertices in between (marked O below).
		 * 
		 *                 +
		 *                / \
		 *               O   \
		 *              /     \
		 * ring edge 3 /       O  ring edge 1
		 *            /         \
		 *           O           \
		 *          /             \
		 *         /               \
		 *        +---O----O----O---+
		 * 			  ring edge 2
		 
		*/
			
		class FRingEdge 
		{

		public:
			FRingEdge() 
			{
			}

			/** FRing edge can be a single point. */
			FRingEdge(int V1) 
			:
			V1(V1), VertNum(1)
			{
			}
			
			FRingEdge(int V1, int V2, TArrayView<int> InInner, bool bReverse = false) 
			:
			V1(V1), V2(V2), Inner(InInner), VertNum(InInner.Num() + 2), bReverse(bReverse)
			{
			}

			FRingEdge(int V1, int V2, EdgePatch Patch)
			:
			FRingEdge(V1, V2, Patch.VIDs, Patch.bIsReversed)
			{
			}

			inline int Num() const 
			{
				return VertNum;
			}

			int operator[](int Index) const
			{
				check(Index >= 0 && Index < VertNum);

				if (Index == 0) 
				{
					return V1;
				} 
				else if (Index == VertNum - 1) 
				{
					return V2;
				}
				else 
				{
					int InnerIndex = Index - 1; // index into the Inner array
					int ActualIndex = bReverse ? Inner.Num() - InnerIndex - 1: InnerIndex; // potentially reverse index
					check(ActualIndex >= 0 && ActualIndex < VertNum);
					return Inner[ActualIndex];
				}
			}

		private:
			
			int V1 = IndexConstants::InvalidID;
			int V2 = IndexConstants::InvalidID;
			TArrayView<int> Inner;
			int VertNum = 0; // Number of elements
			bool bReverse = false;
		};

		/* A ring consists of 3 ring edges */
		struct FRing 
		{
			FRingEdge UV;
			FRingEdge VW;
			FRingEdge UW;
		};


		FConcentricRingsTessellationPattern(const FDynamicMesh3* InMesh) 
		:
		FTessellationPattern(InMesh)
		{
		}


		virtual ~FConcentricRingsTessellationPattern() 
		{
		}

		virtual EOperationValidationResult Validate() const override
		{
			return EOperationValidationResult::Ok;
		}

		void Init(const TArray<int>& InEdgeTessLevels, const TArray<int>& InInnerTessLevels) 
		{
			EdgeTessLevels = InEdgeTessLevels;
			InnerTessLevels = InInnerTessLevels;

			// If the inner area of the triangle patch is not being tessellated but at least one of its edges is, then 
			// we insert a single vertex in the middle and connect it to all of the edge vertices. Therefore, we find 
			// those triangles and set their inner tessellation level to 1. This would tell the TessellateTriPatch function 
			// to insert a vertex and generate the triangle fan.
			for (const int TriangleID : Mesh->TriangleIndicesItr()) 
			{
				if (InnerTessLevels[TriangleID] <= 0)
				{	
					const FIndex3i TriEdges = Mesh->GetTriEdges(TriangleID);

					if (EdgeTessLevels[TriEdges[0]] || EdgeTessLevels[TriEdges[1]] || EdgeTessLevels[TriEdges[2]])
					{
						InnerTessLevels[TriangleID] = 1;
					}
				}
			}
		}

		/** 
		 * Convenience method to convert an inner tessellation level from the number of vertices to the number of segments.
		 * 
		 * 		O---O---O  3 vertices => 2 segments
		 */
		inline int GetInnerLevelAsSegments(const int InTriangleID) const
		{
			return InnerTessLevels[InTriangleID] - 1;
		}

		virtual int GetNumberOfNewVerticesForEdgePatch(const int InEdgeID) const override 
		{
			if (Mesh->IsEdge(InEdgeID) == false) 
			{
				checkNoEntry();
				return InvalidIndex;
			}

			return EdgeTessLevels[InEdgeID];
		}


		virtual int GetNumberOfNewVerticesForTrianglePatch(const int InTriangleID) const override 
		{
			if (Mesh->IsTriangle(InTriangleID) == false) 
			{
				checkNoEntry();
				return InvalidIndex;
			}

			const int TessLevel = GetInnerLevelAsSegments(InTriangleID);
			if (TessLevel < 0) 
			{
				return 0;
			} 
			else if (TessLevel == 0) 
			{	
				return 1; // one vertex in the middle
			}

			// We iterate through rings and count how many vertices we are introducing per ring
			int NumNewTriangleVertices = 0;
			for (int RingLevel = TessLevel; RingLevel > 0; RingLevel -= 2) // next inner ring has 2 less segments
			{
				// 3 corners plus TessLevel - 1 vertcies along each of the 3 edges.
				NumNewTriangleVertices += 3 + 3 * (RingLevel - 1);
				
				// If the current ring only has 2 segments then we will be inserting one extra vertex in the middle
				NumNewTriangleVertices += RingLevel == 2 ? 1 : 0;
			}

			return NumNewTriangleVertices;
		}


		virtual int GetNumberOfPatchTriangles(const int InTriangleID) const override 
		{
			if (Mesh->IsTriangle(InTriangleID) == false) 
			{
				checkNoEntry();
				return InvalidIndex;
			}
		
			const int TessLevel = GetInnerLevelAsSegments(InTriangleID);

			// Count how many triangles are we generating when connecting outer ring with the first inner ring
			int NumNewTrianglesPerFace = 0;
			for (int EdgeIdx = 0; EdgeIdx < 3; ++EdgeIdx) 
			{
				const int EdgeID = Mesh->GetTriEdge(InTriangleID, EdgeIdx);
				const int OuterEdgeTriangles = this->GetNumberOfNewVerticesForEdgePatch(EdgeID) + 1;
				const int InnerEdgeTriangles = TessLevel;
				NumNewTrianglesPerFace += OuterEdgeTriangles + InnerEdgeTriangles;
			}

			// Iterate over every pair of inner rings and count how many triangles we are generating between them
			int RingLevel = TessLevel;
			while (RingLevel > 0) 
			{	
				const int CurRingLevel = RingLevel;
				const int NextRingLevel = RingLevel - 2;
				
				if (NextRingLevel < 0) 
				{	
					// no more rings left, so the current ring is a single triangle at the center
					checkSlow(CurRingLevel == 1);
					NumNewTrianglesPerFace += 1;
				}
				else 
				{
					NumNewTrianglesPerFace += 3 * (CurRingLevel + NextRingLevel);
				}
				
				RingLevel -= 2;
			}

			return NumNewTrianglesPerFace;
		}


		virtual void TessellateEdgePatch(EdgePatch& EdgePatch) const override
		{
			const int EdgeTessLevel = EdgeTessLevels[EdgePatch.EdgeID];
			
			checkSlow(EdgeTessLevel == EdgePatch.LinearCoord.Num());
			
			const int NumSegments = EdgeTessLevel + 1;
			const double Step = 1.0 / NumSegments;
			
			for (int Idx = 0; Idx < EdgeTessLevel; ++Idx)
			{
				double Value = (Idx + 1)*Step;
				EdgePatch.LinearCoord[Idx] = Value;
			}
		}


		virtual void TessellateTriPatch(TrianglePatch& TriPatch) const override
		{   
			FRing OuterRing;
			OuterRing.UV = FRingEdge(TriPatch.UVWCorners[0], TriPatch.UVWCorners[1], TriPatch.UVEdge);
			OuterRing.VW = FRingEdge(TriPatch.UVWCorners[1], TriPatch.UVWCorners[2], TriPatch.VWEdge);
			OuterRing.UW = FRingEdge(TriPatch.UVWCorners[2], TriPatch.UVWCorners[0], TriPatch.UWEdge);

			TArray<FRing> RingArray;
			RingArray.Add(OuterRing);

			// Track which vertex index we are currently working with
			int InnerIdx = 0;
		
            // We are working with an abstract patch whose vertex coordinates are the same as their barycentric coordinates
			FVector3d OuterU = FVector3d(1.0, 0.0, 0.0);
			FVector3d OuterV = FVector3d(0.0, 1.0, 0.0);
			FVector3d OuterW = FVector3d(0.0, 0.0, 1.0);

		    // Generate inner rings. Each subsequent inner ring will have its level reduced by 2.
            // The last inner ring can either be a single triangle or a single point.
			FVector3d InnerU, InnerV, InnerW;
			for (int TesLevel = GetInnerLevelAsSegments(TriPatch.TriangleID); TesLevel >= 0; TesLevel -= 2)
			{
				// Given the 3 corner vertices of the previous ring, compute the 3 corner vertices of this inner ring
				this->ComputeInnerConcentricTriangle(OuterU, OuterV, OuterW, TesLevel + 1, InnerU, InnerV, InnerW);

				FRing InnerRing;

				int StartIdx = InnerIdx; // save the index of the U corner since we will wrap back to it
				
				const double Step = TesLevel == 0 ? 0.0 : 1.0 / TesLevel;

				// Compute the barycentric coordinates for all vertices in the UV ring edge
				{
					TriPatch.BaryCoord[InnerIdx] = InnerU;
					for (int Idx = 1; Idx < TesLevel; ++Idx)
					{
						TriPatch.BaryCoord[InnerIdx + Idx] = InnerU + Idx*Step*(InnerV - InnerU);
					}
					TriPatch.BaryCoord[InnerIdx + TesLevel] = InnerV;


					if (TesLevel == 0) 
					{
						InnerRing.UV = FRingEdge(TriPatch.VIDs[InnerIdx]); // just a single vertex in the middle
					}
					else 
					{
						InnerRing.UV = FRingEdge(TriPatch.VIDs[InnerIdx], TriPatch.VIDs[InnerIdx + TesLevel], TriPatch.VIDs.Slice(InnerIdx + 1, TesLevel - 1));
					}
					
					InnerIdx += TesLevel;
				}

				// Compute the barycentric coordinates for all vertices in the VW ring edge
				{
					TriPatch.BaryCoord[InnerIdx] = InnerV;
					for (int Idx = 1; Idx < TesLevel; ++Idx)
					{
						TriPatch.BaryCoord[InnerIdx + Idx] = InnerV + Idx*Step*(InnerW - InnerV);
					}
					TriPatch.BaryCoord[InnerIdx + TesLevel] = InnerW;

					if (TesLevel == 0) 
					{
						InnerRing.VW = FRingEdge(TriPatch.VIDs[InnerIdx]);
					}
					else 
					{
						InnerRing.VW = FRingEdge(TriPatch.VIDs[InnerIdx], TriPatch.VIDs[InnerIdx + TesLevel], TriPatch.VIDs.Slice(InnerIdx + 1, TesLevel - 1));
					}

					InnerIdx += TesLevel;
				}

				// Compute the barycentric coordinates for all vertices in the UW ring edge
				{
					TriPatch.BaryCoord[InnerIdx] = InnerW;
					for (int Idx = 1; Idx < TesLevel; ++Idx)
					{
						TriPatch.BaryCoord[InnerIdx + Idx] = InnerW + Idx*Step*(InnerU - InnerW);
					}

					if (TesLevel == 0)
					{
						InnerRing.UW = FRingEdge(TriPatch.VIDs[InnerIdx]);
					}
					else 
					{
						InnerRing.UW = FRingEdge(TriPatch.VIDs[InnerIdx], TriPatch.VIDs[StartIdx], TriPatch.VIDs.Slice(InnerIdx + 1, TesLevel - 1));
					}

					InnerIdx += TesLevel;
				}

				OuterU = InnerU;
				OuterV = InnerV;
				OuterW = InnerW;

				RingArray.Add(InnerRing);
			}

            // Iterate over pair of rings and tessellate the area between them
			int TriangleIdx = 0; // Offset into TriPatch.Triangles where we will be adding new triangles
			for (int RingIdx = 0; RingIdx < RingArray.Num() - 1; ++RingIdx)
			{
				TriangleIdx += this->StitchRingEdges(RingArray[RingIdx].UV, RingArray[RingIdx + 1].UV, TriangleIdx, TriPatch.Triangles);
				TriangleIdx += this->StitchRingEdges(RingArray[RingIdx].VW, RingArray[RingIdx + 1].VW, TriangleIdx, TriPatch.Triangles);
				TriangleIdx += this->StitchRingEdges(RingArray[RingIdx].UW, RingArray[RingIdx + 1].UW, TriangleIdx, TriPatch.Triangles);
			}

            // If the tessellation level is odd, then we need to create one more triangle with the last inner ring vertices
			if (GetInnerLevelAsSegments(TriPatch.TriangleID)% 2 != 0) 
			{	
				const FRing& LastRing = RingArray.Last();
				TriPatch.Triangles[TriangleIdx] = FIndex3i(LastRing.UV[0], LastRing.UV[1], LastRing.VW[1]);
			}
		}


		/**
		 * Given two ring edges, "stitch" them together by generating a sequence of triangles connecting all vertices.
		 * 
		 *    Outer ring edge         O  O                            O--O  
		 *                                      	  ==>            /\  /\
		 *                                                          /  \/  \
		 *    Inner ring edge 1    O    O    O                     O----O---O 
		 * 
		 * @return the number of triangles generated
		 */
		int StitchRingEdges(const FRingEdge& InOuter, const FRingEdge& InInner, int Offset, TArrayView<FIndex3i>& OutTriangles) const 
		{
			const int OuterNum = InOuter.Num();
			const int InnerNum = InInner.Num();
			int TessDiff = InnerNum - OuterNum;
			
			const int NumTriangles = OuterNum - 1 + InnerNum - 1; // we know how many triangles to expect
			
            // Track which vertex are we currently at in the outer and inner vertex arrays
			int OuterIdx = 0;
			int InnerIdx = 0;

		    // We move along the outer and inner vertices and generate triangles. For each triangle, we need to choose which 
            // ring edge do we pick the third vertex from. Is it the OuterIdx + 1 or the InnerIdx + 1 vertex? We use the 
            // updated TessDiff variable to make the choice. If it's negative we choose the outer vertex, otherwise the inner vertex.
            // For more information see the Sec. 5.1 and Figure 8 in "Watertight Tessellation using Forward Differencing, H. Moreton, 2001."
            //
			//       OuterIdx	  
			//         \
			//          O  O     
			//                          
			//                          
			//       O    O    O 
			//      /
			//   InnerIdx
			
			for (int TriIndex = 0; TriIndex < NumTriangles; ++TriIndex) 
			{
				// If TessDiff is negative and the next vertex along outer side is available or we simply run out of inner vertices,
				// then use outer vertex as the third vertex to create a triangle 
				if ((TessDiff < 0 && OuterIdx + 1 < OuterNum) || InnerIdx == InnerNum - 1) 
				{
					OutTriangles[Offset + TriIndex] = FIndex3i(InOuter[OuterIdx], InOuter[OuterIdx + 1], InInner[InnerIdx]);
					TessDiff += 2*InnerNum;
					OuterIdx += 1;
				}
				else
				{
					ensure((TessDiff >= 0 && InnerIdx + 1 < InnerNum) || OuterIdx == OuterNum - 1);
					OutTriangles[Offset + TriIndex] = FIndex3i(InInner[InnerIdx], InOuter[OuterIdx], InInner[InnerIdx + 1]);
					TessDiff -= 2*OuterNum;
					InnerIdx += 1;
				}
			}
			
			return NumTriangles;
		}
	
		/** An array of per triangle (inner) tessellation levels. The size must match the maximum triangle ID of the mesh. */
		TArray<int> InnerTessLevels;

		/** An array of per edge tessellation levels. The size must match the maximum edge ID of the mesh. */
		TArray<int> EdgeTessLevels;
	};




	/** 
	 * Use the tessellation pattern to tessellate the mesh and generate the tessellation data. 
	 * 
	 * @return false if the tessellation failed or was cancelled by the user
	 */
	bool TessellateGeometry(const FDynamicMesh3* Mesh, 
							const FTessellationPattern* Pattern, 
							const bool bUseParallel, 
							FProgressCancel* Progress,
							FCompactMaps& OutCompactInfo, 
							FTessellationData& TessData,
							FDynamicMesh3* OutMesh,
							FSelectiveTessellate::FTessellationInformation& TessInfo) 
	{	

		TessData.Init(Pattern);

		// Tessellate edge patches
		TArray<int> EdgesToTessellate = TessData.EdgesToTessellate.Array();
		ParallelFor(EdgesToTessellate.Num(), [&](int32 Index)
		{
			if (Progress && Progress->Cancelled()) 
			{
				return;
			}

			FTessellationPattern::EdgePatch Patch;

			const int EdgeID = EdgesToTessellate[Index];
			Patch.EdgeID = EdgeID;
			Patch.LinearCoord = TessData.MapEdgeCoordBufferBlock(EdgeID);
			Patch.VIDs = TessData.MapEdgeIDBufferBlock(EdgeID);
			
			Pattern->TessellateEdgePatch(Patch);
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}
		
		// Tessellate triangle patches
		TArray<int> TrianglesToTessellate = TessData.TrianglesToTessellate.Array();
		ParallelFor(TrianglesToTessellate.Num(), [&](int32 Index)
		{
			if (Progress && Progress->Cancelled()) 
			{
				return;
			}

			FTessellationPattern::TrianglePatch Patch;
			
			const int TriangleID = TrianglesToTessellate[Index]; 
			
			Patch.TriangleID = TriangleID;
			
			FIndex3i TriVertices = Mesh->GetTriangle(TriangleID);
			Patch.UVWCorners = TriVertices;
			
			const FIndex3i& TriEdges = Mesh->GetTriEdgesRef(TriangleID);
			
			// UV Edge
			if (TessData.EdgesToTessellate.Contains(TriEdges[0]))
			{
				Patch.UVEdge.EdgeID = TriEdges[0];
				Patch.UVEdge.LinearCoord = TessData.MapEdgeCoordBufferBlock(TriEdges[0]);
				Patch.UVEdge.VIDs = TessData.MapEdgeIDBufferBlock(TriEdges[0]);
				Patch.UVEdge.bIsReversed = Mesh->GetEdgeV(TriEdges[0])[0] != TriVertices[0];
			}

			// VW Edge
			if (TessData.EdgesToTessellate.Contains(TriEdges[1]))
			{
				Patch.VWEdge.EdgeID = TriEdges[1];
				Patch.VWEdge.LinearCoord = TessData.MapEdgeCoordBufferBlock(TriEdges[1]);
				Patch.VWEdge.VIDs = TessData.MapEdgeIDBufferBlock(TriEdges[1]);
				Patch.VWEdge.bIsReversed = Mesh->GetEdgeV(TriEdges[1])[0] != TriVertices[1];
			}
			
			// UW Edge
			if (TessData.EdgesToTessellate.Contains(TriEdges[2]))
			{
				Patch.UWEdge.EdgeID = TriEdges[2];
				Patch.UWEdge.LinearCoord = TessData.MapEdgeCoordBufferBlock(TriEdges[2]);
				Patch.UWEdge.VIDs = TessData.MapEdgeIDBufferBlock(TriEdges[2]);
				Patch.UWEdge.bIsReversed = Mesh->GetEdgeV(TriEdges[2])[0] != TriVertices[2];
			}

			//  Inner Triangle Vertices
			Patch.BaryCoord = TessData.MapInnerCoordBufferBlock(TriangleID);
			Patch.VIDs = TessData.MapInnerIDBufferBlock(TriangleID);

			// Inner Triangles
			Patch.Triangles = TessData.MapTrianglesBufferBlock(TriangleID);

			// Tessellate the patch, i.e. generate inner vertices and triangles
			Pattern->TessellateTriPatch(Patch);
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		if (SelectiveTessellateLocals::ConstructTessellatedMesh(Mesh, 
														        Progress, 
															    TessData, 
															    OutCompactInfo, 
															    OutMesh, 
															    TessInfo) == false)
		{
			return false;
		}

		return true;
	}


	/** 
	 * Use the tessellation pattern to tessellate an overlay. 
	 * 
	 * @return false if the tessellation failed or was cancelled by the user
	 */
	template<typename RealType, int ElementSize> 
	bool TessellateOverlay(const FDynamicMesh3* Mesh, 
						   const TDynamicMeshOverlay<RealType, ElementSize>* Overlay,
						   const FTessellationPattern* Pattern, 
						   FTessellationData& MeshTessData,
						   const bool bUseParallel, 
						   FProgressCancel* Progress,
						   const FCompactMaps& CompactInfo,
						   TDynamicMeshOverlay<RealType, ElementSize>* OutOverlay)
	{	
		
		//TODO: The overlay tessellation data should only compute and contain the connectivity information since we 
	 	// can simply  reuse coordinates from the geometry tessellation (i.e. overlay elements live on the vertices). 
		SelectiveTessellateLocals::FOverlayTessellationData OverlayTessData(Mesh, Overlay);
		OverlayTessData.Init(Pattern);

		// Tessellate edge patches
		TArray<int> EdgesToTessellate = OverlayTessData.EdgesToTessellate.Array();
		ParallelFor(EdgesToTessellate.Num(), [&](int32 Index)
		{
			if (Progress && Progress->Cancelled()) 
			{
				return;
			}

			const int EdgeID = EdgesToTessellate[Index];
			
			const FIndex2i EdgeTri = Mesh->GetEdgeT(EdgeID);
			
			if (Overlay->IsSetTriangle(EdgeTri.A))
			{
				FTessellationPattern::EdgePatch PatchA;
				PatchA.EdgeID = EdgeID;
				PatchA.LinearCoord = OverlayTessData.MapEdgeCoordBufferBlock(EdgeID);
				PatchA.VIDs = OverlayTessData.MapEdgeIDBufferBlock(EdgeID, EdgeTri.A);
				Pattern->TessellateEdgePatch(PatchA);
			}

			if (EdgeTri.B != FDynamicMesh3::InvalidID && Overlay->IsSetTriangle(EdgeTri.B))
			{
				FTessellationPattern::EdgePatch PatchB;
				PatchB.EdgeID = EdgeID;
				PatchB.LinearCoord = OverlayTessData.MapEdgeCoordBufferBlock(EdgeID);
				PatchB.VIDs = OverlayTessData.MapEdgeIDBufferBlock(EdgeID, EdgeTri.B);
				Pattern->TessellateEdgePatch(PatchB);
			}
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		// Tessellate triangle patches
		TArray<int> TrianglesToTessellate = OverlayTessData.TrianglesToTessellate.Array();
		ParallelFor(TrianglesToTessellate.Num(), [&](int32 Index)
		{
			if (Progress && Progress->Cancelled()) 
			{
				return;
			}
			
			FTessellationPattern::TrianglePatch Patch;
			
			const int TriangleID = TrianglesToTessellate[Index]; 
			
			Patch.TriangleID = TriangleID;
			
			FIndex3i TriVertices = Mesh->GetTriangle(TriangleID);
			Patch.UVWCorners = Overlay->GetTriangle(TriangleID);
			
			const FIndex3i& TriEdges = Mesh->GetTriEdgesRef(TriangleID);
			
			// UV Edge
			if (OverlayTessData.EdgesToTessellate.Contains(TriEdges[0]))
			{
				Patch.UVEdge.EdgeID = TriEdges[0];
				Patch.UVEdge.LinearCoord = OverlayTessData.MapEdgeCoordBufferBlock(TriEdges[0]);
				Patch.UVEdge.VIDs = OverlayTessData.MapEdgeIDBufferBlock(TriEdges[0], TriangleID);
				Patch.UVEdge.bIsReversed = Mesh->GetEdgeV(TriEdges[0])[0] != TriVertices[0];
			}

			// VW Edge
			if (OverlayTessData.EdgesToTessellate.Contains(TriEdges[1]))
			{
				Patch.VWEdge.EdgeID = TriEdges[1];
				Patch.VWEdge.LinearCoord = OverlayTessData.MapEdgeCoordBufferBlock(TriEdges[1]);
				Patch.VWEdge.VIDs = OverlayTessData.MapEdgeIDBufferBlock(TriEdges[1], TriangleID);
				Patch.VWEdge.bIsReversed = Mesh->GetEdgeV(TriEdges[1])[0] != TriVertices[1];
			}
			
			// UW Edge
			if (OverlayTessData.EdgesToTessellate.Contains(TriEdges[2]))
			{
				Patch.UWEdge.EdgeID = TriEdges[2];
				Patch.UWEdge.LinearCoord = OverlayTessData.MapEdgeCoordBufferBlock(TriEdges[2]);
				Patch.UWEdge.VIDs = OverlayTessData.MapEdgeIDBufferBlock(TriEdges[2], TriangleID);
				Patch.UWEdge.bIsReversed = Mesh->GetEdgeV(TriEdges[2])[0] != TriVertices[2];
			}

			//  Inner Triangle Vertices
			Patch.BaryCoord = OverlayTessData.MapInnerCoordBufferBlock(TriangleID);
			Patch.VIDs = OverlayTessData.MapInnerIDBufferBlock(TriangleID);

			// Inner Triangles
			Patch.Triangles = OverlayTessData.MapTrianglesBufferBlock(TriangleID);

			// Tessellate the patch, i.e. generate inner vertices and triangles
			Pattern->TessellateTriPatch(Patch);
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		if (SelectiveTessellateLocals::ConstructTessellatedOverlay(Mesh, 
															  	   Overlay,
															  	   MeshTessData, 
															  	   OverlayTessData, 
															  	   CompactInfo, 
																   Progress, 
															  	   OutOverlay) == false) 
		{
			return false;
		}

		return true;
	}


	/**
	 * Construct a new triangle attribute from the tessellation data and compact information from the geometry tessellation.
	 */
	template<typename RealType, int ElementSize> 
	void ConstructTriangleAttribute(const FDynamicMesh3* Mesh, 
						   			 const TDynamicMeshTriangleAttribute<RealType, ElementSize>* Attribute,
						   			 FTessellationData& TessData,
						   			 const bool bUseParallel, 
						   			 const FCompactMaps& CompactInfo,
						   			 TDynamicMeshTriangleAttribute<RealType, ElementSize>* OutAttribute) 
	{
		RealType Value[ElementSize];

		for (const int TID : Mesh->TriangleIndicesItr())
		{
			Attribute->GetValue(TID, Value);

			if (TessData.TrianglesToTessellate.Contains(TID))
			{
				TArrayView<int> TriangleIDBlock = TessData.MapTriangleIDBufferBlock(TID);
				for (int Index = 0; Index < TriangleIDBlock.Num(); ++Index) 
				{
					const int NewTID = TriangleIDBlock[Index];
					const int ToTID = CompactInfo.GetTriangleMapping(NewTID);		
					OutAttribute->SetValue(ToTID, Value);
				}
			}
			else 
			{
				const int ToTID = CompactInfo.GetTriangleMapping(TID);		
				OutAttribute->SetValue(ToTID, Value);
			}
		}
	}

	/**
	 * Given the original FDynamicMesh3 and the tessellation data, construct new tessellated FDynamicMesh3 geometry.
	 * Output mesh will be compact. 
	 * 
	 * @param CompactInfo Stores vertex and triangle mappings.
	 * @return false if the user cancelled the operation.
	 */
	bool ConstructTessellatedMesh(const FDynamicMesh3* Mesh,
								  FProgressCancel* Progress,
								  FTessellationData& TessData,
								  FCompactMaps& CompactInfo,
								  FDynamicMesh3* ResultMesh,
								  FSelectiveTessellate::FTessellationInformation& TessInfo)
	{
		ResultMesh->Clear();

		const int ResultNumVertexIDs = Mesh->MaxVertexID() + TessData.EdgeIDs.Num() + TessData.InnerIDs.Num();
		CompactInfo.ResetVertexMap(ResultNumVertexIDs, false);

		const int ResultNumTriangleIDs = Mesh->MaxTriangleID() + TessData.Triangles.Num();
		CompactInfo.ResetTriangleMap(ResultNumTriangleIDs, false);

		// Append all of the vertices from the input mesh
		for (int VertexID = 0; VertexID < Mesh->MaxVertexID(); ++VertexID)
		{
			if (Mesh->IsVertex(VertexID))
			{
				CompactInfo.SetVertexMapping(VertexID, ResultMesh->AppendVertex(Mesh->GetVertex(VertexID)));
			}
			else 
			{
				CompactInfo.SetVertexMapping(VertexID, FCompactMaps::InvalidID);
			}
		}

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		// Append all of the vertices along tessellated edges
		for (const int EdgeID : TessData.EdgesToTessellate)
		{
			TArrayView<double> CoordBlock = TessData.MapEdgeCoordBufferBlock(EdgeID);
			TArrayView<int> IDBlock = TessData.MapEdgeIDBufferBlock(EdgeID);
			
			const FIndex2i EdgeV = Mesh->GetEdgeV(EdgeID);

			for (int Index = 0; Index < CoordBlock.Num(); ++Index)
			{	
				double Alpha = CoordBlock[Index];
				Alpha = FMath::Clamp(Alpha, 0.0, 1.0);

				const int VertexID = IDBlock[Index]; 
				
				const FVector3d Vertex = (1.0 - Alpha)*Mesh->GetVertex(EdgeV[0]) + Alpha*Mesh->GetVertex(EdgeV[1]);
				
				const int NewVertexID = ResultMesh->AppendVertex(Vertex);

				CompactInfo.SetVertexMapping(VertexID, NewVertexID);

				if (TessInfo.SelectedVertices) 
				{
					TessInfo.SelectedVertices->Add(NewVertexID);
				}
			}
		}

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		// Append all of the vertices inside tessellated triangles
		for (const int TriangleID : TessData.TrianglesToTessellate)
		{
			TArrayView<FVector3d> CoordBlock = TessData.MapInnerCoordBufferBlock(TriangleID);
			TArrayView<int> IDBlock = TessData.MapInnerIDBufferBlock(TriangleID);
			checkSlow(CoordBlock.Num() == IDBlock.Num());

			const FIndex3i TriangleV = Mesh->GetTriangle(TriangleID);

			for (int Index = 0; Index < CoordBlock.Num(); ++Index)
			{	
				const FVector3d Bary = CoordBlock[Index];
				const int VertexID = IDBlock[Index];

				const FVector3d Vertex = Bary[0] * Mesh->GetVertex(TriangleV[0]) + 
										 Bary[1] * Mesh->GetVertex(TriangleV[1]) + 
										 Bary[2] * Mesh->GetVertex(TriangleV[2]);
				
				const int NewVertexID = ResultMesh->AppendVertex(Vertex);
				CompactInfo.SetVertexMapping(VertexID, NewVertexID);

				if (TessInfo.SelectedVertices) 
				{
					TessInfo.SelectedVertices->Add(NewVertexID);
				}
			}
		}

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		if (Mesh->HasTriangleGroups()) 
		{
			ResultMesh->EnableTriangleGroups();
		}

		// Append all the triangles from the input mesh that we are not tessellating
		for (const int TriangleID : Mesh->TriangleIndicesItr())
		{
			if (TessData.TrianglesToTessellate.Contains(TriangleID) == false)
			{
				const FIndex3i Tri = Mesh->GetTriangle(TriangleID);
				const int TriGrp = Mesh->GetTriangleGroup(TriangleID);
				const FIndex3i MappedTri = CompactInfo.GetVertexMapping(Tri);
				const int NewTriID = ResultMesh->AppendTriangle(MappedTri, TriGrp);
				CompactInfo.SetTriangleMapping(TriangleID, NewTriID);

				if (TessInfo.TriangleIDToNewTriangleIDs)
				{
					TessInfo.TriangleIDToNewTriangleIDs->FindOrAdd(TriangleID).Add(NewTriID);
				}
			}
		}
		
		// Append all the new triangles generated during the tessellation
		for (const int TriangleID : TessData.TrianglesToTessellate) 
		{
			TArrayView<FIndex3i> TrianglesBlock = TessData.MapTrianglesBufferBlock(TriangleID);
			TArrayView<int> TriangleIDBlock = TessData.MapTriangleIDBufferBlock(TriangleID);
			checkSlow(TrianglesBlock.Num() == TriangleIDBlock.Num());
			const int TriGrp = Mesh->GetTriangleGroup(TriangleID);
			
			for (int Index = 0; Index < TriangleIDBlock.Num(); ++Index)
			{
				const int NewTriangleID = TriangleIDBlock[Index];
				const FIndex3i Tri = TrianglesBlock[Index];
				const FIndex3i MappedTri = CompactInfo.GetVertexMapping(Tri);
				const int MappedTriId = ResultMesh->AppendTriangle(MappedTri, TriGrp);
				CompactInfo.SetTriangleMapping(NewTriangleID, MappedTriId);

				if (TessInfo.TriangleIDToNewTriangleIDs)
				{
					TessInfo.TriangleIDToNewTriangleIDs->FindOrAdd(TriangleID).Add(MappedTriId);
				}
			}	 
		}
		
		if (TessInfo.SelectedVertices) 
		{
			// Add all of the original vertices of the triangles we are tessellating
			TSet<int> TempSetSelectedVertices; // first add them to tset to avoid duplicates
			for (const int TriangleID : TessData.TrianglesToTessellate) 
			{
				const FIndex3i Tri = Mesh->GetTriangle(TriangleID);
				const FIndex3i MappedTri = CompactInfo.GetVertexMapping(Tri);
				
				TempSetSelectedVertices.Add(MappedTri[0]);
				TempSetSelectedVertices.Add(MappedTri[1]);
				TempSetSelectedVertices.Add(MappedTri[2]);
			}

			for (int VID : TempSetSelectedVertices) 
			{
				TessInfo.SelectedVertices->Add(VID);
			}
		}

		//TODO: instead of generating the mesh from scratch, we could remove triangles we are tessellating from the 
		// input mesh and append the new ones. This can be an option when the number of triangles tessellated is small,
		// compared to the total number of triangles.

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		return true;
	}


	/**
	 * Construct a new overlay from the tessellation data and compact information from the geometry tessellation.
	 * 
	 * @return false if fails or the user cancells the operation
	 */
	template<typename RealType, int ElementSize> 
	bool ConstructTessellatedOverlay(const FDynamicMesh3* Mesh, 
									 const TDynamicMeshOverlay<RealType, ElementSize>* Overlay,
									 FTessellationData& MeshTessData, 
									 FOverlayTessellationData<RealType, ElementSize>& OverlayTessData, 
									 const FCompactMaps& CompactInfo,
									 FProgressCancel* Progress, 
								     TDynamicMeshOverlay<RealType, ElementSize>* ResultOverlay)  
	{	
		ResultOverlay->ClearElements();

		// Need to track the ID of the appended elements to handle non-compact meshes
		TMap<int, int> MapE; 
		MapE.Reserve(Overlay->ElementCount() + OverlayTessData.EdgeIDOffsets.Num());
				
		// Buffers to be reused
		RealType Element1[ElementSize];
		RealType Element2[ElementSize];
		RealType Element3[ElementSize];
		RealType Out[ElementSize];
		
		// First add all the existing elements
		for (const int ElementID : Overlay->ElementIndicesItr())
		{
			Overlay->GetElement(ElementID, Out);
			MapE.Add(ElementID, ResultOverlay->AppendElement(Out));
		}
		
		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}
		
		// Add all the new elements inserted along the edges by the tessellator
		for (const int EdgeID : OverlayTessData.EdgesToTessellate)
		{
			const FIndex2i EdgeTri = Mesh->GetEdgeT(EdgeID);
			const FIndex2i EdgeV = Mesh->GetEdgeV(EdgeID);
			TArrayView<double> CoordBlock = OverlayTessData.MapEdgeCoordBufferBlock(EdgeID);

			if (Overlay->IsSetTriangle(EdgeTri.A))  
			{
				TArrayView<int> IDBlock = OverlayTessData.MapEdgeIDBufferBlock(EdgeID, EdgeTri.A);
				checkSlow(IDBlock.Num() == CoordBlock.Num());
				
				Overlay->GetElementAtVertex(EdgeTri.A, EdgeV[0], Element1);
				Overlay->GetElementAtVertex(EdgeTri.A, EdgeV[1], Element2);

				for (int Index = 0; Index < IDBlock.Num(); ++Index)
				{	
					const RealType Alpha = (RealType)CoordBlock[Index];
					const int ElementID = IDBlock[Index]; 
					
					LerpElements<RealType, ElementSize>(Element1, Element2, Out, Alpha);
					
					MapE.Add(ElementID, ResultOverlay->AppendElement(Out));
				}
			}	

			if (EdgeTri.B != FDynamicMesh3::InvalidID && Overlay->IsSetTriangle(EdgeTri.B) && Overlay->IsSeamEdge(EdgeID))
			{
				TArrayView<int> IDBlock = OverlayTessData.MapEdgeIDBufferBlock(EdgeID, EdgeTri.B);
				checkSlow(IDBlock.Num() == CoordBlock.Num());

				Overlay->GetElementAtVertex(EdgeTri.B, EdgeV[0], Element1);
				Overlay->GetElementAtVertex(EdgeTri.B, EdgeV[1], Element2);

				for (int Index = 0; Index < IDBlock.Num(); ++Index)
				{	
					const RealType Alpha = (RealType)CoordBlock[Index];
					const int ElementID = IDBlock[Index]; 
					
					LerpElements<RealType, ElementSize>(Element1, Element2, Out, Alpha);
					
					MapE.Add(ElementID, ResultOverlay->AppendElement(Out));
				}
			}
		}

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		// Add all the elements added inside of the triangles by the tessellator
		for (const int TriangleID : OverlayTessData.TrianglesToTessellate)
		{
			const FIndex3i TriangleV = Mesh->GetTriangle(TriangleID);
			
			Overlay->GetElementAtVertex(TriangleID, TriangleV[0], Element1);
			Overlay->GetElementAtVertex(TriangleID, TriangleV[1], Element2);
			Overlay->GetElementAtVertex(TriangleID, TriangleV[2], Element3);

			TArrayView<FVector3d> CoordBlock = OverlayTessData.MapInnerCoordBufferBlock(TriangleID);
			TArrayView<int> IDBlock = OverlayTessData.MapInnerIDBufferBlock(TriangleID);

			checkSlow(CoordBlock.Num() == IDBlock.Num());
			for (int Index = 0; Index < CoordBlock.Num(); ++Index)
			{
				const FVector3d Bary = CoordBlock[Index]; 
				const int ElementID = IDBlock[Index];

				BaryElements<RealType, ElementSize>(Element1, Element2, Element3, Out, Bary[0], Bary[1], Bary[2]);

				MapE.Add(ElementID, ResultOverlay->AppendElement(Out));
			}
		}

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		// Set the overlay triangles and point them to the correct element ids
		for (const int TriangleID : Mesh->TriangleIndicesItr())
		{
			if (Overlay->IsSetTriangle(TriangleID) && OverlayTessData.TrianglesToTessellate.Contains(TriangleID) == false)
			{
				const FIndex3i ElementTri = Overlay->GetTriangle(TriangleID);
				const int ToTID = CompactInfo.GetTriangleMapping(TriangleID);
				ResultOverlay->SetTriangle(ToTID, FIndex3i(MapE[ElementTri.A], MapE[ElementTri.B], MapE[ElementTri.C]));
			}
		}

		for (const int TriangleID : OverlayTessData.TrianglesToTessellate)
		{
			if (ensure(Overlay->IsSetTriangle(TriangleID))) // TrianglesToTessellate set should already contain only triangle IDs set in the overlay
			{ 
				TArrayView<FIndex3i> TrianglesBlock = OverlayTessData.MapTrianglesBufferBlock(TriangleID);
				TArrayView<int> TriangleIDBlock = MeshTessData.MapTriangleIDBufferBlock(TriangleID);

				checkSlow(TrianglesBlock.Num() == TriangleIDBlock.Num());

				for (int Index = 0; Index < TrianglesBlock.Num(); ++Index)
				{
					const FIndex3i Tri = TrianglesBlock[Index];

					const int NewTriangleID = TriangleIDBlock[Index];
					const int ToTID = CompactInfo.GetTriangleMapping(NewTriangleID);

					ResultOverlay->SetTriangle(ToTID, FIndex3i(MapE[Tri.A], MapE[Tri.B], MapE[Tri.C]));
				}
			}
		}

		
		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		return true;
	}

	
	/** 
	 * General purpose function to interpolate any per-vertex data. The function will iterate over all vertices 
	 * inserted along the edges and call InterpolateEdgeFunc. Then it will iterate over all of the vertices inserted 
	 * inside of the triangles and call InterpolateInnerFunc.
	 * 
	 * @param InterpolateEdgeFunc A callback function with the signature void(int V1, int V2, int NewV, double U).
	 * 						      V1,V2 are the edge vertex indices into the original input mesh. NewV is the mapped 
	 * 							  vertex id into the tessellated mesh for which we are asking the function to set the 
	 * 							  value for. U is the linear interpolation coefficient of NewV with respect to V1,V2. 
	 * 
	 * @param InterpolateInnerFunc A callback function with the signature void(int V1, int V2, int V3, int NewV, double U, double V, double W).
	 * 						       V1,V2,V3 are the vertex indices into the original input mesh. NewV is the mapped 
	 * 							   vertex id into the tessellated mesh for which we are asking the function to set the 
	 * 							   value for. U,V,W are the barycentric coordinates of NewV with respect to V1,V2,V3.
	 */
	void InterpolateVertexData(const FDynamicMesh3* Mesh,
							   const TFunctionRef<void(int V1, int V2, int NewV, double U)> InterpolateEdgeFunc, 
							   const TFunctionRef<void(int V1, int V2, int V3, int NewV, double U, double V, double W)> InterpolateInnerFunc, 
						   	   FTessellationData& TessData,
						   	   const bool bUseParallel,
						   	   const FCompactMaps& CompactInfo)
	{
		// Set value for the vertices inserted along edges via interpolation
		ParallelFor(TessData.EdgesToTessellate.Num(), [&](int32 EIndex)
		{
			const int EdgeID = TessData.EdgesToTessellate[FSetElementId::FromInteger(EIndex)];
			
			TArrayView<double> CoordBlock = TessData.MapEdgeCoordBufferBlock(EdgeID);
			TArrayView<int> IDBlock = TessData.MapEdgeIDBufferBlock(EdgeID);
			checkSlow(CoordBlock.Num() == IDBlock.Num());

			const FIndex2i EdgeV = Mesh->GetEdgeV(EdgeID);

			for (int Index = 0; Index < CoordBlock.Num(); ++Index)
			{	
				const double Alpha = CoordBlock[Index];
				const int VID =  IDBlock[Index]; 
				const int ToVID = CompactInfo.GetVertexMapping(VID);

				InterpolateEdgeFunc(EdgeV.A, EdgeV.B, ToVID, Alpha);
			}
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		// Set value for the vertices inserted inside of the triangles via interpolation 
		ParallelFor(TessData.TrianglesToTessellate.Num(), [&](int32 TIndex)
		{
			const int TriangleID = TessData.TrianglesToTessellate[FSetElementId::FromInteger(TIndex)];

			TArrayView<FVector3d> CoordBlock = TessData.MapInnerCoordBufferBlock(TriangleID);
			TArrayView<int> IDBlock = TessData.MapInnerIDBufferBlock(TriangleID);
			checkSlow(CoordBlock.Num() == IDBlock.Num());

			const FIndex3i TriangleV = Mesh->GetTriangle(TriangleID);

			for (int Index = 0; Index < CoordBlock.Num(); ++Index)
			{	
				const FVector3d BaryCoords = CoordBlock[Index];
				const int VID = IDBlock[Index];
				const int ToVID = CompactInfo.GetVertexMapping(VID);

				InterpolateInnerFunc(TriangleV.A, TriangleV.B, TriangleV.C, ToVID, BaryCoords[0], BaryCoords[1], BaryCoords[2]);
			}
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
	}


	/** Interpolate any data stored in the TDynamicMeshVertexAttribute */
	template<typename RealType, int ElementSize>
	void ConstructDynamicMeshVertexAttribute(const FDynamicMesh3* Mesh,
								  		     const TDynamicMeshVertexAttribute<RealType, ElementSize>* Attribute,
						   		  		     FTessellationData& TessData,
						   		  		     const bool bUseParallel,
						   		  		     const FCompactMaps& CompactInfo,
											 TDynamicMeshVertexAttribute<RealType, ElementSize>* OutAttribute)
	{
		auto InterpolateEdgeFunc = [Attribute, OutAttribute] (int V1, int V2, int NewV, double U)
		{
			RealType Value1[ElementSize];
			RealType Value2[ElementSize];
			RealType OutValue[ElementSize];

			Attribute->GetValue(V1, Value1);
			Attribute->GetValue(V2, Value2);
			
			LerpElements<RealType, ElementSize>(Value1, Value2, OutValue, U);

			OutAttribute->SetValue(NewV, OutValue);
		};

		auto InterpolateInnerFunc = [Attribute, OutAttribute](int V1, int V2, int V3, int NewV, double U, double V, double W)
		{
			RealType Value1[ElementSize];
			RealType Value2[ElementSize];
			RealType Value3[ElementSize];
			RealType OutValue[ElementSize];

			Attribute->GetValue(V1, Value1);
			Attribute->GetValue(V2, Value2);

			Attribute->GetValue(V3, Value3);
			BaryElements<RealType, ElementSize>(Value1, Value2, Value3, OutValue, U, V, W);

			OutAttribute->SetValue(NewV, OutValue);
		};

		RealType Value[ElementSize];
		for (const int VID : Mesh-> VertexIndicesItr())
		{
			Attribute->GetValue(VID, Value);
			const int ToVID = CompactInfo.GetVertexMapping(VID);
			OutAttribute->SetValue(ToVID, Value);
		}

		InterpolateVertexData(Mesh, InterpolateEdgeFunc, InterpolateInnerFunc, TessData, bUseParallel, CompactInfo);
	}


	/** Interpolate skin weights */
	void ConstructVertexSkinWeightsAttribute(const FDynamicMesh3* Mesh,
								  		     const FDynamicMeshVertexSkinWeightsAttribute* Attribute,
						   		  		     FTessellationData& TessData,
						   		  		     const bool bUseParallel,
						   		  		     const FCompactMaps& CompactInfo,
											 FDynamicMeshVertexSkinWeightsAttribute* OutAttribute)
	{
		using FBoneWeights = UE::AnimationCore::FBoneWeights;
		
		auto InterpolateEdgeFunc = [Attribute, OutAttribute] (int V1, int V2, int NewV, double U)
		{
			FBoneWeights Value1;
			FBoneWeights Value2;
			FBoneWeights OutValue;

			Attribute->GetValue(V1, Value1);
			Attribute->GetValue(V2, Value2);
		
			U = FMath::Clamp(U, 0.0, 1.0);
			OutValue = FBoneWeights::Blend(Value1, Value2, (float)U);

			OutAttribute->SetValue(NewV, OutValue);
		};

		auto InterpolateInnerFunc = [Attribute, OutAttribute] (int V1, int V2, int V3, int NewV, double U, double V, double W)
		{
			FBoneWeights Value1, Value2, Value3, OutValue;

			Attribute->GetValue(V1, Value1);
			Attribute->GetValue(V2, Value2);
			Attribute->GetValue(V3, Value3);
			
			OutValue = FBoneWeights::Blend(Value1, Value2, Value3, (float)U, (float)V, (float)W);

			OutAttribute->SetValue(NewV, OutValue);
		};

		FBoneWeights Value;
		for (const int VID : Mesh-> VertexIndicesItr())
		{
			Attribute->GetValue(VID, Value);
			const int ToVID = CompactInfo.GetVertexMapping(VID);
			OutAttribute->SetValue(ToVID, Value);
		}

		InterpolateVertexData(Mesh, InterpolateEdgeFunc, InterpolateInnerFunc, TessData, bUseParallel, CompactInfo);
	}


	/** Interpolate per-vertex normals */
	void ConstructPerVertexNormals(const FDynamicMesh3* Mesh,
						   		   FTessellationData& TessData,
						   		   const bool bUseParallel,
						   		   const FCompactMaps& CompactInfo,
								   FDynamicMesh3* OutMesh)
	{
		auto InterpolateEdgeFunc = [Mesh, OutMesh] (int V1, int V2, int NewV, double U)
		{
			FVector3f OutValue;
			U = FMath::Clamp(U, 0.0, 1.0);
			OutValue =  (1.0 - U) * Mesh->GetVertexNormal(V1)  + U * Mesh->GetVertexNormal(V2);
			OutMesh->SetVertexNormal(NewV, OutValue);
		};

		auto InterpolateInnerFunc = [Mesh, OutMesh] (int V1, int V2, int V3, int NewV, double U, double V, double W)
		{
			FVector3f OutValue;
			checkSlow(FMath::Abs(U + V + W - 1.0) < KINDA_SMALL_NUMBER);
			OutValue = U * Mesh->GetVertexNormal(V1) + V * Mesh->GetVertexNormal(V2) + W * Mesh->GetVertexNormal(V3);
			OutMesh->SetVertexNormal(NewV, OutValue);
		};

		for (const int VID : Mesh->VertexIndicesItr())
		{
			const int ToVID = CompactInfo.GetVertexMapping(VID);
			OutMesh->SetVertexNormal(ToVID, Mesh->GetVertexNormal(VID));
		}

		InterpolateVertexData(Mesh, InterpolateEdgeFunc, InterpolateInnerFunc, TessData, bUseParallel, CompactInfo);
	}


	/** Interpolate per-vertex UVs */
	void ConstructPerVertexUVs(const FDynamicMesh3* Mesh,
						   	   FTessellationData& TessData,
						   	   const bool bUseParallel,
						   	   const FCompactMaps& CompactInfo,
							   FDynamicMesh3* OutMesh)
	{
		auto InterpolateEdgeFunc = [Mesh, OutMesh] (int V1, int V2, int NewV, double U)
		{
			FVector2f OutValue;
			U = FMath::Clamp(U, 0.0, 1.0);
			OutValue =  float(1.0 - U) * Mesh->GetVertexUV(V1)  + float(U) * Mesh->GetVertexUV(V2);
			OutMesh->SetVertexUV(NewV, OutValue);
		};

		auto InterpolateInnerFunc = [Mesh, OutMesh] (int V1, int V2, int V3, int NewV, double U, double V, double W)
		{
			FVector2f OutValue;
			checkSlow(FMath::Abs(U + V + W - 1.0) < KINDA_SMALL_NUMBER);
			OutValue = float(U) * Mesh->GetVertexUV(V1) + float(V) * Mesh->GetVertexUV(V2) + float(W) * Mesh->GetVertexUV(V3);
			OutMesh->SetVertexUV(NewV, OutValue);
		};

		for (const int VID : Mesh-> VertexIndicesItr())
		{
			const int ToVID = CompactInfo.GetVertexMapping(VID);
			OutMesh->SetVertexUV(ToVID, Mesh->GetVertexUV(VID));
		}

		InterpolateVertexData(Mesh, InterpolateEdgeFunc, InterpolateInnerFunc, TessData, bUseParallel, CompactInfo);
	}


	/** Interpolate per-vertex colors */
	void ConstructPerVertexColors(const FDynamicMesh3* Mesh,
						   	      FTessellationData& TessData,
						   	      const bool bUseParallel,
						   	      const FCompactMaps& CompactInfo,
							      FDynamicMesh3* OutMesh)
	{
		auto InterpolateEdgeFunc = [Mesh, OutMesh] (int V1, int V2, int NewV, double U)
		{
			FVector4f OutValue;
			U = FMath::Clamp(U, 0.0, 1.0);
			OutValue =  float(1.0 - U) * Mesh->GetVertexColor(V1)  + float(U) * Mesh->GetVertexColor(V2);
			OutMesh->SetVertexColor(NewV, OutValue);
		};

		auto InterpolateInnerFunc = [Mesh, OutMesh] (int V1, int V2, int V3, int NewV, double U, double V, double W)
		{
			FVector4f OutValue;
			checkSlow(FMath::Abs(U + V + W - 1.0) < KINDA_SMALL_NUMBER);
			OutValue = float(U) * Mesh->GetVertexColor(V1) + float(V) * Mesh->GetVertexColor(V2) + float(W) * Mesh->GetVertexColor(V3);
			OutMesh->SetVertexColor(NewV, OutValue);
		};

		for (const int VID : Mesh-> VertexIndicesItr())
		{
			const int ToVID = CompactInfo.GetVertexMapping(VID);
			OutMesh->SetVertexColor(ToVID, Mesh->GetVertexColor(VID));
		}

		InterpolateVertexData(Mesh, InterpolateEdgeFunc, InterpolateInnerFunc, TessData, bUseParallel, CompactInfo);
	}

	/** 
	 * Run the main tessellation logic for the geometry, overlays and attributes. 
	 * 
	 * @param OutMesh The resulting tessellated mesh.
	 * @return false if the tessellation failed or the user cancelled it
	 */
	bool Tessellate(const FDynamicMesh3* InMesh, 
					const FTessellationPattern* Pattern, 
					const bool bUseParallel, 
					FProgressCancel* Progress, 
					FDynamicMesh3* OutMesh,
					FSelectiveTessellate::FTessellationInformation& TessInfo)
	{
		OutMesh->Clear();

		FCompactMaps CompactInfo;

		SelectiveTessellateLocals::FTessellationData TessData(InMesh);
		if (TessellateGeometry(InMesh, Pattern, bUseParallel, Progress, CompactInfo, TessData, OutMesh, TessInfo) == false) 
		{
			return false;
		}

		if (InMesh->HasAttributes()) 
		{	
			OutMesh->EnableAttributes(); 

			const FDynamicMeshAttributeSet* InAttributes = InMesh->Attributes(); 
			FDynamicMeshAttributeSet* OutAttributes = OutMesh->Attributes(); 
			
			if (InAttributes->NumNormalLayers()) 
			{
				OutAttributes->SetNumNormalLayers(InAttributes->NumNormalLayers());
				for (int Idx = 0; Idx < InAttributes->NumNormalLayers(); ++Idx) 
				{	
					if (TessellateOverlay(InMesh, InAttributes->GetNormalLayer(Idx), Pattern, TessData, bUseParallel, Progress, CompactInfo,  OutAttributes->GetNormalLayer(Idx)) == false) 
					{
						return false;
					}
				} 
			}

			if (InAttributes->NumUVLayers()) 
			{
				OutAttributes->SetNumUVLayers(InAttributes->NumUVLayers());
				for (int Idx = 0; Idx < InAttributes->NumUVLayers(); ++Idx) 
				{	
					if (TessellateOverlay(InMesh, InAttributes->GetUVLayer(Idx), Pattern, TessData, bUseParallel, Progress, CompactInfo, OutAttributes->GetUVLayer(Idx)) == false)
					{
						return false;
					}
				}
			}

			if (InAttributes->HasPrimaryColors()) 
			{
				OutAttributes->EnablePrimaryColors();
				if (TessellateOverlay(InMesh, InAttributes->PrimaryColors(), Pattern, TessData, bUseParallel, Progress, CompactInfo, OutAttributes->PrimaryColors()) == false) 
				{
					return false;
				}
			}

			if (InAttributes->HasMaterialID())	 
			{
				OutAttributes->EnableMaterialID();
				OutAttributes->GetMaterialID()->SetName(InAttributes->GetMaterialID()->GetName());
				ConstructTriangleAttribute(InMesh, InAttributes->GetMaterialID(), TessData, bUseParallel, CompactInfo, OutAttributes->GetMaterialID());
			
				if (Progress && Progress->Cancelled()) 
				{
					return false;
				}
			}

			
			if (InAttributes->NumPolygroupLayers())  
			{
				OutAttributes->SetNumPolygroupLayers(InAttributes->NumPolygroupLayers());
				for (int Idx = 0; Idx < InAttributes->NumPolygroupLayers(); ++Idx) 
				{	
					OutAttributes->GetPolygroupLayer(Idx)->SetName(InAttributes->GetPolygroupLayer(Idx)->GetName());
					ConstructTriangleAttribute(InMesh, InAttributes->GetPolygroupLayer(Idx), TessData, bUseParallel, CompactInfo, OutAttributes->GetPolygroupLayer(Idx));
				}

				if (Progress && Progress->Cancelled()) 
				{
					return false;
				}
			}

			for (const TTuple<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttributeInfo : InAttributes->GetSkinWeightsAttributes())
			{
				FDynamicMeshVertexSkinWeightsAttribute* SkinAttribute = new FDynamicMeshVertexSkinWeightsAttribute(OutMesh);
				ConstructVertexSkinWeightsAttribute(InMesh, AttributeInfo.Value.Get(), TessData, bUseParallel, CompactInfo, SkinAttribute);
				SkinAttribute->SetName(AttributeInfo.Value.Get()->GetName());
				OutAttributes->AttachSkinWeightsAttribute(AttributeInfo.Key, SkinAttribute);
			}

			for (const TTuple<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttributeInfo : InAttributes->GetMorphTargetAttributes())
			{
				FDynamicMeshMorphTargetAttribute* MorphTargetAttribute = new FDynamicMeshMorphTargetAttribute(OutMesh);
				ConstructDynamicMeshVertexAttribute<float, 3>(InMesh, AttributeInfo.Value.Get(), TessData, bUseParallel, CompactInfo, MorphTargetAttribute);
				MorphTargetAttribute->SetName(AttributeInfo.Value.Get()->GetName());
				OutAttributes->AttachMorphTargetAttribute(AttributeInfo.Key, MorphTargetAttribute);
			}

			if (Progress && Progress->Cancelled()) 
			{
				return false;
			}

			if (InAttributes->NumWeightLayers() > 0)
			{
				OutAttributes->SetNumWeightLayers(InAttributes->NumWeightLayers());
				
				for (int Idx = 0; Idx < InAttributes->NumWeightLayers(); ++Idx)
				{
					ConstructDynamicMeshVertexAttribute<float, 1>(InMesh, InAttributes->GetWeightLayer(Idx), TessData, bUseParallel, CompactInfo, OutAttributes->GetWeightLayer(Idx));
					OutAttributes->GetWeightLayer(Idx)->SetName(InAttributes->GetWeightLayer(Idx)->GetName());
				}
			}

			if (Progress && Progress->Cancelled()) 
			{
				return false;
			}
		}
		
		if (InMesh->HasVertexNormals()) 
		{	
			OutMesh->EnableVertexNormals(FVector3f::Zero());
			ConstructPerVertexNormals(InMesh, TessData, bUseParallel, CompactInfo, OutMesh);
		}
		
		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		if (InMesh->HasVertexUVs()) 
		{	
			OutMesh->EnableVertexUVs(FVector2f::Zero());
			ConstructPerVertexUVs(InMesh, TessData, bUseParallel, CompactInfo, OutMesh);
		}

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		if (InMesh->HasVertexColors()) 
		{	
			OutMesh->EnableVertexColors(FVector4f::Zero());
			ConstructPerVertexColors(InMesh, TessData, bUseParallel, CompactInfo, OutMesh);
		}

		if (Progress && Progress->Cancelled()) 
		{
			return false;
		}

		return true;
	}
}


//
// GLSL pattern
//

TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateConcentricRingsTessellationPattern(const FDynamicMesh3* InMesh,	
												   											    const TArray<int>& InEdgeTessLevels,
                                                   											    const TArray<int>& InInnerTessLevels)
{
	TUniquePtr<SelectiveTessellateLocals::FConcentricRingsTessellationPattern> Pattern = MakeUnique<SelectiveTessellateLocals::FConcentricRingsTessellationPattern>(InMesh);
	Pattern->Init(InEdgeTessLevels, InInnerTessLevels);
	
	return Pattern;
}


TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateConcentricRingsTessellationPattern(const FDynamicMesh3* InMesh,
												   											    const int InTessellationLevel)
{
	TArray<int> InEdgeTessLevels;
	InEdgeTessLevels.Init(InTessellationLevel, InMesh->MaxEdgeID());
	
	TArray<int> InInnerTessLevels;
	InInnerTessLevels.Init(InTessellationLevel, InMesh->MaxTriangleID());
	
	return FSelectiveTessellate::CreateConcentricRingsTessellationPattern(InMesh, InEdgeTessLevels, InInnerTessLevels);
}

 
TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateConcentricRingsTessellationPattern(const FDynamicMesh3* InMesh,  
								   							  					    		    const int InTessellationLevel,
								   							  					    		    const TArray<int>& InTriangleList) 
{
	if (InTriangleList.IsEmpty()) 
	{
		return nullptr;
	}

	TArray<int> EdgeTessLevels;
	EdgeTessLevels.Init(0, InMesh->MaxEdgeID());
	for (const int TriangleID : InTriangleList) 
	{
		const FIndex3i EdgesIdx = InMesh->GetTriEdges(TriangleID);
		EdgeTessLevels[EdgesIdx[0]] = InTessellationLevel;
		EdgeTessLevels[EdgesIdx[1]] = InTessellationLevel;
		EdgeTessLevels[EdgesIdx[2]] = InTessellationLevel;
	}

	TArray<int> TriangleTessLevels;
	TriangleTessLevels.Init(0, InMesh->MaxTriangleID());

	for (const int TriangleID : InTriangleList) 
	{
		TriangleTessLevels[TriangleID] = InTessellationLevel; 
	}

	return FSelectiveTessellate::CreateConcentricRingsTessellationPattern(InMesh, EdgeTessLevels, TriangleTessLevels);	
}


TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateConcentricRingsTessellationPattern(const TFunctionRef<int(const int EdgeID)> InEdgeFunc, 
                                                               					    		    const TFunctionRef<int(const int TriangleID)> InTriFunc) 
{

	// TODO: not implemented yet
	checkNoEntry();
	return nullptr;
}


TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateConcentricRingsPatternFromTriangleGroup(const FDynamicMesh3* InMesh,
																					 			     const int InTessellationLevel,
                                                                                     			     const int InPolygroupID) 
{
	if (InMesh->HasTriangleGroups() == false)
	{
		return nullptr;
	}

	TArray<int> TriangleList;
	for (const int TID : InMesh->TriangleIndicesItr()) 
	{
		const int PolygroupID = InMesh->GetTriangleGroup(TID);
		
		if (PolygroupID == InPolygroupID) 
		{
			TriangleList.Add(TID);
		}
	}

	return FSelectiveTessellate::CreateConcentricRingsTessellationPattern(InMesh, InTessellationLevel, TriangleList);
}


TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateConcentricRingsPatternFromPolyGroup(const FDynamicMesh3* InMesh,
                                                                           						 const int InTessellationLevel,
                                                                           						 const FString& InLayerName,
                                                                           						 const int InPolygroupID) 
{
	if (InMesh->HasAttributes() == false)
	{
		return nullptr;
	}

	const FDynamicMeshPolygroupAttribute* PolygroupAttribute = nullptr;

	for (int Idx = 0; Idx < InMesh->Attributes()->NumPolygroupLayers(); ++Idx) 
	{
		if (InMesh->Attributes()->GetPolygroupLayer(Idx)->GetName().ToString() == InLayerName) 
		{
			PolygroupAttribute = InMesh->Attributes()->GetPolygroupLayer(Idx);
			break;
		}
	}

	if (PolygroupAttribute == nullptr) 
	{
		return nullptr;
	}

	TArray<int> TriangleList;
	for (const int TID : InMesh->TriangleIndicesItr()) 
	{
		const int TrianglePolygroupID = PolygroupAttribute->GetValue(TID);
		
		if (TrianglePolygroupID == InPolygroupID) 
		{
			TriangleList.Add(TID);
		}
	}

	return FSelectiveTessellate::CreateConcentricRingsTessellationPattern(InMesh, InTessellationLevel, TriangleList);
}


TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateConcentricRingsPatternFromMaterial(const FDynamicMesh3* InMesh,
																					 		    const int InTessellationLevel,
                                                                                     		    const int MaterialID) 
{
	if (InMesh->HasAttributes() == false)
	{
		return nullptr;
	}

	if (InMesh->Attributes()->HasMaterialID() == false)
	{
		return nullptr;
	}

	TArray<int> TriangleList;
	for (const int TID : InMesh->TriangleIndicesItr()) 
	{
		const int TriangleMaterialID = InMesh->Attributes()->GetMaterialID()->GetValue(TID);
		
		if (TriangleMaterialID == MaterialID) 
		{
			TriangleList.Add(TID);
		}
	}

	return FSelectiveTessellate::CreateConcentricRingsTessellationPattern(InMesh, InTessellationLevel, TriangleList);
}

TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateConcentricRingsPatternFromSelectionAndMaterial(const FDynamicMesh3* InMesh,
																								 const int InTessellationLevel,
																								 const int MaterialID,
																								 const TArray<int>& SelectedTriangles) 
{
	if (InMesh->HasAttributes() == false || InMesh->Attributes()->HasMaterialID() == false || SelectedTriangles.IsEmpty())
	{
		return nullptr;
	}

	TArray<int> SelectedTrianglesWithMaterial;
	for (const int TID : SelectedTriangles)
	{
		const int TriangleMaterialID = InMesh->Attributes()->GetMaterialID()->GetValue(TID);
		if (TriangleMaterialID == MaterialID)
		{
			SelectedTrianglesWithMaterial.Add(TID);
		}
	}

	return FSelectiveTessellate::CreateConcentricRingsTessellationPattern(InMesh, InTessellationLevel, SelectedTrianglesWithMaterial);
}

//
// Inner uniform pattern
//

TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateInnerUnifromTessellationPattern(const FDynamicMesh3* InMesh,
													  	   						 		     const TArray<int>& InEdgeTessLevels,
                                                      	   									 const TArray<int>& InInnerTessLevels)
{
	// TODO: not implemented yet
	checkNoEntry();
	return nullptr;
}




//
// Uniform pattern
//
TUniquePtr<FTessellationPattern> CreateUniformTessellationPattern(const FDynamicMesh3* InMesh,
																  const int InTessellationLevel,
																  const TArray<int>& InTriangleList) 
{
	// TODO: not implemented yet
	checkNoEntry();
	return nullptr;
}

TUniquePtr<FTessellationPattern> FSelectiveTessellate::CreateRedGreenTessellationPattern(const FDynamicMesh3* InMesh, const TArray<int>& InTriangleTessLevels)
{
	TUniquePtr<FRedGreenTessellationPattern> Pattern = MakeUnique<FRedGreenTessellationPattern>(InMesh, InTriangleTessLevels);
	return Pattern;
}


void FTessellationPattern::ComputeInnerConcentricTriangle(const FVector3d& V1,
                                        				  const FVector3d& V2,
                                        				  const FVector3d& V3,
                                        				  const int EdgeTessLevel,
                                        				  FVector3d& InnerU,
                                        				  FVector3d& InnerV,
                                        				  FVector3d& InnerW) const
{	
	//TODO: Handle the case where V1, V2, V3 form a degenerate triangle 
    
	// Given two lines defined by a point and a normal find their intersection. It is guaranteed 
    // that both lines are on the same plane and that they are not parallel.
	auto IntersectLines = [] (const FVector3d& V1, const FVector3d& N1, const FVector3d& V2, const FVector3d& N2) 
	{
		FLine3d Line1(V1, N1);
		FLine3d Line2(V2, N2);
		FDistLine3Line3d LineDist(Line1, Line2); 
		LineDist.ComputeResult();
		FVector3d OffsetPoint = 0.5 * (LineDist.Line1ClosestPoint + LineDist.Line2ClosestPoint);
		return OffsetPoint;
	};


	if (EdgeTessLevel < 0) 
	{
		checkSlow(false);
		return;
	}

	if (EdgeTessLevel == 0) // edges are not tessellated
	{
		InnerU = V1;
		InnerV = V2;
		InnerW = V3;
		return;
	}
	else if (EdgeTessLevel == 1) // just a vertex in the middle
	{
		FVector3d Center = (V1 + V2 + V3)/3.0;
		InnerU = Center;
		InnerV = Center;
		InnerW = Center;
		return;
	}

	// Inserted vertices on each subdivided edge closest to the edge corners
	FVector3d EdgeDir1 = (V2 - V1) / (EdgeTessLevel + 1);
	FVector3d UV1 = V1 + EdgeDir1;
	FVector3d UV2 = V1 + EdgeTessLevel*EdgeDir1;

	FVector3d EdgeDir2 = (V3 - V1) / (EdgeTessLevel + 1);
	FVector3d UW1 = V1 + EdgeDir2;
	FVector3d UW2 = V1 + EdgeTessLevel*EdgeDir2;

	FVector3d EdgeDir3 = (V3 - V2) / (EdgeTessLevel + 1);
	FVector3d VW1 = V2 + EdgeDir3;
	FVector3d VW2 = V2 + EdgeTessLevel*EdgeDir3;
	
	// Normal vector of the plane formed by the triangle
	FVector3d PlaneNormal = Normalized(EdgeDir1.Cross(EdgeDir2));
	
	// Edge normal vectors
	FVector3d OuterUVNormal = Normalized(PlaneNormal.Cross(EdgeDir1));
	FVector3d OuterUWNormal = Normalized(EdgeDir2.Cross(PlaneNormal));
	FVector3d OuterVWNormal = Normalized(PlaneNormal.Cross(EdgeDir3));

	// Vertices of the inner triangle
	InnerU = IntersectLines(UV1, OuterUVNormal, UW1, OuterUWNormal);
	InnerV = IntersectLines(UV2, OuterUVNormal, VW1, OuterVWNormal);
	InnerW = IntersectLines(UW2, OuterUWNormal, VW2, OuterVWNormal);
}


bool FSelectiveTessellate::Cancelled()
{
	return (Progress == nullptr) ? false : Progress->Cancelled();
}


bool FSelectiveTessellate::Compute()
{	
	if (Validate() != EOperationValidationResult::Ok) 
	{
		return false;
	}

	// Check for the empty meshes
	if (bInPlace && ResultMesh->TriangleCount() == 0) 
	{
		return true;
	}

	if (bInPlace == false && Mesh->TriangleCount() == 0) 
	{
		return true;
	}

	// Make a copy of the mesh since we are tessellating in place. The copy will be used to restore the mesh to its 
	// original state in case the user cancelled the operation.
	TUniquePtr<FDynamicMesh3> ResultMeshCopy = nullptr;
	if (bInPlace)
	{
		ResultMeshCopy = MakeUnique<FDynamicMesh3>();
		ResultMeshCopy->Copy(*ResultMesh);
		Mesh = ResultMeshCopy.Get();
	}

	bool bTessResult = SelectiveTessellateLocals::Tessellate(Mesh, Pattern, bUseParallel, Progress, ResultMesh, TessInfo);

	if (Cancelled() || bTessResult == false)
	{
		if (bInPlace) 
		{ 
			// Restore the input mesh
			checkSlow(ResultMeshCopy != nullptr);
			*ResultMesh = MoveTemp(*ResultMeshCopy);
		}
		else 
		{
			ResultMesh->Clear();
		}

		return false;
	}
	
	return true;
}