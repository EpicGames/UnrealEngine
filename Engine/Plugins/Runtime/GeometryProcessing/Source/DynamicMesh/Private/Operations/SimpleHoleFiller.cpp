// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/SimpleHoleFiller.h"
#include "DynamicMeshEditor.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Operations/HoleFillUtil.h"

using namespace UE::Geometry;


bool FSimpleHoleFiller::Fill(int GroupID)
{
	if (GroupID < 0 && Mesh->HasTriangleGroups())
	{
		GroupID = Mesh->AllocateTriangleGroup();
	}

	if (Loop.GetVertexCount() < 3)
	{
		return false;
	}

	// this just needs one triangle
	if (Loop.GetVertexCount() == 3)
	{
		FIndex3i Tri(Loop.Vertices[0], Loop.Vertices[2], Loop.Vertices[1]);
		int NewTID = Mesh->AppendTriangle(Tri, GroupID);
		if (NewTID < 0)
		{
			return false;
		}
		NewTriangles = { NewTID };
		NewVertex = FDynamicMesh3::InvalidID;
		if (bAutoFillPrimaryColors)
		{
			FillColors();
		}
		return true;
	}

	// [TODO] 4-case? could check nbr normals to figure out best internal edge...

	bool bOK = false;
	if (FillType == EFillType::PolygonEarClipping)
	{
		bOK = Fill_EarClip(GroupID);
	}
	else
	{
		bOK = Fill_Fan(GroupID);
	}
	if (bOK && bAutoFillPrimaryColors)
	{
		FillColors();
	}

	return bOK;
}


bool FSimpleHoleFiller::Fill_Fan(int GroupID)
{
	// compute centroid
	FVector3d c = FVector3d::Zero();
	for (int i = 0; i < Loop.GetVertexCount(); ++i)
	{
		c += Mesh->GetVertex(Loop.Vertices[i]);
	}
	c *= 1.0 / Loop.GetVertexCount();

	// add centroid vtx
	NewVertex = Mesh->AppendVertex(c);

	// stitch triangles
	FDynamicMeshEditor Editor(Mesh);
	FDynamicMeshEditResult AddFanResult;
	if (!Editor.AddTriangleFan_OrderedVertexLoop(NewVertex, Loop.Vertices, GroupID, AddFanResult))
	{
		constexpr bool bPreserveManifold = false;
		Mesh->RemoveVertex(NewVertex, false);
		NewVertex = FDynamicMesh3::InvalidID;
		return false;
	}
	NewTriangles = AddFanResult.NewTriangles;

	return true;
}




bool FSimpleHoleFiller::Fill_EarClip(int GroupID)
{
	TArray<FVector3d> Vertices;
	int32 NumVertices = Loop.GetVertexCount();
	for (int32 i = 0; i < NumVertices; ++i)
	{
		Vertices.Add(Mesh->GetVertex(Loop.Vertices[i]));
	}

	TArray<FIndex3i> Triangles;
	PolygonTriangulation::TriangulateSimplePolygon(Vertices, Triangles);

	for (FIndex3i PolyTriangle : Triangles)
	{
		FIndex3i MeshTriangle(
			Loop.Vertices[PolyTriangle.A],
			Loop.Vertices[PolyTriangle.B],
			Loop.Vertices[PolyTriangle.C]);
		int32 NewTriangle = Mesh->AppendTriangle(MeshTriangle, GroupID);
		if (NewTriangle >= 0)
		{
			NewTriangles.Add(NewTriangle);
		}
	}

	return NewTriangles.Num() == Triangles.Num();
}


void FSimpleHoleFiller::FillColors()
{
	HoleFillUtil::FillColorOverlay(*Mesh, Loop.Edges, NewTriangles, NewVertex);
}


bool FSimpleHoleFiller::UpdateAttributes(TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>>& VidUVMaps)
{
	if (!Mesh->HasAttributes() || NewTriangles.Num() == 0)
	{
		return false;
	}

	FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();

	if (!ensure(VidUVMaps.Num() == Attributes->NumUVLayers()))
	{
		return false;
	}

	for (int i = 0; i < Attributes->NumUVLayers(); ++i)
	{
		FDynamicMeshUVOverlay* UVLayer = Attributes->GetUVLayer(i);
		FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>& UVMap = VidUVMaps[i];

		// Create an entry for the newly added center vert if we need it
		if (NewVertex != IndexConstants::InvalidID && !UVMap.Contains(NewVertex))
		{
			FVector2f NewVertUV = FVector2f::Zero();
			for (int32 Vid : Loop.Vertices)
			{
				NewVertUV += UVMap[Vid].Value;
			}
			NewVertUV /= (float)Loop.Vertices.Num();
			UVMap.Add(NewVertex, FMeshRegionBoundaryLoops::ElementIDAndValue<FVector2f>(
				IndexConstants::InvalidID, NewVertUV));
		}

		for (int32 Tid : NewTriangles)
		{
			FIndex3i TriVids = Mesh->GetTriangle(Tid);
			FIndex3i TriUVs;
			for (int32 j = 0; j < 3; ++j)
			{
				int32 Vid = TriVids[j];

				if (!UVMap.Contains(Vid))
				{
					return false;
				}

				FMeshRegionBoundaryLoops::ElementIDAndValue<FVector2f>& VertUVInfo = UVMap[Vid];
				if (VertUVInfo.Key == IndexConstants::InvalidID)
				{
					TriUVs[j] = UVLayer->AppendElement(VertUVInfo.Value);
					VertUVInfo.Key = TriUVs[j];
				}
				else if (UVLayer->IsElement(VertUVInfo.Key))
				{
					TriUVs[j] = VertUVInfo.Key;
				}
				else
				{
					return false;
				}
			}
			UVLayer->SetTriangle(Tid, TriUVs);
		}
	}

	FDynamicMeshEditor MeshEditor(Mesh);
	MeshEditor.SetTriangleNormals(NewTriangles);
	return true;
}