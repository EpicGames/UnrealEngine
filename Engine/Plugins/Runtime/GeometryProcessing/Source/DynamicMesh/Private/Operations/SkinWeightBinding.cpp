// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/SkinWeightBinding.h"

#include "Algo/RemoveIf.h"
#include "BoneWeights.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "Spatial/DenseGrid3.h"
#include "Spatial/MeshWindingNumberGrid.h"
#include "Spatial/OccupancyGrid3.h"


namespace UE::Geometry::SkinWeightBindingLocals
{

// A simple FIFO queue. Maintains a set of blocks, rather than allocate for each element.
template<typename T, int32 BlockSize=256>
class TFIFOQueue
{
public:
	TFIFOQueue() = default;

	void Push(T InElem)
	{
		// Container Empty?
		if (Blocks.IsEmpty())
		{
			Blocks.SetNum(1);
			Blocks[0].bIsFree = false;
			PushIndex = -1;
		}
		// Or are we at the end of current block?
		else if (PushIndex == (BlockSize - 1))
		{
			PushBlock = AllocateNewPushBlock();
			PushIndex = -1;
		}

		PushIndex++;

		Blocks[PushBlock].Data[PushIndex] = InElem;
	}

	T Pop()
	{
		check(!IsEmpty());

		if (PopIndex == (BlockSize - 1))
		{
			// Reached the end. Free this block and move onto the next one.
			PopBlock = FreeCurrentPopBlock();
			PopIndex = -1;
		}

		PopIndex++;
		return Blocks[PopBlock].Data[PopIndex];
	}

	bool TryPop(T &OutValue)
	{
		if (IsEmpty())
		{
			return false;
		}

		OutValue = Pop();
		return true;
	}

	void Reset()
	{
		Blocks.Reset();
		PushBlock = 0;
		PushIndex = -1;
		PopBlock = 0;
		PopIndex = -1;
		FreeCount = 0;
	}
	

	bool IsEmpty() const
	{
		return PushBlock == PopBlock && PushIndex == PopIndex;
	}

private:
	int32 AllocateNewPushBlock()
	{
		checkSlow(PushBlock != -1);

		int NewBlockIndex = INDEX_NONE;
		
		if (FreeCount > 0)
		{
			// Find a free block
			for (int32 Index = 0; Index < Blocks.Num(); Index++)
			{
				if (Blocks[Index].bIsFree)
				{
					NewBlockIndex = Index;
					FreeCount--;
					break;
				}
			}
			checkfSlow(NewBlockIndex != INDEX_NONE, TEXT("We should have found a free block."));
		}
		else
		{
			NewBlockIndex = Blocks.Num();
			Blocks.AddDefaulted();
		}

		Blocks[PushBlock].NextBlock = NewBlockIndex;
		Blocks[NewBlockIndex].bIsFree = false;
		Blocks[NewBlockIndex].NextBlock = INDEX_NONE;
		return NewBlockIndex;
	}

	int32 FreeCurrentPopBlock()
	{
		const int32 NextBlock = Blocks[PopBlock].NextBlock;
		checkSlow(NextBlock != -1);
		Blocks[PopBlock].bIsFree = true;
		FreeCount++;

		return NextBlock;
	}
	

	int32 PushBlock = 0;
	int32 PushIndex = -1;

	int32 PopBlock = 0;
	int32 PopIndex = -1;

	int32 FreeCount = 0;
	
	struct FBlock
	{
		FBlock()
		{
			Data = new T[BlockSize];
		}
		~FBlock()
		{
			delete [] Data;
		}
		FBlock(FBlock &&InOther) noexcept
		{
			Data = InOther.Data;
			InOther.Data = nullptr;
			NextBlock = InOther.NextBlock;
			bIsFree = InOther.bIsFree;
		}
		// disallow copy/copy-assignment
		FBlock(const FBlock&) = delete;
		FBlock& operator=(const FBlock&) = delete;
		
		bool bIsFree = true;
		int32 NextBlock = INDEX_NONE;
		T *Data;
	};
	
	TArray<FBlock> Blocks;
};


static float DistanceToLineSegment(const FVector& P, const FVector& A, const FVector& B)
{
	const FVector M = B - A;
	const FVector T = P - A;

	const float C1 = (float)FVector::DotProduct(M, T);
	if (C1 <= 0.0f)
	{
		return (float)FVector::Dist(P, A);
	}

	const float C2 = (float)FVector::DotProduct(M, M);
	if (C2 <= C1)
	{
		return (float)FVector::Dist(P, B);
	}

	// Project the point onto the line and get the distance between them.
	const FVector PT = A + M * (C1 / C2);
	return (float)FVector::Dist(P, PT);
}


// List of bones as used by the binding class. In this case for each bone transform, we want to
// store a list of line segments going from the bone transform to all the child bone transforms.
struct FTransformHierarchyQuery
{
	explicit FTransformHierarchyQuery(TConstArrayView<SkinBinding::FBonePoseInfo> InTransformHierarchy)
	{
		for (int Index = 0; Index < InTransformHierarchy.Num(); Index++)
		{
			FTransform Xform = InTransformHierarchy[Index].LocalTransform;
			int32 ParentIndex = InTransformHierarchy[Index].ParentIndex;

			while (ParentIndex != INDEX_NONE)
			{
				Xform = Xform * InTransformHierarchy[ParentIndex].LocalTransform;
				ParentIndex = InTransformHierarchy[ParentIndex].ParentIndex;
			}

			BoneFans.Add({ Xform.GetLocation() });
		}

		// Fill in the fan tips, as needed.
		for (int Index = 0; Index < InTransformHierarchy.Num(); Index++)
		{
			const int32 ParentIndex = InTransformHierarchy[Index].ParentIndex;
			if (ParentIndex != INDEX_NONE)
			{
				BoneFans[ParentIndex].TipsPos.Add(BoneFans[Index].RootPos);
			}
		}		
	}

	bool IsEndEffector(const int32 InBoneIndex) const
	{
		return BoneFans[InBoneIndex].TipsPos.IsEmpty();
	}

	float GetDistanceToBoneFan(const int32 InBoneIndex, const FVector& InPoint) const
	{
		return BoneFans[InBoneIndex].GetDistance(InPoint);
	}
	
	FBox GetBoneFanBBox(const int32 InBoneIndex) const
	{
		return BoneFans[InBoneIndex].GetBBox();
	}

	bool GetBoneFanIntersectsBox(const int32 InBoneIndex, const FBox &InBox) const
	{
		return BoneFans[InBoneIndex].IntersectsBox(InBox);
	}

private:
	struct FBoneFan
	{
		FVector RootPos;
		TArray<FVector> TipsPos;

		float GetDistance(const FVector& InPoint) const
		{
			if (TipsPos.IsEmpty())
			{
				return (float)FVector::Distance(RootPos, InPoint);
			}
			else
			{
				float Distance = std::numeric_limits<float>::max();
				for (const FVector& TipPos: TipsPos)
				{
					Distance = FMath::Min(Distance, DistanceToLineSegment(InPoint, RootPos, TipPos));
				}
				return Distance;
			}
		}

		FBox GetBBox() const
		{
			FBox Box(RootPos, RootPos);
			for (const FVector& TipPos: TipsPos)
			{
				Box += TipPos;
			}

			return Box;
		}

		bool IntersectsBox(const FBox &InBox) const
		{
			if (TipsPos.IsEmpty())
			{
				return FMath::PointBoxIntersection(RootPos, InBox);
			}

			if (GetBBox().Intersect(InBox))
			{
				for (const FVector& TipPos: TipsPos)
				{
					if (FMath::LineBoxIntersection(InBox, RootPos, TipPos, TipPos - RootPos))
					{
						return true;
					}
				}
			}
			return false;
		}				
	};

	 TArray<FBoneFan> BoneFans;
};


struct FCreateSkinWeights_Closest_WorkData final :
	TThreadSingleton<FCreateSkinWeights_Closest_WorkData>
{
	TArray<TPair<FBoneIndexType, float>> RawBoneWeights;
	TArray<AnimationCore::FBoneWeight> BoneWeights;

	void NormalizeWeightsAndLimitCount(const float InTotalWeight, const int32 InMaxWeights)
	{
		// Normalize
		for (TPair<FBoneIndexType, float> &BoneWeight: RawBoneWeights)
		{
			BoneWeight.Value /= InTotalWeight;
		}
		
		RawBoneWeights.Sort([](const TPair<FBoneIndexType, float> &A, const TPair<FBoneIndexType, float> &B)
		{
			return A.Value > B.Value;
		});

		BoneWeights.Reset(InMaxWeights);
		for (int32 BoneIndex = 0; BoneIndex < FMath::Min(InMaxWeights, RawBoneWeights.Num()); BoneIndex++)
		{
			const TPair<FBoneIndexType, float>& BoneWeight = RawBoneWeights[BoneIndex];
			BoneWeights.Add(UE::AnimationCore::FBoneWeight(BoneWeight.Key, BoneWeight.Value));
		}
			
	}
};

float ComputeWeightStiffness(const float InWeight, const float InStiffness)
{
	return (1.0f - InStiffness) * InWeight + InStiffness * InWeight * InWeight;
}

void CopyBonesToMesh(FDynamicMesh3& Mesh, TConstArrayView<SkinBinding::FBonePoseInfo> TransformHierarchy)
{
	// Initialize bone attributes
	FDynamicMeshAttributeSet* AttribSet = Mesh.Attributes();

	const int32 NumBones = TransformHierarchy.Num();
	AttribSet->EnableBones(NumBones);

	FDynamicMeshBoneNameAttribute* BoneNames = AttribSet->GetBoneNames();
	FDynamicMeshBoneParentIndexAttribute* BoneParentIndices = AttribSet->GetBoneParentIndices();
	FDynamicMeshBonePoseAttribute* BonePoses = AttribSet->GetBonePoses();

	for (int BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
	{
		BoneNames->SetValue(BoneIdx, TransformHierarchy[BoneIdx].Name);
		BoneParentIndices->SetValue(BoneIdx, TransformHierarchy[BoneIdx].ParentIndex);
		BonePoses->SetValue(BoneIdx, TransformHierarchy[BoneIdx].LocalTransform);
	}
}

FDynamicMeshVertexSkinWeightsAttribute* GetOrCreateSkinWeightsAttribute(
	FDynamicMesh3& InMesh,
	FName InProfileName
	)
{
	FDynamicMeshVertexSkinWeightsAttribute *Attribute = InMesh.Attributes()->GetSkinWeightsAttribute(InProfileName);
	if (!Attribute)
	{
		Attribute = new FDynamicMeshVertexSkinWeightsAttribute(&InMesh);
		InMesh.Attributes()->AttachSkinWeightsAttribute(InProfileName, Attribute);
	}
	
	return Attribute;
}

// Generalized helper for direct-distance bone weights, with enumerator to customize what bones are considered for each vertex
template<typename EnumerateType>
void CreateSkinWeightsImpl_DirectDistance_General(FDynamicMesh3& InMesh, TConstArrayView<SkinBinding::FBonePoseInfo> TransformHierarchy, FName ProfileName,
	float InStiffness,
	const AnimationCore::FBoneWeightsSettings& InSettings,
	const FTransformHierarchyQuery& Skeleton,
	EnumerateType EnumerateBonesForVertex)
{
	using namespace AnimationCore;

	FDynamicMeshVertexSkinWeightsAttribute *SkinWeights = GetOrCreateSkinWeightsAttribute(InMesh, ProfileName);
		
	const int32 MaxVertexID = InMesh.MaxVertexID();
	const int32 MaxInfluences = InSettings.GetMaxWeightCount();

	// Use the diagonal size of the bbox to make the bone distance falloff scale invariant.
	const float DiagBounds = (float)FMath::Max(InMesh.GetBounds(true).DiagonalLength(), UE_SMALL_NUMBER);
	
	ParallelFor(MaxVertexID, [&](const int32 VertexIdx)
	{
		if (!InMesh.IsVertex(VertexIdx))
		{
			return;
		}
		const FVector3d& Pos = InMesh.GetVertex(VertexIdx);

		FCreateSkinWeights_Closest_WorkData &WorkData = FCreateSkinWeights_Closest_WorkData::Get();
		
		if (MaxInfluences > 1)
		{
			WorkData.RawBoneWeights.Reset(TransformHierarchy.Num());
		}
		else
		{
			WorkData.RawBoneWeights.Init(MakeTuple(static_cast<FBoneIndexType>(0), 1.0f), 1);
		}
		
		float TotalWeight = 0.0f;
		EnumerateBonesForVertex(VertexIdx, [&](int32 BoneIndex)
		{
			// Normalize the distance by the diagonal size of the bbox to maintain scale invariance.
			float Weight = Skeleton.GetDistanceToBoneFan(BoneIndex, Pos) / DiagBounds;

			// Avoid div-by-zero but allow for the possibility that multiple bones may
			// touch this vertex.
			Weight = FMath::Max(Weight, KINDA_SMALL_NUMBER);

			// Compute the actual weight, factoring in the stiffness value. W = (1/S(D))^2
			// Where S(x) is the stiffness function.
			Weight = FMath::Square(1.0f / ComputeWeightStiffness(Weight, InStiffness));

			if (MaxInfluences > 1)
			{
				TotalWeight += Weight;
				WorkData.RawBoneWeights.Add(MakeTuple(static_cast<FBoneIndexType>(BoneIndex), Weight));
			}
			else if (Weight > TotalWeight)
			{
				// For single influences, we only care about the strongest influence.
				TotalWeight = Weight;
				WorkData.RawBoneWeights[0] = MakeTuple(static_cast<FBoneIndexType>(BoneIndex), Weight);
			}
		});

		WorkData.NormalizeWeightsAndLimitCount(TotalWeight, InSettings.GetMaxWeightCount());

		SkinWeights->SetValue(VertexIdx, FBoneWeights::Create(WorkData.BoneWeights, InSettings));
	});
}


// Direct Distance skin weight implementation where vertices can bind to every bone
void CreateSkinWeightsImpl_DirectDistance(
	FDynamicMesh3& InMesh, TConstArrayView<SkinBinding::FBonePoseInfo> TransformHierarchy, FName ProfileName,
	float InStiffness,
	const AnimationCore::FBoneWeightsSettings& InSettings)
{
	const int32 MaxInfluences = InSettings.GetMaxWeightCount();
	const FTransformHierarchyQuery Skeleton(TransformHierarchy);

	CreateSkinWeightsImpl_DirectDistance_General(InMesh, TransformHierarchy, ProfileName, InStiffness, InSettings, Skeleton,
	[MaxInfluences, &Skeleton, NumBones = TransformHierarchy.Num()](int32 VertexIndex, TFunctionRef<void(int32)> ProcessBone)
	{
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			// For single influences, avoid end effectors to avoid closest-distance fighting with the tips of their
			// parent bones.
			if (MaxInfluences == 1 && Skeleton.IsEndEffector(BoneIndex))
			{
				continue;
			}

			ProcessBone(BoneIndex);
		}
	});
}

// helper to filter the relevant bones when using the Constrained/bone-groups binding methods --
// When MaxInfluences == 1, filter end effectors from bone groups that also contain non-end-effector bones.
// OutFilteredGroups is populated only when filtering is needed; the returned view points into either
// OutFilteredGroups or the original BoneGroups.
static TConstArrayView<TArray<int32, TInlineAllocator<16>>> FilterBoneGroupEndEffectors(
	int32 MaxInfluences,
	TConstArrayView<TArray<int32, TInlineAllocator<16>>> BoneGroups,
	const FTransformHierarchyQuery& Skeleton,
	TArray<TArray<int32, TInlineAllocator<16>>>& OutFilteredGroups
)
{
	if (MaxInfluences == 1)
	{
		OutFilteredGroups.Append(BoneGroups);
		for (TArray<int32, TInlineAllocator<16>>&BoneGroup : OutFilteredGroups)
		{
			if (BoneGroup.Num() > 1)
			{
				int32 NonEndEffectors = 0;
				for (int32 Bone : BoneGroup)
				{
					NonEndEffectors += (int32)!Skeleton.IsEndEffector(Bone);
				}
				// if there are non-end-effector bones in this group, remove the end effectors
				if (NonEndEffectors > 0 && NonEndEffectors < BoneGroup.Num())
				{
					BoneGroup.SetNum(Algo::RemoveIf(BoneGroup, [&Skeleton](int32 Bone) {return Skeleton.IsEndEffector(Bone);}));
				}
			}
		}
		return OutFilteredGroups;
	}
	return BoneGroups;
}


// Enumerate bone group for a vertex, w/ fallback to standard un-constrained bones when vertex has no valid bone group
static void EnumerateVertexBoneGroup(
	int32 VertexIndex,
	TConstArrayView<int32> VIDtoGroupIndex,
	TConstArrayView<TArray<int32, TInlineAllocator<16>>> UseBoneGroups,
	const FTransformHierarchyQuery& Skeleton,
	int32 NumBones,
	int32 MaxInfluences,
	TFunctionRef<void(int32)> ProcessBone)
{
	int32 GroupIndex = VIDtoGroupIndex[VertexIndex];
	// for vertices that weren't assigned a (valid) group, fall back to a full search over bones
	if (!UseBoneGroups.IsValidIndex(GroupIndex))
	{
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			// For single influences, avoid end effectors to avoid closest-distance fighting with the tips of their
			// parent bones.
			if (MaxInfluences == 1 && Skeleton.IsEndEffector(BoneIndex))
			{
				continue;
			}

			ProcessBone(BoneIndex);
		}
	}
	else
	{
		for (int32 BoneIndex : UseBoneGroups[GroupIndex])
		{
			ProcessBone(BoneIndex);
		}
	}
}


// Direct Distance skin weight implementation where vertices can only bind to bones in limited 'bone groups'
void CreateSkinWeightsImpl_DirectDistance_Constrained(
	FDynamicMesh3& InMesh, TConstArrayView<SkinBinding::FBonePoseInfo> TransformHierarchy, FName ProfileName,
	float InStiffness,
	const AnimationCore::FBoneWeightsSettings& InSettings,
	TConstArrayView<TArray<int32, TInlineAllocator<16>>> BoneGroups, TConstArrayView<int32> VIDtoGroupIndex
	)
{
	if (VIDtoGroupIndex.IsEmpty() || BoneGroups.IsEmpty() || !ensure(VIDtoGroupIndex.Num() == InMesh.MaxVertexID()))
	{
		// fall back to unconstrained
		CreateSkinWeightsImpl_DirectDistance(InMesh, TransformHierarchy, ProfileName, InStiffness, InSettings);
		return;
	}

	const int32 MaxInfluences = InSettings.GetMaxWeightCount();
	const FTransformHierarchyQuery Skeleton(TransformHierarchy);
	TArray<TArray<int32, TInlineAllocator<16>>> FilteredBoneGroups;
	TConstArrayView<TArray<int32, TInlineAllocator<16>>> UseBoneGroups =
		FilterBoneGroupEndEffectors(MaxInfluences, BoneGroups, Skeleton, FilteredBoneGroups);

	CreateSkinWeightsImpl_DirectDistance_General(InMesh, TransformHierarchy, ProfileName, InStiffness, InSettings, Skeleton,
	[&VIDtoGroupIndex, &UseBoneGroups, &Skeleton, NumBones=TransformHierarchy.Num(), MaxInfluences](int32 VertexIndex, TFunctionRef<void(int32)> ProcessBone)
	{
		EnumerateVertexBoneGroup(VertexIndex, VIDtoGroupIndex, UseBoneGroups, Skeleton, NumBones, MaxInfluences, ProcessBone);
	});
}

// Generalized helper for geodesic-voxel bone weights, with predicates to customize which bones are computed and considered per vertex.
// Note: EnumerateBonesForVertex *must* only enumerate bones for which ShouldComputeBone returned true,
// otherwise Phase 2 will read uninitialized weight data.
template<typename ShouldComputeBoneType, typename EnumerateType>
void CreateSkinWeightsImpl_GeodesicVoxel_General(
	FDynamicMesh3& InMesh, TConstArrayView<SkinBinding::FBonePoseInfo> TransformHierarchy, FName ProfileName,
	float InStiffness, int32 InVoxelResolution,
	const AnimationCore::FBoneWeightsSettings& InSettings,
	const FTransformHierarchyQuery& Skeleton,
	ShouldComputeBoneType ShouldComputeBone,
	EnumerateType EnumerateBonesForVertex
	)
{
	using namespace AnimationCore;

	FDynamicMeshVertexSkinWeightsAttribute *SkinWeights = GetOrCreateSkinWeightsAttribute(InMesh, ProfileName);
		
	const int32 MaxVertexID = InMesh.MaxVertexID();
	const int32 MaxInfluences = InSettings.GetMaxWeightCount();

	// Use the diagonal size of the bbox to make the bone distance falloff scale invariant.
	const float DiagonalBounds = (float)FMath::Max(InMesh.GetBounds(true).DiagonalLength(), UE_SMALL_NUMBER);

	// This is grossly inefficient but tricky to do otherwise, since each bone distance
	// computation is done per-thread. We could possibly solve this by chunking instead
	// and accumulating partial results.
	TArray<float> Weights;
	Weights.SetNumUninitialized(MaxVertexID * TransformHierarchy.Num());

	const FOccupancyGrid3 Occupancy(InMesh, InVoxelResolution);

	const FVector3i Dimensions = Occupancy.GetOccupancyStateGrid().GetDimensions();

	// Phase 1: Compute per-bone geodesic distance fields via BFS through the voxel grid.
	ParallelFor(TransformHierarchy.Num(), [&](int32 BoneIndex) {
		if (!ShouldComputeBone(BoneIndex))
		{
			return;
		}
		
		TFIFOQueue<FVector3i> WorkingSet;  
		FDenseGrid3f BoneDistance(Dimensions.X, Dimensions.Y, Dimensions.Z, DiagonalBounds);

		// Mark all the cells that the bone intersects with distance of 0 and put them
		// on the work queue.
		const FBox BoneBox = Skeleton.GetBoneFanBBox(BoneIndex);
		const FVector3i BoneMin = Occupancy.GetCellIndexFromPoint(BoneBox.Min);
		const FVector3i BoneMax = Occupancy.GetCellIndexFromPoint(BoneBox.Max);

		for (int32 I = BoneMin.X; I <= BoneMax.X; I++)
		{
			for (int32 J = BoneMin.Y; J <= BoneMax.Y; J++)
			{
				for (int32 K = BoneMin.Z; K <= BoneMax.Z; K++)
				{
					const FVector3i Candidate(I, J, K);
					if (BoneDistance.IsValidIndex(Candidate))
					{
						const FBox3d CellBox{Occupancy.GetCellBoxFromIndex(Candidate)};
						if (Skeleton.GetBoneFanIntersectsBox(BoneIndex, CellBox))
						{
							WorkingSet.Push(Candidate);
							BoneDistance[Candidate] = 0.0f;
						}
					}
				}
			}
		}

		// Iterate over all the voxels until we have constructed shortest distance paths
		// throughout the level set.
		while (!WorkingSet.IsEmpty())
		{
			const FVector3i WorkItem = WorkingSet.Pop();

			// Loop through each of the neighbours (6 face neighbours, 12 edge neighbors,
			// and 8 corner neighbours) and see if any of them are closer to the bone
			// than their current marked distance.
			float CurrentDistance = BoneDistance[WorkItem];
			for (int32 N = 0; N < 26; N++)
			{
				FVector3i Offset(IndexUtil::GridOffsets26[N]);
				FVector3i Candidate(WorkItem + Offset);

				if (!BoneDistance.IsValidIndex(Candidate))
				{
					continue;
				}
					
				// Ensure this entry is either a part of the interior or boundary domain.
				if (Occupancy.GetOccupancyStateGrid()[Candidate] == FOccupancyGrid3::EDomain::Exterior)
				{
					continue;
				}

				const float CellDistance = (FVector3f(Offset) * Occupancy.GetCellSize()).Length();
				const float CandidateDistance = CurrentDistance + CellDistance;
				const float OldDistance = BoneDistance[Candidate]; 

				if (OldDistance > CandidateDistance)
				{
					WorkingSet.Push(Candidate);
					BoneDistance[Candidate] = CandidateDistance;
				}
			}
		}

		// Loop through all the vertices, find the voxel each belongs to, and compute
		// the distance from the voxel to the vertex (assuming the distance stored in the
		// voxel is based on traversing from voxel center to voxel center).
		for (int32 VertexIdx = 0; VertexIdx < MaxVertexID; VertexIdx++)
		{
			if (!InMesh.IsVertex(VertexIdx))
			{
				continue;
			}
			const FVector3d& Pos = InMesh.GetVertex(VertexIdx);
			const FVector3i CellIndex = Occupancy.GetCellIndexFromPoint(Pos);
			const FVector3d CellCenter{Occupancy.GetCellCenterFromIndex(CellIndex)};

			if (ensure(BoneDistance.IsValidIndex(CellIndex)))
			{
				float Distance = BoneDistance[CellIndex];
				Distance += (float)FVector3d::Distance(CellCenter, Pos);
				
				// Normalize the distance by the diagonal size of the bbox to maintain scale invariance.
				float Weight = Distance / DiagonalBounds;

				// Avoid div-by-zero but allow for the possibility that multiple bones may
				// touch this vertex.
				Weight = FMath::Max(Weight, UE_KINDA_SMALL_NUMBER);

				// Compute the actual weight, factoring in the stiffness value. W = (1/S(D))^2
				// Where S(x) is the stiffness function.
				Weight = FMath::Square(1.0f / ComputeWeightStiffness(Weight, InStiffness));

				Weights[VertexIdx * TransformHierarchy.Num() + BoneIndex] = Weight;
			}
		}
	});	

	// Phase 2: Aggregate per-vertex weights across bones and normalize.
	ParallelFor(MaxVertexID, [&](const int32 VertexIdx)
	{
		if (!InMesh.IsVertex(VertexIdx))
		{
			return;
		}
		FCreateSkinWeights_Closest_WorkData &WorkData = FCreateSkinWeights_Closest_WorkData::Get();
			
		if (MaxInfluences > 1)
		{
			WorkData.RawBoneWeights.Reset(TransformHierarchy.Num());
		}
		else
		{
			WorkData.RawBoneWeights.Init(MakeTuple(static_cast<FBoneIndexType>(0), 1.0f), 1);
		}

		float TotalWeight = 0.0f;
		EnumerateBonesForVertex(VertexIdx, [&](int32 BoneIndex)
		{
			const float Weight = Weights[VertexIdx * TransformHierarchy.Num() + BoneIndex];
			
			if (MaxInfluences > 1)
			{
				TotalWeight += Weight;
				WorkData.RawBoneWeights.Add(MakeTuple(static_cast<FBoneIndexType>(BoneIndex), Weight));
			}
			else if (Weight > TotalWeight)
			{
				TotalWeight = Weight;
				WorkData.RawBoneWeights[0] = MakeTuple(static_cast<FBoneIndexType>(BoneIndex), Weight);
			}
		});

		WorkData.NormalizeWeightsAndLimitCount(TotalWeight, InSettings.GetMaxWeightCount());

		SkinWeights->SetValue(VertexIdx, FBoneWeights::Create(WorkData.BoneWeights, InSettings));
	});
}


// Geodesic Voxel skin weight implementation where vertices can bind to every bone
void CreateSkinWeightsImpl_GeodesicVoxel(
	FDynamicMesh3& InMesh, TConstArrayView<SkinBinding::FBonePoseInfo> TransformHierarchy, FName ProfileName,
	float InStiffness, int32 InVoxelResolution,
	const AnimationCore::FBoneWeightsSettings& InSettings
	)
{
	const int32 MaxInfluences = InSettings.GetMaxWeightCount();
	const FTransformHierarchyQuery Skeleton(TransformHierarchy);

	CreateSkinWeightsImpl_GeodesicVoxel_General(InMesh, TransformHierarchy, ProfileName, InStiffness, InVoxelResolution, InSettings, Skeleton,
		// ShouldComputeBone: skip end effectors for single influence
		[MaxInfluences, &Skeleton](int32 BoneIndex) -> bool
		{
			return !(MaxInfluences == 1 && Skeleton.IsEndEffector(BoneIndex));
		},
		// EnumerateBonesForVertex: iterate all bones, skipping end effectors for single influence
		[MaxInfluences, &Skeleton, NumBones = TransformHierarchy.Num()](int32 VertexIndex, TFunctionRef<void(int32)> ProcessBone)
		{
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				if (MaxInfluences == 1 && Skeleton.IsEndEffector(BoneIndex))
				{
					continue;
				}
				ProcessBone(BoneIndex);
			}
		});
}


// Geodesic Voxel skin weight implementation where vertices can only bind to bones in limited 'bone groups'
void CreateSkinWeightsImpl_GeodesicVoxel_Constrained(
	FDynamicMesh3& InMesh, TConstArrayView<SkinBinding::FBonePoseInfo> TransformHierarchy, FName ProfileName,
	float InStiffness, int32 InVoxelResolution,
	const AnimationCore::FBoneWeightsSettings& InSettings,
	TConstArrayView<TArray<int32, TInlineAllocator<16>>> BoneGroups, TConstArrayView<int32> VIDtoGroupIndex
	)
{
	if (VIDtoGroupIndex.IsEmpty() || BoneGroups.IsEmpty() || !ensure(VIDtoGroupIndex.Num() == InMesh.MaxVertexID()))
	{
		// fall back to unconstrained
		CreateSkinWeightsImpl_GeodesicVoxel(InMesh, TransformHierarchy, ProfileName, InStiffness, InVoxelResolution, InSettings);
		return;
	}

	const int32 MaxInfluences = InSettings.GetMaxWeightCount();
	const FTransformHierarchyQuery Skeleton(TransformHierarchy);
	TArray<TArray<int32, TInlineAllocator<16>>> FilteredBoneGroups;
	TConstArrayView<TArray<int32, TInlineAllocator<16>>> UseBoneGroups =
		FilterBoneGroupEndEffectors(MaxInfluences, BoneGroups, Skeleton, FilteredBoneGroups);

	// Track whether bones appear in any group, so Phase 1 can skip unused bones
	TArray<bool> ActiveBones;
	ActiveBones.SetNumZeroed(TransformHierarchy.Num());
	for (const TArray<int32, TInlineAllocator<16>>& Group : UseBoneGroups)
	{
		for (int32 Bone : Group)
		{
			ActiveBones[Bone] = true;
		}
	}

	// If we have any vertices w/out a valid group, they will need an unconstrained search; update ActiveBones accordingly
	bool bHasUnassignedVerts = false;
	for (int32 GroupIndex : VIDtoGroupIndex)
	{
		if (!UseBoneGroups.IsValidIndex(GroupIndex))
		{
			bHasUnassignedVerts = true;
			break;
		}
	}
	if (bHasUnassignedVerts)
	{
		for (int32 BoneIndex = 0; BoneIndex < TransformHierarchy.Num(); ++BoneIndex)
		{
			if (MaxInfluences > 1 || !Skeleton.IsEndEffector(BoneIndex))
			{
				ActiveBones[BoneIndex] = true;
			}
		}
	}

	CreateSkinWeightsImpl_GeodesicVoxel_General(InMesh, TransformHierarchy, ProfileName, InStiffness, InVoxelResolution, InSettings, Skeleton,
		// ShouldComputeBone: only compute bones that appear in at least one group
		[&ActiveBones](int32 BoneIndex) -> bool
		{
			return ActiveBones[BoneIndex];
		},
		// EnumerateBonesForVertex: iterate bones in the vertex's assigned group (or all bones if unassigned)
		[&VIDtoGroupIndex, &UseBoneGroups, &Skeleton, MaxInfluences, NumBones=TransformHierarchy.Num()](int32 VertexIndex, TFunctionRef<void(int32)> ProcessBone)
		{
			EnumerateVertexBoneGroup(VertexIndex, VIDtoGroupIndex, UseBoneGroups, Skeleton, NumBones, MaxInfluences, ProcessBone);
		});
}

}

namespace UE::Geometry::SkinBinding
{

void CreateSkinWeights(FDynamicMesh3& InMesh, TConstArrayView<SkinBinding::FBonePoseInfo> TransformHierarchy, FName ProfileName, const FBindSettings& BindSettings)
{
	using namespace UE::Geometry::SkinWeightBindingLocals;

	const float ClampedStiffness = FMath::Clamp(BindSettings.Stiffness, 0.0f, 1.0f);

	UE::AnimationCore::FBoneWeightsSettings BoneSettings;
	BoneSettings.SetMaxWeightCount(BindSettings.MaxInfluences);

	switch(BindSettings.BindType)
	{
	default:
		ensureMsgf(false, TEXT("Unknown BindType in CreateSkinWeights"));
		// fall through to DirectDistance
	case ESkinBindingType::DirectDistance:
		CreateSkinWeightsImpl_DirectDistance(InMesh, TransformHierarchy, ProfileName, ClampedStiffness, BoneSettings);
		break;
			
	case ESkinBindingType::GeodesicVoxel:
		CreateSkinWeightsImpl_GeodesicVoxel(InMesh, TransformHierarchy, ProfileName, ClampedStiffness, BindSettings.VoxelResolution, BoneSettings);
		break;
	}

	CopyBonesToMesh(InMesh, TransformHierarchy);
}

void CreateConstrainedSkinWeights(FDynamicMesh3& InMesh, TConstArrayView<FBonePoseInfo> TransformHierarchy, FName ProfileName, const FBindSettings& BindSettings,
	TConstArrayView<TArray<int32, TInlineAllocator<16>>> BoneGroups, TConstArrayView<int32> VIDtoGroupIndex)
{
	using namespace UE::Geometry::SkinWeightBindingLocals;

	const float ClampedStiffness = FMath::Clamp(BindSettings.Stiffness, 0.0f, 1.0f);

	UE::AnimationCore::FBoneWeightsSettings BoneSettings;
	BoneSettings.SetMaxWeightCount(BindSettings.MaxInfluences);

	switch(BindSettings.BindType)
	{
	default:
		ensureMsgf(false, TEXT("Unknown BindType in CreateConstrainedSkinWeights"));
		// fall through to DirectDistance
	case ESkinBindingType::DirectDistance:
		CreateSkinWeightsImpl_DirectDistance_Constrained(
			InMesh, TransformHierarchy, ProfileName, ClampedStiffness, BoneSettings, BoneGroups, VIDtoGroupIndex);
		break;
	case ESkinBindingType::GeodesicVoxel:
		CreateSkinWeightsImpl_GeodesicVoxel_Constrained(
			InMesh, TransformHierarchy, ProfileName, ClampedStiffness, BindSettings.VoxelResolution, BoneSettings, BoneGroups, VIDtoGroupIndex);
		break;
	}

	CopyBonesToMesh(InMesh, TransformHierarchy);
}



} // namespace UE::Geometry::SkinBinding
