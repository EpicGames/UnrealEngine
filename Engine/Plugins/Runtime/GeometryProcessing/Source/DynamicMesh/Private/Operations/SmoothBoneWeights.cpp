// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/SmoothBoneWeights.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Util/ProgressCancel.h"
#include "Solvers/PrecomputedMeshWeightData.h"
#include "Math/UnrealMathUtility.h"
#include "Parameterization/MeshLocalParam.h"
#include "DynamicMesh/MeshNormals.h"
#include "Tasks/Task.h"
#include "Async/TaskGraphInterfaces.h"

using namespace UE::AnimationCore;
using namespace UE::Geometry;

namespace SmoothBoneWeightsLocals 
{	
	template <typename BoneIndexType, typename BoneWeightType>
	void NormalizeWeights(TMap<BoneIndexType, BoneWeightType>& InOutWeights)
	{
		BoneWeightType TotalWeight = (BoneWeightType) 0.f;
		for (const TTuple<BoneIndexType, BoneWeightType>& Weight : InOutWeights)
		{
			TotalWeight += Weight.Value;
		}

		if (!FMath::IsNearlyZero(TotalWeight))
		{
			for (TTuple<BoneIndexType, BoneWeightType>& Weight : InOutWeights)
			{
				Weight.Value /= TotalWeight;
			}
		}
	}
	
	/** Data source implementation for bone weights data stored in the FDynamicMeshVertexSkinWeightsAttribute. */
	class FSkinWeightsAttributeDataSource : public TBoneWeightsDataSource<FBoneIndexType, float>
	{
	public:
		FSkinWeightsAttributeDataSource(const FDynamicMeshVertexSkinWeightsAttribute* InAttribute)
		:
		Attribute(InAttribute) 
		{
			checkSlow(Attribute);
		}

		virtual ~FSkinWeightsAttributeDataSource() = default; 

		virtual int32 GetBoneNum(const int32 VertexID) override
		{
			FBoneWeights Weights;
			Attribute->GetValue(VertexID, Weights);
			return Weights.Num();
		}

		virtual FBoneIndexType GetBoneIndex(const int32 VertexID, const int32 Index) override
		{
			FBoneWeights Weights;
			Attribute->GetValue(VertexID, Weights);
			return Weights[Index].GetBoneIndex();
		}

		virtual float GetBoneWeight(const int32 VertexID, const int32 Index) override
		{
			FBoneWeights Weights;
			Attribute->GetValue(VertexID, Weights);
			return Weights[Index].GetWeight();
		}

		virtual float GetWeightOfBoneOnVertex(const int32 VertexID, const FBoneIndexType BoneIndex) override
		{
			FBoneWeights Weights;
			Attribute->GetValue(VertexID, Weights);
			for (const FBoneWeight& BoneWeight : Weights)
			{
				if (BoneWeight.GetBoneIndex() == BoneIndex)
				{
					return BoneWeight.GetWeight();;
				}
			}

			return 0.f;
		}

	protected:
		const FDynamicMeshVertexSkinWeightsAttribute* Attribute = nullptr;
	};
}


//
// TSmoothBoneWeights
//

template <typename BoneIndexType, typename BoneWeightType>
TSmoothBoneWeights<BoneIndexType, BoneWeightType>::TSmoothBoneWeights(const FDynamicMesh3* InSourceMesh,
																	  TBoneWeightsDataSource<BoneIndexType, BoneWeightType>* InDataSource)
:
SourceMesh(InSourceMesh),
DataSource(InDataSource)
{
	checkSlow(InSourceMesh);
}

template <typename BoneIndexType, typename BoneWeightType>
bool TSmoothBoneWeights<BoneIndexType, BoneWeightType>::Cancelled()
{
	return (Progress == nullptr) ? false : Progress->Cancelled();
}

template <typename BoneIndexType, typename BoneWeightType>
EOperationValidationResult TSmoothBoneWeights<BoneIndexType, BoneWeightType>::Validate()
{	
	if (SourceMesh == nullptr) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (DataSource == nullptr)
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	return EOperationValidationResult::Ok;
}

template <typename BoneIndexType, typename BoneWeightType>
void TSmoothBoneWeights<BoneIndexType, BoneWeightType>::ComputeEdgeWeightsIfNeeded()
{
	if (EdgeWeightMethod == ESmoothBoneWeightsEdgeWeightMethod::CotanWeights
		&& CotangentEdgeWeights.Num() != SourceMesh->MaxEdgeID())
	{
		UE::MeshDeformation::ConstructEdgeCotanWeightsDataArray(*SourceMesh, CotangentEdgeWeights);
	}
}

template <typename BoneIndexType, typename BoneWeightType>
bool TSmoothBoneWeights<BoneIndexType, BoneWeightType>::SmoothWeightsAtVertex(const int32 VertexID,
																			  const BoneWeightType VertexFalloff,
																			  TMap<BoneIndexType, BoneWeightType>& OutFinalWeights)
{
	// The non-locked overload is a special case of the locked overload with no locked bones.
	const TSet<BoneIndexType> EmptyLockedBones;
	return SmoothWeightsAtVertex(VertexID, VertexFalloff, EmptyLockedBones, OutFinalWeights);
}

template <typename BoneIndexType, typename BoneWeightType>
bool TSmoothBoneWeights<BoneIndexType, BoneWeightType>::SmoothWeightsAtVertex(
	const int32 VertexID,
	const BoneWeightType VertexFalloff,
	const TSet<BoneIndexType>& LockedBones,
	TMap<BoneIndexType, BoneWeightType>& OutFinalWeights)
{
	using namespace SmoothBoneWeightsLocals;

	ComputeEdgeWeightsIfNeeded();

	OutFinalWeights.Reset();

	// Identify locked influences on the source vertex; they are preserved exactly
	// and the remaining (1 - TotalLockedWeight) is the budget for unlocked bones.
	TMap<BoneIndexType, BoneWeightType> LockedWeights;
	BoneWeightType TotalLockedWeight = (BoneWeightType)0.0;
	for (int32 Index = 0; Index < DataSource->GetBoneNum(VertexID); ++Index)
	{
		const BoneIndexType BoneIndex = DataSource->GetBoneIndex(VertexID, Index);
		if (LockedBones.Contains(BoneIndex))
		{
			const BoneWeightType Weight = DataSource->GetBoneWeight(VertexID, Index);
			LockedWeights.Add(BoneIndex, Weight);
			TotalLockedWeight += Weight;
		}
	}
	const BoneWeightType AvailableWeight = FMath::Max((BoneWeightType)0.0, (BoneWeightType)1.0 - TotalLockedWeight);

	// Compute the neighborhood-average distribution over unlocked bones using a
	// standard Laplacian formulation: a single edge-weight denominator summed
	// across all contributing neighbors. Self is intentionally excluded — the
	// lerp at the end provides old-value retention, and including self with a
	// fixed weight would break the cotan formulation.
	TMap<BoneIndexType, BoneWeightType> WeightedBoneSum;
	BoneWeightType TotalEdgeWeight = (BoneWeightType)0.0;

	for (const int32 NeighborVertexID : SourceMesh->VtxVerticesItr(VertexID))
	{
		BoneWeightType EdgeWeight = (BoneWeightType)1.0;
		if (EdgeWeightMethod == ESmoothBoneWeightsEdgeWeightMethod::CotanWeights)
		{
			const int32 EdgeId = SourceMesh->FindEdge(VertexID, NeighborVertexID);
			if (EdgeId != FDynamicMesh3::InvalidID)
			{
				// Cotangent weights can be negative; clamp to zero to keep the
				// smoothing operator on the simplex (same as FSmoothDynamicMeshAttributes).
				EdgeWeight = static_cast<BoneWeightType>(FMath::Max(0.0, CotangentEdgeWeights[EdgeId]));
			}
			else
			{
				EdgeWeight = (BoneWeightType)0.0;
			}
		}

		if (EdgeWeight <= (BoneWeightType)0.0)
		{
			continue;
		}

		for (int32 Index = 0; Index < DataSource->GetBoneNum(NeighborVertexID); ++Index)
		{
			const BoneIndexType BoneIndex = DataSource->GetBoneIndex(NeighborVertexID, Index);
			if (LockedBones.Contains(BoneIndex))
			{
				continue;
			}
			const BoneWeightType Weight = DataSource->GetBoneWeight(NeighborVertexID, Index);
			if (Weight > MinimumWeightThreshold)
			{
				WeightedBoneSum.FindOrAdd(BoneIndex) += EdgeWeight * Weight;
			}
		}
		TotalEdgeWeight += EdgeWeight;
	}

	// Build the neighborhood-average target, scaled to the unlocked weight budget.
	// (When there are no locked bones, AvailableWeight == 1 and this collapses to
	// a plain Laplacian average that already sums to 1.)
	TMap<BoneIndexType, BoneWeightType> NeighborhoodTarget;
	if (!FMath::IsNearlyZero(TotalEdgeWeight))
	{
		const BoneWeightType Scale = AvailableWeight / TotalEdgeWeight;
		NeighborhoodTarget.Reserve(WeightedBoneSum.Num());
		for (const TTuple<BoneIndexType, BoneWeightType>& Entry : WeightedBoneSum)
		{
			NeighborhoodTarget.Add(Entry.Key, Entry.Value * Scale);
		}
	}

	// If neighbors gave no usable signal, smoothing is a no-op for this vertex.
	// Populate OutFinalWeights with the vertex's current weights (locked + unlocked)
	// so callers that overwrite the vertex (e.g. FSmoothDynamicMeshVertexSkinWeights,
	// which builds a fresh FBoneWeights from this map and calls Attribute->SetValue)
	// don't wipe out bones that are absent from the map. Callers that record
	// per-bone edits (e.g. the paint tool) will simply record no-op edits.
	if (NeighborhoodTarget.IsEmpty())
	{
		const int32 NumBones = DataSource->GetBoneNum(VertexID);
		OutFinalWeights.Reserve(NumBones);
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			OutFinalWeights.Add(
				DataSource->GetBoneIndex(VertexID, Index),
				DataSource->GetBoneWeight(VertexID, Index));
		}
		if (Cancelled())
		{
			return false;
		}
		return true;
	}

	// Clamp the lerp factor to [0,1]. Values outside this range cause the lerp
	// to extrapolate, producing weights outside the simplex (negative or > 1)
	// which then feed back into subsequent strokes and cause unbounded oscillation.
	const BoneWeightType ClampedFalloff = FMath::Clamp(VertexFalloff, (BoneWeightType)0.0, (BoneWeightType)1.0);

	// Lerp from the source vertex's current unlocked weights toward the
	// neighborhood target. We must include every unlocked bone present on the
	// source vertex (so bones with no neighborhood representation fade toward 0)
	// and every bone in the neighborhood target (so new influences can flow in).
	for (int32 Index = 0; Index < DataSource->GetBoneNum(VertexID); ++Index)
	{
		const BoneIndexType BoneIndex = DataSource->GetBoneIndex(VertexID, Index);
		if (LockedBones.Contains(BoneIndex))
		{
			continue;
		}
		const BoneWeightType OldWeight = DataSource->GetBoneWeight(VertexID, Index);
		const BoneWeightType TargetWeight = NeighborhoodTarget.FindRef(BoneIndex);
		OutFinalWeights.Add(BoneIndex, FMath::Lerp<BoneWeightType>(OldWeight, TargetWeight, ClampedFalloff));
	}
	for (const TTuple<BoneIndexType, BoneWeightType>& Entry : NeighborhoodTarget)
	{
		if (!OutFinalWeights.Contains(Entry.Key))
		{
			OutFinalWeights.Add(Entry.Key, FMath::Lerp<BoneWeightType>((BoneWeightType)0.0, Entry.Value, ClampedFalloff));
		}
	}

	BoneWeightType UnlockedSum = (BoneWeightType)0.0;
	for (TTuple<BoneIndexType, BoneWeightType>& Entry : OutFinalWeights)
	{
		ensureMsgf(Entry.Value >= -static_cast<BoneWeightType>(UE_KINDA_SMALL_NUMBER),
			TEXT("Smoothing produced out-of-range weight %f for bone %d on vertex %d before renormalize."),
			static_cast<float>(Entry.Value), Entry.Key, VertexID);
		Entry.Value = FMath::Max((BoneWeightType)0.0, Entry.Value);
		UnlockedSum += Entry.Value;
	}
	if (!FMath::IsNearlyZero(UnlockedSum))
	{
		const BoneWeightType Rescale = AvailableWeight / UnlockedSum;
		for (TTuple<BoneIndexType, BoneWeightType>& Entry : OutFinalWeights)
		{
			Entry.Value *= Rescale;
		}
	}

	// Append locked bones unchanged. Combined with the unlocked budget above,
	// the total weight on the vertex sums to 1.
	OutFinalWeights.Append(LockedWeights);

	if (Cancelled())
	{
		return false;
	}

	return true;
}


//
// FSmoothDynamicMeshVertexSkinWeights
//

FSmoothDynamicMeshVertexSkinWeights::FSmoothDynamicMeshVertexSkinWeights(const FDynamicMesh3* InSourceMesh, const FName InProfile)
:
TSmoothBoneWeights(InSourceMesh, nullptr)
{
	if (SourceMesh && SourceMesh->Attributes())
	{
		Attribute = SourceMesh->Attributes()->GetSkinWeightsAttribute(InProfile);
		if (Attribute)
		{
			SkinWeightsAttributeDataSource = MakeUnique<SmoothBoneWeightsLocals::FSkinWeightsAttributeDataSource>(Attribute);
			DataSource = SkinWeightsAttributeDataSource.Get();
		}
	}
}

FSmoothDynamicMeshVertexSkinWeights::FSmoothDynamicMeshVertexSkinWeights(const FDynamicMesh3* InSourceMesh, FDynamicMeshVertexSkinWeightsAttribute* InAttribute)
:
TSmoothBoneWeights(InSourceMesh, nullptr),
Attribute(InAttribute)
{
	if (InAttribute)
	{
		SkinWeightsAttributeDataSource = MakeUnique<SmoothBoneWeightsLocals::FSkinWeightsAttributeDataSource>(InAttribute);
		DataSource = SkinWeightsAttributeDataSource.Get();
	}
}


EOperationValidationResult FSmoothDynamicMeshVertexSkinWeights::Validate()
{	
	if (Attribute == nullptr) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (MaxNumInfluences <= 0)
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	return TSmoothBoneWeights::Validate();
}

bool FSmoothDynamicMeshVertexSkinWeights::SmoothWeightsAtVertex(const int32 VertexID, const float VertexFalloff)
{	
	TMap<FBoneIndexType, float> FinalWeights;
	if (TSmoothBoneWeights<FBoneIndexType, float>::SmoothWeightsAtVertex(VertexID, VertexFalloff, FinalWeights))
	{
		FBoneWeightsSettings BoneSettings;
		BoneSettings.SetNormalizeType(EBoneWeightNormalizeType::None);
		FBoneWeights WeightArray;
		
		for (const TTuple<FBoneIndexType, float>& FinalWeight : FinalWeights)
		{
			WeightArray.SetBoneWeight(FBoneWeight(FinalWeight.Key, FinalWeight.Value), BoneSettings);
		}
		
		BoneSettings.SetNormalizeType(EBoneWeightNormalizeType::Always);
		BoneSettings.SetMaxWeightCount(MaxNumInfluences);

		// make sure we do not exceed the max influence limit
		WeightArray.Renormalize(BoneSettings);

		Attribute->SetValue(VertexID, WeightArray);

		return true;
	}

	return false;
}

bool FSmoothDynamicMeshVertexSkinWeights::SmoothWeightsAtVerticesWithinDistance(const TArray<int32>& Vertices, 
																  				const float Strength, 
																  				const double FloodFillUpToDistance,
																				const int32 NumIterations)
{
	TSet<int32> VerticesToSmooth;
	VerticesToSmooth.Append(Vertices);

	const double FloodFillUpToDistanceSquared = FloodFillUpToDistance * FloodFillUpToDistance;

	// We want to add vertices to the VerticesToSmooth within FloodFillUpToDistance away from each vertex in the 
	// Vertices array
	if (FloodFillUpToDistance > 0)
	{
		// Now this process is quite fast, so cancellation can be check at the beginning
		if (Cancelled())
		{
			return false;
		}

		const int32 NumVertices = Vertices.Num();
		const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), NumVertices), 1);
		constexpr int32 MinVerticesByTask = 20;
		const int32 VerticesByTask = FMath::Max(FMath::DivideAndRoundUp(NumVertices, NumTasks), MinVerticesByTask);
		const int32 NumBatches = FMath::DivideAndRoundUp(NumVertices, VerticesByTask);
		TArray<UE::Tasks::FTask> PendingTasks;
		std::atomic<int32> NumAccessSet = 0;
		for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
		{
			const int32 StartIndex = BatchIndex * VerticesByTask;
			int32 EndIndex = (BatchIndex + 1) * VerticesByTask;
			EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(NumVertices, EndIndex) : EndIndex;
			UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, StartIndex, EndIndex, &Vertices, &VerticesToSmooth, FloodFillUpToDistance, FloodFillUpToDistanceSquared, &NumAccessSet]()
			{
				for (int32 Index = StartIndex; Index < EndIndex; ++Index)
				{
					const int32 SeedVID = Vertices[Index];
					
					// If at least one neighboring vertex is not part of the set of vertices to smooth then we need to flood
					bool bNeedToFlood = false;
					for (const int32 NeighborVertexID : SourceMesh->VtxVerticesItr(SeedVID))
					{
						bool bVerticesToSmoothContainsNeighborVertexID;
						while (true)
						{
							// Make sure to read in the set only when we are not writing in it, INDEX_NONE is used when are are writing in the set
							// Store the number of thread which are reading from the set in NumAccessSet
							if (int32 NumAccessSetExpected = NumAccessSet.load(std::memory_order_relaxed); NumAccessSetExpected != INDEX_NONE)
							{
								// Increment the NumAccessSet to read
								if (NumAccessSet.compare_exchange_weak(NumAccessSetExpected, NumAccessSetExpected+1, std::memory_order_relaxed))
								{
									bVerticesToSmoothContainsNeighborVertexID = VerticesToSmooth.Contains(NeighborVertexID);
									NumAccessSet.fetch_sub(1, std::memory_order_relaxed); // Decrement the NumAccessSet to release
									break;
								}
							}
						}
						if (!bVerticesToSmoothContainsNeighborVertexID &&
							DistanceSquared(SourceMesh->GetVertex(SeedVID), SourceMesh->GetVertex(NeighborVertexID)) < FloodFillUpToDistanceSquared)
						{
							bNeedToFlood = true;
							break;
						}
					}

					if (bNeedToFlood)
					{
						FVector3d Normal = FMeshNormals::ComputeVertexNormal(*SourceMesh, SeedVID);
						FFrame3d SeedFrame = SourceMesh->GetVertexFrame(SeedVID, false, &Normal);

						TMeshLocalParam<FDynamicMesh3> Param(SourceMesh);
						Param.ParamMode = ELocalParamTypes::ExponentialMapUpwindAvg;
						Param.ComputeToMaxDistance(SeedVID, SeedFrame, FloodFillUpToDistance);

						// Only points within FloodFillUpToDistance should have UVs set
						TArray<int32> PointsWithinDistance;
						Param.GetPointsWithUV(PointsWithinDistance);

						while (true)
						{
							// If none set are reading from the set try to write new data and set the atomic to INDEX_NONE
							if (int32 Expected = NumAccessSet.load(std::memory_order_relaxed); Expected == 0)
							{
								if (NumAccessSet.compare_exchange_weak(Expected, INDEX_NONE, std::memory_order_relaxed))
								{
									VerticesToSmooth.Append(PointsWithinDistance);
									NumAccessSet.store(0, std::memory_order_relaxed); // Release the atomic for reading or writing
									break;
								}
							}
						}
					}
				}
			});
			PendingTasks.Add(PendingTask);
		}
		UE::Tasks::Wait(PendingTasks);
	}

	for (int32 Itr = 0; Itr < NumIterations; ++Itr)
	{
		for (const int32 VertexID : VerticesToSmooth)
		{
			if (!FSmoothDynamicMeshVertexSkinWeights::SmoothWeightsAtVertex(VertexID, Strength))
			{
				return false;
			}
		}
	}

	return true;
}


// template instantiation
template class DYNAMICMESH_API UE::Geometry::TSmoothBoneWeights<FBoneIndexType, float>;
template class DYNAMICMESH_API UE::Geometry::TSmoothBoneWeights<int, float>;