// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/SmoothDynamicMeshAttributes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshOverlay.h"
#include "Solvers/PrecomputedMeshWeightData.h"
#include "Util/ProgressCancel.h"
#include "Async/ParallelFor.h"
#include "MeshQueries.h"
#include "Math/UnrealMathUtility.h"
#include "Operations/AverageOverlayToVertices.h"

using namespace UE::Geometry;
using namespace UE::MeshDeformation; 

FSmoothDynamicMeshAttributes::FSmoothDynamicMeshAttributes(const FDynamicMesh3& InMesh)
:
Mesh(&InMesh)
{
}

bool FSmoothDynamicMeshAttributes::Cancelled()
{
	return (Progress == nullptr) ? false : Progress->Cancelled();
}

template<typename RealType, int ElementSize>
bool FSmoothDynamicMeshAttributes::SmoothOverlay(TDynamicMeshOverlay<RealType, ElementSize>* InOverlay, const TArray<bool>& DimensionsToSmooth)
{
	if (NumIterations <= 0 || Strength <= 0 || Mesh->EdgeCount() == 0)
	{
		return true;
	}

	if (!(DimensionsToSmooth.IsEmpty() || DimensionsToSmooth.Num() == ElementSize))
	{
		return false;
	}

    //
    // Check if we need to pre-compute cotangent weights or edge lengths. Avoid re-compute on subsequent calls.
    //
    if (EdgeWeightMethod == EEdgeWeights::CotanWeights && CotangentEdgeWeights.Num() != Mesh->MaxEdgeID())
    {
        ConstructEdgeCotanWeightsDataArray(*Mesh, CotangentEdgeWeights);
    }
    if (EdgeWeightMethod == EEdgeWeights::EdgeLength && EdgeLengthWeights.Num() != Mesh->MaxEdgeID())
    {   
        double TotalLength;
        TMeshQueries<FDynamicMesh3>::GetAllEdgeLengths(*Mesh, EdgeLengthWeights, TotalLength);
        EdgeLengthThreshold = 0.001*(TotalLength/Mesh->EdgeCount()); // minimum acceptable edge length for weight computation
    }

    //
    // Compute an array of vertex indices that contains the selection and the all the one-ring neighbors
    //
    ExpandedSelection.Reset();
    VIDToExpandedSelectionIdx.Reset();
    if (Selection.Num())
    {
        TMeshQueries<FDynamicMesh3>::ExpandVertexSelectionToNeighbors(*Mesh, Selection, ExpandedSelection, VIDToExpandedSelectionIdx);
    }
    
    //
    // For every vertex in ExpandedSelection, compute the area/angle weighted average of the overlay elements on that vertex
    //
    TArray<RealType> VertexVectorValue;
    TArray<bool> HasElements;
    FAverageOverlayToVertices AverageOverlayOp(*Mesh);
    if (Selection.Num())
    {
        AverageOverlayOp.Selection = MoveTemp(ExpandedSelection);
    }
    if (!AverageOverlayOp.AverageOverlay(InOverlay, VertexVectorValue, HasElements))
    {
        return false;
    }

    //
	// Iterate over every vertex (in Selection or the whole mesh if the Selection is empty) and smooth
    //
    const int32 NumVertsToSmooth = Selection.Num() ? Selection.Num() : Mesh->MaxVertexID();
    TArray<RealType> NewVertexVectorValue = VertexVectorValue;
    for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
    {
        ParallelFor(NumVertsToSmooth, [&](int32 VIdx)
        {
            const int32 VertexID = Selection.Num() ? Selection[VIdx] : VIdx;

            if (!Mesh->IsVertex(VertexID))
            {
                return;
            }
        
            RealType WeightII = 0.0;        // Total weight value for all elements
            RealType VecAccum[ElementSize]; // Value of weighted elements
            for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
            {
                VecAccum[EIdx] = static_cast<RealType>(0.0);
            }

            for (const int32 EdgeId : Mesh->VtxEdgesItr(VertexID))
            {   
				const FIndex2i EdgeV = Mesh->GetEdgeV(EdgeId);
				const int32 NextV = (EdgeV.A == VertexID) ? EdgeV.B : EdgeV.A;
                const int32 NextSelectionID = MeshVIDToSelectionIdx(NextV);

                // check if the vertex has at least one element and needs to be considered in the calculation
                if (HasElements[NextSelectionID]) 
                {
                    RealType WeightIJ = static_cast<RealType>(1.0f); // EEdgeWeights::Uniform
                    if (EdgeWeightMethod == EEdgeWeights::CotanWeights) 
                    {
                        // cotangent weights can be negative, hence we clamp them to zero
                        WeightIJ = static_cast<RealType>(FMath::Clamp(CotangentEdgeWeights[EdgeId], (RealType)0.0, CotangentEdgeWeights[EdgeId])); 
                    }
                    else if (EdgeWeightMethod == EEdgeWeights::EdgeLength)
                    {
                        WeightIJ = static_cast<RealType>(1.0/FMath::Max(EdgeLengthThreshold, EdgeLengthWeights[EdgeId]));
                    }
                    
                    for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
                    {
                        VecAccum[EIdx] += VertexVectorValue[ElementSize*NextSelectionID + EIdx] * WeightIJ;
                    }
                    WeightII += WeightIJ;
                }
               
            }

            if (!FMath::IsNearlyZero((double)WeightII))
            {
                const RealType InvWeightII = RealType(1.0/WeightII);
                for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
                {
				    VecAccum[EIdx] *= InvWeightII;
                }
            }
            
            const int32 SelectionIdx = MeshVIDToSelectionIdx(VertexID);
			for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
			{
                const int32 Start = ElementSize*SelectionIdx;
				NewVertexVectorValue[Start + EIdx] = FMath::Lerp(VertexVectorValue[Start + EIdx], VecAccum[EIdx], Strength);
			}
        }, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		VertexVectorValue = NewVertexVectorValue;
    }


    //
    // Write the smoothed data back into the overlay
    //
    ParallelFor(NumVertsToSmooth, [&](int32 Idx)
    {
        const int32 VertexID = Selection.Num() ? Selection[Idx] : Idx;
        if (!Mesh->IsVertex(VertexID))
        {
            return;
        }

        const int32 SelectionIdx = MeshVIDToSelectionIdx(VertexID);

        TArray<int32> Elements;
        InOverlay->GetVertexElements(VertexID, Elements);
		RealType ElementData[ElementSize];
       
        for (const int32 ElementID : Elements)
        { 
			InOverlay->GetElement(ElementID, ElementData);

            if (SplitVertexModel == ESplitVertexModel::SetAllToSmoothed)  
            {   
				for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
				{
					if (DimensionsToSmooth.IsEmpty() || DimensionsToSmooth[EIdx])
					{
						ElementData[EIdx] = NewVertexVectorValue[SelectionIdx * ElementSize + EIdx];
					}
				}
            }
            else if (SplitVertexModel == ESplitVertexModel::BlendWithSmoothed)
            {
				for (int32 EIdx = 0; EIdx < ElementSize; ++EIdx)
				{
					if (DimensionsToSmooth.IsEmpty() || DimensionsToSmooth[EIdx])
					{
						ElementData[EIdx] = FMath::Lerp(ElementData[EIdx], NewVertexVectorValue[SelectionIdx * ElementSize + EIdx], BlendWithSmoothedStrength);
					}
				}
            }
            else
            {
                checkNoEntry(); // unsupported type
            }

            InOverlay->SetElement(ElementID, ElementData);
        }
    }, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	return true;
}


// template instantiation
template DYNAMICMESH_API bool UE::Geometry::FSmoothDynamicMeshAttributes::SmoothOverlay<float,4>(class UE::Geometry::TDynamicMeshOverlay<float,4>*, class TArray<bool, class TSizedDefaultAllocator<32> > const&);
template DYNAMICMESH_API bool UE::Geometry::FSmoothDynamicMeshAttributes::SmoothOverlay<float,3>(class UE::Geometry::TDynamicMeshOverlay<float,3>*, class TArray<bool, class TSizedDefaultAllocator<32> > const&);
template DYNAMICMESH_API bool UE::Geometry::FSmoothDynamicMeshAttributes::SmoothOverlay<float,2>(class UE::Geometry::TDynamicMeshOverlay<float,2>*, class TArray<bool, class TSizedDefaultAllocator<32> > const&);
template DYNAMICMESH_API bool UE::Geometry::FSmoothDynamicMeshAttributes::SmoothOverlay<float,1>(class UE::Geometry::TDynamicMeshOverlay<float,1>*, class TArray<bool, class TSizedDefaultAllocator<32> > const&);
template DYNAMICMESH_API bool UE::Geometry::FSmoothDynamicMeshAttributes::SmoothOverlay<double,3>(class UE::Geometry::TDynamicMeshOverlay<double,3>*, class TArray<bool, class TSizedDefaultAllocator<32> > const&);
