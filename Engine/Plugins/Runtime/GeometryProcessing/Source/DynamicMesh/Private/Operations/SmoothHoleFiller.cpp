// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/SmoothHoleFiller.h"
#include "MeshBoundaryLoops.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "Selections/MeshEdgeSelection.h"
#include "Operations/SimpleHoleFiller.h"
#include "Operations/OffsetMeshRegion.h"
#include "Operations/MeshRegionOperator.h"
#include "Solvers/ConstrainedMeshSmoother.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "SubRegionRemesher.h"
#include "ProjectionTargets.h"
#include "MeshConstraintsUtil.h"
#include "DynamicMeshEditor.h"

using namespace UE::Geometry;

namespace
{
	// hard-coded remeshing parameters
	const int InitialRemeshPasses = 5;
	const int PostSmoothRemeshPasses = 10;
	const int SmoothSolveIterations = 2;
	const bool bRemeshAfterSmooth = true;
	const double RemeshingSmoothAlpha = 0.75;


	/// Apply LaplacianMeshSmoother to subset of mesh triangles. 
	/// 
	/// \param Mesh The mesh to smooth
	/// \param InputTriangles Triangles defining code region to smooth. This region can grow by specifying nIncludeExteriorRings > 0.
	/// \param nConstrainLoops Number of one-rings to soft-constrain inside the smooth region.
	/// \param nIncludeExteriorRings Number of one-rings to grow the region.
	/// \param bConstrainExteriorRings Whether to pin the vertices outside of the original "InputTriangles" region.
	/// \param InteriorSmoothness Baseline "smoothness" scalar (\propto 1/weight) for vertices inside the initial region boundary.
	/// \param BorderWeight Constraint weight for vertices on the initial region boundary
	///
	/// - Border of subset always has soft constraint with borderWeight, but is then snapped back to original vtx pos 
	///   after solve.
	/// - nConstrainLoops inner loops are also soft-constrained, with weight falloff via square roots (defines continuity)
	/// - interiorWeight is soft constraint added to all vertices
	///
	void RegionSmooth(FDynamicMesh3* Mesh,
		const FMeshFaceSelection& InputTriangles,
		int nConstrainLoops,
		int nIncludeExteriorRings,
		bool bConstrainExteriorRings,
		double InteriorSmoothness,
		double BorderWeight = 10.0)
	{
		TSet<int> FixedVerts;
		FMeshFaceSelection SmoothTriangles = InputTriangles;

		if (nIncludeExteriorRings > 0)
		{
			if (bConstrainExteriorRings)
			{
				// add constraints to vertices which are in expandVerts but not in startVerts (i.e. the "exterior ring" 
				// vertices)

				FMeshEdgeSelection BoundaryEdges(Mesh);
				BoundaryEdges.SelectBoundaryTriEdges(SmoothTriangles);
				SmoothTriangles.ExpandToOneRingNeighbours(nIncludeExteriorRings);

				FMeshVertexSelection StartVerts(Mesh);
				StartVerts.SelectTriangleVertices(InputTriangles);
				StartVerts.DeselectEdges(BoundaryEdges.AsArray());

				FMeshVertexSelection ExpandVerts(Mesh);
				ExpandVerts.SelectTriangleVertices(SmoothTriangles);

				for (int ExpandedRegionVertexID : ExpandVerts.AsSet())
				{
					if (!StartVerts.IsSelected(ExpandedRegionVertexID))
					{
						FixedVerts.Add(ExpandedRegionVertexID);
					}
				}
			}
			else
			{
				SmoothTriangles.ExpandToOneRingNeighbours(nIncludeExteriorRings);
			}
		}


		// Submesh
		FMeshRegionOperator Region(Mesh, SmoothTriangles.AsArray());
		FDynamicMesh3& SmoothMesh = Region.Region.GetSubmesh();

		ELaplacianWeightScheme UseScheme = ELaplacianWeightScheme::IDTCotanget;
		TUniquePtr<UE::Solvers::IConstrainedMeshSolver> Smoother = UE::MeshDeformation::ConstructConstrainedMeshSmoother(
			UseScheme, SmoothMesh);
		check(Smoother);

		// map fixed verts to submesh
		TSet<int> SubFixedVerts;
		for (int BaseVertexID : FixedVerts)
		{
			SubFixedVerts.Add(Region.Region.MapVertexToSubmesh(BaseVertexID));
		}

		// Constrain borders
		double Weight = BorderWeight;
		TSet<int> Constrained;

		for (int BaseVertexID : Region.Region.GetBaseBorderVertices())
		{
			int SubVertexID = Region.Region.MapVertexToSubmesh(BaseVertexID);
			Smoother->AddConstraint(SubVertexID, Weight, SmoothMesh.GetVertex(SubVertexID), true);
			Constrained.Add(SubVertexID);
		}

		if (Constrained.Num() > 0 && nConstrainLoops > 0)
		{
			Weight = FMath::Sqrt(Weight);
			for (int k = 0; k < nConstrainLoops; ++k)
			{
				TSet<int> NextLayer;

				for (int SubVertexID : Constrained)
				{
					for (int NeighborVertexID : SmoothMesh.VtxVerticesItr(SubVertexID))
					{
						if (Constrained.Contains(NeighborVertexID) == false)
						{
							if (Smoother->IsConstrained(NeighborVertexID) == false)
							{
								Smoother->AddConstraint(NeighborVertexID, Weight, SmoothMesh.GetVertex(NeighborVertexID), 
									SubFixedVerts.Contains(NeighborVertexID));
							}
							NextLayer.Add(NeighborVertexID);
						}
					}
				}

				Constrained.Append(NextLayer);
				Weight = FMath::Sqrt(Weight);
			}
		}

		// constraint weight for interior vertices away from border
		// (this is an empirically-determined hack that seems to work OK to normalize the smoothing result for variable vertex count...)
		double NonlinearT = InteriorSmoothness * InteriorSmoothness;
		double ScaledPower = (NonlinearT / 50.0) * Mesh->VertexCount();
		double InteriorWeight = (ScaledPower < FMathf::ZeroTolerance) ? 999999.0 : (1.0 / ScaledPower);

		// soft constraint on all interior vertices, if requested
		if (InteriorWeight > 0)
		{
			for (int VertexID : SmoothMesh.VertexIndicesItr())
			{
				if (Smoother->IsConstrained(VertexID) == false)
				{
					Smoother->AddConstraint(VertexID, InteriorWeight, SmoothMesh.GetVertex(VertexID), SubFixedVerts.Contains(VertexID));
				}
			}
		}
		else if (SubFixedVerts.Num() > 0)
		{
			for (int VertexID : SubFixedVerts)
			{
				if (Smoother->IsConstrained(VertexID) == false)
				{
					Smoother->AddConstraint(VertexID, 0, SmoothMesh.GetVertex(VertexID), true);
				}
			}
		}

		TArray<FVector3d> PositionBuffer;
		bool bOK = Smoother->Deform(PositionBuffer);

		for (int VertexID : SmoothMesh.VertexIndicesItr())
		{
			SmoothMesh.SetVertex(VertexID, PositionBuffer[VertexID]);
		}

		Region.BackPropropagateVertices(true);
	}
}


FSmoothHoleFiller::FSmoothHoleFiller(FDynamicMesh3& Mesh, const FEdgeLoop& FillLoop) :
	Mesh(Mesh),
	FillLoop(FillLoop)
{
}


void FSmoothHoleFiller::ConstrainSubregionSeams(FSubRegionRemesher& Remesher, bool bConstrainROIBoundary)
{
	FMeshConstraints Constraints;
	const bool bAllowSplits = !bConstrainROIBoundary;
	constexpr bool bAllowSmoothing = false;

	// Constrain seam edges and vertices in the EdgeROI
	FMeshConstraintsUtil::ConstrainSeamsInEdgeROI(Constraints,
		Mesh,
		Remesher.GetCurrentEdgeROI().Array(),
		bAllowSplits,
		bAllowSmoothing);

	// Constrain TriangleROI boundaries in the EdgeROI
	FMeshConstraintsUtil::ConstrainROIBoundariesInEdgeROI(Constraints, Mesh, Remesher.GetCurrentEdgeROI(),
		Remesher.GetCurrentTriangleROI(), bAllowSplits, bAllowSmoothing);

	// Finally, fully constrain all mesh boundaries in the EdgeROI. Do this so we don't disrupt another loop that we 
	// might want to fill later.
	for (int EdgeID : Remesher.GetCurrentEdgeROI())
	{
		if (Mesh.IsBoundaryEdge(EdgeID))
		{
			Constraints.SetOrUpdateEdgeConstraint(EdgeID, FEdgeConstraint::FullyConstrained());
		}
	}

	if (Constraints.HasConstraints())
	{
		Remesher.SetExternalConstraints(MoveTemp(Constraints));
	}
}

void FSmoothHoleFiller::DefaultConfigureRemesher(FSubRegionRemesher& Remesher, bool bConstrainROIBoundary)
{
	check(RemeshingTargetEdgeLength > 0.0);
	Remesher.SetTargetEdgeLength(RemeshingTargetEdgeLength);

	Remesher.SmoothSpeedT = RemeshingSmoothAlpha;
	Remesher.bEnableSmoothing = (RemeshingSmoothAlpha > 0.0);
	Remesher.SmoothType = FRemesher::ESmoothTypes::MeanValue;

	ConstrainSubregionSeams(Remesher, bConstrainROIBoundary);

	Remesher.ProjectionMode = FMeshRefinerBase::ETargetProjectionMode::NoProjection;
}

bool FSmoothHoleFiller::Fill(int32 GroupID)
{
	// first do an easy hole fill
	FSimpleHoleFiller Filler(&Mesh, FillLoop);
	if (Filler.Fill(GroupID) == false)
	{
		return false;
	}

	if (FillLoop.Vertices.Num() <= 3)
	{
		NewTriangles = Filler.NewTriangles;
		return true;
	}

	// Initialize target edge length
	double MinLength, MaxLength, AvgLength;
	TMeshQueries<FDynamicMesh3>::EdgeLengthStatsFromEdges(Mesh, FillLoop.Edges, MinLength, MaxLength, AvgLength);
	check(FillOptions.FillDensityScalar > 0.0);
	RemeshingTargetEdgeLength = AvgLength / FillOptions.FillDensityScalar;

	// Get the initial triangle selection for remeshing/smoothing
	FMeshFaceSelection TriangleSelection(&Mesh);
	TriangleSelection.Select(Filler.NewTriangles);
	NewTriangles = Filler.NewTriangles;

	// if we aren't trying to stay inside hole, expand out a bit,
	// which allows us to clean up ugly edges
	if (!FillOptions.bConstrainToHoleInterior)
	{
		TriangleSelection.ExpandToOneRingNeighbours(FillOptions.RemeshingExteriorRegionWidth);
		TriangleSelection.LocalOptimize(true, true);
	}

	// remesh the initial coarse fill region
	if (InitialRemeshPasses > 0)
	{
		TUniquePtr<FSubRegionRemesher> Remesher;
		if (FillOptions.bConstrainToHoleInterior)
		{
			Remesher = MakeUnique<FRestrictedSubRegionRemesher>(&Mesh, TriangleSelection.AsSet());
		}
		else
		{
			Remesher = MakeUnique<FSubRegionRemesher>(&Mesh);
			FMeshVertexSelection VertexSelection(&Mesh, TriangleSelection);	// All triangle vertices
			Remesher->SetInitialVertexROI(VertexSelection.AsSet());
			Remesher->InitializeFromVertexROI();
		}

		Remesher->UpdateROI();
		DefaultConfigureRemesher(*Remesher, true);

		for (int k = 0; k < InitialRemeshPasses; ++k)
		{
			if (k > 0)
			{
				Remesher->UpdateROI();
				// If the ROI has expanded, it may include seam edges that were not constrained in the original Remesher set up
				ConstrainSubregionSeams(*Remesher, true);
			}
			Remesher->BasicRemeshPass();
		}

		NewTriangles = Remesher->GetCurrentTriangleROI().Array();
		TriangleSelection.DeselectAll();
		TriangleSelection.Select(NewTriangles);

		if (!FillOptions.bConstrainToHoleInterior)
		{
			TriangleSelection.LocalOptimize(true, true);
		}
	}

	// Now iteratively smooth and remesh

	if (FillOptions.bConstrainToHoleInterior)
	{
		for (int k = 0; k < SmoothSolveIterations; ++k)
		{
			bool bFinal = (k == (SmoothSolveIterations - 1));
			SmoothAndRemeshPreserveRegion(TriangleSelection, bFinal);

			TriangleSelection.DeselectAll();
			TriangleSelection.Select(NewTriangles);
		}
	}
	else
	{
		SmoothAndRemesh(TriangleSelection);

		TriangleSelection.DeselectAll();
		TriangleSelection.Select(NewTriangles);
	}

	// Filter NewTriangles by new groupID
	TArray<int> AllRemeshedTriangles = NewTriangles;
	NewTriangles.Reset();
	for (int TriangleID : AllRemeshedTriangles)
	{
		if (Mesh.GetTriangleGroup(TriangleID) == GroupID)
		{
			NewTriangles.Emplace(TriangleID);
		}
	}

	return true;
}


void FSmoothHoleFiller::SmoothAndRemeshPreserveRegion(FMeshFaceSelection& TriangleSelection, bool bFinal)
{
	check(FillOptions.bConstrainToHoleInterior);

	RegionSmooth(&Mesh, TriangleSelection, FillOptions.SmoothingInteriorRegionWidth, 
		FillOptions.SmoothingExteriorRegionWidth, true, FillOptions.InteriorSmoothness);

	if (bRemeshAfterSmooth)
	{
		FRestrictedSubRegionRemesher Remesher(&Mesh, TriangleSelection.AsSet());
		Remesher.UpdateROI();
		DefaultConfigureRemesher(Remesher, true);
		

		FDynamicMesh3 ProjectionTargetMeshCopy;
		TUniquePtr<FDynamicMeshAABBTree3> ProjectionTargetSpatial = nullptr;
		TUniquePtr<FMeshProjectionTarget> ProjectionTarget = nullptr;

		if (bFinal && FillOptions.bProjectDuringRemesh)
		{
			// TODO: Get only a mesh subset for projection rather than copying the whole mesh
			ProjectionTargetMeshCopy.Copy(Mesh, false, false, false, false);
			ProjectionTargetSpatial = MakeUnique<FDynamicMeshAABBTree3>(&ProjectionTargetMeshCopy, true);
			ProjectionTarget = MakeUnique<FMeshProjectionTarget>(&ProjectionTargetMeshCopy, ProjectionTargetSpatial.Get());

			Remesher.SetProjectionTarget(ProjectionTarget.Get());
			Remesher.ProjectionMode = FMeshRefinerBase::ETargetProjectionMode::AfterRefinement;
		}
		else
		{
			Remesher.ProjectionMode = FMeshRefinerBase::ETargetProjectionMode::NoProjection;
		}

		for (int k = 0; k < PostSmoothRemeshPasses; ++k)
		{
			if (k > 0)
			{
				Remesher.UpdateROI();
				// If the ROI has expanded, it may include seam edges that were not constrained in the original Remesher set up
				ConstrainSubregionSeams(Remesher, true);
			}
			Remesher.BasicRemeshPass();
		}

		NewTriangles = Remesher.GetCurrentTriangleROI().Array();
	}
	else
	{
		NewTriangles = TriangleSelection.AsArray();
	}
}


void FSmoothHoleFiller::SmoothAndRemesh(FMeshFaceSelection& TriangleSelection)
{
	check(!FillOptions.bConstrainToHoleInterior);

	RegionSmooth(&Mesh, TriangleSelection, FillOptions.SmoothingInteriorRegionWidth, 
		FillOptions.SmoothingExteriorRegionWidth, false, FillOptions.InteriorSmoothness);

	if (bRemeshAfterSmooth)
	{
		FSubRegionRemesher Remesher(&Mesh);
		FMeshVertexSelection VertexSelection(&Mesh, TriangleSelection);
		Remesher.SetInitialVertexROI(VertexSelection.AsSet());
		Remesher.InitializeFromVertexROI();
		Remesher.UpdateROI();

		DefaultConfigureRemesher(Remesher, false);

		FDynamicMesh3 ProjectionTargetMeshCopy;
		TUniquePtr<FDynamicMeshAABBTree3> ProjectionTargetSpatial = nullptr;
		TUniquePtr<FMeshProjectionTarget> ProjectionTarget = nullptr;

		if (FillOptions.bProjectDuringRemesh)
		{
			// TODO: Get only a mesh subset for projection rather than copying the whole mesh
			ProjectionTargetMeshCopy.Copy(Mesh, false, false, false, false);
			ProjectionTargetSpatial = MakeUnique<FDynamicMeshAABBTree3>(&ProjectionTargetMeshCopy, true);
			ProjectionTarget = MakeUnique<FMeshProjectionTarget>(&ProjectionTargetMeshCopy, ProjectionTargetSpatial.Get());

			Remesher.SetProjectionTarget(ProjectionTarget.Get());
			Remesher.ProjectionMode = FMeshRefinerBase::ETargetProjectionMode::AfterRefinement;
		}
		else
		{
			Remesher.ProjectionMode = FMeshRefinerBase::ETargetProjectionMode::NoProjection;
		}

		for (int k = 0; k < PostSmoothRemeshPasses; ++k)
		{
			if (k > 0)
			{
				Remesher.UpdateROI();
				// If the ROI has expanded, it may include seam edges that were not constrained in the original Remesher set up
				ConstrainSubregionSeams(Remesher, false);
			}
			Remesher.BasicRemeshPass();
		}

		NewTriangles = Remesher.GetCurrentTriangleROI().Array();
	}
	else
	{
		NewTriangles = TriangleSelection.AsArray();
	}
}

