// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/TransferBoneWeights.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Async/ParallelFor.h"
#include "Util/ProgressCancel.h"
#include "BoneIndices.h"
#include "IndexTypes.h"
#include "TransformTypes.h"
#include "Algo/Count.h"
#include "Solvers/Internal/QuadraticProgramming.h"
#include "Solvers/LaplacianMatrixAssembly.h"
#include "Operations/SmoothBoneWeights.h"

using namespace UE::AnimationCore;
using namespace UE::Geometry;

namespace TransferBoneWeightsLocals 
{
	/**
	 * Given a triangle and point on a triangle (via barycentric coordinates), compute the bone weights for the point.
	 * 
	 * @param OutWeights Interpolated weights for a vertex with Bary barycentric coordinates
	 * @param TriVertices The vertices of a triangle containing the point we are interpolating the weights for
	 * @param Bary Barycentric coordinates of the point
	 * @param Attribute Attribute containing bone weights of the mesh that TriVertices belong to
	 * @param MaxNumInfluences The maximum allowed number of influences per vertex stored in OutWeights
	 * @param SourceIndexToBone Optional map from bone index to bone name for the source mesh
	 * @param TargetBoneToIndex OPtional map from bone name to bone index for the target mesh
	 * @param bNormalizeToOne If true, OutWeights will be normalized to sum to 1.
	 */
    void InterpolateBoneWeights(FBoneWeights& OutWeights,
								const FIndex3i& TriVertices,
								const FVector3f& Bary,
								const FDynamicMeshVertexSkinWeightsAttribute* Attribute,
								const int32 MaxNumInfluences,
								const TArray<FName>* SourceIndexToBone = nullptr,
								const TMap<FName, FBoneIndexType>* TargetBoneToIndex = nullptr,
								bool bNormalizeToOne = true)
	{
		FBoneWeights Weight1, Weight2, Weight3;
		Attribute->GetValue(TriVertices[0], Weight1);
		Attribute->GetValue(TriVertices[1], Weight2);
		Attribute->GetValue(TriVertices[2], Weight3);

		FBoneWeightsSettings BlendSettings;
		BlendSettings.SetNormalizeType(bNormalizeToOne ? EBoneWeightNormalizeType::Always : EBoneWeightNormalizeType::None);
		BlendSettings.SetBlendZeroInfluence(true);
		BlendSettings.SetMaxWeightCount(MaxNumInfluences);
		OutWeights = FBoneWeights::Blend(Weight1, Weight2, Weight3, Bary[0], Bary[1], Bary[2], BlendSettings);

		// TODO: Blend method can potentially skip applying renormalization and prunning weights to match MaxNumInfluences
		// using the BlendSettings so force renormalization. Remove this once FBoneWeights::Blend is fixed.
		OutWeights.Renormalize(BlendSettings);
		 
		// Check if we need to remap the indices
		if (SourceIndexToBone && TargetBoneToIndex) 
		{
			FBoneWeightsSettings BoneSettings;
			BoneSettings.SetNormalizeType(EBoneWeightNormalizeType::None);

			FBoneWeights MappedWeights;

			for (int32 WeightIdx = 0; WeightIdx < OutWeights.Num(); ++WeightIdx)
			{
				const FBoneWeight& BoneWeight = OutWeights[WeightIdx];

				FBoneIndexType FromIdx = BoneWeight.GetBoneIndex();
				uint16 FromWeight = BoneWeight.GetRawWeight();

				checkSlow(FromIdx < SourceIndexToBone->Num());
				if (FromIdx < SourceIndexToBone->Num())
				{
					FName BoneName = (*SourceIndexToBone)[FromIdx];
					if (TargetBoneToIndex->Contains(BoneName))
					{
						FBoneIndexType ToIdx = (*TargetBoneToIndex)[BoneName];
						FBoneWeight MappedBoneWeight(ToIdx, FromWeight);
						MappedWeights.SetBoneWeight(MappedBoneWeight, BoneSettings);
					}
					else 
					{	
						UE_LOGF(LogGeometry, Error, "FTransferBoneWeights: Bone name %ls does not exist in the target mesh.", *BoneName.ToString());
					}
				}
			}


			if (MappedWeights.Num() == 0)
			{
				// If no bone mappings were found, add a single entry for the root bone
				MappedWeights.SetBoneWeight(FBoneWeight(0, 1.0f), FBoneWeightsSettings());
			}
			else if (OutWeights.Num() != MappedWeights.Num() && bNormalizeToOne)
			{	
				// In case some of the bones were not mapped we need to renormalize
				MappedWeights.Renormalize(FBoneWeightsSettings());
			}

			OutWeights = MappedWeights;
		}
	}

	
	FDynamicMeshVertexSkinWeightsAttribute* GetOrCreateSkinWeightsAttribute(FDynamicMesh3& InMesh, const FName& InProfileName)
	{
		checkSlow(InMesh.HasAttributes());
		FDynamicMeshVertexSkinWeightsAttribute* Attribute = InMesh.Attributes()->GetSkinWeightsAttribute(InProfileName);
		if (Attribute == nullptr)
		{
			Attribute = new FDynamicMeshVertexSkinWeightsAttribute(&InMesh);
			InMesh.Attributes()->AttachSkinWeightsAttribute(InProfileName, Attribute);
		}
		return Attribute;
	}

	static FVector3f ToUENormal(const FVector3d& Normal)
	{
		return FVector3f((float)Normal.X, (float)Normal.Y, (float)Normal.Z);
	}
}

FTransferBoneWeights::FTransferBoneWeights(const FDynamicMesh3* InSourceMesh, 
								   const FName& InSourceProfileName,
								   const FDynamicMeshAABBTree3* InSourceBVH)
:
SourceMesh(InSourceMesh),
SourceProfileName(InSourceProfileName),
SourceBVH(InSourceBVH)
{
	// If the BVH for the source mesh was not specified then create one
	if (SourceBVH == nullptr)
	{
		InternalSourceBVH = MakeUnique<FDynamicMeshAABBTree3>(SourceMesh);
	}
}

FTransferBoneWeights::~FTransferBoneWeights() 
{
}

bool FTransferBoneWeights::Cancelled()
{
	return (Progress == nullptr) ? false : Progress->Cancelled();
}

EOperationValidationResult FTransferBoneWeights::Validate()
{	
	if (SourceMesh == nullptr) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	// Either BVH was passed by the caller or was created internally in the constructor
	if (SourceBVH == nullptr && InternalSourceBVH.IsValid() == false) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (SourceMesh->HasAttributes() == false) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}
	
	if (SourceMesh->Attributes()->GetSkinWeightsAttribute(SourceProfileName) == nullptr) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (bIgnoreBoneAttributes == false && SourceMesh->Attributes()->HasBones() == false) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	return EOperationValidationResult::Ok;
}

bool FTransferBoneWeights::TransferWeightsToMesh(FDynamicMesh3& InOutTargetMesh, const FName& InTargetProfileName)
{	
	using TransferBoneWeightsLocals::ToUENormal;

	if (Validate() != EOperationValidationResult::Ok) 
	{
		return false;
	}

	if (!InOutTargetMesh.HasAttributes())
	{
		InOutTargetMesh.EnableAttributes(); 
	}

	if (!bIgnoreBoneAttributes && !InOutTargetMesh.Attributes()->HasBones())
	{
		return false; // the target mesh must have bone attributes
	}

	// If we need to compare normals, make sure both the target and the source meshes have per-vertex normals data
	TUniquePtr<FMeshNormals> InternalTargetMeshNormals;
	if (NormalThreshold >= 0)
	{
		if (!SourceMesh->HasVertexNormals() && !InternalSourceMeshNormals)
		{
			// only do this once for the source mesh in case of subsequent calls to the method
			InternalSourceMeshNormals = MakeUnique<FMeshNormals>(SourceMesh);
			InternalSourceMeshNormals->ComputeVertexNormals();
		}

		if (!InOutTargetMesh.HasVertexNormals())
		{
			InternalTargetMeshNormals = MakeUnique<FMeshNormals>(&InOutTargetMesh);
			InternalTargetMeshNormals->ComputeVertexNormals();
		}
	}
	
	FDynamicMeshVertexSkinWeightsAttribute* TargetSkinWeights = TransferBoneWeightsLocals::GetOrCreateSkinWeightsAttribute(InOutTargetMesh, InTargetProfileName);
	checkSlow(TargetSkinWeights);
	
	// Map the bone name to its index for the target mesh.
	// Will be null if either the target and the source skeletons are the same or the caller forced the attributes to be ignored
	TUniquePtr<TMap<FName, uint16>> TargetBoneToIndex;
	if (!bIgnoreBoneAttributes)
	{	
		const TArray<FName>& SourceBoneNames = SourceMesh->Attributes()->GetBoneNames()->GetAttribValues();
		const TArray<FName>& TargetBoneNames = InOutTargetMesh.Attributes()->GetBoneNames()->GetAttribValues();

		if (SourceBoneNames != TargetBoneNames)
		{
			TargetBoneToIndex = MakeUnique<TMap<FName, uint16>>();
			TargetBoneToIndex->Reserve(TargetBoneNames.Num());

			for (int32 BoneID = 0; BoneID < TargetBoneNames.Num(); ++BoneID)
			{
				const FName& BoneName = TargetBoneNames[BoneID];
				if (TargetBoneToIndex->Contains(BoneName))
				{
					checkSlow(false);
					return false; // there should be no duplicates
				}
				TargetBoneToIndex->Add(BoneName, static_cast<uint16>(BoneID));
			}
		}
	}

	bool bFailed = false;

	// compute the transfer only for the subset of vertices if necessary  
	const bool bUseSubset = !TargetVerticesSubset.IsEmpty();
	const int32 NumVerticesToTransfer = bUseSubset ? TargetVerticesSubset.Num() : InOutTargetMesh.MaxVertexID();
	
	if (TransferMethod == ETransferBoneWeightsMethod::ClosestPointOnSurface)
	{
		MatchedVertices.Init(false, NumVerticesToTransfer);
		
		ParallelFor(NumVerticesToTransfer, [this, &InOutTargetMesh, &TargetBoneToIndex, &TargetSkinWeights, &InternalTargetMeshNormals, bUseSubset](int32 InVertexID)
		{
			if (Cancelled()) 
			{
				return;
			}

			const int32 VertexID = bUseSubset ? TargetVerticesSubset[InVertexID] : InVertexID;
			if (InOutTargetMesh.IsVertex(VertexID)) 
			{
				const FVector3d Point = InOutTargetMesh.GetVertex(VertexID);

				FVector3f Normal = FVector3f::UnitY();
				if (NormalThreshold >= 0) 
				{
					const bool bHasNormals = InOutTargetMesh.HasVertexNormals();
					if (ensure(bHasNormals || InternalTargetMeshNormals))
					{
						Normal = bHasNormals ? InOutTargetMesh.GetVertexNormal(VertexID) : ToUENormal(InternalTargetMeshNormals->GetNormals()[VertexID]);
					}
				}

				FBoneWeights Weights;
				if (TransferWeightsToPoint(Weights, Point, TargetBoneToIndex.Get(), Normal))
				{
					TargetSkinWeights->SetValue(VertexID, Weights);
					MatchedVertices[VertexID] = true;
				}
			}

		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		// If the caller requested to simply find the closest point for all vertices then the number of matched vertices
		// must be equal to the target mesh vertex count
		if (SearchRadius < 0 && NormalThreshold < 0)
		{
			int32 NumMatched = 0;
			for (bool Flag : MatchedVertices)
			{
				if (Flag) 
				{ 
					NumMatched++;
				}
			}
			bFailed = NumMatched != NumVerticesToTransfer;
		}
	} 
	else if (TransferMethod == ETransferBoneWeightsMethod::InpaintWeights)
	{
		/**
         *  Given two meshes, Mesh1 without weights and Mesh2 with weights, assume they are aligned in 3d space.
         *  For every vertex on Mesh1 find the closest point on the surface of Mesh2 within a radius R. If the difference 
         *  between the normals of the two points is below the threshold, then it's a match. Otherwise no match.
         *  So now we have two sets of vertices on Mesh1. One with a match on the source mesh and one without a match.
         *  For all the vertices with a match, copy weights over. For all the vertices without the match, do nothing.
         *  Now, for all the vertices without a match, try to approximate the weights by smoothly interpolating between 
         *  the weights at the known vertices via solving a quadratic problem. 
         *  
         *  The solver minimizes an energy
         *      trace(W^t Q W)
         *      W \in R^(nxm) is a matrix where n is the number of vertices and m is the number of bones. So (i,j) entry is 
         *                    the influence (weight) of a vertex i by bone j
         *      Q \in R^(nxn) is a matrix that combines both Dirichlet and Laplacian energies, Q = -L + L*M^(-1)*L
         *                    where L is a cotangent Laplacian and M is a mass matrix
         *  
         *  subject to constraints
         *      All weights at a single vertex sum to 1: sum(W(i,:)) = 1
         *      All weights must be non-negative: W(i,j) >=0 for any i, j
         *      Any vertex for which we found a match must have fixed weights that can't be changed, 
         *      i.e. W(i,j) = KnownWeights(i,j) where i is a vertex for which we found a match on the body.
		 */
	
		MatchedVertices.Init(false, InOutTargetMesh.MaxVertexID());
		
		// Check if the target mesh contains the user specifed force inpaint weight map 
		const FDynamicMeshWeightAttribute* ForceInpaintLayer = nullptr;
		if (!ForceInpaintWeightMapName.IsNone() && (ForceInpaint.IsEmpty() || ForceInpaint.Num() != InOutTargetMesh.MaxVertexID())) // ForceInpaint array takes priority if valid
		{
			for (int32 Idx = 0; Idx < InOutTargetMesh.Attributes()->NumWeightLayers(); ++Idx)
			{	
				const FDynamicMeshWeightAttribute* WeightLayer = InOutTargetMesh.Attributes()->GetWeightLayer(Idx);
				if (WeightLayer && WeightLayer->GetName() == ForceInpaintWeightMapName)
				{
					ForceInpaintLayer = WeightLayer;
					break;
				}
			}
		}

		// because the inpaint algorithm can extract data from regions outside the target vertex subset, a temporary attribute profile is used to modify the weights.
		// NOTE: make sure to copy the weights of the vertex subset into the complete TargetSkinWeights attribute before exciting the function. (see CopySubsetWeightsIfNeeded) 
		FDynamicMeshVertexSkinWeightsAttribute SubsetTargetSkinWeights;
		if (bUseSubset)
		{
			SubsetTargetSkinWeights.Copy(*TargetSkinWeights);
		}
		FDynamicMeshVertexSkinWeightsAttribute* EditedSkinWeights = bUseSubset ? &SubsetTargetSkinWeights : TargetSkinWeights;

		// For every vertex on the target mesh try to find the match on the source mesh using the distance and normal checks
		ParallelFor(InOutTargetMesh.MaxVertexID(), [this, &InOutTargetMesh, &ForceInpaintLayer, &TargetBoneToIndex, &EditedSkinWeights, &InternalTargetMeshNormals](int32 VertexID)
		{
			if (Cancelled()) 
			{
				return;
			}
			
			if (InOutTargetMesh.IsVertex(VertexID)) 
			{
				// check if we need to force the vertex to not have a match
				if (ForceInpaint.Num() == InOutTargetMesh.MaxVertexID() && ForceInpaint[VertexID] != 0)
				{
					return;
				}
				else if (ForceInpaintLayer != nullptr)
				{
					float Value;
					ForceInpaintLayer->GetValue(VertexID, &Value);
					if (Value != 0)
					{
						return;
					}
				}

				const FVector3d Point = InOutTargetMesh.GetVertex(VertexID);
				FVector3f Normal = FVector3f::UnitY();
				if (NormalThreshold >= 0) 
				{
					const bool bHasNormals = InOutTargetMesh.HasVertexNormals();
					if (ensure(bHasNormals || InternalTargetMeshNormals.IsValid()))
					{
						Normal = bHasNormals ? InOutTargetMesh.GetVertexNormal(VertexID) : ToUENormal(InternalTargetMeshNormals->GetNormals()[VertexID]);
					}
				}

				FBoneWeights Weights;
				if (TransferWeightsToPoint(Weights, Point, TargetBoneToIndex.Get(), Normal))
				{
					EditedSkinWeights->SetValue(VertexID, Weights);
					MatchedVertices[VertexID] = true;
				}
			}
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		if (Cancelled()) 
		{
			return false;
		}

		int32 NumMatched = 0;
		if (!bUseSubset)
		{
			for (bool Flag : MatchedVertices)
			{
				if (Flag) 
				{ 
					NumMatched++;
				}
			}
		}
		else
		{
			NumMatched = (int32)Algo::CountIf(TargetVerticesSubset, [this](int32 VertexID)
			{
				return MatchedVertices.IsValidIndex(VertexID) && MatchedVertices[VertexID];
			});
		}

		// If no vertices matched, we have nothing to inpaint.
		if (NumMatched == 0)
		{
			return false;
		}

		auto CopySubsetWeightsIfNeeded = [Subset = TargetVerticesSubset, &EditedSkinWeights, &TargetSkinWeights, &InOutTargetMesh]()
		{
			if (EditedSkinWeights && EditedSkinWeights != TargetSkinWeights)
			{
				for (const int32 VertexID: Subset)
				{
					if (InOutTargetMesh.IsVertex(VertexID))
					{
						FBoneWeights Weights;
						EditedSkinWeights->GetValue(VertexID, Weights);
						TargetSkinWeights->SetValue(VertexID, Weights);
					}
				}
			}
		};
		
		// If all vertices were matched then nothing else to do
		if (NumMatched == NumVerticesToTransfer)
		{
			// copy weights from the subset skin weights if using subset
			CopySubsetWeightsIfNeeded();
			return true;
		}

		// Compute linearization so we can store constraints at linearized indices
		FVertexLinearization VtxLinearization(InOutTargetMesh, false);
		const TArray<int32>& ToMeshV = VtxLinearization.ToId();
		const TArray<int32>& ToIndex = VtxLinearization.ToIndex();
		
		// Setup the sparse matrix FixedValues of known (matched) weight values and the array (FixedIndices) of the matched vertex IDs
		const int32 TargetNumBones = InOutTargetMesh.Attributes()->GetBoneNames()->GetAttribValues().Num();
		FSparseMatrixD FixedValues;
		FixedValues.resize(NumMatched, TargetNumBones);
		std::vector<Eigen::Triplet<FSparseMatrixD::Scalar>> FixedValuesTriplets;
		FixedValuesTriplets.reserve(NumMatched);
		
		TArray<int> FixedIndices;
		FixedIndices.Reserve(NumMatched);

		for (int32 VertexID = 0; VertexID < InOutTargetMesh.MaxVertexID(); ++VertexID)
		{
			if (InOutTargetMesh.IsVertex(VertexID) && MatchedVertices[VertexID])
			{
				FBoneWeights Data;
				EditedSkinWeights->GetValue(VertexID, Data);

				const int32 NumBones = Data.Num();
				checkSlow(NumBones > 0);
				const int32 CurIdx = FixedIndices.Num();
				for (int32 BoneID = 0; BoneID < NumBones; ++BoneID)
				{
					const int BoneIdx = Data[BoneID].GetBoneIndex();
					const double BoneWeight = Data[BoneID].GetWeight();
					FixedValuesTriplets.emplace_back(CurIdx, BoneIdx, BoneWeight);
				}

				checkSlow(VertexID < ToIndex.Num());
				FixedIndices.Add(ToIndex[VertexID]);
			}
		}
		FixedValues.setFromTriplets(FixedValuesTriplets.begin(), FixedValuesTriplets.end());

		const int32 NumVerts = VtxLinearization.NumVerts();
		FEigenSparseMatrixAssembler CotangentAssembler(NumVerts, NumVerts);
		FEigenSparseMatrixAssembler LaplacianAssembler(NumVerts, NumVerts);

		UE::Tasks::FTask CotangentTask, LaplacianTask;
		if (bUseIntrinsicLaplacian)
		{
			CotangentTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&InOutTargetMesh, &VtxLinearization, &CotangentAssembler]()
			{
				// Construct the Cotangent weights matrix
				UE::MeshDeformation::ConstructFullIDTCotangentLaplacian<double>(InOutTargetMesh, VtxLinearization, CotangentAssembler,
					UE::MeshDeformation::ECotangentWeightMode::Default,
					UE::MeshDeformation::ECotangentAreaMode::NoArea);
			});

			
			LaplacianTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&InOutTargetMesh, &VtxLinearization, &LaplacianAssembler]()
			{
				// Construct the Laplacian with cotangent weights scaled by the voronoi area (i.e. M^(-1)*L matrix where M is the mass/stiffness matrix)
				UE::MeshDeformation::ConstructFullIDTCotangentLaplacian<double>(InOutTargetMesh, VtxLinearization, LaplacianAssembler,
																				UE::MeshDeformation::ECotangentWeightMode::Default,
																				UE::MeshDeformation::ECotangentAreaMode::VoronoiArea);
			});
		}
		else 
		{
			CotangentTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&InOutTargetMesh, &VtxLinearization, &CotangentAssembler]()
			{
				UE::MeshDeformation::ConstructFullCotangentLaplacian<double>(InOutTargetMesh, VtxLinearization, CotangentAssembler,
					UE::MeshDeformation::ECotangentWeightMode::Default,
					UE::MeshDeformation::ECotangentAreaMode::NoArea);
			});
			
			LaplacianTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&InOutTargetMesh, &VtxLinearization, &LaplacianAssembler]()
			{
				UE::MeshDeformation::ConstructFullCotangentLaplacian<double>(InOutTargetMesh, VtxLinearization, LaplacianAssembler,
					UE::MeshDeformation::ECotangentWeightMode::Default,
					UE::MeshDeformation::ECotangentAreaMode::VoronoiArea);
			});
		}

		FSparseMatrixD CotangentMatrix, MassCotangentMatrix;
		UE::Tasks::FTask CotangentExtractTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&CotangentMatrix, &CotangentAssembler]()
		{
			CotangentAssembler.ExtractResult(CotangentMatrix);
		}, CotangentTask);

		UE::Tasks::FTask LaplacianExtractTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&MassCotangentMatrix, &LaplacianAssembler]()
		{
			LaplacianAssembler.ExtractResult(MassCotangentMatrix);
		}, LaplacianTask);

		TArray<UE::Tasks::FTask> PendingTasks = { CotangentExtractTask, LaplacianExtractTask };
		UE::Tasks::Wait(PendingTasks);

		// -L * L* M^(-1)*L energy
		FSparseMatrixD Energy = -1*CotangentMatrix + CotangentMatrix*MassCotangentMatrix;

		// Solve the QP problem with fixed constraints
		FSparseMatrixD TargetWeights;
		TArray<int> VaribleRows;

		// We want the solution TargetWeights matrix to only contain the rows representing the variable (non-fixed) rows
		constexpr bool bVariablesOnly = true;
 		bFailed = !FQuadraticProgramming::SolveWithFixedConstraints(Energy, nullptr, FixedIndices, FixedValues, TargetWeights, bVariablesOnly, KINDA_SMALL_NUMBER, &VaribleRows);
		checkSlow((VaribleRows.Num() + FixedIndices.Num()) == Energy.rows());

		if (!bFailed)
		{
			// Transpose so we can efficiently iterate over the col-major matrix. Each column now contains per-vertex weights.
			// Otherwise, we are iterating over rows of a col-major matrix which is slow.
			FSparseMatrixD TargetWeightsTransposed = TargetWeights.transpose(); 

			FBoneWeightsSettings BoneSettings;
			BoneSettings.SetNormalizeType(EBoneWeightNormalizeType::None);
			
			FBoneWeightsSettings RenormalizeBoneSettings;
			RenormalizeBoneSettings.SetMaxWeightCount(MaxNumInfluences);

			// Iterate over every column containing all bone weights for the vertex
			for (int32 ColIdx = 0; ColIdx < TargetWeightsTransposed.outerSize(); ++ColIdx)
			{
				FBoneWeights WeightArray;

				// Iterate over only non-zero rows (i.e. bone indices with non-zero weights)
				for (FSparseMatrixD::InnerIterator Itr(TargetWeightsTransposed, ColIdx); Itr; ++Itr)
				{
					const FBoneIndexType BoneIdx = static_cast<FBoneIndexType>(Itr.row());
					const float Weight = static_cast<float>(Itr.value());
					FBoneWeight Bweight(BoneIdx, Weight);
					WeightArray.SetBoneWeight(Bweight, BoneSettings);
				}

				WeightArray.Renormalize(RenormalizeBoneSettings);

				const int32 VertexIDLinearalized = bVariablesOnly ? static_cast<int32>(VaribleRows[ColIdx]) : ColIdx; // linearized vertex ID (matrix row) of the variable in the Energy matrix
				const int32 VertexID = static_cast<int32>(ToMeshV[VertexIDLinearalized]);
				EditedSkinWeights->SetValue(VertexID, WeightArray);
			}

			// copy weights from the subset skin weights if using subset
			CopySubsetWeightsIfNeeded();
			
			// Optional post-processing smoothing of the weights at the vertices without a match
			if (NumSmoothingIterations > 0 && SmoothingStrength > 0)
			{
				TArray<int32> VerticesToSmooth;
				const int32 NumNotMatched = InOutTargetMesh.VertexCount() - NumMatched;
				VerticesToSmooth.Reserve(NumNotMatched);
				for (int32 VertexID = 0; VertexID < InOutTargetMesh.MaxVertexID(); ++VertexID)
				{
					if (InOutTargetMesh.IsVertex(VertexID) && !MatchedVertices[VertexID])
					{
						VerticesToSmooth.Add(VertexID);
					}
				}

				FSmoothDynamicMeshVertexSkinWeights SmoothWeights(&InOutTargetMesh, InTargetProfileName);
				SmoothWeights.MaxNumInfluences = MaxNumInfluences;
				if (ensure(SmoothWeights.Validate() == EOperationValidationResult::Ok))
				{
					ensure(SmoothWeights.SmoothWeightsAtVerticesWithinDistance(VerticesToSmooth, SmoothingStrength, SearchRadius, NumSmoothingIterations));
				}
			}
		}
	}
	else 
	{
		checkNoEntry(); // unsupported method
	}

	if (Cancelled() || bFailed) 
	{
		return false;
	}
		
	return true;
}

bool FTransferBoneWeights::TransferWeightsToPoint(UE::AnimationCore::FBoneWeights& OutWeights, 
												  const FVector3d& InPoint, 
												  const TMap<FName, uint16>* TargetBoneToIndex,
												  const FVector3f& InNormal)
{	
	using TransferBoneWeightsLocals::ToUENormal;

	// Find the containing triangle and the barycentric coordinates of the closest point
	int32 TriID; 
	FVector3d Bary;
	if (!FindClosestPointOnSourceSurface(InPoint, TargetToWorld, TriID, Bary))
	{
		return false;
	}
	FVector3f BaryF = FVector3f((float)Bary[0], (float)Bary[1], (float)Bary[2]);
	const FIndex3i TriVertex = SourceMesh->GetTriangle(TriID);

	const FDynamicMeshVertexSkinWeightsAttribute* SourceSkinWeights = SourceMesh->Attributes()->GetSkinWeightsAttribute(SourceProfileName);
	const TArray<FName>* SourceBoneNames = nullptr;
	if (!bIgnoreBoneAttributes) 
	{
		SourceBoneNames = &SourceMesh->Attributes()->GetBoneNames()->GetAttribValues();
	}

	if (SearchRadius < 0 && NormalThreshold < 0)
	{
		// If the radius and normals are ignored, simply interpolate the weights and return the result
		TransferBoneWeightsLocals::InterpolateBoneWeights(OutWeights, TriVertex, BaryF, SourceSkinWeights, MaxNumInfluences, SourceBoneNames, TargetBoneToIndex);
	}
	else
	{
		bool bPassedRadiusCheck = true;
		if (SearchRadius >= 0)
		{
			const FVector3d MatchedPoint = SourceMesh->GetTriBaryPoint(TriID, Bary[0], Bary[1], Bary[2]);
			bPassedRadiusCheck = (InPoint - MatchedPoint).Length() <= SearchRadius;
		}

		bool bPassedNormalsCheck = true;
		if (NormalThreshold >= 0)
		{
			FVector3f Normal0 = FVector3f::UnitY();
			FVector3f Normal1 = FVector3f::UnitY();
			FVector3f Normal2 = FVector3f::UnitY();
			const bool bHasSourceNormals = SourceMesh->HasVertexNormals();
			if (ensure(bHasSourceNormals || InternalSourceMeshNormals.IsValid()))
			{
				Normal0 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertex[0]) : ToUENormal(InternalSourceMeshNormals->GetNormals()[TriVertex[0]]);
				Normal1 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertex[1]) : ToUENormal(InternalSourceMeshNormals->GetNormals()[TriVertex[1]]);
				Normal2 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertex[2]) : ToUENormal(InternalSourceMeshNormals->GetNormals()[TriVertex[2]]);
			}

			const FVector3f MatchedNormal = Normalized(BaryF[0]*Normal0 + BaryF[1]*Normal1 + BaryF[2]*Normal2);
			const FVector3f InNormalNormalized = Normalized(InNormal);
			const float NormalAngle = FMathf::ACos(InNormalNormalized.Dot(MatchedNormal));
			bPassedNormalsCheck = (double)NormalAngle <= NormalThreshold;

			if (!bPassedNormalsCheck && LayeredMeshSupport)
			{
				// try again with a flipped normal
				bPassedNormalsCheck = (double)(TMathUtil<float>::Pi - NormalAngle) <= NormalThreshold;
			}
		}
		
		if (bPassedRadiusCheck && bPassedNormalsCheck)
		{
			TransferBoneWeightsLocals::InterpolateBoneWeights(OutWeights, TriVertex, BaryF, SourceSkinWeights, MaxNumInfluences, SourceBoneNames, TargetBoneToIndex);
		}
		else
		{
			return false;
		}
	}

	return true;
}

template<typename BoneIndexType, typename BoneFloatWeightType, typename PosVectorType, typename NormalVectorType>
bool FTransferBoneWeights::TransferWeightsToPoint(TArray<BoneIndexType>& OutBones, 
												  TArray<BoneFloatWeightType>& OutWeights,
												  const UE::Math::TVector<PosVectorType>& InPoint,
												  const TMap<FName, uint16>* TargetBoneToIndex,
												  const UE::Math::TVector<NormalVectorType>& InNormal)
{
	FBoneWeights BoneWeights;
	if (!this->TransferWeightsToPoint(BoneWeights, FVector3d(InPoint.X, InPoint.Y, InPoint.Z), TargetBoneToIndex, FVector3f((float)InNormal.X, (float)InNormal.Y, (float)InNormal.Z)))
	{
		return false;
	}

	const int32 NumEntries = BoneWeights.Num();

	OutBones.SetNum(NumEntries);
	OutWeights.SetNum(NumEntries);

	for (int32 BoneIdx = 0; BoneIdx < NumEntries; ++BoneIdx)
	{
		OutBones[BoneIdx] = static_cast<BoneIndexType>(BoneWeights[BoneIdx].GetBoneIndex());
		OutWeights[BoneIdx] = static_cast<BoneFloatWeightType>(BoneWeights[BoneIdx].GetWeight());
	}

	return true;
}

bool FTransferBoneWeights::FindClosestPointOnSourceSurface(const FVector3d& InPoint, const FTransformSRT3d& InToWorld, int32& NearTriID, FVector3d& Bary)
{
	IMeshSpatial::FQueryOptions Options;
	double NearestDistSqr;
	
	const FVector3d WorldPoint = InToWorld.TransformPosition(InPoint);
	if (SourceBVH != nullptr) 
	{ 
		NearTriID = SourceBVH->FindNearestTriangle(WorldPoint, NearestDistSqr, Options);
	}
	else 
	{
		NearTriID = InternalSourceBVH->FindNearestTriangle(WorldPoint, NearestDistSqr, Options);
	}

	if (!ensure(NearTriID != IndexConstants::InvalidID))
	{
		return false;
	}

	const FDistPoint3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleDistance(*SourceMesh, NearTriID, WorldPoint);
	const FVector3d NearestPnt = Query.ClosestTrianglePoint;
	const FIndex3i TriVertex = SourceMesh->GetTriangle(NearTriID);

	Bary = VectorUtil::BarycentricCoords(NearestPnt, SourceMesh->GetVertexRef(TriVertex.A),
													 SourceMesh->GetVertexRef(TriVertex.B),
													 SourceMesh->GetVertexRef(TriVertex.C));

	return true;
}

// template instantiation
template DYNAMICMESH_API bool UE::Geometry::FTransferBoneWeights::TransferWeightsToPoint<int,float,float,float>(class TArray<int,class TSizedDefaultAllocator<32> > &,class TArray<float,class TSizedDefaultAllocator<32> > &,struct UE::Math::TVector<float> const &,class TMap<class FName,unsigned short,class FDefaultSetAllocator,struct TDefaultMapHashableKeyFuncs<class FName,unsigned short,0> > const *,struct UE::Math::TVector<float> const &);