// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/TransferDynamicMeshAttributes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Util/ProgressCancel.h"
#include "Async/ParallelFor.h"
#include "TransformTypes.h"
#include "Algo/Count.h"
#include "Solvers/Internal/QuadraticProgramming.h"
#include "Solvers/LaplacianMatrixAssembly.h"
#include "Operations/SmoothDynamicMeshAttributes.h"

using namespace UE::Geometry;

namespace TransferValuesLocals 
{
	static constexpr int32 NumElements = 4;
	
	/**
	 * Given a triangle and point on a triangle (via barycentric coordinates), compute the color.
	 * 
	 * @param OutColor Interpolated color for a vertex with Bary barycentric coordinates
	 * @param TriElements The vertices of a triangle containing the point we are interpolating the colors for
	 * @param Bary Barycentric coordinates of the point
	 * @param InColorAttribute Attribute containing colors of the mesh that TriVertices belong to
	 */

	void InterpolateVertexAttribute(FVector4f& OutColor,
									const FIndex3i& TriElements,
									const FVector3f& Bary,
									const FDynamicMeshColorOverlay* InColorAttribute)
	{
		FVector4f Value1, Value2, Value3;

		InColorAttribute->GetElement(TriElements[0], Value1);
		InColorAttribute->GetElement(TriElements[1], Value2);
		InColorAttribute->GetElement(TriElements[2], Value3);

		const float Alpha = Bary[0], Beta = Bary[1], Theta  = Bary[2];
		for (int32 Idx = 0; Idx < NumElements; ++Idx)
		{
			OutColor[Idx] = Alpha * Value1[Idx] + Beta * Value2[Idx] + Theta * Value3[Idx];
		}
	}

	FDynamicMeshColorOverlay* GetOrCreateColorAttribute(FDynamicMesh3& InMesh, const bool InSplit)
	{
		checkSlow(InMesh.HasAttributes());
		FDynamicMeshAttributeSet* MeshAttributes = InMesh.Attributes();
		if (!MeshAttributes->HasPrimaryColors())
		{
			MeshAttributes->EnablePrimaryColors();

			// Start with a clean attribute and elements we can write to.
			MeshAttributes->PrimaryColors()->CreatePerVertex(0.f);
		}
		
		if (InSplit)
		{
			// create vertex instances for each face if needed
			FDynamicMeshColorOverlay* ColorOverlay = MeshAttributes->PrimaryColors();
		
			ColorOverlay->SplitVerticesWithPredicate(
				[](int ElementIdx, int TriID) { return true; },
				[ColorOverlay](int ElementIdx, int TriID, float* FillVect)
				{
					const FVector4f CurValue = ColorOverlay->GetElement(ElementIdx);
					FillVect[0] = CurValue.X; FillVect[1] = CurValue.Y; FillVect[2] = CurValue.Z; FillVect[3] = CurValue.W;
				});
		}
		
		return MeshAttributes->PrimaryColors();
	}

	static FVector3f ToUENormal(const FVector3d& Normal)
	{
		return FVector3f((float)Normal.X, (float)Normal.Y, (float)Normal.Z);
	}

	void GetBiasedElementPositions(const FDynamicMesh3& InOutTargetMesh, const float InRatio, TArray<FVector>& OutElementPositions)
	{
		const FDynamicMeshColorOverlay* ColorOverlay = InOutTargetMesh.Attributes()->PrimaryColors();
		if (!ensure(ColorOverlay))
		{
			return;
		}
		
		// get centroid of each face
		TArray<FVector> TrianglesCentroids;
		TrianglesCentroids.SetNumUninitialized(InOutTargetMesh.MaxTriangleID());
		for (int TriIdx : InOutTargetMesh.TriangleIndicesItr())
		{
			TrianglesCentroids[TriIdx] = InOutTargetMesh.GetTriCentroid(TriIdx);
		}

		// store the triangle for each element  
		TArray<int32> ElementToTriangle;
		ElementToTriangle.Init(INDEX_NONE, ColorOverlay->MaxElementID());
		for (int TriIdx : InOutTargetMesh.TriangleIndicesItr())
		{
			const FIndex3i TriElements = ColorOverlay->GetTriangle(TriIdx);
			ElementToTriangle[TriElements[0]] = TriIdx;
			ElementToTriangle[TriElements[1]] = TriIdx;
			ElementToTriangle[TriElements[2]] = TriIdx;
		}
		
		// clamp the bias between UE_SMALL_NUMBER and 1.0
		const double Ratio = FMath::Clamp(static_cast<double>(FMath::Abs(InRatio)), UE_KINDA_SMALL_NUMBER, 1.f);
		
		// compute biased vertex instance positions (per ElementID)
		// note that MaxElementID() is used here instead of ElementCount()
		OutElementPositions.Reset();
		OutElementPositions.Init(FVector::Zero(), ColorOverlay->MaxElementID());
		for (const int32 ElementID : ColorOverlay->ElementIndicesItr())
		{
			const int32 ParentVertex = ColorOverlay->GetParentVertex(ElementID);

			// initialize with the parent vertex position
			FVector& VertexPos = OutElementPositions[ElementID];
			VertexPos = InOutTargetMesh.GetVertexRef(ParentVertex);

			// get the face the element belongs to
			const int32 TriangleIndex = ElementToTriangle[ElementID];
			if (TriangleIndex != INDEX_NONE)
			{
				const FVector& Centroid = TrianglesCentroids[TriangleIndex];

				VertexPos = FMath::Lerp(VertexPos, Centroid, Ratio);
			}
		}
	}

	struct FTaskContext
	{
		TArray<int32> ElementIDs;
	};
}

FTransferVertexColorAttribute::FTransferVertexColorAttribute(
	const FDynamicMesh3* InSourceMesh,
	const FDynamicMeshAABBTree3* InSourceBVH)
	: SourceMesh(InSourceMesh)
	, SourceBVH(InSourceBVH)
{
	// If the BVH for the source mesh was not specified then create one
	if (SourceBVH == nullptr)
	{
		InternalSourceBVH = MakeUnique<FDynamicMeshAABBTree3>(SourceMesh);
	}
}

FTransferVertexColorAttribute::~FTransferVertexColorAttribute() 
{}

bool FTransferVertexColorAttribute::Cancelled()
{
	return (Progress == nullptr) ? false : Progress->Cancelled();
}

EOperationValidationResult FTransferVertexColorAttribute::Validate()
{	
	if (SourceMesh == nullptr) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	// Either BVH was passed by the caller or was created internally in the constructor
	if (SourceBVH == nullptr && InternalSourceBVH.IsValid() == false) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (!SourceMesh->HasAttributes()) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	if (!SourceMesh->Attributes()->HasPrimaryColors()) 
	{
		return EOperationValidationResult::Failed_UnknownReason;
	}

	return EOperationValidationResult::Ok;
}

bool FTransferVertexColorAttribute::TransferColorsToMesh(FDynamicMesh3& InOutTargetMesh)
{	
	using namespace TransferValuesLocals;

	if (Validate() != EOperationValidationResult::Ok) 
	{
		return false;
	}

	if (!InOutTargetMesh.HasAttributes())
	{
		InOutTargetMesh.EnableAttributes(); 
	}

	// If we need to compare normals, make sure both the target and the source meshes have per-vertex normals data
	TUniquePtr<FMeshNormals> InternalTargetMeshNormals;
	if (NormalThreshold >= 0)
	{
		if (!SourceMesh->HasVertexNormals() && !InternalSourceMeshNormals)
		{
			// only do this once for the source mesh in case of subsequent calls to the method
			InternalSourceMeshNormals = MakeUnique<FMeshNormals>(SourceMesh);
			InternalSourceMeshNormals->ComputeVertexNormals();
		}

		if (!InOutTargetMesh.HasVertexNormals())
		{
			InternalTargetMeshNormals = MakeUnique<FMeshNormals>(&InOutTargetMesh);
			InternalTargetMeshNormals->ComputeVertexNormals();
		}
	}

	FDynamicMeshColorOverlay* TargetColors = GetOrCreateColorAttribute(InOutTargetMesh, bHardEdges);
	
	checkSlow(TargetColors);

	bool bFailed = false;

	// compute the transfer only for the subset of vertices if necessary  
	const bool bUseSubset = !TargetVerticesSubset.IsEmpty();
	const int32 NumVerticesToTransfer = bUseSubset ? TargetVerticesSubset.Num() : InOutTargetMesh.MaxVertexID();
	
	if (TransferMethod == ETransferMethod::ClosestPointOnSurface)
	{
		const int32 NumMatched = TransferUsingClosestPoint(InOutTargetMesh, InternalTargetMeshNormals);
		const int32 NumVerticesToMatch = bHardEdges ? TargetColors->ElementCount() : NumVerticesToTransfer;
		
		// If the caller requested to simply find the closest point for all vertices then the number of matched vertices
		// must be equal to the target mesh vertex count
		if (SearchRadius < 0 && NormalThreshold < 0)
		{
			bFailed = NumMatched != NumVerticesToMatch;
		}
	} 
	else if (TransferMethod == ETransferMethod::Inpaint)
	{
		/**
         *  Given two meshes, Mesh1 without colors and Mesh2 with colors, assume they are aligned in 3d space.
         *  For every vertex on Mesh1 find the closest point on the surface of Mesh2 within a radius R. If the difference 
         *  between the normals of the two points is below the threshold, then it's a match. Otherwise no match.
         *  So now we have two sets of vertices on Mesh1. One with a match on the source mesh and one without a match.
         *  For all the vertices with a match, copy values over. For all the vertices without the match, do nothing.
         *  Now, for all the vertices without a match, try to approximate the values by smoothly interpolating between 
         *  the values at the known vertices via solving a quadratic problem. 
         *  
         *  The solver minimizes an energy
         *      trace(W^t Q W)
         *      W \in R^(nxm) is a matrix where n is the number of vertices and m is the number of elements.
         *      Q \in R^(nxn) is a matrix that combines both Dirichlet and Laplacian energies, Q = -L + L*M^(-1)*L
         *                    where L is a cotangent Laplacian and M is a mass matrix
         *  
		 */

		MatchedVertices.Init(false, InOutTargetMesh.MaxVertexID());
		TArray<FVector4f> MatchedColors;
		MatchedColors.Init(FVector4f::Zero(), InOutTargetMesh.MaxVertexID());
 
		// because the inpaint algorithm can extract data from regions outside the target vertex subset, a temporary attribute is used to modify the values.
		// NOTE: make sure to copy the values of the vertex subset into the complete TargetColors attribute before exciting the function. (see CopySubsetColorsIfNeeded)
		FDynamicMeshColorOverlay SubsetTargetColors;
		if (bUseSubset)
		{
			SubsetTargetColors.Copy(*TargetColors);
		}
		FDynamicMeshColorOverlay* EditedColors = bUseSubset ? &SubsetTargetColors : TargetColors;

		// Task context for the parallel for loops down below, to avoid repeatedly re-allocating an array.
		TArray<FTaskContext> TaskContexts;
		
		// For every vertex on the target mesh try to find the match on the source mesh using the distance and normal checks
		ParallelForWithTaskContext(TaskContexts, InOutTargetMesh.MaxVertexID(), [this, &InOutTargetMesh, &EditedColors, &InternalTargetMeshNormals, &MatchedColors](FTaskContext& Context, int32 VertexID)
		{
			if (Cancelled()) 
			{
				return;
			}
			
			if (InOutTargetMesh.IsVertex(VertexID)) 
			{
				// check if we need to force the vertex to not have a match
				if (ForceInpaint.Num() == InOutTargetMesh.MaxVertexID() && ForceInpaint[VertexID] != 0)
				{
					return;
				}
 
				const FVector3d Point = InOutTargetMesh.GetVertex(VertexID);
				FVector3f Normal = FVector3f::UnitY();
				if (NormalThreshold >= 0) 
				{
					const bool bHasNormals = InOutTargetMesh.HasVertexNormals();
					if (ensure(bHasNormals || InternalTargetMeshNormals.IsValid()))
					{
						Normal = bHasNormals ? InOutTargetMesh.GetVertexNormal(VertexID) : ToUENormal(InternalTargetMeshNormals->GetNormals()[VertexID]);
					}
				}
 
				FVector4f& Color = MatchedColors[VertexID];
				if (TransferColorToPoint(Color, Point, Normal))
				{
					EditedColors->GetVertexElements(VertexID, Context.ElementIDs);

					for (int32 ElementID: Context.ElementIDs)
					{
						EditedColors->SetElement(ElementID, Color);
					}
					MatchedVertices[VertexID] = true;
				}
			}
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
 
		if (Cancelled()) 
		{
			return false;
		}
 
		int32 NumMatched = 0;
		if (!bUseSubset)
		{
			for (bool Flag : MatchedVertices)
			{
				if (Flag) 
				{ 
					NumMatched++;
				}
			}
		}
		else
		{
			NumMatched = (int32)Algo::CountIf(TargetVerticesSubset, [this](int32 VertexID)
			{
				return MatchedVertices.IsValidIndex(VertexID) && MatchedVertices[VertexID];
			});
		}
 
		// If no vertices matched, we have nothing to inpaint.
		if (NumMatched == 0)
		{
			return false;
		}
 
		auto CopySubsetColorsIfNeeded = [Subset = TargetVerticesSubset, &EditedColors, &TargetColors, &InOutTargetMesh]()
		{
			TArray<int32> ElementIDs;
			if (EditedColors && EditedColors != TargetColors)
			{
				for (const int32 VertexID: Subset)
				{
					if (InOutTargetMesh.IsVertex(VertexID))
					{
						TargetColors->GetVertexElements(VertexID, ElementIDs);

						for (int32 ElementID: ElementIDs)
						{
							FVector4f Color;
							EditedColors->GetElement(ElementID, Color);
							TargetColors->SetElement(ElementID, Color);
						}
					}
				}
			}
		};
		
		// If all vertices were matched then nothing else to do
		if (NumMatched == NumVerticesToTransfer)
		{
			// copy colors from the subset attribute if using subset
			CopySubsetColorsIfNeeded();
			return true;
		}
 
		// Compute linearization so we can store constraints at linearized indices
		FVertexLinearization VtxLinearization(InOutTargetMesh, false);
		const TArray<int32>& ToMeshV = VtxLinearization.ToId();
		const TArray<int32>& ToIndex = VtxLinearization.ToIndex();
		
		// Setup the sparse matrix FixedValues of known (matched) colors and the array (FixedIndices) of the matched vertex IDs
		FSparseMatrixD FixedValues;
		FixedValues.resize(NumMatched, NumElements);
		std::vector<Eigen::Triplet<FSparseMatrixD::Scalar>> FixedValuesTriplets;
		FixedValuesTriplets.reserve(NumMatched);
		
		TArray<int> FixedIndices;
		FixedIndices.Reserve(NumMatched);
 
		for (int32 VertexID = 0; VertexID < InOutTargetMesh.MaxVertexID(); ++VertexID)
		{
			if (InOutTargetMesh.IsVertex(VertexID) && MatchedVertices[VertexID])
			{
				const FVector4f& Color = MatchedColors[VertexID];
 
				const int32 CurIdx = FixedIndices.Num();
				for (int32 Idx = 0; Idx < NumElements; ++Idx)
				{
					FixedValuesTriplets.emplace_back(CurIdx, Idx, Color[Idx]);
				}
 
				checkSlow(VertexID < ToIndex.Num());
				FixedIndices.Add(ToIndex[VertexID]);
			}
		}
		FixedValues.setFromTriplets(FixedValuesTriplets.begin(), FixedValuesTriplets.end());
 
		const int32 NumVerts = VtxLinearization.NumVerts();
		FEigenSparseMatrixAssembler CotangentAssembler(NumVerts, NumVerts);
		FEigenSparseMatrixAssembler LaplacianAssembler(NumVerts, NumVerts);
 
		if (bUseIntrinsicLaplacian)
		{
			// Construct the Cotangent values matrix
			UE::MeshDeformation::ConstructFullIDTCotangentLaplacian<double>(InOutTargetMesh, VtxLinearization, CotangentAssembler,
																			UE::MeshDeformation::ECotangentWeightMode::Default, 
																			UE::MeshDeformation::ECotangentAreaMode::NoArea);
 
			// Construct the Laplacian with cotangent values scaled by the voronoi area (i.e. M^(-1)*L matrix where M is the mass/stiffness matrix)
			UE::MeshDeformation::ConstructFullIDTCotangentLaplacian<double>(InOutTargetMesh, VtxLinearization, LaplacianAssembler,
																			UE::MeshDeformation::ECotangentWeightMode::Default,
																			UE::MeshDeformation::ECotangentAreaMode::VoronoiArea);
		}
		else 
		{
			UE::MeshDeformation::ConstructFullCotangentLaplacian<double>(InOutTargetMesh, VtxLinearization, CotangentAssembler,
																		 UE::MeshDeformation::ECotangentWeightMode::Default,
																		 UE::MeshDeformation::ECotangentAreaMode::NoArea);
 
			UE::MeshDeformation::ConstructFullCotangentLaplacian<double>(InOutTargetMesh, VtxLinearization, LaplacianAssembler,
																		 UE::MeshDeformation::ECotangentWeightMode::Default,
																		 UE::MeshDeformation::ECotangentAreaMode::VoronoiArea);
		}
 
		FSparseMatrixD CotangentMatrix, MassCotangentMatrix;
		CotangentAssembler.ExtractResult(CotangentMatrix);
		LaplacianAssembler.ExtractResult(MassCotangentMatrix);
 
		// -L * L* M^(-1)*L energy
		FSparseMatrixD Energy = -1*CotangentMatrix + CotangentMatrix*MassCotangentMatrix;
 
		// Solve the QP problem with fixed constraints
		FSparseMatrixD TargetValues;
		TArray<int> VariableRows;
 
		// We want the solution TargetValues matrix to only contain the rows representing the variable (non-fixed) rows
		constexpr bool bVariablesOnly = true;
 		bFailed = !FQuadraticProgramming::SolveWithFixedConstraints(Energy, nullptr, FixedIndices, FixedValues, TargetValues, bVariablesOnly, KINDA_SMALL_NUMBER, &VariableRows);
		checkSlow((VariableRows.Num() + FixedIndices.Num()) == Energy.rows());
 
		if (!bFailed)
		{
			// Transpose so we can efficiently iterate over the col-major matrix. Each column now contains per-vertex values.
			// Otherwise, we are iterating over rows of a col-major matrix which is slow.
			FSparseMatrixD TargetValuesTransposed = TargetValues.transpose(); 
 
			// Iterate over every column containing all values for the vertex
			TArray<int32> ElementIDs; 
			for (int32 ColIdx = 0; ColIdx < TargetValuesTransposed.outerSize(); ++ColIdx)
			{
				FVector4f Data(0.f);
				FInt32Vector4 N(0);
				
				// Iterate over only non-zero rows (i.e. non-zero values)
				for (FSparseMatrixD::InnerIterator Itr(TargetValuesTransposed, ColIdx); Itr; ++Itr)
				{
					const int32 Index = static_cast<int32>(Itr.row());
					const float Value = static_cast<float>(Itr.value());
					Data[Index] += Value;
					N[Index]++;
				}

				// normalize
				for (int32 Index = 0; Index < NumElements; ++Index)
				{
					if (N[Index] > 1)
					{
						Data[Index] /= static_cast<float>(N[Index]); 
					}
				}
				
				const int32 VertexIDLinearalized = bVariablesOnly ? static_cast<int32>(VariableRows[ColIdx]) : ColIdx; // linearized vertex ID (matrix row) of the variable in the Energy matrix
				const int32 VertexID = ToMeshV[VertexIDLinearalized];

				EditedColors->GetVertexElements(VertexID, ElementIDs);
				for (int32 ElementID: ElementIDs)
				{
					EditedColors->SetElement(ElementID, Data);
				}
			}
 
			// copy values from the subset attribute if using subset
			CopySubsetColorsIfNeeded();
			
			// Optional post-processing smoothing of the values at the vertices without a match
			if (NumSmoothingIterations > 0 && SmoothingStrength > 0)
			{
				TArray<int32> VerticesToSmooth;
				const int32 NumNotMatched = InOutTargetMesh.VertexCount() - NumMatched;
				VerticesToSmooth.Reserve(NumNotMatched);
				for (int32 VertexID = 0; VertexID < InOutTargetMesh.MaxVertexID(); ++VertexID)
				{
					if (InOutTargetMesh.IsVertex(VertexID) && !MatchedVertices[VertexID])
					{
						VerticesToSmooth.Add(VertexID);
					}
				}

				FSmoothDynamicMeshAttributes BlurOp(InOutTargetMesh);
				BlurOp.NumIterations = NumSmoothingIterations;
				BlurOp.Strength = SmoothingStrength;
				BlurOp.EdgeWeightMethod = FSmoothDynamicMeshAttributes::EEdgeWeights::CotanWeights; //expose as param
				BlurOp.Selection = MoveTemp(VerticesToSmooth);

				TArray<bool> ValuesToSmooth;
				ValuesToSmooth.Init(true, NumElements);
				
				ensure( BlurOp.SmoothOverlay(TargetColors, ValuesToSmooth) );
			}
		}
	}
	else 
	{
		checkNoEntry(); // unsupported method
	}

	if (Cancelled() || bFailed) 
	{
		return false;
	}
		
	return true;
}

bool FTransferVertexColorAttribute::TransferColorToPoint(FVector4f& OutColor, const FVector3d& InPoint, const FVector3f& InNormal) const 
{	
	using namespace TransferValuesLocals;

	// Find the containing triangle and the barycentric coordinates of the closest point
	int32 TriID; 
	FVector3d Bary;
	if (!FindClosestPointOnSourceSurface(InPoint, TargetToWorld, TriID, Bary))
	{
		return false;
	}
	
	const FVector3f BaryF((float)Bary[0], (float)Bary[1], (float)Bary[2]);

	const FDynamicMeshColorOverlay* SourceColors = SourceMesh->Attributes()->PrimaryColors();
	const FIndex3i ColorTriElements = SourceColors->GetTriangle(TriID);
	if (ColorTriElements.A == IndexConstants::InvalidID)
	{
		return false;
	}
	if (!ensure(ColorTriElements.B != IndexConstants::InvalidID && ColorTriElements.C != IndexConstants::InvalidID))
	{
		return false;
	}
	if (SearchRadius < 0 && NormalThreshold < 0)
	{
		// If the radius and normals are ignored, simply interpolate the values and return the result
		InterpolateVertexAttribute(OutColor, ColorTriElements, BaryF, SourceColors);
	}
	else
	{
		bool bPassedRadiusCheck = true;
		if (SearchRadius >= 0)
		{
			const FVector3d MatchedPoint = SourceMesh->GetTriBaryPoint(TriID, Bary[0], Bary[1], Bary[2]);
			bPassedRadiusCheck = (InPoint - MatchedPoint).Length() <= SearchRadius;
		}

		bool bPassedNormalsCheck = true;
		if (NormalThreshold >= 0)
		{
			FVector3f Normal0 = FVector3f::UnitY();
			FVector3f Normal1 = FVector3f::UnitY();
			FVector3f Normal2 = FVector3f::UnitY();
			const bool bHasSourceNormals = SourceMesh->HasVertexNormals();
			if (ensure(bHasSourceNormals || InternalSourceMeshNormals.IsValid()))
			{
				const FIndex3i TriVertices = SourceMesh->GetTriangle(TriID);
				Normal0 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertices[0]) : ToUENormal(InternalSourceMeshNormals->GetNormals()[TriVertices[0]]);
				Normal1 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertices[1]) : ToUENormal(InternalSourceMeshNormals->GetNormals()[TriVertices[1]]);
				Normal2 = bHasSourceNormals ? SourceMesh->GetVertexNormal(TriVertices[2]) : ToUENormal(InternalSourceMeshNormals->GetNormals()[TriVertices[2]]);
			}

			const FVector3f MatchedNormal = Normalized(BaryF[0]*Normal0 + BaryF[1]*Normal1 + BaryF[2]*Normal2);
			const FVector3f InNormalNormalized = Normalized(InNormal);
			const float NormalAngle = FMathf::ACos(InNormalNormalized.Dot(MatchedNormal));
			bPassedNormalsCheck = (double)NormalAngle <= NormalThreshold;

			if (!bPassedNormalsCheck && LayeredMeshSupport)
			{
				// try again with a flipped normal
				bPassedNormalsCheck = (double)(TMathUtil<float>::Pi - NormalAngle) <= NormalThreshold;
			}
		}
		
		if (bPassedRadiusCheck && bPassedNormalsCheck)
		{
			InterpolateVertexAttribute(OutColor, ColorTriElements, BaryF, SourceColors);
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool FTransferVertexColorAttribute::FindClosestPointOnSourceSurface(const FVector3d& InPoint, const FTransformSRT3d& InToWorld, int32& NearTriID, FVector3d& Bary) const
{
	IMeshSpatial::FQueryOptions Options;
	double NearestDistSqr;
	
	const FVector3d WorldPoint = InToWorld.TransformPosition(InPoint);
	if (SourceBVH != nullptr) 
	{ 
		NearTriID = SourceBVH->FindNearestTriangle(WorldPoint, NearestDistSqr, Options);
	}
	else 
	{
		NearTriID = InternalSourceBVH->FindNearestTriangle(WorldPoint, NearestDistSqr, Options);
	}

	if (!ensure(NearTriID != IndexConstants::InvalidID))
	{
		return false;
	}

	const FDistPoint3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleDistance(*SourceMesh, NearTriID, WorldPoint);
	const FVector3d NearestPnt = Query.ClosestTrianglePoint;
	const FIndex3i TriVertex = SourceMesh->GetTriangle(NearTriID);

	Bary = VectorUtil::BarycentricCoords(NearestPnt, SourceMesh->GetVertexRef(TriVertex.A),
													 SourceMesh->GetVertexRef(TriVertex.B),
													 SourceMesh->GetVertexRef(TriVertex.C));

	return true;
}

int32 FTransferVertexColorAttribute::TransferUsingClosestPoint(FDynamicMesh3& InOutTargetMesh, const TUniquePtr<FMeshNormals>& InTargetMeshNormals)
{
	using namespace TransferValuesLocals;
	
	if (!ensure(TransferMethod == ETransferMethod::ClosestPointOnSurface))
	{
		return 0;
	}
	
	FDynamicMeshAttributeSet* MeshAttributes = InOutTargetMesh.Attributes();
	if (!ensure(MeshAttributes->HasPrimaryColors()))
	{
		return 0;
	}
		
	FDynamicMeshColorOverlay* TargetColors = MeshAttributes->PrimaryColors();

	// transfer using element ids instead of vertices
	if (bHardEdges)
	{
		// compute per-vertex instance biased positions
		TArray<FVector> BiasedPositions;
		GetBiasedElementPositions(InOutTargetMesh, BiasRatio, BiasedPositions);

		// note that MaxElementID() is used here instead of ElementCount()
		const int32 NumElementsToTransfer = TargetColors->MaxElementID();
		MatchedVertices.Init(false, NumElementsToTransfer);

		ParallelFor(NumElementsToTransfer, [this, &InOutTargetMesh, &TargetColors, &InTargetMeshNormals, &BiasedPositions](int32 InElementID)
		{
			if (Cancelled()) 
			{
				return;
			}

			if (TargetColors->IsElement(InElementID))
			{
				const int32 VertexID = TargetColors->GetParentVertex(InElementID);
				if (InOutTargetMesh.IsVertex(VertexID))
				{
					const FVector3d& BiasedPoint = BiasedPositions.IsValidIndex(InElementID) ? BiasedPositions[InElementID] : InOutTargetMesh.GetVertexRef(VertexID);

					FVector3f Normal = FVector3f::UnitY();
					if (NormalThreshold >= 0) 
					{
						const bool bHasNormals = InOutTargetMesh.HasVertexNormals();
						if (ensure(bHasNormals || InTargetMeshNormals))
						{
							Normal = bHasNormals ? InOutTargetMesh.GetVertexNormal(VertexID) : ToUENormal(InTargetMeshNormals->GetNormals()[VertexID]);
						}
					}

					FVector4f Color;
					if (TransferColorToPoint(Color, BiasedPoint, Normal))
					{
						TargetColors->SetElement(InElementID, Color);								
						MatchedVertices[InElementID] = true;
					}
				}
			}

		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		// compute matching number
		int32 NumMatched = 0;
		for (const int32 ElementID : TargetColors->ElementIndicesItr())
		{
			if (MatchedVertices[ElementID])
			{
				NumMatched++;
			}
		}

		return NumMatched;
	}


	// compute the transfer only for the subset of vertices if necessary  
	const bool bUseSubset = !TargetVerticesSubset.IsEmpty();
	const int32 NumVerticesToTransfer = bUseSubset ? TargetVerticesSubset.Num() : InOutTargetMesh.MaxVertexID();

	TArray<FTaskContext> TaskContexts;

	MatchedVertices.Init(false, NumVerticesToTransfer);

	ParallelForWithTaskContext(TaskContexts, NumVerticesToTransfer, [this, &InOutTargetMesh, &TargetColors, &InTargetMeshNormals, bUseSubset](FTaskContext& Context, int32 InVertexID)
	{
		if (Cancelled()) 
		{
			return;
		}

		const int32 VertexID = bUseSubset ? TargetVerticesSubset[InVertexID] : InVertexID;
		if (InOutTargetMesh.IsVertex(VertexID)) 
		{
			const FVector3d Point = InOutTargetMesh.GetVertex(VertexID);

			FVector3f Normal = FVector3f::UnitY();
			if (NormalThreshold >= 0) 
			{
				const bool bHasNormals = InOutTargetMesh.HasVertexNormals();
				if (ensure(bHasNormals || InTargetMeshNormals))
				{
					Normal = bHasNormals ? InOutTargetMesh.GetVertexNormal(VertexID) : ToUENormal(InTargetMeshNormals->GetNormals()[VertexID]);
				}
			}

			FVector4f Color;
			if (TransferColorToPoint(Color, Point, Normal))
			{
				TargetColors->GetVertexElements(VertexID, Context.ElementIDs);

				for (int32 ElementID: Context.ElementIDs)
				{
					TargetColors->SetElement(ElementID, Color);
				}
			
				MatchedVertices[VertexID] = true;
			}
		}

	}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	int32 NumMatched = 0;
	for (bool Flag : MatchedVertices)
	{
		if (Flag) 
		{ 
			NumMatched++;
		}
	}
	
	return NumMatched;
}