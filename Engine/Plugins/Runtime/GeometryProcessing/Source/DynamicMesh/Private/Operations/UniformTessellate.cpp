// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/UniformTessellate.h"
#include "VectorTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "Async/ParallelFor.h"
#include "Util/ProgressCancel.h"

using namespace UE::Geometry;

namespace UniformTessellateLocals 
{
	// Collection of utility functions for triangular sequence (3 6 10 15 ...) which form a pattern of numbers 
	// that form triangles.
	//
	// 3 vertices, 1 triangle   6 vertices, 4 triangles
	//                                   o  
	//        o                         o o 
	//       o o                       o o o
	// TriangleNumber = 2        TriangleNumber = 3
	namespace TriangularPatternUtils 
	{	
		// Total number of vertices
		int NumVertices(const int TriangleNumber) 
		{	
			return TriangleNumber*(TriangleNumber + 1)/2;
		}

		// Number of inner vertices in the triangle (not corners or on edges)
		int NumInnerVertices(const int TriangleNumber) 
		{
			// 3 corner vertices plus TriangleNumber - 2 vertices on each of the 3 edges
			int NumVerticesAlongEdges = 3 + 3*(TriangleNumber - 2); 
			return NumVertices(TriangleNumber) - NumVerticesAlongEdges;
		}

		// Total number of triangles formed using the vertices along the pattern
		int NumTriangles(const int TriangleNumber) 
		{	
			return (TriangleNumber - 1) * (TriangleNumber - 1);
		}
	};

	/** 
	 * Abstract class containing common functionality shared by all Generators.
	 */
	class FBaseGenerator 
	{
	public:
		
		const FDynamicMesh3* InMesh = nullptr; // Input mesh to be tessellated
		
		int TessellationNum = 0;
		
		FProgressCancel* Progress = nullptr; // Set this to be able to cancel the generator

		bool bUseParallel = false; // Enable multithreading

		FBaseGenerator(const FDynamicMesh3* Mesh, const int TessellationNum, FProgressCancel* InProgress, const bool bUseParallel) 
		:
		InMesh(Mesh), TessellationNum(TessellationNum), Progress(InProgress), bUseParallel(bUseParallel)
		{
		} 

		virtual ~FBaseGenerator() 
		{
		}

		// If this returns true, the user cancelled the generator
		virtual bool Cancelled() const
		{
			return (Progress == nullptr) ? false : Progress->Cancelled();
		}

		// If this returns false, the input parameters are invalid
		bool Validate() 
		{
			return InMesh && InMesh->IsCompact() && TessellationNum >= 0;
		}

		/**
		 * If needed, reverse the offset of the new vertex added along the input mesh edge.
		 * 
		 * @param bIsReversed Is offset consistent with the input triangle edge direction. 
		 * @param Offset Which new vertex that we added along the edge we are interested in. 
		 */
		int OrderedEdgeOffset(const bool bIsReversed, const int Offset) const
		{
			checkSlow(Offset < TessellationNum);
			return bIsReversed ? TessellationNum - 1 - Offset : Offset;
		}

		/**
		 * For a triangle with vertices [v1,v2,v3] we assume in our calculations that the directed edges are [v1 v2], 
		 * [v2 v3], [v1 v3]. Where v1 is the vertex we are starting to sweep from towards the bottom edge [v2 v3].
		 * But for an edge [v1 v2] we could have added the vertices starting from v2 towards v1 in 
		 * GenerateEdgeElements(). So we need to know from which end we added vertices along the edges of the parent 
		 * triangle.
		 *
		 * @param InTriEdges Edge indices into the input mesh
		 * @param InTriVertices Vertex indices into the parent mesh we are tessellating
		 * @param OutIsEdgeReversed Array of 3 booleans matching the order of InTriEdges
		 */
		void GetEdgeOrder(const FIndex3i& InTriEdges, const FIndex3i& InTriVertices, TStaticArray<bool, 3>& OutIsEdgeReversed)
		{
			OutIsEdgeReversed[0] = InMesh->GetEdgeV(InTriEdges[0])[0] != InTriVertices[0];
			OutIsEdgeReversed[1] = InMesh->GetEdgeV(InTriEdges[1])[0] != InTriVertices[1];
			OutIsEdgeReversed[2] = InMesh->GetEdgeV(InTriEdges[2])[0] != InTriVertices[0];
		}
		
		virtual bool Generate() = 0;
	};

	/**
	 * Abstract class containing common functions for generating new elements for each vertex added along the edges or inside
	 * the input triangles. Vertex can contain one or more elements(i.e. normal, uv, color overlays) or a single 
	 * element (i.e. vertex coordinates). 
	 */
	template<typename RealType, int ElementSize>
	class FElementsGenerator : public FBaseGenerator
	{
	public:

		int InElementCount = 0; // Input mesh total number of elements
		int OutElementCount = 0; // Output mesh total number of elements
		int OutEdgeElementCount = 0; // Output mesh total number of new edge elements
		int OutInnerElementCount = 0; // Number of new inner per-triangle elements (per single triangle not total)

		// The array containing per-vertex elements.
		// It's split in 3 chunks of data containing: 
		//  ---------------------------------------------------------------------
		//  | Elements from the input | Elements added   | Elements added       |
		//  | mesh (copied over)      | along the edges  | inside the triangles |
		// ---------------------------------------------------------------------
		TArray<RealType> Elements;

		// Track which elements we set in the Elements array. 
		TArray<bool> ElementIsSet; 

		FElementsGenerator(const FDynamicMesh3* Mesh, const int TessellationNum, FProgressCancel* InProgress, const bool bUseParallel) 
		:
		FBaseGenerator(Mesh, TessellationNum, InProgress, bUseParallel)
		{
		}
		
		virtual ~FElementsGenerator() 
		{
		}

		virtual void SetBufferSizes(const int NumElements)
		{
			checkSlow(NumElements * ElementSize > 0);
			Elements.SetNum(NumElements * ElementSize);
			ElementIsSet.Init(false, NumElements);
		}

		// Child classes overwrite this to initialize element counts (InElementCount, OutElementCount ... )
		virtual void Initialize() = 0;

		/**
		 * Returns the index into the Elements array of the element (normal, uv, colors, vertex coordinate etc) 
		 * added along the edge. Must be overwritten by the child class since the behavior varies on the the number of 
		 * elements we are generating for each vertex.
		 * 
		 * @param InEdgeID The ID of the input mesh edge the element belongs to.
		 * @param InTriangleID If the edge is a seam edge, we need to know which side (which triangle) we are dealing with.
		 * @param VertexOffset Which of the new elements that we added along the edge we are interested in. 
		 */
		virtual int GetElementIDOnEdge(const int InEdgeID, const int InTriangleID, const int VertexOffset) const = 0;

		virtual void SetElementOnEdge(const int InEdgeID, const int InTriangleID, const int VertexOffset, const RealType* Value) 
		{
			const int Index = this->GetElementIDOnEdge(InEdgeID, InTriangleID, VertexOffset);
			checkSlow(Index >= InElementCount && Index < InElementCount + OutEdgeElementCount); 
			SetElement(Index, Value); 
		}

		/**
		 * Returns the index into the Triangles array of the element added inside the input mesh triangle.
		 * This default implementation assumes we are adding one element per new vertex inside the triangle.  
		 * 
		 *            v1
		 *            o        3 elements (A, B, C) added inside the input mesh triangle (v1,v2,v3)
		 *           x x       
		 *          x A x      A - VertexOffset == 0 
		 *         x C B x     B - VertexOffset == 1 
		 *        o x x x o    C - VertexOffset == 2  
		 *       v3       v2 
		 *
		 * @param InTriangleID The ID of the input mesh triangle the new element belongs to.
		 * @param VertexOffset Which of the new elements that we added inside the triangle we are interested in. 
		 */
		virtual int GetElementIDOnTriangle(const int InTriangleID, const int VertexOffset) const 
		{
			checkSlow(InTriangleID >= 0 && VertexOffset >= 0 && VertexOffset < OutInnerElementCount);
			return InElementCount + OutEdgeElementCount + OutInnerElementCount*InTriangleID + VertexOffset;
		}

		virtual void SetElementOnTriangle(const int InTriangleID, const int VertexOffset, const RealType* Value)
		{
			const int Index = GetElementIDOnTriangle(InTriangleID, VertexOffset);
			checkSlow(Index >= InElementCount + OutEdgeElementCount && Index < OutElementCount); 
			SetElement(Index, Value); 
		}
		
		// Get the element ID at the vertex of the input triangle
		virtual int GetInputMeshElementID(const int TriangleID, const int VertexID) const = 0;

		// Get the element value at the vertex of the input triangle
		virtual void GetInputMeshElementValue(const int TriangleID, const int VertexID, RealType* Value) const = 0;
		
		void SetElement(const int Index, const RealType* Value)
		{	
			const int Offset = Index * ElementSize;
			ElementIsSet[Index] = true;
			for (int Idx = 0; Idx < ElementSize; ++Idx)
			{
				Elements[Offset + Idx] = Value[Idx];
			}
		}

		template<typename AsType>
		void SetElement(const int Index, const AsType& TypeValue)
		{	
			const int Offset = Index * ElementSize;
			ElementIsSet[Index] = true;
			for (int Idx = 0; Idx < ElementSize; ++Idx)
			{
				Elements[Offset + Idx] = TypeValue[Idx];
			}
		}

		void GetElement(const int Index, RealType* Value) const
		{
			if (ElementIsSet[Index] == false) 
			{
				checkSlow(false); // we are trying to access an element that wasn't set
				return;
			}

			const int Offset = Index * ElementSize;
			checkSlow(Offset + ElementSize <= Elements.Num());
			for (int Idx = 0; Idx < ElementSize; ++Idx) 
			{
				Value[Idx] = Elements[Offset + Idx];
			}
		}

		template<typename AsType>
		void GetElement(const int Index, AsType& TypeValue) const
		{
			if (ElementIsSet[Index] == false)
			{
				checkSlow(false); // we are trying to access an element that wasn't set
				return;
			}

			const int Offset = Index * ElementSize;
			checkSlow(Offset + ElementSize <= Elements.Num());
			for (int Idx = 0; Idx < ElementSize; ++Idx) 
			{
				TypeValue[Idx] = Elements[Offset + Idx];
			}
		}

		template<typename AsType>
		AsType GetElement(const int Index) const
		{
			AsType Value;
			GetElement(Index, Value);
			return Value;
		}

		void LerpElements(const RealType* Element1, const RealType* Element2, RealType* OutElement, RealType Alpha) 
		{
			Alpha = FMath::Clamp(Alpha, RealType(0), RealType(1));
			RealType OneMinusAlpha = (RealType)1 - Alpha;

			for (int Idx = 0; Idx < ElementSize; ++Idx) 
			{
				OutElement[Idx] = OneMinusAlpha * Element1[Idx] + Alpha * Element2[Idx];
			}
		} 

		/**
		 * Generate new elements for vertices (marked with "x" below) along each unique edge of the input mesh 
		 * (input triangle corners are marked with "o" below). The number of the new vertices per edge is equal to the 
		 * tessellation number. The number of the new elements per vertex can vary, the default implementation below assumes
		 * one element per vertex. 
		 * 
		 *          o   
		 *         x x  
		 *        x   x   TessellationNum = 2, 6 new elements added in total (assuming 1 element per vertex)
		 *       o x x o
		 */
		virtual bool GenerateEdgeElements() 
		{
			ParallelFor(InMesh->MaxEdgeID(), [&](int32 EdgeID) 
			{
				if (this->Cancelled()) 
				{
					return;
				}

				checkSlow(InMesh->IsEdge(EdgeID));

				const FIndex2i EdgeV = InMesh->GetEdgeV(EdgeID);
				const FIndex2i EdgeTri = InMesh->GetEdgeT(EdgeID);
		
				RealType Element1[ElementSize];
				RealType Element2[ElementSize];
				GetInputMeshElementValue(EdgeTri.A, EdgeV.A, Element1);
				GetInputMeshElementValue(EdgeTri.A, EdgeV.B, Element2);

				RealType OutElement[ElementSize];
				for (int VertexOffset = 0; VertexOffset < TessellationNum; ++VertexOffset)
				{
					const RealType Alpha = (RealType)(VertexOffset + 1) / RealType(TessellationNum + 1);
					LerpElements(Element1, Element2, OutElement, Alpha);
					SetElementOnEdge(EdgeID, EdgeTri.A, VertexOffset, OutElement); 
				}

			}, this->bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);


			if (this->Cancelled()) 
			{
				return false;
			}

			return true;
		}

		/**
		 * Generate new elements inside of each input mesh triangle (marked with "*"" below) assuming edge elements were 
		 * previously added (marked with "x" below). For the input triangle (corners marked with "o" below) with 
		 * vertices [v1 v2 v3] we pick v1 as our starting vertex and "sweep" towards the opposite edge [v2 v3] while 
		 * generating new inner elements. We iterate through levels where each level is a line segment between two edge 
		 * elements added in GenerateEdgeElements(). Along each line segment, depending on the level, we generate the 
		 * appropriate number of elements.
		 *
		 *    TessellationNum = 4
		 *
		 *            v1 
		 *   |        o        Skip
		 *   |       x x       Level 0 (no new elements needs to be added so ignore) 
		 *   |      x * x      Level 1 (1 new element added)
		 *   |     x * * x     Level 2 (2 new elements added)
		 *   v    x * * * x    Level 4 (3 new elements added, so 6 new elements added in total)
		 *       o x x x x o   Skip
		 *      v3         v2 
		 */
		virtual bool GenerateTriangleElements()
		{
			if (TessellationNum <= 1) 
			{
				return true; // No inner triangle vertices need to be generated
			}
			
			ParallelFor(InMesh->MaxTriangleID(), [&](int32 TriangleID) 
			{
				if (this->Cancelled()) 
				{
					return;
				}
				
				if (this->IsValidTriangle(TriangleID) == false) 
				{
					return;
				}

				const FIndex3i TriVertices = InMesh->GetTriangle(TriangleID);
				const FIndex3i TriEdges = InMesh->GetTriEdges(TriangleID);

				TStaticArray<bool, 3> IsEdgeReversed;
				GetEdgeOrder(TriEdges, TriVertices, IsEdgeReversed);
				
				int ElementIDCounter = 0; // Track how many new elements in total we added so far for this triangle
				for (int Level = 1; Level < TessellationNum; ++Level)
				{
					// The number of new vertices added along each Level is same as the Level number
					const int NumNewLevelVertices = Level; 
					const int ID1 = this->GetElementIDOnEdge(TriEdges[0], TriangleID, OrderedEdgeOffset(IsEdgeReversed[0], Level));  
					const int ID2 = this->GetElementIDOnEdge(TriEdges[2], TriangleID, OrderedEdgeOffset(IsEdgeReversed[2], Level));

					RealType Element1[ElementSize];
					RealType Element2[ElementSize];
					GetElement(ID1, Element1);
					GetElement(ID2, Element2);

					RealType OutElement[ElementSize];
					for (int VertexOffset = 0; VertexOffset < NumNewLevelVertices; ++VertexOffset) 
					{
						const RealType Alpha = RealType(VertexOffset + 1) / RealType(NumNewLevelVertices + 1);
						LerpElements(Element1, Element2, OutElement, Alpha);
						this->SetElementOnTriangle(TriangleID, ElementIDCounter, OutElement);
						ElementIDCounter++;
					}
				}
			}, this->bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread); 

			if (this->Cancelled()) 
			{
				return false;
			}

			return true;
		}


		virtual bool IsValidTriangle(int32 TriangleID) const
		{
			return InMesh->IsTriangle(TriangleID);
		}
	};

	/**
	 * Abstract class containing common functions for computing triangle element-index triplets of the tessellated mesh. 
	 */
	template<typename RealType, int ElementSize>
	class FTrianglesGenerator : public FElementsGenerator<RealType, ElementSize>
	{
	public:

		using BaseType = FElementsGenerator<RealType, ElementSize>;
		using BaseType::InMesh;
		using BaseType::Elements;
		using BaseType::ElementIsSet;
		using BaseType::TessellationNum;
		using BaseType::OrderedEdgeOffset;
		using BaseType::GetEdgeOrder;
		using BaseType::SetBufferSizes;

		int OutTriangleCount = 0; // Total number of triangle triplets in the output mesh
		int OutInnerTriangleCount = 0; // Number of triangle triplets introduced per input triangle after tessellation

		// Array of triangle corner Elements, stored as tuples of indices into Elements array.
		TArray<FIndex3i> Triangles;

		FTrianglesGenerator(const FDynamicMesh3* Mesh, const int TessellationNum, FProgressCancel* InProgress, const bool bUseParallel) 
		:							 
		FElementsGenerator<RealType, ElementSize>(Mesh, TessellationNum, InProgress, bUseParallel)
		{
			const int TriangleNumber = TessellationNum + 2; // plus 2 corner vertices on the edge ends
			OutInnerTriangleCount = TriangularPatternUtils::NumTriangles(TriangleNumber);
			OutTriangleCount = OutInnerTriangleCount * InMesh->TriangleCount();
		}

		virtual ~FTrianglesGenerator() 
		{
		}
		
		void SetTriangleElements(const int Index, const int A, const int B, const int C)
		{
			checkSlow(Index < OutTriangleCount);
			Triangles[Index] = FIndex3i(A, B, C);
		}

		virtual void SetBufferSizes(const int NumTriangles, const int NumElements)
		{
			this->SetBufferSizes(NumElements);
			checkSlow(NumTriangles > 0);
			Triangles.SetNum(NumTriangles);
		}

		/**
		 * Return the Triangle ID of the new triangle added inside the input mesh triangle.
		 * 
		 *           v1
		 *           o            
		 *          / \        a - TriangleOffset == 0
		 *         / a \       b - TriangleOffset == 1
		 *        x-----x      c - TriangleOffset == 2
		 *       / \ c / \     d - TriangleOffset == 3
		 *      / d \ / b \
		 *     x-----*-----x      
		 *    v3           v2
		 *
		 * @param TriangleID The ID of the input mesh triangle.  
		 * @param TriangleOffset Which of the new output triangles we are interested in (must be less than OutInnerTriangleCount). 
		 */
		int GetOutTriangleID(const int TriangleID, const int TriangleOffset) const
		{
			const int OutTriangleID = OutInnerTriangleCount*TriangleID + TriangleOffset;
			checkSlow(TriangleOffset < OutInnerTriangleCount && OutTriangleID < OutTriangleCount);
			return OutTriangleID;
		}

		/**
		 * Generate the single triangle at the top that is adjacent to the vertex we start the sweep from.
		 *
		 * @param TriangleID Parent triangle ID.
		 * @param TriVertices Parent triangle vertices.
		 * @param TriEdges Parent triangle edges.
		 * @param IsEdgeReversed Is edge vertex offset consistent with the parent triangle edge direction.
		 */
		void GenerateTopTriangleStrip(const int TriangleID,  
									const FIndex3i& TriVertices, 
									const FIndex3i& TriEdges, 
									const TStaticArray<bool, 3>& IsEdgeReversed) 
		{
			const int ElementID1 = this->GetInputMeshElementID(TriangleID, TriVertices[0]);
			const int ElementID2 = this->GetElementIDOnEdge(TriEdges[0], TriangleID, OrderedEdgeOffset(IsEdgeReversed[0], 0));  
			const int ElementID3 = this->GetElementIDOnEdge(TriEdges[2], TriangleID, OrderedEdgeOffset(IsEdgeReversed[2], 0));
			SetTriangleElements(GetOutTriangleID(TriangleID, 0), ElementID1, ElementID2, ElementID3); 
		}


		/**
		 * Generate triangles along the the strip bounded by two Levels. Each Level is bounded by two edge vertices 
		 * (marked with "x" below) we generated in GenerateEdgeElements() and contains zero or more inner triangle 
		 * (marked with "*" below) vertices we generated in GenerateTriangleElements(). We iterate through every vertex 
		 * in the top level and generate two triangles adjacent to it except for the last vertex which only generates one 
		 * since we always have an odd number of triangles in our strip. We call this method for all strips whose bottom 
		 * level is not the edge of the parent triangle.
		 * 
		 *          <---------
		 *     x-----*-----*-----x  Level 2 (top level)
		 *    / \   / \   / \   / \
		 *   /   \ /   \ /   \ /   \
		 *  x-----*-----*-----*-----x  Level 3 (bottom level)
		 *  
		 * @param TopLevel Only provide the top level index since bottom index is simply TopLevel + 1.
		 * @param TriangleID Parent triangle ID.
		 * @param TriEdges Parent triangle edges.
		 * @param IsEdgeReversed Is edge vertex offset consistent with the parent triangle edge direction.
		 */
		void GenerateTriangleStripForLevel(const int TopLevel,
										const int TriangleID,
										const FIndex3i& TriEdges, 
										const TStaticArray<bool, 3>& IsEdgeReversed) 
		{
			// Top level vertex, normal, uv indices
			const int NumTopLevelNewVertices = TopLevel; // Number of inner triangle vertices is same as the Level number
			const int NumTopLevelVertices = NumTopLevelNewVertices + 2; // Plus 2 edge vertices
			const int TopLevelFirstElement = this->GetElementIDOnEdge(TriEdges[0], TriangleID, OrderedEdgeOffset(IsEdgeReversed[0], TopLevel));
			const int TopLevelLastElement = this->GetElementIDOnEdge(TriEdges[2], TriangleID, OrderedEdgeOffset(IsEdgeReversed[2], TopLevel));

			// Bottom level vertex, normal, uv indices
			const int BtmLevel = TopLevel + 1; 
			const int BtmLevelFirstElement = this->GetElementIDOnEdge(TriEdges[0], TriangleID, OrderedEdgeOffset(IsEdgeReversed[0], BtmLevel));
			const int BtmLevelLastElement = this->GetElementIDOnEdge(TriEdges[2], TriangleID, OrderedEdgeOffset(IsEdgeReversed[2], BtmLevel));
					
			// How many triangles we added so far. Initialized to the sum of all triangles added along the strips above 
			// the current one.
			int TriangleIDCounter = TriangularPatternUtils::NumTriangles(NumTopLevelVertices); 

			// How many inner triangles vertices is contained in all the strips above the current one.
			int InnerTriVertexCounter = TriangularPatternUtils::NumInnerVertices(NumTopLevelVertices);

			// Iterate over every vertex in the top level and generate triangles along the strip that share that vertex
			for (int VertexOffset = 0; VertexOffset < NumTopLevelVertices; ++VertexOffset) 
			{	
				if (VertexOffset == 0) 
				{
					// VID1 ?---x  TopLevelFirstVertex (VertexOffset = 0)
					//       \ / \
					//  VID2  *---x  BtmLevelFirstVertex

					const bool bIsNextVertexLast = VertexOffset + 1 == NumTopLevelVertices - 1; // The vertex can either be an inner triangle vertex or an edge vertex
					
					const int ID1 = bIsNextVertexLast ? TopLevelLastElement : this->GetElementIDOnTriangle(TriangleID, InnerTriVertexCounter);
					const int ID2 = this->GetElementIDOnTriangle(TriangleID, InnerTriVertexCounter + NumTopLevelNewVertices);  
					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter), TopLevelFirstElement, BtmLevelFirstElement, ID2); 					
					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter + 1), TopLevelFirstElement, ID2, ID1); 

					TriangleIDCounter += 2;
				}
				else if (VertexOffset == NumTopLevelVertices - 1)
				{
					//    TopLevelLastVertex x   
					//                      / \
					// BtmLevelLastVertex  o---x VID 
					
					const int NumBtmLevelNewVertices = BtmLevel;
					const int LastNewVertexBtmLevel = InnerTriVertexCounter + NumTopLevelNewVertices + NumBtmLevelNewVertices - 1;
					const int ID = this->GetElementIDOnTriangle(TriangleID, LastNewVertexBtmLevel);  
					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter), TopLevelLastElement, ID, BtmLevelLastElement); 					
					TriangleIDCounter++;
				}
				else
				{	
					// VID4 ?---*  VID1 
					//       \ / \
					//  VID3  *---*  VID2 

					const bool bIsNextVertexLast = VertexOffset + 1 == NumTopLevelVertices - 1; // The vertex can either be an inner triangle vertex or an edge vertex
					const int CurOffset = InnerTriVertexCounter + VertexOffset - 1;
					
					const int ID1 = this->GetElementIDOnTriangle(TriangleID, CurOffset);
					const int ID2 = this->GetElementIDOnTriangle(TriangleID, CurOffset + NumTopLevelNewVertices); 
					const int ID3 = this->GetElementIDOnTriangle(TriangleID, CurOffset + NumTopLevelNewVertices + 1); 
					const int ID4 = bIsNextVertexLast ? TopLevelLastElement : this->GetElementIDOnTriangle(TriangleID, CurOffset + 1);

					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter), ID1, ID2, ID3);
					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter + 1), ID1, ID3, ID4);

					TriangleIDCounter += 2;
				}
			}
		}

		/** 
		 * This is very similar to GenerateTriangleStripForLevel() but specificaly handles the last strip because the bottom
		 * level ends up being the edge of the original triangle and hence indexing is different. Splitting the logic
		 * helps with code clarity.
		 */
		void GenerateBottomTriangleStrip(const int TriangleID,
										const FIndex3i& TriVertices, 
										const FIndex3i& TriEdges, 
										const TStaticArray<bool, 3>& IsEdgeReversed) 
		{	
			// Top level vertex, normal, uv indices
			const int TopLevel = TessellationNum - 1;
			const int NumTopLevelVertices = TopLevel + 2;
			
			const int TopLevelFirstElement = this->GetElementIDOnEdge(TriEdges[0], TriangleID, OrderedEdgeOffset(IsEdgeReversed[0], TopLevel));
			const int TopLevelLastElement = this->GetElementIDOnEdge(TriEdges[2], TriangleID, OrderedEdgeOffset(IsEdgeReversed[2], TopLevel));

			// Bottom level vertex, normal, uv indices
			const int BtmLevel = TopLevel + 1;
			
			const int BtmLevelFirstElement = this->GetInputMeshElementID(TriangleID, TriVertices[1]);
			const int BtmLevelLastElement = this->GetInputMeshElementID(TriangleID, TriVertices[2]);
		
			int TriangleIDCounter = TriangularPatternUtils::NumTriangles(NumTopLevelVertices);
			int InnerTriVertexCounter = TriangularPatternUtils::NumInnerVertices(NumTopLevelVertices);

			// Iterate over every vertex in each Level
			for (int VertexOffset = 0; VertexOffset < NumTopLevelVertices; ++VertexOffset) 
			{	
				if (VertexOffset == 0) 
				{	
					// VID1 ?---x  TopLevelFirstVertex
					//       \ / \
					//   VID2 x---o  BtmLevelFirstVertex

					const bool bIsNextVertexLast = VertexOffset + 1 == NumTopLevelVertices - 1;
					
					const int ID1 = bIsNextVertexLast ? TopLevelLastElement : this->GetElementIDOnTriangle(TriangleID, InnerTriVertexCounter);
					const int ID2 = this->GetElementIDOnEdge(TriEdges[1], TriangleID, OrderedEdgeOffset(IsEdgeReversed[1], 0));
					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter), TopLevelFirstElement, BtmLevelFirstElement, ID2); 
					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter + 1), TopLevelFirstElement, ID2, ID1);
					
					TriangleIDCounter += 2;
				}
				else if (VertexOffset == NumTopLevelVertices - 1)
				{
					//    TopLevelLastVertex x   
					//                      / \
					// BtmLevelLastVertex  o---x VID 

					const int ID = this->GetElementIDOnEdge(TriEdges[1], TriangleID, OrderedEdgeOffset(IsEdgeReversed[1], TessellationNum - 1));
					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter), TopLevelLastElement, ID, BtmLevelLastElement); 					
					TriangleIDCounter++;
				}
				else
				{	
					// VID4 ?---*  VID1 
					//       \ / \
					//   VID3 x---x  VID2
					
					const bool bIsNextVertexLast = VertexOffset + 1 == NumTopLevelVertices - 1;
					const int CurOffset = InnerTriVertexCounter + VertexOffset - 1;
					
					const int ID1 = this->GetElementIDOnTriangle(TriangleID, CurOffset);
					const int ID2 = this->GetElementIDOnEdge(TriEdges[1], TriangleID, OrderedEdgeOffset(IsEdgeReversed[1], VertexOffset - 1));
					const int ID3 = this->GetElementIDOnEdge(TriEdges[1], TriangleID, OrderedEdgeOffset(IsEdgeReversed[1], VertexOffset));
					const int ID4 = bIsNextVertexLast ? TopLevelLastElement : this->GetElementIDOnTriangle(TriangleID, CurOffset + 1);

					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter), ID1, ID2, ID3);
					SetTriangleElements(GetOutTriangleID(TriangleID, TriangleIDCounter + 1), ID1, ID3, ID4);
					
					TriangleIDCounter += 2;
				}
			}
		}

		/*
		* Use generated elements along edges and inside each input mesh triangle to generate new triangles.
		* For an input triangle (corners marked with "o" below) with vertices [v1 v2 v3] and elements [e1 e2 e3] we pick 
		* e1 as our starting vertex and "sweep" towards the opposite edge [v2 v3]/[e2 e3] while "stitching" elements 
		* together to generate new triangles. We do this strip by strip, generating new triangles along each strip.
		* 
		*                        v1
		*                        o
		*                       / \  Top Strip
		*                      /   \
		*            Level 0  x-----x
		*                    / \   / \  Middle Strip 1
		*                   /   \ /   \
		*         Level 1  x-----*-----x
		*                 / \   / \   / \  Middle Strip 2
		*                /   \ /   \ /   \
		*      Level 2  x-----*-----*-----x
		*              / \   / \   / \   / \  Bottom Strip
		*             /   \ /   \ /   \ /   \
		*            o-----x-----x-----x-----o
		*           v3                       v2
		*/
		bool GenerateTriangles()
		{
			if (TessellationNum == 0)
			{
				return true; // No triangles needs to be generated
			}

			ParallelFor(InMesh->MaxTriangleID(), [&](int32 TriangleID) 
			{
				if (this->Cancelled()) 
				{
					return;
				}

				if (this->IsValidTriangle(TriangleID) == false) 
				{
					return;
				}
				 				
				const FIndex3i TriVertices = InMesh->GetTriangle(TriangleID);
				const FIndex3i TriEdges = InMesh->GetTriEdges(TriangleID);

				TStaticArray<bool, 3> IsEdgeReversed;
				GetEdgeOrder(TriEdges, TriVertices, IsEdgeReversed);

				GenerateTopTriangleStrip(TriangleID, TriVertices, TriEdges, IsEdgeReversed);					

				// Genereate the middle strips by iterating over their top levels.
				for (int Level = 0; Level < TessellationNum - 1; ++Level)
				{
					GenerateTriangleStripForLevel(Level, TriangleID, TriEdges, IsEdgeReversed); 
				}
				
				GenerateBottomTriangleStrip(TriangleID, TriVertices, TriEdges, IsEdgeReversed);			

			}, this->bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

			if (this->Cancelled()) 
			{
				return false;
			}

			return true;
		}
	};

	/**
	 * Generator to handle the interpolation of the data stored in TDynamicMeshOverlay.
	 */
	template<typename RealType, int ElementSize> 
	class FOverlayGenerator : public FTrianglesGenerator<RealType, ElementSize>
	{
	public:
		using BaseType = FTrianglesGenerator<RealType, ElementSize>;
		using BaseType::InMesh;
		using BaseType::TessellationNum;
		using BaseType::InElementCount; 
		using BaseType::OutElementCount; 
		using BaseType::OutEdgeElementCount; 
		using BaseType::OutInnerElementCount;
		using BaseType::Triangles;
		using BaseType::OutTriangleCount;
		using BaseType::OutInnerTriangleCount;
		using BaseType::ElementIsSet;
		
		// Track the offset into the Elements array for a block of data containing elements along the edge 
		// (1 or more for each vertex added along the edge). We can have a variable number of elements per-vertex 
		// depending on edge being the seam edge or not.
		TArray<int> EdgeToElementOffset; 

		TBitArray<FDefaultBitArrayAllocator> SeamEdges; // Which edges in the input mesh are seams edges

		const TDynamicMeshOverlay<RealType, ElementSize>* Overlay;
		
		FOverlayGenerator(const FDynamicMesh3* Mesh, 
						  const int TessellationNum,
						  FProgressCancel* InProgress, 
						  const bool bUseParallel,
						  const TDynamicMeshOverlay<RealType, ElementSize>* InOverlay) 
		:
		FTrianglesGenerator<RealType, ElementSize>(Mesh, TessellationNum, InProgress, bUseParallel), Overlay(InOverlay)
		{
		}
		
		virtual ~FOverlayGenerator() 
		{
		}

		virtual void Initialize() override
		{	
			SeamEdges = TBitArray<FDefaultBitArrayAllocator>(false, InMesh->EdgeCount());
			EdgeToElementOffset.SetNum(InMesh->EdgeCount());

			int LastOffset = 0;
			for (int InEdgeID = 0; InEdgeID < InMesh->EdgeCount(); ++InEdgeID)
			{
				// If edge is a seam and non boundary then each new inserted vertex along the edge will generate 
				// 2 new elements, otherwise only one.
				EdgeToElementOffset[InEdgeID] = LastOffset; 
								
				const bool bIsNotBndry = InMesh->GetEdgeT(InEdgeID).B != FDynamicMesh3::InvalidID;
				const bool bIsSeam = Overlay->IsSeamEdge(InEdgeID) && bIsNotBndry;
				SeamEdges[InEdgeID] = bIsSeam;
				LastOffset += bIsSeam ? 2*TessellationNum : TessellationNum;	
			}

			InElementCount = Overlay->MaxElementID();
			OutEdgeElementCount = LastOffset;

			const int TriangleNumber = TessellationNum + 2; // plus 2 vertices on the ends
			OutInnerElementCount = TriangularPatternUtils::NumInnerVertices(TriangleNumber);
			OutElementCount = InElementCount + OutEdgeElementCount + OutInnerElementCount * InMesh->TriangleCount();

			this->SetBufferSizes(OutTriangleCount, OutElementCount);

			RealType ElementData[ElementSize];
			for (int32 ElementID : Overlay->ElementIndicesItr())
			{
				Overlay->GetElement(ElementID, ElementData);
				this->SetElement(ElementID, ElementData);
			}
		}

		/**
		* Vertices added along seam edges contain 2 elements. Vertices on non-seam edges contain only one.
		* We use EdgeToElementOffset to correctly index into the Elements array.
		*
		*          Seam edge                                  | Non seam edge
		*          Vertex1            Vertex2                 | Vertex1   Vertex2
		*         ---------------------------------------------------------------------
		*         | Normal1  Normal2 | Normal1  Normal2 | ... | Normal1 | Normal1 |....
		*         ---------------------------------------------------------------------
		*        /                                            /
		*       /                                           EdgeToElementOffset[InNonSeamEdgeID]
		*    EdgeToElementOffset[InSeamEdgeID] 
		*/
		virtual int GetElementIDOnEdge(const int EdgeID, const int TriangleID, const int VertexOffset) const override
		{	
			checkSlow(EdgeID >= 0 && VertexOffset >= 0 && VertexOffset < TessellationNum);
			const bool bIsSeam = SeamEdges[EdgeID];
			const FIndex2i EdgeTri = InMesh->GetEdgeT(EdgeID);
			const int SeamIdx = EdgeTri.A == TriangleID ? 0 : 1;

			const int NormalOffset = bIsSeam ? 2*VertexOffset + SeamIdx : VertexOffset;
			return InElementCount + EdgeToElementOffset[EdgeID] + NormalOffset; 
		}

		virtual int GetInputMeshElementID(const int TriangleID, const int VertexID) const override 
		{
			return Overlay->GetElementIDAtVertex(TriangleID, VertexID);
		}

		virtual void GetInputMeshElementValue(const int TriangleID, const int VertexID, RealType* Value) const override
		{
			Overlay->GetElementAtVertex(TriangleID, VertexID, Value);
		}
		
		virtual bool GenerateEdgeElements() override
		{
			ParallelFor(InMesh->MaxEdgeID(), [&](int32 EdgeID) 
			{
				if (this->Cancelled()) 
				{
					return;
				}

				checkSlow(InMesh->IsEdge(EdgeID));

				const FIndex2i EdgeV = InMesh->GetEdgeV(EdgeID);
				const FIndex2i EdgeTri = InMesh->GetEdgeT(EdgeID);

				RealType Element1[ElementSize]; 
				RealType Element2[ElementSize]; 
				RealType Out[ElementSize];
				
				if (Overlay->IsSetTriangle(EdgeTri.A)) 
				{
					GetInputMeshElementValue(EdgeTri.A, EdgeV.A, Element1);
					GetInputMeshElementValue(EdgeTri.A, EdgeV.B, Element2);

					for (int VertexOffset = 0; VertexOffset < TessellationNum; ++VertexOffset)
					{
						const RealType Tau = (RealType)(VertexOffset + 1) / RealType(TessellationNum + 1);
						this->LerpElements(Element1, Element2, Out, Tau);
						this->SetElementOnEdge(EdgeID, EdgeTri.A, VertexOffset, Out);	
					}
				}

				if (SeamEdges[EdgeID] && EdgeTri.B != FDynamicMesh3::InvalidID && Overlay->IsSetTriangle(EdgeTri.B))
				{
					GetInputMeshElementValue(EdgeTri.B, EdgeV.A, Element1);
					GetInputMeshElementValue(EdgeTri.B, EdgeV.B, Element2);
					
					for (int VertexOffset = 0; VertexOffset < TessellationNum; ++VertexOffset)
					{
						const RealType Tau = (RealType)(VertexOffset + 1) / RealType(TessellationNum + 1);
						this->LerpElements(Element1, Element2, Out, Tau);
						this->SetElementOnEdge(EdgeID, EdgeTri.B, VertexOffset, Out);
					}
				}
			}, this->bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);  

			if (this->Cancelled()) 
			{
				return false;
			}

			return true;
		}

		// Copy the content of the Generator (elements and triangles) to the overlay.
		void CopyToAttribute(TDynamicMeshOverlay<RealType, ElementSize>* OutOverlay) 
		{
			if (IsOverlayValid() == false || OutOverlay == nullptr) 
			{
				return; // empty input or output overlay, nothing to do
			}
			
			OutOverlay->ClearElements();
			
			TArray<int32> ElementMap;
			ElementMap.SetNumUninitialized(OutElementCount);
			RealType Value[ElementSize];
			for (int32 EID = 0; EID < OutElementCount; ++EID)
			{
				if (ElementIsSet[EID]) 
				{
					this->GetElement(EID, Value);
					ElementMap[EID] = OutOverlay->AppendElement(Value);
				}
				else 
				{
					ElementMap[EID] = FDynamicMesh3::InvalidID;
				}
			}
			
			for (int32 TriangleID = 0; TriangleID < InMesh->MaxTriangleID(); ++TriangleID) 
			{	
				// Check if the triangle is valid and set in the overlay
				if (this->IsValidTriangle(TriangleID)) 
				{
					// Iterate over each subtriangle we generated for each input mesh triangle
					for (int TriangleOffset = 0; TriangleOffset < OutInnerTriangleCount; ++TriangleOffset)
					{	
						int32 OutTriangleID = this->GetOutTriangleID(TriangleID, TriangleOffset);
						FIndex3i ElementIDs = Triangles[OutTriangleID];
						FIndex3i MappedElementIDs = FIndex3i(ElementMap[ElementIDs[0]], ElementMap[ElementIDs[1]], ElementMap[ElementIDs[2]]);
						
						bool bIsValidMapping = MappedElementIDs[0] != FDynamicMesh3::InvalidID &&
							      			   MappedElementIDs[1] != FDynamicMesh3::InvalidID && 
							      			   MappedElementIDs[2] != FDynamicMesh3::InvalidID;
						checkSlow(bIsValidMapping);

						if (bIsValidMapping) 
						{
							ensure(OutOverlay->SetTriangle(OutTriangleID, MappedElementIDs) == EMeshResult::Ok);
						}
					}
				}
			}
		}	
		
		bool IsOverlayValid() const
		{
			return Overlay && Overlay->ElementCount();
		}

		virtual bool Generate() override
		{	
			if (this->Validate() == false) 
			{
				return false;
			}

			if (IsOverlayValid() == false) 
			{
				return true; // Nothing to do if the input overlay is null or empty
			}

			this->Initialize();

			if (this->GenerateEdgeElements() == false) 
			{
				return false;
			}
			
			if (this->GenerateTriangleElements() == false) 
			{
				return false;
			}
			
			if (this->GenerateTriangles() == false) 
			{
				return false;
			}
				
			return true;
		}

		
		virtual bool IsValidTriangle(int32 TriangleID) const override
		{
			return InMesh->IsTriangle(TriangleID) && Overlay->IsSetTriangle(TriangleID);
		}
	};

	/**
	 * Generator to handle the interpolation of the vertex position data and generation of the new triangle topology.
	 */
	template<typename RealType>
	class FMeshVerticesGenerator : public FTrianglesGenerator<RealType, 3>
	{
	public:
		using BaseType = FTrianglesGenerator<RealType, 3>;
		using BaseType::InMesh;
		using BaseType::TessellationNum;
		using BaseType::InElementCount; 
		using BaseType::OutElementCount; 
		using BaseType::OutEdgeElementCount; 
		using BaseType::OutInnerElementCount;
		using BaseType::OutTriangleCount;

		// Map vertex IDs inserted along the edges to the input mesh triangle ID(s) that share that edge.
  		TMap<int32, FIndex2i>* VertexEdgeMap; 

		// Map vertex IDs to IDs of the triangles those vertices inserted into.
  		TMap<int32, int32>* VertexTriangleMap;
		
		FMeshVerticesGenerator(const FDynamicMesh3* Mesh, 
							   const int TessellationNum, 
							   FProgressCancel* InProgress, 
  							   TMap<int32, FIndex2i>* VertexEdgeMap,
  							   TMap<int32, int32>* VertexTriangleMap,
  							   const bool bUseParallel) 
		:
		FTrianglesGenerator<RealType, 3>(Mesh, TessellationNum, InProgress, bUseParallel), 
  		VertexEdgeMap(VertexEdgeMap),
  		VertexTriangleMap(VertexTriangleMap)
		{
		}
			
		virtual ~FMeshVerticesGenerator()
		{
		}

		virtual void Initialize() override
		{	
			const int TriangleNumber = TessellationNum + 2; // plus 2 vertices on the ends
			InElementCount = InMesh->VertexCount(); 
			OutEdgeElementCount = TessellationNum * InMesh->EdgeCount(); 
			OutInnerElementCount = TriangularPatternUtils::NumInnerVertices(TriangleNumber); 
			OutElementCount = InElementCount + OutEdgeElementCount + OutInnerElementCount*InMesh->TriangleCount();	

			this->SetBufferSizes(OutTriangleCount, OutElementCount);
			
			// Copy over the input mesh elements
			RealType Value[3];
			for (int VertexID = 0; VertexID < InMesh->VertexCount(); ++VertexID)
			{
				GetInputMeshElementValue(VertexID, Value);
				this->SetElement(VertexID, Value);
			}

			if (VertexEdgeMap) 
			{
				VertexEdgeMap->Reserve(OutEdgeElementCount);
			}
			
			if (VertexTriangleMap) 
			{
				VertexTriangleMap->Reserve(OutInnerElementCount*InMesh->TriangleCount());
			}
		}

		/**
		 * Returns the index into the Vertices array of the vertex added along the edge. 
		 *
		 *                          VertexOffset = 0
		 *                         /     VertexOffset = 2
		 *                        /     /    
		 *  Edge(v1,v2)       o  x  x  x  o 
		 *                   v1           v2
		 * 
		 * @param InEdgeID The ID of the input mesh edge the new vertex belongs to.
		 * @param VertexOffset Which of the new vertices that we added along the edge we are interested in. 
		 */
		virtual int GetElementIDOnEdge(const int InEdgeID, const int InTriangleID, const int VertexOffset) const override
		{	
			checkSlow(InEdgeID >= 0 && VertexOffset >= 0 && VertexOffset < TessellationNum);
			return InElementCount + TessellationNum * InEdgeID + VertexOffset; 
		}

		// The element ID at a vertex is simply the vertex ID, ignore the triangle ID
		virtual int GetInputMeshElementID(const int TriangleID, const int VertexID) const override 
		{
			return VertexID;
		}

		// Since we have a single element per-vertex we ignore the triangle ID
		void GetInputMeshElementValue(const int VertexID, RealType* Value) const 
		{
			FVector3d VV = InMesh->GetVertex(VertexID);
			Value[0] = VV[0];
			Value[1] = VV[1];
			Value[2] = VV[2];
		}
		
		virtual void GetInputMeshElementValue(const int TriangleID, const int VertexID, RealType* Value) const override
		{
			GetInputMeshElementValue(VertexID, Value);
		}

		/** Compute the mappings between vertices and triangles of the input and the output meshes. */
		bool ComputeTopologyMappings() 
		{	
			if (VertexEdgeMap) 
			{
				for (int32 EdgeID = 0; EdgeID < InMesh->MaxEdgeID(); ++EdgeID)
				{
					checkSlow(InMesh->IsEdge(EdgeID));
					
					const FIndex2i EdgeTri = InMesh->GetEdgeT(EdgeID);

					for (int VertexOffset = 0; VertexOffset < TessellationNum; ++VertexOffset)
					{	
						const int VertexID = this->GetElementIDOnEdge(EdgeID, EdgeTri.A, VertexOffset);
						VertexEdgeMap->Add(VertexID, EdgeTri);
					}
				}
			}

			if (this->Cancelled()) 
			{
				return false;
			}

			if (VertexTriangleMap) 
			{			
				for (int32 TriangleID = 0; TriangleID < InMesh->MaxTriangleID(); ++TriangleID)
				{
					checkSlow(InMesh->IsTriangle(TriangleID));
					
					int VertexIDCounter = 0;
					for (int Level = 1; Level < TessellationNum; ++Level)
					{
						for (int VertexOffset = 0; VertexOffset < Level; ++VertexOffset, ++VertexIDCounter) 
						{
							const int VertexID = this->GetElementIDOnTriangle(TriangleID, VertexIDCounter);
							VertexTriangleMap->Add(VertexID, TriangleID);
						}
					}
				}
			}
			
			return true;
		}

		virtual bool Generate() override
		{	
			if (this->Validate() == false) 
			{
				return false;
			}

			this->Initialize();

			if (this->GenerateEdgeElements() == false) 
			{
				return false;
			}
			
			if (this->GenerateTriangleElements() == false) 
			{
				return false;
			}
			
			if (this->GenerateTriangles() == false) 
			{
				return false;
			}

			if (ComputeTopologyMappings() == false) 
			{
				return false;
			}
				
			return true;
		}
	};

	/**
	 * General purpose generator to interpolate any per-vertex numeric data of any dimension. 
	 */
	template<typename RealType, int ElementSize>
	class FVertexAttributeGenerator : public FElementsGenerator<RealType, ElementSize>
	{
	public:
		using BaseType = FElementsGenerator<RealType, ElementSize>;
		using BaseType::InMesh;
		using BaseType::TessellationNum;
		using BaseType::InElementCount; 
		using BaseType::OutElementCount; 
		using BaseType::OutEdgeElementCount; 
		using BaseType::OutInnerElementCount;
		using BaseType::Elements;
		
		// Set this to pull the data you are interpolating at an input mesh vertex
		TFunction<void(const int VertexID, RealType* OutValue)> DataFunction;

		FVertexAttributeGenerator(const FDynamicMesh3* Mesh, 
								const int TessellationNum, 
								FProgressCancel* InProgress, 
								const bool bUseParallel, 
								const TFunction<void(int VertexID, RealType* Value)>& InDataFunction) 
		:
		FElementsGenerator<RealType, ElementSize>(Mesh, TessellationNum, InProgress, bUseParallel), DataFunction(InDataFunction)
		{
		}
		
		virtual ~FVertexAttributeGenerator() 
		{
		}	

		virtual void Initialize() override
		{	
			const int TriangleNumber = TessellationNum + 2; // plus 2 vertices on the ends
			InElementCount = InMesh->VertexCount(); 
			OutEdgeElementCount = TessellationNum * InMesh->EdgeCount(); 
			OutInnerElementCount = TriangularPatternUtils::NumInnerVertices(TriangleNumber); 
			OutElementCount = InElementCount + OutEdgeElementCount + OutInnerElementCount*InMesh->TriangleCount();	

			this->SetBufferSizes(OutElementCount);

			RealType Value[ElementSize];
			for (int VertexID = 0; VertexID < InMesh->VertexCount(); ++VertexID)
			{
				GetInputMeshElementValue(VertexID, Value);
				this->SetElement(VertexID, Value);
			}
		}

		virtual int GetElementIDOnEdge(const int EdgeID, const int TriangleID, const int VertexOffset) const override
		{	
			checkSlow(EdgeID >= 0 && VertexOffset >= 0 && VertexOffset < TessellationNum);
			return InElementCount + TessellationNum * EdgeID + VertexOffset; 
		}

		virtual int GetInputMeshElementID(const int TriangleID, const int VertexID) const override 
		{
			return VertexID;
		}

		void GetInputMeshElementValue(const int VertexID, RealType* OutValue) const
		{
			DataFunction(VertexID, OutValue);
		}

		virtual void GetInputMeshElementValue(const int TriangleID, const int VertexID, RealType* OutValue) const override
		{
			GetInputMeshElementValue(VertexID, OutValue);
		}
		
		virtual bool Generate() override
		{	
			if (this->Validate() == false) 
			{
				return false;
			}

			this->Initialize();
			
			if (this->GenerateEdgeElements() == false) 
			{
				return false;
			}
			
			if (this->GenerateTriangleElements() == false) 
			{
				return false;
			}
				
			return true;
		}

		static TUniquePtr<FVertexAttributeGenerator> CreateFromPerVertexNormals(const FDynamicMesh3* Mesh, 
																			const int TessellationNum,
																			FProgressCancel* InProgress, 
																			const bool bUseParallel)  
		{	
			if (Mesh->HasVertexNormals() == false) 
			{
				return nullptr;
			}

			TFunction<void(int, RealType*)> FPntr = [Mesh](const int VertexID, RealType* Value) 
			{
				FVector3f Normal = Mesh->GetVertexNormal(VertexID);
				for (int Idx = 0; Idx < 3; ++Idx) 
				{
					Value[Idx] = Normal[Idx];
				}
			};

			return MakeUnique<FVertexAttributeGenerator>(Mesh, TessellationNum, InProgress, bUseParallel, FPntr);
		}

		static TUniquePtr<FVertexAttributeGenerator> CreateFromPerVertexUVs(const FDynamicMesh3* Mesh, 
																			const int TessellationNum,
																			FProgressCancel* InProgress, 
																			const bool bUseParallel)  
		{	
			if (Mesh->HasVertexUVs() == false) 
			{
				return nullptr;
			}

			TFunction<void(int, RealType*)> FPntr = [Mesh](const int VertexID, RealType* Value) 
			{
				FVector2f UV = Mesh->GetVertexUV(VertexID);
				for (int Idx = 0; Idx < 2; ++Idx) 
				{
					Value[Idx] = UV[Idx];
				}
			};

			return MakeUnique<FVertexAttributeGenerator>(Mesh, TessellationNum, InProgress, bUseParallel, FPntr);
		}

		static TUniquePtr<FVertexAttributeGenerator> CreateFromPerVertexColors(const FDynamicMesh3* Mesh, 
																			   const int TessellationNum,
																			   FProgressCancel* InProgress, 
																			   const bool bUseParallel) 
		{	
			if (Mesh->HasVertexColors() == false) 
			{
				return nullptr;
			}

			TFunction<void(int, RealType*)> FPntr = [Mesh](const int VertexID, RealType* Value) 
			{
				FVector4f Color = Mesh->GetVertexColor(VertexID);
				for (int Idx = 0; Idx < 4; ++Idx) 
				{
					Value[Idx] = Color[Idx];
				}
			};

			return MakeUnique<FVertexAttributeGenerator>(Mesh, TessellationNum, InProgress, bUseParallel, FPntr);
		}

		static TUniquePtr<FVertexAttributeGenerator> CreateFromVertexAttribute(const FDynamicMesh3* Mesh,
			const int TessellationNum,
			FProgressCancel* InProgress,
			const bool bUseParallel,
			const TDynamicMeshVertexAttribute<RealType, ElementSize>* Attribute)
		{
			if (Attribute == nullptr)
			{
				return nullptr;
			}

			TFunction<void(int, RealType*)> FPntr = [Attribute](const int VertexID, RealType* Value)
			{
				Attribute->GetValue(VertexID, Value);
			};

			return MakeUnique<FVertexAttributeGenerator>(Mesh, TessellationNum, InProgress, bUseParallel, FPntr);
		}

		void CopyToAttribute(TDynamicMeshVertexAttribute<RealType, ElementSize>* OutAttribute)
		{
			RealType Value[ElementSize];
			for (int EID = 0; EID < OutElementCount; ++EID)
			{
				BaseType::GetElement(EID, Value);
				OutAttribute->SetValue(EID, Value);
			}
		}


	};

	/**
	 * General purpose generator to interpolate any per-triangle numeric data of any dimension. 
	 */
	template<typename RealType, int ElementSize> 
	class FTriangleAttributeGenerator : public FBaseGenerator
	{
	public:

		int OutTriangleCount = 0; // Total number of triangles in the output mesh
		int OutInnerTriangleCount = 0; // Number of triangles introduced per input triangle after tessellation

		TArray<RealType> AttribValues;

		// Set this to pull the data you are interpolating at an input mesh triangle
		TFunction<void(const int TriangleID, RealType* OutValue)> DataFunction;
		
		FTriangleAttributeGenerator(const FDynamicMesh3* Mesh, 
								    const int TessellationNum,
								    FProgressCancel* InProgress, 
								    const bool bUseParallel,
								    const TFunction<void(int TriangleID, RealType* Value)>& InDataFunction) 
		:
		FBaseGenerator(Mesh, TessellationNum, InProgress, bUseParallel), DataFunction(InDataFunction)
		{
			const int TriangleNumber = TessellationNum + 2; // plus 2 corner vertices on the edge ends
			OutInnerTriangleCount = TriangularPatternUtils::NumTriangles(TriangleNumber);
			OutTriangleCount = OutInnerTriangleCount * InMesh->TriangleCount();
		}

		
		void SetElement(const int Index, const RealType* Data)
		{	
			const int Offset = Index * ElementSize;
			checkSlow(Index >= 0 && Offset + ElementSize <= AttribValues.Num());
			for (int Idx = 0; Idx < ElementSize; ++Idx)
			{
				AttribValues[Offset + Idx] = Data[Idx];
			}
		}

		void GetElement(const int Index, RealType* Data) const
		{
			const int Offset = Index * ElementSize;
			checkSlow(Index >= 0 && Index + ElementSize <= AttribValues.Num());
			for (int Idx = 0; Idx < ElementSize; ++Idx) 
			{
				Data[Idx] = AttribValues[Offset + Idx];
			}
		}

		void SetBufferSizes(const int NumTriangles)
		{
			AttribValues.SetNum(NumTriangles*ElementSize);
		}

		int GetOutTriangleID(const int TriangleID, const int TriangleOffset) const
		{
			const int OutTriangleID = OutInnerTriangleCount*TriangleID + TriangleOffset;
			checkSlow(TriangleOffset < OutInnerTriangleCount && OutTriangleID < OutTriangleCount);
			return OutTriangleID;
		}

		void GenerateTriangles()
		{
			RealType Value[ElementSize];
			for (int TriangleID = 0; TriangleID < InMesh->TriangleCount(); ++TriangleID)
			{
				DataFunction(TriangleID, Value);
				for (int Offset = 0; Offset < OutInnerTriangleCount; ++Offset)
				{
					SetElement(GetOutTriangleID(TriangleID, Offset), Value);
				}
			}
		}

		virtual bool Generate() override
		{	
			if (this->Validate() == false) 
			{
				return false;
			}

			SetBufferSizes(OutTriangleCount);
					
			GenerateTriangles();
				
			return true;
		}
		
		static TUniquePtr<FTriangleAttributeGenerator> 
		CreateFromDynamicMeshTriangleAttribute(const FDynamicMesh3* Mesh, 
											   const int TessellationNum, 
											   FProgressCancel* InProgress, 
											   const bool bUseParallel,
											   const TDynamicMeshTriangleAttribute<RealType, ElementSize>* Attribute) 
		{
			if (Attribute == nullptr) 
			{
				return nullptr;
			}

			TFunction<void(int, RealType*)> FPntr = [Attribute](const int TriangleID, RealType* Value) 
			{
				Attribute->GetValue(TriangleID, Value);
			};

			return MakeUnique<FTriangleAttributeGenerator>(Mesh, TessellationNum, InProgress, bUseParallel, FPntr);
		}

		static TUniquePtr<FTriangleAttributeGenerator> 
		CreateFromMeshTriangleGroups(const FDynamicMesh3* Mesh, 
									 const int TessellationNum,  
									 FProgressCancel* InProgress, 
									 const bool bUseParallel) 
		{
			if (Mesh->HasTriangleGroups() == false) 
			{
				return nullptr;
			}

			TFunction<void(int, RealType*)> FPntr = [Mesh](const int TriangleID, RealType* Value) 
			{
				*Value = Mesh->GetTriangleGroup(TriangleID);
			};

			return MakeUnique<FTriangleAttributeGenerator>(Mesh, TessellationNum, InProgress, bUseParallel, FPntr);
		}

		void CopyToAttribute(TDynamicMeshTriangleAttribute<RealType, ElementSize>* OutAttribute) 
		{
			RealType Value[ElementSize];
			for (int TID = 0; TID < OutTriangleCount; ++TID) 
			{
				GetElement(TID, Value);
				OutAttribute->SetValue(TID, Value);
			}
		}
	};

	/**
	 * Custom generator for the per-vertex skin weight attribute.
	 */
	class FSkinWeightAttributeGenerator : public FBaseGenerator
	{
	public:
		
		int InWeightCount = 0; 
		int OutWeightCount = 0; 
		int OutEdgeWeightCount = 0; 
		int OutInnerWeightCount = 0;

		using FBoneWeights = UE::AnimationCore::FBoneWeights;	

		TArray<FBoneWeights> VertexBoneWeights;

		const FDynamicMeshVertexSkinWeightsAttribute* SkinWeightsAttribute = nullptr;

		FSkinWeightAttributeGenerator(const FDynamicMesh3* Mesh, 
									  const int TessellationNum, 
									  FProgressCancel* InProgress, 
									  const bool bUseParallel,
									  const FDynamicMeshVertexSkinWeightsAttribute* SkinWeightsAttribute) 
		:
		FBaseGenerator(Mesh, TessellationNum, InProgress, bUseParallel), SkinWeightsAttribute(SkinWeightsAttribute)
		{
			const int TriangleNumber = TessellationNum + 2; // plus 2 vertices on the ends
			InWeightCount = InMesh->VertexCount(); 
			OutEdgeWeightCount = TessellationNum * InMesh->EdgeCount(); 
			OutInnerWeightCount = TriangularPatternUtils::NumInnerVertices(TriangleNumber); 
			OutWeightCount = InWeightCount + OutEdgeWeightCount + OutInnerWeightCount*InMesh->TriangleCount();	
		}
		
		virtual ~FSkinWeightAttributeGenerator() 
		{
		}	

		void Initialize()
		{	
			VertexBoneWeights.SetNum(OutWeightCount);
			
			FBoneWeights Weight;
			for (int VertexID = 0; VertexID < InMesh->VertexCount(); ++VertexID)
			{
				SkinWeightsAttribute->GetValue(VertexID, Weight);
				SetWeight(VertexID, Weight);
			}
		}

		void SetWeightOnEdge(const int EdgeID, const int VertexOffset, const FBoneWeights& Value) 
		{
			const int Index = GetWeightIDOnEdge(EdgeID, VertexOffset);
			checkSlow(Index >= InWeightCount && Index < InWeightCount + OutEdgeWeightCount); 
			SetWeight(Index, Value); 
		}

		void SetWeightOnTriangle(const int TriangleID, const int VertexOffset, const FBoneWeights& Value)
		{
			const int Index = GetWeightIDOnTriangle(TriangleID, VertexOffset);
			checkSlow(Index >= InWeightCount + OutEdgeWeightCount && Index < OutWeightCount); 
			SetWeight(Index, Value); 
		}

		int GetWeightIDOnEdge(const int EdgeID, const int VertexOffset) const 
		{	
			checkSlow(EdgeID >= 0 && VertexOffset >= 0 && VertexOffset < TessellationNum);
			return InWeightCount + TessellationNum * EdgeID + VertexOffset; 
		}

		int GetWeightIDOnTriangle(const int TriangleID, const int VertexOffset) const
		{
			checkSlow(TriangleID >= 0 && VertexOffset >= 0 && VertexOffset < OutInnerWeightCount);
			return InWeightCount + OutEdgeWeightCount + OutInnerWeightCount*TriangleID + VertexOffset;
		}

		void SetWeight(const int Index, const FBoneWeights& Value)
		{	
			VertexBoneWeights[Index] = Value;
		}

		void GetWeight(const int Index, FBoneWeights& Value) const
		{	
			Value = VertexBoneWeights[Index];
		}

		void LerpWeights(const FBoneWeights& Weight1, const FBoneWeights& Weight2, FBoneWeights& OutWeight, double Alpha) 
		{
			Alpha = FMath::Clamp(Alpha, 0.0, 1.0);
			OutWeight = FBoneWeights::Blend(Weight1, Weight2, (float)Alpha);
		} 

		bool GenerateEdgeElements() 
		{
			ParallelFor(InMesh->MaxEdgeID(), [&](int32 EdgeID) 
			{
				if (Cancelled()) 
				{
					return;
				}

				checkSlow(InMesh->IsEdge(EdgeID));

				const FIndex2i EdgeV = InMesh->GetEdgeV(EdgeID);
		
				FBoneWeights Weight1, Weight2;
				SkinWeightsAttribute->GetValue(EdgeV.A, Weight1);
				SkinWeightsAttribute->GetValue(EdgeV.B, Weight2);

				FBoneWeights Out;
				for (int VertexOffset = 0; VertexOffset < TessellationNum; ++VertexOffset)
				{
					const double Alpha = (double)(VertexOffset + 1) / (TessellationNum + 1);
					LerpWeights(Weight1, Weight2, Out, Alpha);
					SetWeightOnEdge(EdgeID, VertexOffset, Out); 
				}

			}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);  

			if (Cancelled()) 
			{
				return false;
			}

			return true;
		}

		bool GenerateTriangleElements() 
		{
			if (TessellationNum <= 1) 
			{
				return true; // No inner triangle vertices need to be generated
			}
			
			ParallelFor(InMesh->MaxTriangleID(), [&](int32 TriangleID) 
			{
				if (Cancelled()) 
				{
					return;
				}

				checkSlow(InMesh->IsTriangle(TriangleID));

				const FIndex3i TriVertices = InMesh->GetTriangle(TriangleID);
				const FIndex3i TriEdges = InMesh->GetTriEdges(TriangleID);

				TStaticArray<bool, 3> IsEdgeReversed;
				GetEdgeOrder(TriEdges, TriVertices, IsEdgeReversed);
				
				int ElementIDCounter = 0; // Track how many new elements in total we added so far for this triangle
				for (int Level = 1; Level < TessellationNum; ++Level)
				{
					const int ID1 = GetWeightIDOnEdge(TriEdges[0], OrderedEdgeOffset(IsEdgeReversed[0], Level));  
					const int ID2 = GetWeightIDOnEdge(TriEdges[2], OrderedEdgeOffset(IsEdgeReversed[2], Level));

					FBoneWeights Weight1, Weight2;
					GetWeight(ID1, Weight1);
					GetWeight(ID2, Weight2);

					FBoneWeights OutWeight;

					// The number of new vertices added along each Level is same as the Level number
					const int NumNewLevelVertices = Level; 
					for (int VertexOffset = 0; VertexOffset < NumNewLevelVertices; ++VertexOffset) 
					{
						const double Alpha = (double)(VertexOffset + 1) / (NumNewLevelVertices + 1);
						LerpWeights(Weight1, Weight2, OutWeight, Alpha);
						SetWeightOnTriangle(TriangleID, ElementIDCounter, OutWeight);
						ElementIDCounter++;
					}
				}
			}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

			if (Cancelled()) 
			{
				return false;
			}

			return true;
		}

		void CopyToAttribute(FDynamicMeshVertexSkinWeightsAttribute* SkinAttribute) 
		{
			for (int Idx = 0; Idx < VertexBoneWeights.Num(); ++ Idx) 
			{
				SkinAttribute->SetValue(Idx, VertexBoneWeights[Idx]);
			}
		}

		virtual bool Generate() override
		{	
			if (Validate() == false) 
			{
				return false;
			}

			Initialize();
			
			if (GenerateEdgeElements() == false) 
			{
				return false;
			}
			
			if (GenerateTriangleElements() == false) 
			{
				return false;
			}
				
			return true;
		}
	};

	bool Tessellate(const FDynamicMesh3* InMesh, 
				   const int TessellationNum, 
				   FProgressCancel* Progress, 
				   const bool bUseParallel,
				   TMap<int32, FIndex2i>* OutVertexEdgeMap,
				   TMap<int32, int32>* OutVertexTriangleMap, 
				   FDynamicMesh3* OutMesh) 
	{	
		OutMesh->Clear();

		FMeshVerticesGenerator<double> FTrianglesGenerator(InMesh, TessellationNum, Progress, OutVertexEdgeMap, OutVertexTriangleMap, bUseParallel);
		if (FTrianglesGenerator.Generate() == false) 
		{
			return false;
		}
		
		const int NumVerts = FTrianglesGenerator.OutElementCount;
		for (int Idx = 0; Idx < NumVerts; ++Idx)
		{
			OutMesh->AppendVertex(FTrianglesGenerator.GetElement<FVector3d>(Idx));
		}

		TUniquePtr<FTriangleAttributeGenerator<int, 1>> TriangleGroupGenerator;
		if (InMesh->HasTriangleGroups()) 
		{
			TriangleGroupGenerator = FTriangleAttributeGenerator<int, 1>::CreateFromMeshTriangleGroups(InMesh, TessellationNum, Progress, bUseParallel);
			if (TriangleGroupGenerator->Generate() == false) 
			{
				return false;
			} 

			OutMesh->EnableTriangleGroups();
		}

		for (int Idx = 0; Idx < FTrianglesGenerator.Triangles.Num(); ++Idx)
		{
			int PolyID = 0; 
			if (OutMesh->HasTriangleGroups() && TriangleGroupGenerator != nullptr) 
			{
				TriangleGroupGenerator->GetElement(Idx, &PolyID);
			} 

			ensure(OutMesh->AppendTriangle(FTrianglesGenerator.Triangles[Idx], PolyID) == Idx);
		}

		if (OutMesh->TriangleCount() != FTrianglesGenerator.Triangles.Num()) 
		{
			return false;
		}

		if (InMesh->HasAttributes()) 
		{	
			OutMesh->EnableAttributes(); 

			const FDynamicMeshAttributeSet* InAttributes = InMesh->Attributes(); 
			FDynamicMeshAttributeSet* OutAttributes = OutMesh->Attributes(); 
			
			if (InAttributes->NumNormalLayers()) 
			{
				OutAttributes->SetNumNormalLayers(InAttributes->NumNormalLayers());
				for (int Idx = 0; Idx < InAttributes->NumNormalLayers(); ++Idx) 
				{	
					FOverlayGenerator<float, 3> Generator(InMesh, TessellationNum, Progress, bUseParallel, InAttributes->GetNormalLayer(Idx));
					if (Generator.Generate() == false) 
					{
						return false;
					}

					Generator.CopyToAttribute(OutAttributes->GetNormalLayer(Idx));
				} 
			}

			if (InAttributes->NumUVLayers()) 
			{
				OutAttributes->SetNumUVLayers(InAttributes->NumUVLayers());
				for (int Idx = 0; Idx < InAttributes->NumUVLayers(); ++Idx) 
				{	
					FOverlayGenerator<float, 2> Generator(InMesh, TessellationNum, Progress, bUseParallel, InAttributes->GetUVLayer(Idx));
					if (Generator.Generate() == false) 
					{
						return false;
					}

					Generator.CopyToAttribute(OutAttributes->GetUVLayer(Idx));
				}
			}

			if (InAttributes->HasPrimaryColors()) 
			{
				OutAttributes->EnablePrimaryColors();
				FOverlayGenerator<float, 4> Generator(InMesh, TessellationNum, Progress, bUseParallel, InAttributes->PrimaryColors());
				if (Generator.Generate() == false)
				{
					return false;
				}

				Generator.CopyToAttribute(OutAttributes->PrimaryColors());
			}

			if (InAttributes->HasMaterialID())	 
			{
				OutAttributes->EnableMaterialID();
				TUniquePtr<FTriangleAttributeGenerator<int32, 1>> Generator; 
				Generator = FTriangleAttributeGenerator<int32, 1>::CreateFromDynamicMeshTriangleAttribute(InMesh, TessellationNum, Progress, bUseParallel, InAttributes->GetMaterialID());
				if (Generator->Generate() == false)
				{
					return false;
				}

				Generator->CopyToAttribute(OutAttributes->GetMaterialID());
				OutAttributes->GetMaterialID()->SetName(InAttributes->GetMaterialID()->GetName());
			}

			if (InAttributes->NumPolygroupLayers())  
			{
				OutAttributes->SetNumPolygroupLayers(InAttributes->NumPolygroupLayers());
				for (int Idx = 0; Idx < InAttributes->NumPolygroupLayers(); ++Idx) 
				{	
					TUniquePtr<FTriangleAttributeGenerator<int32, 1>> Generator;
					Generator = FTriangleAttributeGenerator<int32, 1>::CreateFromDynamicMeshTriangleAttribute(InMesh, TessellationNum, Progress, bUseParallel, InAttributes->GetPolygroupLayer(Idx));
					if (Generator->Generate() == false) 
					{
						return false;
					}

					Generator->CopyToAttribute(OutAttributes->GetPolygroupLayer(Idx));
					OutAttributes->GetPolygroupLayer(Idx)->SetName(InAttributes->GetPolygroupLayer(Idx)->GetName());
				}
			}

			for (const TTuple<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& AttributeInfo : InAttributes->GetSkinWeightsAttributes())
			{
				FSkinWeightAttributeGenerator Generator(InMesh, TessellationNum, Progress, bUseParallel, AttributeInfo.Value.Get());
				if (Generator.Generate() == false) 
				{
					return false;
				}

				FDynamicMeshVertexSkinWeightsAttribute* SkinAttribute = new FDynamicMeshVertexSkinWeightsAttribute(OutMesh);			
				Generator.CopyToAttribute(SkinAttribute);
				SkinAttribute->SetName(AttributeInfo.Value.Get()->GetName());
				
				OutAttributes->AttachSkinWeightsAttribute(AttributeInfo.Key, SkinAttribute);
			}

			for (const TTuple<FName, TUniquePtr<FDynamicMeshMorphTargetAttribute>>& AttributeInfo : InAttributes->GetMorphTargetAttributes())
			{
				TUniquePtr<FVertexAttributeGenerator<float, 3>> Generator;
				Generator = FVertexAttributeGenerator<float, 3>::CreateFromVertexAttribute(InMesh, TessellationNum, Progress, bUseParallel, AttributeInfo.Value.Get());
				if (Generator->Generate() == false)
				{
					return false;
				}

				FDynamicMeshMorphTargetAttribute* MorphTargetAttribute = new FDynamicMeshMorphTargetAttribute(OutMesh);			
				Generator->CopyToAttribute(MorphTargetAttribute);
				MorphTargetAttribute->SetName(AttributeInfo.Value.Get()->GetName());
				
				OutAttributes->AttachMorphTargetAttribute(AttributeInfo.Key, MorphTargetAttribute);
			}

			if (InAttributes->NumWeightLayers() > 0)
			{
				OutAttributes->SetNumWeightLayers(InAttributes->NumWeightLayers());
				for (int Idx = 0; Idx < InAttributes->NumWeightLayers(); ++Idx)
				{
					TUniquePtr<FVertexAttributeGenerator<float, 1>> Generator;
					Generator = FVertexAttributeGenerator<float, 1>::CreateFromVertexAttribute(InMesh, TessellationNum, Progress, bUseParallel, InAttributes->GetWeightLayer(Idx));
					if (Generator->Generate() == false)
					{
						return false;
					}

					Generator->CopyToAttribute(OutAttributes->GetWeightLayer(Idx));
					OutAttributes->GetWeightLayer(Idx)->SetName(InAttributes->GetWeightLayer(Idx)->GetName());
				}
			}


		}

		if (InMesh->HasVertexNormals()) 
		{	
			TUniquePtr<FVertexAttributeGenerator<float, 3>> Generator;
			Generator = FVertexAttributeGenerator<float, 3>::CreateFromPerVertexNormals(InMesh, TessellationNum, Progress, bUseParallel);
			if (Generator->Generate() == false) 
			{
				return false;
			}

			OutMesh->EnableVertexNormals(FVector3f::Zero());
			
			for (int Idx = 0; Idx < NumVerts; ++Idx) 
			{
				OutMesh->SetVertexNormal(Idx, Generator->GetElement<FVector3f>(Idx));
			}
		}

		if (InMesh->HasVertexUVs()) 
		{	
			TUniquePtr<FVertexAttributeGenerator<float, 2>> Generator; 
			Generator = FVertexAttributeGenerator<float, 2>::CreateFromPerVertexUVs(InMesh, TessellationNum, Progress, bUseParallel);
			if (Generator->Generate() == false) 
			{
				return false;
			}

			OutMesh->EnableVertexUVs(FVector2f::Zero());

			for (int Idx = 0; Idx < NumVerts; ++Idx) 
			{
				OutMesh->SetVertexUV(Idx, Generator->GetElement<FVector2f>(Idx));
			}
		}

		if (InMesh->HasVertexColors()) 
		{	
			TUniquePtr<FVertexAttributeGenerator<float, 4>> Generator;
			Generator  = FVertexAttributeGenerator<float, 4>::CreateFromPerVertexColors(InMesh, TessellationNum, Progress, bUseParallel);
			if (Generator->Generate() == false) 
			{
				return false;
			}

			OutMesh->EnableVertexColors(FVector4f::Zero());

			for (int Idx = 0; Idx < NumVerts; ++Idx) 
			{
				OutMesh->SetVertexColor(Idx, Generator->GetElement<FVector4f>(Idx));
			}
		}

		return true; 
	}

}

int32 FUniformTessellate::ExpectedNumVertices(const FDynamicMesh3& Mesh, const int32 TessellationNum)
{
	int32 EdgeVertCount = TessellationNum * Mesh.EdgeCount();
	int32 TriangleNumber = TessellationNum + 2; // plus 2 corner vertices on the edge ends
	int32 InnerTriangleVertCount = UniformTessellateLocals::TriangularPatternUtils::NumInnerVertices(TriangleNumber);
	int32 TotalVertCount = Mesh.VertexCount() + EdgeVertCount + InnerTriangleVertCount* Mesh.TriangleCount();
	return TotalVertCount; 	
}
	
int32 FUniformTessellate::ExpectedNumTriangles(const FDynamicMesh3& Mesh, const int32 TessellationNum)
{
	int32 TriangleNumber = TessellationNum + 2; // plus 2 corner vertices on the edge ends
	int32 InnerTriangleCount = UniformTessellateLocals::TriangularPatternUtils::NumTriangles(TriangleNumber);
	int32 TotalTriangleCount = InnerTriangleCount * Mesh.TriangleCount();
	return TotalTriangleCount;
}

bool FUniformTessellate::Cancelled()
{
	return (Progress == nullptr) ? false : Progress->Cancelled();
}

bool FUniformTessellate::Compute()
{	
	if (Validate() != EOperationValidationResult::Ok) 
	{
		return false;
	}

	if (TessellationNum == 0)
	{
		return true; 
	}

	// Check for the empty meshes
	if (bInPlace && ResultMesh->TriangleCount() == 0) 
	{
		return true;
	}

	if (bInPlace == false && Mesh->TriangleCount() == 0) 
	{
		return true;
	}

	// Make a copy of the mesh since we are tessellating in place. 
	// The copy will be used to restore the mesh to its original state in case the user cancelled the operation.
	FDynamicMesh3 ResultMeshCopy;
	if (bInPlace) 
	{
		ResultMeshCopy.Copy(*ResultMesh);
		Mesh = &ResultMeshCopy;
	}

	ResultMesh->Clear();
	VertexMap.Empty();
	VertexEdgeMap.Empty();
	VertexTriangleMap.Empty();
	
	bool bIsValidMesh = false;

	if (Mesh->IsCompact()) 
	{
		bIsValidMesh = UniformTessellateLocals::Tessellate(Mesh, 
														   TessellationNum, 
														   Progress, 
														   bUseParallel, 
														   bComputeMappings ? &VertexEdgeMap : nullptr, 
														   bComputeMappings ? &VertexTriangleMap : nullptr, 
														   ResultMesh);

		if (bIsValidMesh && bComputeMappings) 
		{
			VertexMap.SetNumUninitialized(Mesh->MaxVertexID());
			for (int32 VertexID = 0; VertexID < Mesh->MaxVertexID(); ++VertexID)
			{
				VertexMap[VertexID] = VertexID; 
			}
		}
	}
	else 
	{
		FDynamicMesh3 CompactMesh;
		FCompactMaps CompactInfo;
		CompactMesh.CompactCopy(*Mesh, true, true, true, true, &CompactInfo);

		bIsValidMesh = UniformTessellateLocals::Tessellate(&CompactMesh, 
														   TessellationNum, 
														   Progress, 
														   bUseParallel, 
														   bComputeMappings ? &VertexEdgeMap : nullptr, 
														   bComputeMappings ? &VertexTriangleMap : nullptr, 
														   ResultMesh);


		if (bIsValidMesh && bComputeMappings) 
		{	
			VertexMap.SetNumUninitialized(Mesh->MaxVertexID());
			for (int32 VertexID = 0; VertexID < Mesh->MaxVertexID(); ++VertexID)
			{
				VertexMap[VertexID] = CompactInfo.GetVertexMapping(VertexID); 
			}
			
			// Compute an additional mapping from the compact mesh triangles to the input mesh triangles 
			// i.e the inverse of the FCompactMaps
			TMap<int32, int32> CompactToInputTriangles;
			CompactToInputTriangles.Reserve(CompactMesh.TriangleCount());
			for (int32 TriangleID : Mesh->TriangleIndicesItr()) 
			{
				const int32 CompactTriangleID = CompactInfo.GetTriangleMapping(TriangleID);
				if (CompactTriangleID != FCompactMaps::InvalidID)
				{
					CompactToInputTriangles.Add(CompactTriangleID, TriangleID);
				}
			}
			checkSlow(CompactToInputTriangles.Num() == CompactMesh.TriangleCount());
				
			// Chage the mapping [tessellated triangles --> compact mesh triangles]
			// to the mapping [tessellated triangles --> input mesh triangles (before compacting)]
			for (TMap<int32, FIndex2i>::TIterator It = VertexEdgeMap.CreateIterator(); It; ++It)
			{
				FIndex2i& EdgeTri = It.Value();
				
				// We should always be able to find the correct match, but double check in case the Tessellate method
				// failed to return the correct mapping
				int32* TID1= CompactToInputTriangles.Find(EdgeTri.A);
				ensure(TID1 != nullptr);
				EdgeTri.A = TID1 != nullptr ? *TID1 : EdgeTri.A;
				
				if (EdgeTri.B != FDynamicMesh3::InvalidID) 
				{
					int32* TID2 = CompactToInputTriangles.Find(EdgeTri.B);
					ensure(TID2 != nullptr);
					EdgeTri.B = TID2 != nullptr ? *TID2 : EdgeTri.B;
				}
			}

			for (TMap<int32, int32>::TIterator It = VertexTriangleMap.CreateIterator(); It; ++It)
			{
				int32& CompactTriID = It.Value();
				int32* TID1 = CompactToInputTriangles.Find(CompactTriID);
				ensure(TID1 != nullptr);
				CompactTriID = TID1 != nullptr ? *TID1 : CompactTriID;
			}
		}
	}

	if (bIsValidMesh == false || Cancelled()) 
	{
		if (bInPlace) 
		{ 
			// Restore the input mesh
			ResultMesh->Copy(ResultMeshCopy);
		}
		else 
		{
			ResultMesh->Clear();
		}

		VertexMap.Empty();
		VertexEdgeMap.Empty();
		VertexTriangleMap.Empty();
		
		return false;
	}
	
	return true;
}