// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/WeldEdgeSequence.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMeshEditor.h"

using namespace UE::Geometry;

FWeldEdgeSequence::EWeldResult FWeldEdgeSequence::Weld()
{
	// As soon as any of these helper functions return a non-OK value,
	// we want to forward that to the user and stop operating.

	EWeldResult Result = EWeldResult::Ok;

	if ((Result = CheckInput()) != EWeldResult::Ok)
	{
		return Result;
	}

	if ((Result = SplitSmallerSpan()) != EWeldResult::Ok)
	{
		return Result;
	}

	if ((Result = WeldEdgeSequence()) != EWeldResult::Ok)
	{
		return Result;
	}

	return Result;
}



/** Protected Functions */

FWeldEdgeSequence::EWeldResult FWeldEdgeSequence::CheckInput()
{
	// Selected edges must be boundary edges
	for (int Edge : EdgeSpanToDiscard.Edges)
	{
		if (!ensure(Mesh->IsEdge(Edge)) || !Mesh->IsBoundaryEdge(Edge))
		{
			return EWeldResult::Failed_EdgesNotBoundaryEdges;
		}
	}

	for (int Edge : EdgeSpanToKeep.Edges)
	{
		if (!ensure(Mesh->IsEdge(Edge)) || !Mesh->IsBoundaryEdge(Edge))
		{
			return EWeldResult::Failed_EdgesNotBoundaryEdges;
		}
	}

	// Ensure that the two input spans are oriented according to mesh boundary
	// Guaranteed to be on boundary after two for loops above
	EdgeSpanToDiscard.SetCorrectOrientation();
	EdgeSpanToKeep.SetCorrectOrientation();

	return EWeldResult::Ok;
}

FWeldEdgeSequence::EWeldResult FWeldEdgeSequence::SplitSmallerSpan()
{
	return SplitEdgesToEqualizeSpanLengths(*Mesh, EdgeSpanToKeep, EdgeSpanToDiscard);
}

FWeldEdgeSequence::EWeldResult UE::Geometry::FWeldEdgeSequence::SplitEdgesToEqualizeSpanLengths(FDynamicMesh3& Mesh, FEdgeSpan& Span1, FEdgeSpan& Span2)
{
	// For each new vertex that must be created:
	// The longest simple edge is found and split
	// The newly generated vertex is inserted into the span
	// The newly generated edge is inserted into the span
	// TODO: Could improve this by sorting lengths and keeping those updated as we split (i.e. using
	//  a priority queue)

	FEdgeSpan& SpanToSplit = (Span1.Vertices.Num() < Span2.Vertices.Num()) ? Span1 : Span2;
	int TotalSplits = FMath::Abs(Span1.Vertices.Num() - Span2.Vertices.Num());
	for (int SplitCount = 0; SplitCount < TotalSplits; ++SplitCount)
	{
		double MaxLength = 0.0;
		int LongestEID = -1;
		int LongestIndex = -1;

		// Find longest edge and store length, ID, and index
		for (int EdgeIndex = 0; EdgeIndex < SpanToSplit.Edges.Num(); ++EdgeIndex)
		{
			FVector3d VertA = Mesh.GetVertex(Mesh.GetEdge(SpanToSplit.Edges[EdgeIndex]).Vert.A);
			FVector3d VertB = Mesh.GetVertex(Mesh.GetEdge(SpanToSplit.Edges[EdgeIndex]).Vert.B);
			double EdgeLength = DistanceSquared(VertA, VertB);

			if (MaxLength < EdgeLength)
			{
				MaxLength = EdgeLength;
				LongestEID = SpanToSplit.Edges[EdgeIndex];
				LongestIndex = EdgeIndex;
			}
		}

		// Split longest edge
		FDynamicMesh3::FEdgeSplitInfo SplitInfo;
		EMeshResult Result = Mesh.SplitEdge(LongestEID, SplitInfo);
		if (Result != EMeshResult::Ok)
		{
			return EWeldResult::Failed_CannotSplitEdge;
		}

		// Correctly insert new vertex (between vertices of split edge)
		SpanToSplit.Vertices.Insert(SplitInfo.NewVertex, LongestIndex + 1);

		// Correctly insert new edge
		// OriginalVertices.B is the non-new vertex of the newly inserted edge- use this
		// to determine whether the edge goes before or after the original in our span
		if (SplitInfo.OriginalVertices.B == SpanToSplit.Vertices[LongestIndex])
		{
			SpanToSplit.Edges.Insert(SplitInfo.NewEdges.A, LongestIndex);
		}
		else
		{
			SpanToSplit.Edges.Insert(SplitInfo.NewEdges.A, LongestIndex + 1);
		}
	}

	return EWeldResult::Ok;
}

// No longer used, as it fails to consider the case of intervening triangles in the middle
//  of the sequence, if the edges are the boundary of a band.
FWeldEdgeSequence::EWeldResult FWeldEdgeSequence::CheckForAndCollapseSideTriangles()
{
	// Checks for and deletes edge between VertA and VertB
	auto CheckForAndHandleEdge = [this](int VertA, int VertB)
	{
		int Edge = Mesh->FindEdge(VertA, VertB);
		if (Edge != IndexConstants::InvalidID)
		{
			if (bAllowIntermediateTriangleDeletion == false)
			{
				return EWeldResult::Failed_TriangleDeletionDisabled;
			}

			FIndex2i TrianglePair = Mesh->GetEdgeT(Edge);
			EMeshResult Result = Mesh->RemoveTriangle(TrianglePair.A);
			if (Result != EMeshResult::Ok)
			{
				return EWeldResult::Failed_CannotDeleteTriangle;
			}

			if (Mesh->IsTriangle(TrianglePair.B))
			{
				Result = Mesh->RemoveTriangle(TrianglePair.B);
				if (Result != EMeshResult::Ok)
				{
					return EWeldResult::Failed_CannotDeleteTriangle;
				}
			}
		}

		return EWeldResult::Ok;
	};

	EWeldResult Result = CheckForAndHandleEdge(EdgeSpanToDiscard.Vertices[0], EdgeSpanToKeep.Vertices.Last());
	if (Result != EWeldResult::Ok)
	{
		return Result;
	}
	
	Result = CheckForAndHandleEdge(EdgeSpanToDiscard.Vertices.Last(), EdgeSpanToKeep.Vertices[0]);
	if (Result != EWeldResult::Ok)
	{
		return Result;
	}

	return EWeldResult::Ok;
}

FWeldEdgeSequence::EWeldResult FWeldEdgeSequence::WeldEdgeSequence()
{
	if (!ensure(EdgeSpanToDiscard.Edges.Num() == EdgeSpanToKeep.Edges.Num()
		&& EdgeSpanToDiscard.Vertices.Num() == EdgeSpanToDiscard.Edges.Num() + 1
		&& EdgeSpanToKeep.Vertices.Num() == EdgeSpanToKeep.Edges.Num() + 1))
	{
		return EWeldResult::Failed_Other;
	}

	// There are certain pathological cases in which one edge weld could delete one of the
	//  next edge paired verts before we can use its location and attribute values for interpolation.
	//  For example in the following diagram, welding ab to de will delete the triangle bce, but we
	//  still need to update vertex f:
	//    a_b_c
	//     \|/
	//    d_e_f
	//    |\|\|
	//
	// This is only possible if there is an edge between b and e (so that the triangle can be deleted), and if
	//  there is not another triangle holding on to c. We can handle this case by collapsing a bit out of order-
	//  if we know there's an edge at this pair but there is not an edge at the next pair, we can do the next
	//  pair first, and we know the same issue won't occur there. On the flip side if there is an edge at the
	//  next pair too, then the next vert can't be destroyed by collapsing this one.
	
	// For this and other edge cases, it is safer to do the welding vert by vert instead of edge by edge.
	//  As another example, in this diagram, after merging ab to de, bc no longer exists, but c still needs
	//  welding to f:
	//    a_b_c
	//    |\|/|
	//    d_e_f

	auto ProcessVidPair = [this](int32 KeepVid, int32 DiscardVid, int32 KeepVertIndex) -> EWeldResult
	{
		FDynamicMesh3::FMergeVerticesInfo MergeInfo;
		EMeshResult Result = Mesh->MergeVertices(KeepVid, DiscardVid, InterpolationT, MergeInfo);

		if (Result == EMeshResult::Failed_CollapseTriangle
			|| Result == EMeshResult::Failed_CollapseQuad
			|| Result == EMeshResult::Failed_FoundDuplicateTriangle)
		{
			// Currently collapse doesn't allow us to collapse away an isolated triangle, quad,
			//  or double sided triangle. We can deal with this case, however, simply by deleting
			//  them.
			// TODO: Should maybe have an option for this in CollapseEdge/MergeVertices
			int32 Eid = Mesh->FindEdge(KeepVid, DiscardVid);
			if (!ensure(Mesh->IsEdge(Eid)))
			{
				return EWeldResult::Failed_Other;
			}
			FIndex2i TidsToDelete = Mesh->GetEdgeT(Eid);
			Mesh->RemoveTriangle(TidsToDelete.A);
			if (TidsToDelete.B != IndexConstants::InvalidID)
			{
				Mesh->RemoveTriangle(TidsToDelete.B);
			}
		}
		else if (Result == EMeshResult::Failed_InvalidNeighbourhood && bAllowFailedMerge == true)
		{
			// If we're allowed to, we just place the edges together without welding.
			FVector3d Destination = Lerp(Mesh->GetVertex(KeepVid), Mesh->GetVertex(DiscardVid), InterpolationT);
			Mesh->SetVertex(KeepVid, Destination);
			Mesh->SetVertex(DiscardVid, Destination);

			// Maybe it's unfortunate that we have to output unmerged edges instead of vertices, but
			//  theoretically the edges on either side were not successfully welded.
			auto AddEdgeAtKeepIndex = [this](int32 KeepEidIndex)
			{
				int32 DiscardEidIndex = EdgeSpanToDiscard.Edges.Num() - 1 - KeepEidIndex;
				int32 KeepEid = EdgeSpanToKeep.Edges[KeepEidIndex];
				int32 DiscardEid = EdgeSpanToDiscard.Edges[DiscardEidIndex];
				if (Mesh->IsEdge(KeepEid) && Mesh->IsEdge(DiscardEid))
				{
					UnmergedEdgePairsOut.Add(TPair<int, int>(KeepEid, DiscardEid));
				}
			};
			if (KeepVertIndex > 0)
			{
				AddEdgeAtKeepIndex(KeepVertIndex - 1);
			}
			if (KeepVertIndex < EdgeSpanToKeep.Edges.Num())
			{
				AddEdgeAtKeepIndex(KeepVertIndex);
			}
		}
		else if (Result != EMeshResult::Ok)
		{
			return EWeldResult::Failed_Other;
		}

		return EWeldResult::Ok;
	};

	for (int KeepVertIndex = 0; KeepVertIndex < EdgeSpanToKeep.Vertices.Num(); ++KeepVertIndex)
	{
		// The spans are oriented opposite directions, so iterate in opposite order
		int32 DiscardVertIndex = EdgeSpanToDiscard.Vertices.Num() - 1 - KeepVertIndex;
		int32 KeepVid = EdgeSpanToKeep.Vertices[KeepVertIndex];
		int32 DiscardVid = EdgeSpanToDiscard.Vertices[DiscardVertIndex];

		if (KeepVid == DiscardVid)
		{
			continue;
		}

		if (!ensure(Mesh->IsVertex(KeepVid) && Mesh->IsVertex(DiscardVid)))
		{
			// This shouldn't happen due to our out-of-order collapse strategy, see above
			continue;
		}

		// See above for why we consider processing the next vid first
		bool bProcessedNext = false;
		int32 InterveningEdge = Mesh->FindEdge(KeepVid, DiscardVid);
		if (InterveningEdge != IndexConstants::InvalidID 
			&& KeepVertIndex < EdgeSpanToKeep.Vertices.Num()-1)
		{
			int32 NextKeepVid = EdgeSpanToKeep.Vertices[KeepVertIndex + 1];
			int32 NextDiscardVid = EdgeSpanToDiscard.Vertices[DiscardVertIndex - 1];
			if (NextKeepVid == NextDiscardVid)
			{
				// Consider ourselves to have dealt with the next vertex
				bProcessedNext = true;
			}
			else if (ensure(Mesh->IsVertex(NextKeepVid) && Mesh->IsVertex(NextDiscardVid))
				&& Mesh->FindEdge(NextKeepVid, NextDiscardVid) == IndexConstants::InvalidID)
			{
				// This is a safe vert pair to collapse, since it doesn't have intervening edges.
				//  And if it did, it wouldn't be in danger of losing a vert due to our collapse.
				EWeldResult Result = ProcessVidPair(NextKeepVid, NextDiscardVid, KeepVertIndex + 1);
				if (Result != EWeldResult::Ok)
				{
					return Result;
				}
				bProcessedNext = true;
			}
		}
		
		EWeldResult Result = ProcessVidPair(KeepVid, DiscardVid, KeepVertIndex);
		if (Result != EWeldResult::Ok)
		{
			return Result;
		}

		if (bProcessedNext)
		{
			++KeepVertIndex;
		}
	}//end iterating through vertices

	return EWeldResult::Ok;
}
