// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/WrapMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Solvers/Internal/QuadraticProgramming.h"
#include "Solvers/LaplacianMatrixAssembly.h"
#include "Solvers/Internal/MatrixSolver.h"
#include "Spatial/PointHashGrid3.h"
#include "ToDynamicMesh.h"

using namespace UE::Geometry;

namespace UE::Geometry::Private
{
	static void ConstructCotangentNoAreaLaplacian(const FDynamicMesh3& Mesh, const FVertexLinearization& VertexLinearization, FSparseMatrixD& Laplacian)
	{
		FEigenSparseMatrixAssembler Assembler(VertexLinearization.NumVerts(), VertexLinearization.NumVerts());
		UE::MeshDeformation::ConstructFullCotangentLaplacian<double>(Mesh, VertexLinearization, Assembler,
			UE::MeshDeformation::ECotangentWeightMode::Default,
			UE::MeshDeformation::ECotangentAreaMode::NoArea);
		Assembler.ExtractResult(Laplacian);
		Laplacian *= -1.;
	}
	static void ConstructCotangentWeightedLaplacian(const FDynamicMesh3& Mesh, const FVertexLinearization& VertexLinearization, FSparseMatrixD& Laplacian)
	{
		FEigenSparseMatrixAssembler Assembler(VertexLinearization.NumVerts(), VertexLinearization.NumVerts());
		UE::MeshDeformation::ConstructFullCotangentLaplacian<double>(Mesh, VertexLinearization, Assembler,
			UE::MeshDeformation::ECotangentWeightMode::Default,
			UE::MeshDeformation::ECotangentAreaMode::VoronoiArea);
		Assembler.ExtractResult(Laplacian);
		Laplacian *= -1.;
	}

	static void ConstructAffineInvariantLaplacian(const FDynamicMesh3& Mesh, const FVertexLinearization& VertexLinearization, FSparseMatrixD& Laplacian)
	{
		FEigenSparseMatrixAssembler Assembler(VertexLinearization.NumVerts(), VertexLinearization.NumVerts());
		for (const int32 Vid : Mesh.VertexIndicesItr())
		{
			const int32 NumNeighbors = Mesh.GetVtxEdgeCount(Vid);
			if (NumNeighbors < 4)
			{
				continue;
			}
			const int32 Index = VertexLinearization.GetIndex(Vid);
			const FVector3d& Xi = Mesh.GetVertexRef(Vid);
			FDenseMatrixD LocalDeltaMatrix(3, NumNeighbors); // columns are Xj - Xi where Xi = center vertex pos, Xj = pos of one-ring vertices
			TArray<int32> NeighborIndices;
			NeighborIndices.Reserve(NumNeighbors);
			
			int32 LocalIndex = 0;
			for (const int32 NeighborId : Mesh.VtxVerticesItr(Vid))
			{
				const int32 NIndex = VertexLinearization.GetIndex(NeighborId);
				const FVector3d& Xj = Mesh.GetVertexRef(NeighborId);
				const FVector3d dXji = Xj - Xi;
				NeighborIndices.Add(NIndex);
				LocalDeltaMatrix(0, LocalIndex) = dXji[0];
				LocalDeltaMatrix(1, LocalIndex) = dXji[1];
				LocalDeltaMatrix(2, LocalIndex) = dXji[2];
				++LocalIndex;
			}

			// Calculate right singular vectors (aka eigenvalues of LocalDeltaMatrix^T LocalDeltaMatrix)
			const FDenseMatrixD LocalDeltaMatrixTLocalDeltaMatrix = LocalDeltaMatrix.adjoint() * LocalDeltaMatrix;
			Eigen::SelfAdjointEigenSolver<FDenseMatrixD> EigenSolver(LocalDeltaMatrixTLocalDeltaMatrix);

			// Eigen's eigenvalues are sorted from smallest to largest. Want first NumNeighbors - 3 (should be zero eigenvalues)
			const FDenseMatrixD WNullSpace = EigenSolver.eigenvectors().leftCols(NumNeighbors - 3);

			const FDenseMatrixD LocalLaplacian = WNullSpace * WNullSpace.adjoint();

			double Lii = 0.;
			for (int32 LocalIndexA = 0; LocalIndexA < NumNeighbors; ++LocalIndexA)
			{
				double Lai = 0.;
				for (int32 LocalIndexB = 0; LocalIndexB < NumNeighbors; ++LocalIndexB)
				{
					Assembler.AddEntryFunc(NeighborIndices[LocalIndexA], NeighborIndices[LocalIndexB], LocalLaplacian(LocalIndexA, LocalIndexB));
					Lai += LocalLaplacian(LocalIndexA, LocalIndexB);
				}
				Assembler.AddEntryFunc(NeighborIndices[LocalIndexA], Index, -Lai);
				Assembler.AddEntryFunc(Index, NeighborIndices[LocalIndexA], -Lai);
				Lii += Lai;
			}
			Assembler.AddEntryFunc(Index, Index, Lii);
		}
		Assembler.ExtractResult(Laplacian);
	}

	class FNonManifoldVertexLinearization : public UE::Geometry::FVertexLinearization
	{
	public:
		FNonManifoldVertexLinearization() = default;

		explicit FNonManifoldVertexLinearization(const FDynamicMesh3& Mesh)
		{
			Initialize(Mesh);
		}

		void Initialize(const FDynamicMesh3& Mesh)
		{
			Empty();
			FNonManifoldMappingSupport NonManifoldMappingSupport(Mesh);

			// Count num non-manifold vertices
			int32 NumNonManifoldVertices = 0;
			if (NonManifoldMappingSupport.IsNonManifoldVertexInSource())
			{
				for (const int32 VertexId : Mesh.VertexIndicesItr())
				{
					if (NonManifoldMappingSupport.IsNonManifoldVertexID(VertexId))
					{
						++NumNonManifoldVertices;
					}
				}
			}
			ToIndexMap.Init(IndexConstants::InvalidID, Mesh.MaxVertexID());
			ToIdMap.SetNumUninitialized(Mesh.VertexCount() - NumNonManifoldVertices);

			// Linear index manifold vertices
			int32 N = 0;
			for (const int32 VertexId : Mesh.VertexIndicesItr())
			{
				if (!NonManifoldMappingSupport.IsNonManifoldVertexID(VertexId))
				{
					ToIdMap[N] = VertexId;
					ToIndexMap[VertexId] = N;
					++N;
				}
			}

			// Update ToIndexMap for non-manifold vertices
			for (const int32 VertexId : Mesh.VertexIndicesItr())
			{
				if (NonManifoldMappingSupport.IsNonManifoldVertexID(VertexId))
				{
					ToIndexMap[VertexId] = ToIndexMap[NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(VertexId)];
				}
			}

		}
	};

	// We don't care about UVs, SkinWeights, etc. 
	struct FMeshWeldingWrapper
	{
		typedef int32 TriIDType;
		typedef int32 VertIDType;
		typedef int32 WedgeIDType;
		typedef int32 UVIDType;
		typedef int32 NormalIDType;
		typedef int32 ColorIDType;

		FMeshWeldingWrapper(const FDynamicMesh3& InOriginalMesh, double WeldingThreshold)
			: OriginalMesh(InOriginalMesh)
		{
			const int32 NumVerts = OriginalMesh.VertexCount();
			const int32 MaxVert = OriginalMesh.MaxVertexID();
			TArray<FVector> UniqueVerts;
			OriginalToMerged.Init(INDEX_NONE, MaxVert);
			const double ThreshSq = WeldingThreshold * WeldingThreshold;

			const double HashGridCellSize = FMath::Max(1.5 * WeldingThreshold, 1.5 * UE_THRESH_POINTS_ARE_SAME);
			TPointHashGrid3<int32, double> UniquePointHashGrid(HashGridCellSize, INDEX_NONE);
			UniquePointHashGrid.Reserve(NumVerts);
			for (const int32 VertexId : OriginalMesh.VertexIndicesItr())
			{
				const FVector& Position = OriginalMesh.GetVertexRef(VertexId);
				bool bUnique = true;
				int32 RemapIndex = INDEX_NONE;

				const TPair<int32, double> FoundExisting = UniquePointHashGrid.FindNearestInRadius(Position, WeldingThreshold,
					[&Position, &UniqueVerts](const int32& UniqueVertIndex)
					{
						return (UniqueVerts[UniqueVertIndex] - Position).SizeSquared();
					},
					[](const int32& UniqueVertIndex) { return false; });

				if (FoundExisting.Get<0>() != INDEX_NONE)
				{
					bUnique = false;
					RemapIndex = FoundExisting.Get<0>();
				}

				if (bUnique)
				{
					// Unique
					const int32 UniqueIndex = UniqueVerts.Add(Position);
					UniquePointHashGrid.InsertPoint(UniqueIndex, Position);

					OriginalIndexes.Add(VertexId);
					OriginalToMerged[VertexId] = VertexId;
				}
				else
				{
					OriginalToMerged[VertexId] = OriginalIndexes[RemapIndex];
				}
			}

			TriIds.Reserve(OriginalMesh.TriangleCount());
			for (const int32 TriId : OriginalMesh.TriangleIndicesItr())
			{
				TriIds.Emplace(TriId);
			}

		}
		int32 NumTris() const
		{
			return TriIds.Num();
		}

		int32 NumVerts() const
		{
			return OriginalIndexes.Num();
		}

		int32 NumUVLayers() const
		{
			return 0;
		}

		// --"Vertex Buffer" info 
		const TArray<int32>& GetVertIDs() const
		{
			return OriginalIndexes;
		}

		const FVector3d GetPosition(const VertIDType VtxID) const
		{
			return OriginalMesh.GetVertex(VtxID);
		}

		// --"Index Buffer" info
		const TArray<int32>& GetTriIDs() const
		{
			return TriIds;
		}

		// return false if this TriID is not contained in mesh.
		bool GetTri(const TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const
		{
			if (OriginalMesh.IsTriangle(TriID))
			{
				const FIndex3i& Triangle = OriginalMesh.GetTriangleRef(TriID);
				VID0 = OriginalToMerged[Triangle.A];
				VID1 = OriginalToMerged[Triangle.B];
				VID2 = OriginalToMerged[Triangle.C];
				return true;
			}
			return false;
		}

		bool HasNormals() const
		{
			return false;
		}

		bool HasTangents() const
		{
			return false;
		}

		bool HasBiTangents() const
		{
			return false;
		}

		bool HasColors() const
		{
			return false;
		}

		// Each triangle corner is a wedge. 
		void GetWedgeIDs(const TriIDType& TriID, WedgeIDType& WID0, WedgeIDType& WID1, WedgeIDType& WID2) const
		{
			checkNoEntry();
		}

		// attribute access per-wedge
		// NB:  ToDynamicMesh will attempt to weld identical attributes that are associated with the same vertex
		FVector2f GetWedgeUV(int32 UVLayerIndex, WedgeIDType WID) const
		{
			checkNoEntry();
			return FVector2f();
		}

		FVector3f GetWedgeNormal(WedgeIDType WID) const
		{
			checkNoEntry();
			return FVector3f();
		}

		FVector3f GetWedgeTangent(WedgeIDType WID) const
		{
			checkNoEntry();
			return FVector3f();
		}

		FVector3f GetWedgeBiTangent(WedgeIDType WID) const
		{
			checkNoEntry();
			return FVector3f();
		}

		FVector4f GetWedgeColor(WedgeIDType WID) const
		{
			checkNoEntry();
			return FVector4f();
		}

		// attribute access that exploits shared attributes. 
		// each group of shared attributes presents itself as a mesh with its own attribute vertex buffer.
		// NB:  If the mesh has no shared Attr attributes, then Get{Attr}IDs() should return an empty array.
		// NB:  Get{Attr}Tri() functions should return false if the triangle is not set in the attribute mesh. 
		const TArray<UVIDType>& GetUVIDs(int32 LayerID) const
		{
			return EmptyArray;
		}

		FVector2f GetUV(int32 LayerID, UVIDType UVID) const
		{
			checkNoEntry();
			return FVector2f();
		}

		bool GetUVTri(int32 LayerID, const TriIDType& TID, UVIDType& ID0, UVIDType& ID1, UVIDType& ID2) const
		{
			return false;
		}

		const TArray<NormalIDType>& GetNormalIDs() const
		{
			return OriginalIndexes;
		}

		FVector3f GetNormal(NormalIDType ID) const
		{
			return OriginalMesh.GetVertexNormal(ID);
		}

		bool GetNormalTri(const TriIDType& TriID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const
		{
			return GetTri(TriID, NID0, NID1, NID2);
		}

		const TArray<NormalIDType>& GetTangentIDs() const
		{
			return EmptyArray;
		}

		FVector3f GetTangent(NormalIDType ID) const
		{
			checkNoEntry();
			return FVector3f();
		}

		bool GetTangentTri(const TriIDType& TID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const
		{
			return false;
		}

		const TArray<NormalIDType>& GetBiTangentIDs() const
		{
			return EmptyArray;
		}

		FVector3f GetBiTangent(NormalIDType ID) const
		{
			checkNoEntry();
			return FVector3f();
		}

		bool GetBiTangentTri(const TriIDType& TID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const
		{
			return false;
		}

		const TArray<ColorIDType>& GetColorIDs() const
		{
			return EmptyArray;
		}

		FVector4f GetColor(ColorIDType ID) const
		{
			checkNoEntry();
			return FVector4f();
		}

		bool GetColorTri(const TriIDType& TID, ColorIDType& NID0, ColorIDType& NID1, ColorIDType& NID2) const
		{
			return false;
		}

		// weight maps information
		int32 NumWeightMapLayers() const
		{
			return 0;
		}

		float GetVertexWeight(int32 WeightMapIndex, int32 SrcVertID) const
		{
			checkNoEntry();
			return 0.f;
		}

		FName GetWeightMapName(int32 WeightMapIndex) const
		{
			checkNoEntry();
			return FName();
		}

		// skin weight attributes information
		int32 NumSkinWeightAttributes() const
		{
			return 0;
		}

		UE::AnimationCore::FBoneWeights GetVertexSkinWeight(int32 SkinWeightAttributeIndex, VertIDType VtxID) const
		{
			checkNoEntry();
			return UE::AnimationCore::FBoneWeights();
		}

		FName GetSkinWeightAttributeName(int32 SkinWeightAttributeIndex) const
		{
			checkNoEntry();
			return NAME_None;
		}

		// bone attributes information
		int32 GetNumBones() const
		{
			return 0;
		}

		FName GetBoneName(int32 BoneIdx) const
		{
			checkNoEntry();
			return FName();
		}

		int32 GetBoneParentIndex(int32 BoneIdx) const
		{
			checkNoEntry();
			return INDEX_NONE;
		}

		FTransform GetBonePose(int32 BoneIdx) const
		{
			checkNoEntry();
			return FTransform();
		}

		FVector4f GetBoneColor(int32 BoneIdx) const
		{
			checkNoEntry();
			return FLinearColor::White;
		}
		const FDynamicMesh3& OriginalMesh;
		TArray<int32> OriginalIndexes; // UniqueIndex -> OrigIndex
		TArray<int32> OriginalToMerged; // OriginalIndex -> OriginalIndexes[UniqueVertIndex] 
		TArray<int32> TriIds;

		TArray<int32> EmptyArray;
	};
}


struct FWrapMesh::FImpl
{
	bool bIsWelded = false;
	FDynamicMesh3 WeldedSourceMesh;
	TArray<int32> SourceToWelded;
	Private::FNonManifoldVertexLinearization SourceVtxLinearization;
	FSparseMatrixD SourceLaplacian;
};

FWrapMesh::FWrapMesh(const FDynamicMesh3* InSourceMesh)
{
	SetMesh(InSourceMesh);
}

FWrapMesh::~FWrapMesh() = default;

void FWrapMesh::SetMesh(const FDynamicMesh3* InSourceMesh)
{
	SourceMesh = InSourceMesh;
	if (SourceMesh)
	{
		Initialize();
	}
	else
	{
		Impl.Reset();
	}
}

void FWrapMesh::Initialize()
{
	check(SourceMesh);
	Impl = MakePimpl<FImpl>();
	if (bWeldSourceMesh)
	{
		// Generate welded Dynamic mesh.
		Impl->bIsWelded = true;
		TToDynamicMesh<Private::FMeshWeldingWrapper> SourceToWelded;
		Private::FMeshWeldingWrapper Wrapper(*SourceMesh, SourceMeshWeldingThreshold);
		Impl->WeldedSourceMesh.EnableAttributes();
		constexpr bool bCopyTangents = false;
		SourceToWelded.Convert(Impl->WeldedSourceMesh, Wrapper, [](int32) { return 0; }, [](int32) { return INDEX_NONE; }, bCopyTangents);

		// Generate SourceToWelded map and non-manifold mapping
		Impl->SourceToWelded.Init(INDEX_NONE, SourceMesh->MaxVertexID());
		TArray<int32> NonManifoldMapping;
		NonManifoldMapping.Init(INDEX_NONE, Impl->WeldedSourceMesh.MaxVertexID());
		// Update for unique source vertices
		for (int32 WeldedId = 0; WeldedId < SourceToWelded.ToSrcVertIDMap.Num(); ++WeldedId)
		{
			int32 SrcVertID = SourceToWelded.ToSrcVertIDMap[WeldedId];
			if (Impl->SourceToWelded[SrcVertID] == INDEX_NONE)
			{
				// First time mapping back to this unique source index. Treat this as the "manifold" vertex
				Impl->SourceToWelded[SrcVertID] = WeldedId;
			}
			NonManifoldMapping[WeldedId] = Impl->SourceToWelded[SrcVertID];
		}
		// Update all non-unique source vertices
		for (int32 SourceId = 0; SourceId < Impl->SourceToWelded.Num(); ++SourceId)
		{
			if (SourceMesh->IsVertex(SourceId) && Impl->SourceToWelded[SourceId] == INDEX_NONE)
			{
				Impl->SourceToWelded[SourceId] = Impl->SourceToWelded[Wrapper.OriginalToMerged[SourceId]];
			}
		}


		UE::Geometry::FNonManifoldMappingSupport::AttachNonManifoldVertexMappingData(NonManifoldMapping, Impl->WeldedSourceMesh);
	}

	const FDynamicMesh3& SourceOrWelded = Impl->bIsWelded ? Impl->WeldedSourceMesh : *SourceMesh;
	Impl->SourceVtxLinearization.Initialize(SourceOrWelded);
	const int32 NumVerts = Impl->SourceVtxLinearization.NumVerts();
	
	FEigenSparseMatrixAssembler LaplacianAssembler(NumVerts, NumVerts);

	switch (LaplacianType)
	{
	case ELaplacianType::CotangentNoArea:
		Private::ConstructCotangentNoAreaLaplacian(SourceOrWelded, Impl->SourceVtxLinearization, Impl->SourceLaplacian);
		break;
	default:
	case ELaplacianType::CotangentAreaWeighted:
		Private::ConstructCotangentWeightedLaplacian(SourceOrWelded, Impl->SourceVtxLinearization, Impl->SourceLaplacian);
		break;
	case ELaplacianType::AffineInvariant:
		Private::ConstructAffineInvariantLaplacian(SourceOrWelded, Impl->SourceVtxLinearization, Impl->SourceLaplacian);
		break;
	}
}

void FWrapMesh::WrapToTargetShape(const FDynamicMesh3& TargetShape, const TArray<FWrapMeshCorrespondence>& SourceTargetVertexCorrespondences, FDynamicMesh3& WrappedMesh) const
{
	check(SourceMesh);

	// Initialize with source topology
	WrappedMesh.Copy(*SourceMesh);

	FDynamicMesh3 WeldedWrappedMesh;
	if (Impl->bIsWelded)
	{
		WeldedWrappedMesh.Copy(Impl->WeldedSourceMesh);
	}

	const FDynamicMesh3& SourceOrWelded = Impl->bIsWelded ? Impl->WeldedSourceMesh : *SourceMesh;
	FDynamicMesh3& WrappedOrWelded = Impl->bIsWelded ? WeldedWrappedMesh : WrappedMesh;

	// Make a BVH for the TargetShape for projection
	UE::Geometry::FDynamicMeshAABBTree3 TargetAABBTree(&TargetShape);

	const int32 NumVerts = Impl->SourceVtxLinearization.NumVerts();
	const TArray<int32>& ToMeshV = Impl->SourceVtxLinearization.ToId();
	const TArray<int32>& ToIndex = Impl->SourceVtxLinearization.ToIndex();

	FEigenSparseMatrixAssembler CorrespondencesAssembler(NumVerts, NumVerts);
	CorrespondencesAssembler.ReserveEntriesFunc(SourceTargetVertexCorrespondences.Num());

	FDenseMatrixD CorrespondenceRHS;
	CorrespondenceRHS.resize(NumVerts, 3);
	CorrespondenceRHS.setZero();
	for (const FWrapMeshCorrespondence& Correspondence : SourceTargetVertexCorrespondences)
	{
		if (SourceMesh->IsVertex(Correspondence.SourceVertexIndex) && TargetShape.IsVertex(Correspondence.TargetVertexIndex))
		{
			// Need to convert to welded ids
			const int32 SourceVertexId = Impl->bIsWelded ? Impl->SourceToWelded[Correspondence.SourceVertexIndex] : Correspondence.SourceVertexIndex;
			const int32 SourceIndex = ToIndex[SourceVertexId];
			CorrespondencesAssembler.AddEntryFunc(SourceIndex, SourceIndex, CorrespondenceStiffness);
			const FVector3d& TargetPos = TargetShape.GetVertexRef(Correspondence.TargetVertexIndex);
			CorrespondenceRHS(SourceIndex, 0) = CorrespondenceStiffness * TargetPos[0];
			CorrespondenceRHS(SourceIndex, 1) = CorrespondenceStiffness * TargetPos[1];
			CorrespondenceRHS(SourceIndex, 2) = CorrespondenceStiffness * TargetPos[2];
		}
	}
	FSparseMatrixD CorrespondenceMatrix;
	CorrespondencesAssembler.ExtractResult(CorrespondenceMatrix);


	const EMatrixSolverType MatrixSolverType = EMatrixSolverType::FastestPSD;
	TUniquePtr<IMatrixSolverBase> Solver = ConstructMatrixSolver(MatrixSolverType);

	FDenseMatrixD ProjectedPointsRHS;
	ProjectedPointsRHS.resize(NumVerts, 3);
	double ScoreParameter = InitialProjectionStiffness;
	bool bFailed = false;
	for (int32 OuterIter = 0; OuterIter < MaxNumOuterIterations; ++OuterIter)
	{
		bool bFinished = false;
		for (int32 InnerIter = 0; InnerIter < NumInnerIterations; ++InnerIter)
		{
			FEigenSparseMatrixAssembler ProjectionScoreAssembler(NumVerts, NumVerts);
			ProjectionScoreAssembler.ReserveEntriesFunc(NumVerts);

			FDenseMatrixD RHS = CorrespondenceRHS;

			double MaxProjectionResidualSq = 0.;

			// Update projected points and projection score
			for (int32 Vid : WrappedOrWelded.VertexIndicesItr())
			{
				const FVector3d& Xi = WrappedOrWelded.GetVertexRef(Vid);
				const int32 Index = ToIndex[Vid];
				const FVector3d& Pi = TargetAABBTree.FindNearestPoint(Xi);
				const double ProjectionResidualSq = FVector3d::DistSquared(Pi, Xi);
				const double ProjectionScore = ScoreParameter / (1. + ScoreParameter * ProjectionResidualSq);
				MaxProjectionResidualSq = FMath::Max(MaxProjectionResidualSq, ProjectionResidualSq);

				ProjectionScoreAssembler.AddEntryFunc(Index, Index, ProjectionScore);
				RHS(Index, 0) += ProjectionScore * Pi[0];
				RHS(Index, 1) += ProjectionScore * Pi[1];
				RHS(Index, 2) += ProjectionScore * Pi[2];
			}

			if (MaxProjectionResidualSq < FMath::Square(ProjectionTolerance))
			{
				bFinished = true;
				break;
			}


			FSparseMatrixD ProjectionScoreMatrix;
			ProjectionScoreAssembler.ExtractResult(ProjectionScoreMatrix);

			FSparseMatrixD SystemMatrix = LaplacianStiffness * Impl->SourceLaplacian + ProjectionScoreMatrix + CorrespondenceMatrix;

			constexpr bool bIsSymmetric = true;
			Solver->SetUp(SystemMatrix, bIsSymmetric);

			FColumnVectorD ResultVector[3];
			for (int32 Index3 = 0; Index3 < 3; ++Index3)
			{
				Solver->Solve(RHS.col(Index3), ResultVector[Index3]);
				if (!ensure(Solver->bSucceeded()))
				{
					bFailed = true;
					break;
				}
			}

			if (bFailed)
			{
				break;
			}

			for (int32 Vid : WrappedOrWelded.VertexIndicesItr())
			{
				const int32 Index = ToIndex[Vid];
				WrappedOrWelded.SetVertex(Vid, FVector3d(ResultVector[0][Index], ResultVector[1][Index], ResultVector[2][Index]));
			}
		}
		if (bFailed || bFinished)
		{
			break;
		}
		// Ramp up ScoreParameter
		ScoreParameter *= ProjectionStiffnessMuliplier;
	}

	if (Impl->bIsWelded)
	{
		for (const int32 VertexId : WrappedMesh.VertexIndicesItr())
		{
			WrappedMesh.SetVertex(VertexId, WeldedWrappedMesh.GetVertex(Impl->SourceToWelded[VertexId]));
		}
	}
}