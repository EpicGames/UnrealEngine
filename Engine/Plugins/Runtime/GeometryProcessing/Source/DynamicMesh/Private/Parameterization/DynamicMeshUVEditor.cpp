// Copyright Epic Games, Inc. All Rights Reserved.


#include "Parameterization/DynamicMeshUVEditor.h"
#include "DynamicSubmesh3.h"

#include "DynamicMesh/MeshNormals.h"
#include "MeshBoundaryLoops.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "MeshQueries.h"
#include "MeshWeights.h"
#include "CompGeom/ConvexHull2.h"

#include "Parameterization/MeshDijkstra.h"
#include "Parameterization/MeshLocalParam.h"
#include "Parameterization/MeshUVPacking.h"
#include "Solvers/MeshParameterizationSolvers.h"
#include "Parameterization/MeshRegionGraph.h"
#include "DynamicMeshEditor.h"

#include "Async/ParallelFor.h"

using namespace UE::Geometry;

namespace FDynamicMeshUVEditorLocals
{
	enum { MAX_TEXCOORDS = 4, MAX_STATIC_TEXCOORDS = 8 };
}

FDynamicMeshUVEditor::FDynamicMeshUVEditor(FDynamicMesh3* MeshIn, int32 UVLayerIndexIn, bool bCreateIfMissing)
{
	Mesh = MeshIn;
	UVOverlayIndex = UVLayerIndexIn;

	if (Mesh->HasAttributes() && Mesh->Attributes()->NumUVLayers() > UVOverlayIndex)
	{
		UVOverlay = Mesh->Attributes()->GetUVLayer(UVOverlayIndex);
	}

	if (UVOverlay == nullptr && bCreateIfMissing)
	{
		CreateUVLayer(UVOverlayIndex);
		UVOverlay = Mesh->Attributes()->GetUVLayer(UVOverlayIndex);
		check(UVOverlay);
	}
}


FDynamicMeshUVEditor::FDynamicMeshUVEditor(FDynamicMesh3* MeshIn, FDynamicMeshUVOverlay* UVOverlayIn)
{
	Mesh = MeshIn;
	UVOverlay = UVOverlayIn;
	UVOverlayIndex = -1;

	if (Mesh->HasAttributes())
	{
		for (int32 UVIndex = 0; UVIndex < Mesh->Attributes()->NumUVLayers(); ++UVIndex) {
			if (Mesh->Attributes()->GetUVLayer(UVIndex) == UVOverlay)
			{
				UVOverlayIndex = UVIndex;
				break;
			}
		}
	}
	check(UVOverlayIndex != -1);
	ensure(UVOverlay->GetParentMesh() == Mesh);
}


void FDynamicMeshUVEditor::CreateUVLayer(int32 LayerIndex)
{
	check(LayerIndex < FDynamicMeshUVEditorLocals::MAX_STATIC_TEXCOORDS)

	if (Mesh->HasAttributes() == false)
	{
		Mesh->EnableAttributes();
	}

	if (Mesh->Attributes()->NumUVLayers() <= LayerIndex)
	{
		Mesh->Attributes()->SetNumUVLayers(LayerIndex+1);
	}
}


int32 FDynamicMeshUVEditor::AddUVLayer()
{
	int32 OldUVOverlayIndex = UVOverlayIndex;
	int32 TotalLayerCount = Mesh->Attributes()->NumUVLayers();
	if (TotalLayerCount < FDynamicMeshUVEditorLocals::MAX_STATIC_TEXCOORDS) {
		CreateUVLayer(TotalLayerCount); // The passed in value is an index, not count
		SwitchActiveLayer(TotalLayerCount);
		SetPerTriangleUVs(0.0, nullptr);
		SwitchActiveLayer(OldUVOverlayIndex);
		return TotalLayerCount;
	}
	else
	{
		return -1;
	}
}

void FDynamicMeshUVEditor::SwitchActiveLayer(int32 UVOverlayIndexIn)
{
	UVOverlay = Mesh->Attributes()->GetUVLayer(UVOverlayIndexIn);
	UVOverlayIndex = UVOverlayIndexIn;
	check(UVOverlay);
}

int32 FDynamicMeshUVEditor::RemoveUVLayer()
{
	int32 TotalLayerCount = Mesh->Attributes()->NumUVLayers();
	if (TotalLayerCount == 1)
	{
		return 0; // Don't remove the last layer, if there's only one.
	}

	for (int32 LayerID = UVOverlayIndex + 1; LayerID < TotalLayerCount; ++LayerID)
	{
		FDynamicMeshUVOverlay* SourceOverlay = Mesh->Attributes()->GetUVLayer(LayerID);
		UVOverlay = Mesh->Attributes()->GetUVLayer(LayerID - 1);
		CopyUVLayer(SourceOverlay);
	}

	Mesh->Attributes()->SetNumUVLayers(TotalLayerCount-1);	
	int32 NewLayerIndex = UVOverlayIndex - 1 < 0 ? 0 : UVOverlayIndex - 1;
	SwitchActiveLayer(NewLayerIndex);

	return NewLayerIndex;
}

void FDynamicMeshUVEditor::ResetUVs()
{
	if (ensure(UVOverlay))
	{
		UVOverlay->ClearElements();
	}
}

void FDynamicMeshUVEditor::ResetUVs(const TArray<int32>& Triangles)
{
	if (ensure(UVOverlay))
	{
		UVOverlay->ClearElements(Triangles);
	}
}


bool FDynamicMeshUVEditor::CopyUVLayer(FDynamicMeshUVOverlay* FromUVOverlay)
{
	if (FromUVOverlay == nullptr || FromUVOverlay == UVOverlay)
	{
		return false;
	}

	TArray<int32> ElementIDMap;
	ElementIDMap.SetNum(FromUVOverlay->MaxElementID());

	UVOverlay->ClearElements();
	for (int32 ElementID : FromUVOverlay->ElementIndicesItr())
	{
		FVector2f UV = FromUVOverlay->GetElement(ElementID);
		int32 NewID = UVOverlay->AppendElement(UV);
		ElementIDMap[ElementID] = NewID;
	}

	for (int32 TriangleID : Mesh->TriangleIndicesItr())
	{
		if (FromUVOverlay->IsSetTriangle(TriangleID))
		{
			FIndex3i UVTriangle = FromUVOverlay->GetTriangle(TriangleID);
			UVTriangle.A = ElementIDMap[UVTriangle.A];
			UVTriangle.B = ElementIDMap[UVTriangle.B];
			UVTriangle.C = ElementIDMap[UVTriangle.C];
			UVOverlay->SetTriangle(TriangleID, UVTriangle);
		}
	}

	return true;
}

void FDynamicMeshUVEditor::TransformUVElements(const TArray<int32>& ElementIDs, TFunctionRef<FVector2f(const FVector2f&)> TransformFunc)
{
	for (int32 elemid : ElementIDs)
	{
		if (UVOverlay->IsElement(elemid))
		{
			FVector2f UV = UVOverlay->GetElement(elemid);
			UV = TransformFunc(UV);
			UVOverlay->SetElement(elemid, UV);
		}
	}
}


void FDynamicMeshUVEditor::SetToPerVertexUVs(TArray<int32>& VertexToUVOut, bool& bIsIdentityMapOut, FUVEditResult* Result)
{
	bIsIdentityMapOut = true;
	VertexToUVOut.Init(IndexConstants::InvalidID, Mesh->MaxVertexID());

	UVOverlay->ClearElements();
	for (int32 VertexID : Mesh->VertexIndicesItr())
	{
		int32 UVID = UVOverlay->AppendElement(FVector2f::Zero());
		VertexToUVOut[VertexID] = UVID;
		bIsIdentityMapOut &= (UVID == VertexID);

		if (Result)
		{
			Result->NewUVElements.Add(UVID);
		}
	}

	for (int32 TriangleID : Mesh->TriangleIndicesItr())
	{
		FIndex3i Tri = Mesh->GetTriangle(TriangleID);
		Tri.A = VertexToUVOut[Tri.A];
		Tri.B = VertexToUVOut[Tri.B];
		Tri.C = VertexToUVOut[Tri.C];
		UVOverlay->SetTriangle(TriangleID, Tri);
	}

}


template<typename EnumerableType>
void InternalSetPerTriangleUVs(EnumerableType TriangleIDs, const FDynamicMesh3* Mesh, FDynamicMeshUVOverlay* UVOverlay, double ScaleFactor, FUVEditResult* Result)
{
	TMap<int32, int32> BaseToOverlayVIDMap;
	TArray<int32> NewUVIndices;

	UVOverlay->ClearElements(TriangleIDs);
	
	for (int32 TriangleID : TriangleIDs)
	{
		FIndex3i MeshTri = Mesh->GetTriangle(TriangleID);
		FFrame3d TriProjFrame = Mesh->GetTriFrame(TriangleID, 0);

		FIndex3i ElemTri;
		for (int32 j = 0; j < 3; ++j)
		{
			FVector2d UV = TriProjFrame.ToPlaneUV(Mesh->GetVertex(MeshTri[j]), 2);
			UV *= ScaleFactor;
			ElemTri[j] = UVOverlay->AppendElement((FVector2f)UV);
			NewUVIndices.Add(ElemTri[j]);
		}
		UVOverlay->SetTriangle(TriangleID, ElemTri);
	}

	if (Result != nullptr)
	{
		Result->NewUVElements = MoveTemp(NewUVIndices);
	}
}


void FDynamicMeshUVEditor::SetPerTriangleUVs(const TArray<int32>& Triangles, double ScaleFactor, FUVEditResult* Result)
{
	if (ensure(UVOverlay) == false) return;
	if (!Triangles.Num()) return;

	InternalSetPerTriangleUVs(Triangles, Mesh, UVOverlay, ScaleFactor, Result);
}


void FDynamicMeshUVEditor::SetPerTriangleUVs(double ScaleFactor, FUVEditResult* Result)
{
	if (ensure(UVOverlay) == false) return;
	if (Mesh->TriangleCount() <= 0) return;

	InternalSetPerTriangleUVs(Mesh->TriangleIndicesItr(), Mesh, UVOverlay, ScaleFactor, Result);
}

void FDynamicMeshUVEditor::SetTriangleUVsFromProjection(const TArray<int32>& Triangles, const FFrame3d& ProjectionFrame, FUVEditResult* Result)
{
	SetTriangleUVsFromPlanarProjection(Triangles, [](const FVector3d& P) { return P; },  
		ProjectionFrame, FVector2d::One(), Result);
}

void FDynamicMeshUVEditor::SetTriangleUVsFromPlanarProjection(
	const TArray<int32>& Triangles, 
	TFunctionRef<FVector3d(const FVector3d&)> PointTransform,
	const FFrame3d& ProjectionFrame, 
	const FVector2d& Dimensions, 
	FUVEditResult* Result)
{
	if (ensure(UVOverlay) == false) return;
	if (!Triangles.Num()) return;

	ResetUVs(Triangles);

	double ScaleX = (FMathd::Abs(Dimensions.X) > FMathf::ZeroTolerance) ? (1.0 / Dimensions.X) : 1.0;
	double ScaleY = (FMathd::Abs(Dimensions.Y) > FMathf::ZeroTolerance) ? (1.0 / Dimensions.Y) : 1.0;

	TMap<int32, int32> BaseToOverlayVIDMap;
	TArray<int32> NewUVIndices;

	for (int32 TID : Triangles)
	{
		FIndex3i BaseTri = Mesh->GetTriangle(TID);
		FIndex3i ElemTri;
		for (int32 j = 0; j < 3; ++j)
		{
			const int32* FoundElementID = BaseToOverlayVIDMap.Find(BaseTri[j]);
			if (FoundElementID == nullptr)
			{
				FVector3d Pos = Mesh->GetVertex(BaseTri[j]);
				FVector3d TransformPos = PointTransform(Pos);
				FVector2d UV = ProjectionFrame.ToPlaneUV(TransformPos, 2);
				UV.X *= ScaleX;
				UV.Y *= ScaleY;
				ElemTri[j] = UVOverlay->AppendElement(FVector2f(UV));
				NewUVIndices.Add(ElemTri[j]);
				BaseToOverlayVIDMap.Add(BaseTri[j], ElemTri[j]);
			}
			else
			{
				ElemTri[j] = *FoundElementID;
			}
		}
		UVOverlay->SetTriangle(TID, ElemTri);
	}

	if (Result != nullptr)
	{
		Result->NewUVElements = MoveTemp(NewUVIndices);
	}
}

void FDynamicMeshUVEditor::TransferTriangleUVsFromMeshViaDirectionProjection(TArrayView<const int32> Triangles, TFunctionRef<FVector3d(const FVector3d&)> TransformTargetToSourceSpacePosition,
	const FVector3d& ProjectionDirection, double ProjectionOffset, TFunctionRef<FVector3d(const FVector3d&)> TransformProjectionToSourceSpaceVector,
	const FDynamicMeshAABBTree3& SourceMeshSpatial, int32 SourceMeshUVChannel,
	const FTransferFromMeshViaProjectionSettings& Settings, FUVEditResult* Result)
{
	if (!ensure(UVOverlay) || Triangles.IsEmpty())
	{
		return;
	}

	const FDynamicMesh3* SourceMesh = SourceMeshSpatial.GetMesh();
	check(SourceMesh);
	const FDynamicMeshUVOverlay* SourceUVs = SourceMesh->HasAttributes() ? SourceMesh->Attributes()->GetUVLayer(SourceMeshUVChannel) : nullptr;
	if (!SourceUVs)
	{
		return;
	}

	// test for empty range
	if (Settings.MinDistance >= Settings.MaxDistance)
	{
		return;
	}

	FVector3d SourceProjDirection = TransformProjectionToSourceSpaceVector(ProjectionDirection.GetSafeNormal());
	double ToSourceDistanceScale = SourceProjDirection.Length();
	if (ToSourceDistanceScale < UE_DOUBLE_SMALL_NUMBER)
	{
		return;
	}
	SourceProjDirection /= ToSourceDistanceScale;
	const FVector3d ConstantProjOffset = SourceProjDirection * ToSourceDistanceScale * ProjectionOffset;

	// Depending on the projection range, we may need to project backwards and forwards, and may need to offset the projections
	struct FProjectionInfo
	{
		double Sign, Offset, Max;
	};
	FProjectionInfo ProjInfo[2];
	int32 NumDirs = 0;
	if (Settings.MinDistance < 0 && Settings.MaxDistance > 0)
	{
		NumDirs = 2;
		ProjInfo[0].Sign = -1;
		ProjInfo[1].Sign = 1;
		ProjInfo[0].Offset = 0;
		ProjInfo[1].Offset = 0;
		ProjInfo[0].Max = -Settings.MinDistance * ToSourceDistanceScale;
		ProjInfo[1].Max = Settings.MaxDistance * ToSourceDistanceScale;
	}
	else
	{
		NumDirs = 1;
		if (Settings.MinDistance < 0)
		{
			ProjInfo[0].Sign = -1;
			ProjInfo[0].Offset = -Settings.MaxDistance * ToSourceDistanceScale;
			ProjInfo[0].Max = -Settings.MinDistance * ToSourceDistanceScale;
		}
		else
		{
			ProjInfo[0].Sign = 1;
			ProjInfo[0].Offset = Settings.MinDistance * ToSourceDistanceScale;
			ProjInfo[0].Max = Settings.MaxDistance * ToSourceDistanceScale;
		}
	}

	if (Settings.bResetUVsForUnmatched)
	{
		// Reset up-front if we are resetting for un-matched triangles (otherwise we rely on the below UVOverlay->SetTriangle calls to clear unused elements as needed)
		UVOverlay->ClearElements(Triangles);
	}

	TMap<int32, int32> BaseToOverlayVIDMap;
	TArray<int32> NewUVIndices;

	IMeshSpatial::FQueryOptions QueryOptions;
	QueryOptions.TriangleFilterF = Settings.SourceMeshTriFilter;


	for (int32 TID : Triangles)
	{
		FIndex3i BaseTri = Mesh->GetTriangle(TID);
		FIndex3i ElemTri = FIndex3i::Invalid();
		FVector2f FoundUVs[3];
		int32 NumFound = 0;

		for (int32 j = 0; j < 3; ++j)
		{
			const int32* FoundElementID = BaseToOverlayVIDMap.Find(BaseTri[j]);
			bool bFound = FoundElementID != nullptr;
			if (!bFound)
			{
				int32 BestTID = INDEX_NONE;
				double BestT = FMathd::MaxReal;
				FVector3f BestBary;
				for (int32 ProjectionIdx = 0; ProjectionIdx < NumDirs; ++ProjectionIdx)
				{
					FVector3d Pos = Mesh->GetVertex(BaseTri[j]);
					FProjectionInfo& Proj = ProjInfo[ProjectionIdx];
					FVector3d TransformPosStart = TransformTargetToSourceSpacePosition(Pos) + ConstantProjOffset + SourceProjDirection * Proj.Offset * Proj.Sign;
					FRay3d Ray(TransformPosStart, SourceProjDirection * Proj.Sign);
					double SearchDist = Proj.Max;
					
					double NearestT;
					int32 NearestTID = INDEX_NONE;
					FVector3d BaryCoords;
					QueryOptions.MaxDistance = SearchDist;
					if (SourceMeshSpatial.FindNearestHitTriangle(Ray, NearestT, NearestTID, BaryCoords, QueryOptions))
					{
						if (BestTID == INDEX_NONE || NearestT < BestT)
						{
							BestT = NearestT;
							BestTID = NearestTID;
							BestBary = (FVector3f)BaryCoords;
						}
					}
				}
				if (BestTID != INDEX_NONE && SourceUVs->IsSetTriangle(BestTID))
				{
					SourceUVs->GetTriBaryInterpolate<float>(BestTID, &BestBary.X, &FoundUVs[j].X);
					NumFound++;
				}
			}
			else
			{
				ElemTri[j] = *FoundElementID;
				NumFound += bool(ElemTri[j] >= 0);
			}
		}
		// If we have mapped a UV coordinate for every vertex, then append any new UV elements and set the UV overlay triangle
		if (NumFound == 3)
		{
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				if (ElemTri[SubIdx] == INDEX_NONE)
				{
					ElemTri[SubIdx] = UVOverlay->AppendElement(FoundUVs[SubIdx]);
					NewUVIndices.Add(ElemTri[SubIdx]);
					BaseToOverlayVIDMap.Add(BaseTri[SubIdx], ElemTri[SubIdx]);
				}
			}
			UVOverlay->SetTriangle(TID, ElemTri);
		}
	}

	if (Result != nullptr)
	{
		Result->NewUVElements = MoveTemp(NewUVIndices);
	}

}


bool FDynamicMeshUVEditor::EstimateGeodesicCenterFrameVertex(const FDynamicMesh3& Mesh, FFrame3d& FrameOut, int32& VertexIDOut, bool bAlignToUnitAxes)
{
	VertexIDOut = *Mesh.VertexIndicesItr().begin();
	FVector3d Normal = FMeshNormals::ComputeVertexNormal(Mesh, VertexIDOut);

	FMeshBoundaryLoops LoopsCalc(&Mesh, true);
	if (LoopsCalc.GetLoopCount() == 0)
	{
		FrameOut = Mesh.GetVertexFrame(VertexIDOut, false, &Normal);
		return false;
	}
	const FEdgeLoop& Loop = LoopsCalc[LoopsCalc.GetMaxVerticesLoopIndex()];
	TArray<FVector2d> SeedPoints;
	for (int32 vid : Loop.Vertices)
	{
		SeedPoints.Add(FVector2d(vid, 0.0));
	}

	TMeshDijkstra<FDynamicMesh3> Dijkstra(&Mesh);
	Dijkstra.ComputeToMaxDistance(SeedPoints, TNumericLimits<float>::Max());
	int32 MaxDistVID = Dijkstra.GetMaxGraphDistancePointID();
	if ( ensure(Mesh.IsVertex(MaxDistVID)) == false )
	{
		FrameOut = Mesh.GetVertexFrame(VertexIDOut, false, &Normal);
		return false;
	}
	VertexIDOut = MaxDistVID;
	Normal = FMeshNormals::ComputeVertexNormal(Mesh, MaxDistVID);
	FrameOut = Mesh.GetVertexFrame(MaxDistVID, false, &Normal);

	// try to generate consistent frame alignment...
	if (bAlignToUnitAxes)
	{
		FrameOut.ConstrainedAlignPerpAxes(0, 1, 2, FVector3d::UnitX(), FVector3d::UnitY(), 0.95);
	}

	return true;
}


bool FDynamicMeshUVEditor::EstimateGeodesicCenterFrameVertex(const FDynamicMesh3& Mesh, const TArray<int32>& Triangles, FFrame3d& FrameOut, int32& VertexIDOut, bool bAlignToUnitAxes)
{
	FDynamicSubmesh3 SubmeshCalc(&Mesh, Triangles, (int)EMeshComponents::None, false);
	FDynamicMesh3& Submesh = SubmeshCalc.GetSubmesh();
	FFrame3d SeedFrame;
	int32 FrameVertexID;
	bool bFrameOK = EstimateGeodesicCenterFrameVertex(Submesh, SeedFrame, FrameVertexID, true);
	if (!bFrameOK)
	{
		return false;
	}
	VertexIDOut = SubmeshCalc.MapVertexToBaseMesh(FrameVertexID);
	FrameOut = SeedFrame;
	return true;
}


bool FDynamicMeshUVEditor::SetTriangleUVsFromExpMap(const TArray<int32>& Triangles, const FExpMapOptions& Options, FUVEditResult* Result)
{
	if (ensure(UVOverlay) == false) return false;
	if (Triangles.Num() == 0) return false;

	ResetUVs(Triangles);

	FDynamicSubmesh3 SubmeshCalc(Mesh, Triangles, (int)EMeshComponents::None, false);
	FDynamicMesh3& Submesh = SubmeshCalc.GetSubmesh();
	FMeshNormals::QuickComputeVertexNormals(Submesh);

	if (Options.NormalSmoothingRounds > 0)
	{
		FMeshNormals::SmoothVertexNormals(Submesh, Options.NormalSmoothingRounds, Options.NormalSmoothingAlpha);
	}

	FFrame3d SeedFrame;
	int32 FrameVertexID = FDynamicMesh3::InvalidID;
	bool bFrameOK = EstimateGeodesicCenterFrameVertex(Submesh, SeedFrame, FrameVertexID, true);
	if (!Submesh.IsVertex(FrameVertexID))
	{
		return false;
	}

	TMeshLocalParam<FDynamicMesh3> Param(&Submesh);
	Param.ParamMode = ELocalParamTypes::ExponentialMapUpwindAvg;
	Param.ComputeToMaxDistance(FrameVertexID, SeedFrame, TNumericLimits<float>::Max());

	TArray<int32> VtxElementIDs;
	TArray<int32> NewElementIDs;
	VtxElementIDs.Init(IndexConstants::InvalidID, Submesh.MaxVertexID());
	for (int32 vid : Submesh.VertexIndicesItr())
	{
		if (Param.HasUV(vid))
		{
			// This is super suspicious. GetUV returns TNumericLimits<double> as an invalid UV, which then gets
			// cast down here as a float, losing invalids entirely. Under precise float math this causes a compilation
			// error, which goes away if you add an intermediary because it no longer gets fully inlined causing the double to get
			// placed into the float.
			// .. separately, not sure about this in the vector header where it just willy nilly casts around. Should probably have this
			// do a saturation on the input ?
			/*
				// Conversion from other type.
				template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
				explicit TVector2(const TVector2<FArg>& From)
				: TVector2<T>((T)From.X, (T)From.Y)
				{
				}
			*/
			FVector2d UVd = Param.GetUV(vid);
			FVector2f UV;

			// Stil doesn't handle when we're at _min_, but at least this doesn't cause a compilation error.
			double MaxFloat = TNumericLimits<float>::Max();
			UV.X = static_cast<float>(UVd.X > MaxFloat ? MaxFloat : UVd.X);
			UV.Y = static_cast<float>(UVd.Y > MaxFloat ? MaxFloat : UVd.Y);

			VtxElementIDs[vid] = UVOverlay->AppendElement(UV);
			NewElementIDs.Add(VtxElementIDs[vid]);
		}
	}

	int32 NumFailed = 0;
	for (int32 tid : Submesh.TriangleIndicesItr())
	{
		FIndex3i SubTri = Submesh.GetTriangle(tid);
		FIndex3i UVTri = { VtxElementIDs[SubTri.A], VtxElementIDs[SubTri.B], VtxElementIDs[SubTri.C] };
		if (UVTri.A == IndexConstants::InvalidID || UVTri.B == IndexConstants::InvalidID || UVTri.C == IndexConstants::InvalidID)
		{
			NumFailed++;
			continue;
		}

		int32 BaseTID = SubmeshCalc.MapTriangleToBaseMesh(tid);
		UVOverlay->SetTriangle(BaseTID, UVTri);
	}

	if (Result != nullptr)
	{
		Result->NewUVElements = MoveTemp(NewElementIDs);
	}

	// If we started from a bad frame, always report this as a failure because the quality will be very bad
	// Otherwise report failure if some of the triangle UVs were not set
	return bFrameOK && (NumFailed == 0);
}



bool FDynamicMeshUVEditor::SetTriangleUVsFromExpMap(
	const TArray<int32>& Triangles,
	TFunctionRef<FVector3d(const FVector3d&)> PointTransform,
	const FFrame3d& ProjectionFrame,
	const FVector2d& Dimensions,
	int32 NormalSmoothingRounds,
	double NormalSmoothingAlpha,
	double FrameNormalBlendWeight,
	FUVEditResult* Result)
{
	if (ensure(UVOverlay) == false) return false;
	if (Triangles.Num() == 0) return false;

	ResetUVs(Triangles);

	double ScaleX = (FMathd::Abs(Dimensions.X) > FMathf::ZeroTolerance) ? (1.0 / Dimensions.X) : 1.0;
	double ScaleY = (FMathd::Abs(Dimensions.Y) > FMathf::ZeroTolerance) ? (1.0 / Dimensions.Y) : 1.0;

	FDynamicSubmesh3 SubmeshCalc(Mesh, Triangles, (int)EMeshComponents::None, false);
	FDynamicMesh3& Submesh = SubmeshCalc.GetSubmesh();
	MeshTransforms::ApplyTransform(Submesh, PointTransform, [&](const FVector3f& V) { return V;});
	FMeshNormals::QuickComputeVertexNormals(Submesh);

	FMeshNormals::SmoothVertexNormals(Submesh, NormalSmoothingRounds, NormalSmoothingAlpha);

	FDynamicMeshAABBTree3 Spatial(&Submesh, true);
	double NearDistSqr;
	int32 SeedTID = Spatial.FindNearestTriangle(ProjectionFrame.Origin, NearDistSqr);
	FDistPoint3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleDistance(Submesh, SeedTID, ProjectionFrame.Origin);
	FIndex3i TriVerts = Submesh.GetTriangle(SeedTID);

	FFrame3d ParamSeedFrame = ProjectionFrame;
	ParamSeedFrame.Origin = Query.ClosestTrianglePoint;
	// correct for inverted frame
	if (ParamSeedFrame.Z().Dot(Submesh.GetTriNormal(SeedTID)) < 0)
	{
		ParamSeedFrame.Rotate(FQuaterniond(ParamSeedFrame.X(), 180.0, true));
	}

	// apply normal blending
	FrameNormalBlendWeight = FMathd::Clamp(FrameNormalBlendWeight, 0, 1);
	if (FrameNormalBlendWeight > 0)
	{
		FVector3d FrameZ = ParamSeedFrame.Z();
		for (int32 vid : Submesh.VertexIndicesItr())
		{
			FVector3d N = (FVector3d)Submesh.GetVertexNormal(vid);
			N = Lerp(N, FrameZ, FrameNormalBlendWeight);
			Submesh.SetVertexNormal(vid, (FVector3f)N);
		}
	}

	TMeshLocalParam<FDynamicMesh3> Param(&Submesh);
	Param.ParamMode = ELocalParamTypes::ExponentialMapUpwindAvg;
	Param.ComputeToMaxDistance(ParamSeedFrame, TriVerts, TNumericLimits<float>::Max());

	TArray<int32> VtxElementIDs;
	TArray<int32> NewElementIDs;
	VtxElementIDs.Init(IndexConstants::InvalidID, Submesh.MaxVertexID());
	for (int32 vid : Submesh.VertexIndicesItr())
	{
		if (Param.HasUV(vid))
		{
			FVector2d UV = Param.GetUV(vid);
			UV.X *= ScaleX;
			UV.Y *= ScaleY;
			VtxElementIDs[vid] = UVOverlay->AppendElement(FVector2f(UV));
			NewElementIDs.Add(VtxElementIDs[vid]);
		}
	}

	int32 NumFailed = 0;
	for (int32 tid : Submesh.TriangleIndicesItr())
	{
		FIndex3i SubTri = Submesh.GetTriangle(tid);
		FIndex3i UVTri = { VtxElementIDs[SubTri.A], VtxElementIDs[SubTri.B], VtxElementIDs[SubTri.C] };
		if (UVTri.A == IndexConstants::InvalidID || UVTri.B == IndexConstants::InvalidID || UVTri.C == IndexConstants::InvalidID)
		{
			NumFailed++;
			continue;
		}

		int32 BaseTID = SubmeshCalc.MapTriangleToBaseMesh(tid);
		UVOverlay->SetTriangle(BaseTID, UVTri);
	}

	if (Result != nullptr)
	{
		Result->NewUVElements = MoveTemp(NewElementIDs);
	}

	return (NumFailed == 0);
}




bool FDynamicMeshUVEditor::SetTriangleUVsFromFreeBoundaryConformal(const TArray<int32>& Triangles, FUVEditResult* Result)
{
	return SetTriangleUVsFromFreeBoundaryConformal(Triangles, false,  Result);
}

bool FDynamicMeshUVEditor::SetTriangleUVsFromFreeBoundaryConformal(const TArray<int32>& Triangles, bool bUseExistingUVTopology, FUVEditResult* Result) 
{
	FSetUVsFromConformalOptions Options;
	Options.bUseExistingUVTopology = bUseExistingUVTopology;
	Options.bUseSpectral = false;
	Options.bPreserveIrregularity = false;
	return SetTriangleUVsFromConformal(Triangles, Options, Result);
}

bool UE::Geometry::FDynamicMeshUVEditor::SetTriangleUVsFromFreeBoundaryConformal(const TArray<int32>& Triangles, 
	const TSet<int32>& PinnedElementIDs, FUVEditResult* Result)
{
	FSetUVsFromConformalOptions Options;
	Options.bUseExistingUVTopology = true;
	Options.bUseSpectral = false;
	Options.bPreserveIrregularity = false;
	Options.PinnedElementIDs = &PinnedElementIDs;
	return SetTriangleUVsFromConformal(Triangles, Options, Result);
}

bool FDynamicMeshUVEditor::SetTriangleUVsFromFreeBoundarySpectralConformal(const TArray<int32>& Triangles, bool bUseExistingUVTopology, bool bPreserveIrregularity, FUVEditResult* Result) 
{
	FSetUVsFromConformalOptions Options;
	Options.bUseExistingUVTopology = bUseExistingUVTopology;
	Options.bUseSpectral = true;
	Options.bPreserveIrregularity = bPreserveIrregularity;
	return SetTriangleUVsFromConformal(Triangles, Options, Result);
}

bool FDynamicMeshUVEditor::SetTriangleUVsFromConformal(const TArray<int32>& Triangles, const FSetUVsFromConformalOptions& Options, FUVEditResult* Result)
{
	bool bUseExistingUVTopology = Options.bUseExistingUVTopology;

	if (ensure(UVOverlay) == false) return false;
	if (Triangles.Num() == 0) return false;

	if (!bUseExistingUVTopology)
	{
		ResetUVs(Triangles);
	}

	FDynamicMesh3 Submesh(EMeshComponents::None);
	TMap<int32, int32> BaseToSubmeshV;
	TArray<int32> SubmeshToBaseV;
	TArray<int32> SubmeshToBaseT;

	for (int32 tid : Triangles)
	{
		if (bUseExistingUVTopology && !UVOverlay->IsSetTriangle(tid))
		{
			continue;
		}
		FIndex3i Triangle = (bUseExistingUVTopology) ? UVOverlay->GetTriangle(tid) : Mesh->GetTriangle(tid);
		FIndex3i NewTriangle;
		for (int32 j = 0; j < 3; ++j)
		{
			const int32* FoundIdx = BaseToSubmeshV.Find(Triangle[j]);
			if (FoundIdx)
			{
				NewTriangle[j] = *FoundIdx;
			}
			else
			{
				FVector3d Position = Mesh->GetVertex((bUseExistingUVTopology) ? UVOverlay->GetParentVertex(Triangle[j]) : Triangle[j]);
				int32 NewVtxID = Submesh.AppendVertex(Position);
				check(NewVtxID == SubmeshToBaseV.Num());
				SubmeshToBaseV.Add(Triangle[j]);
				BaseToSubmeshV.Add(Triangle[j], NewVtxID);
				NewTriangle[j] = NewVtxID;
			}
		}

		int32 NewTriID = Submesh.AppendTriangle(NewTriangle);
		check(NewTriID == SubmeshToBaseT.Num());
		SubmeshToBaseT.Add(tid);
	}

	// is there a quick check we can do to ensure that we have a single connected component?
	
	TUniquePtr<UE::Solvers::IConstrainedMeshUVSolver> Solver = nullptr;

	FMeshBoundaryLoops Loops(&Submesh, true);
	int32 LongestLoopIndex = Loops.GetLongestLoopIndex();
	if (LongestLoopIndex == INDEX_NONE)
	{ 
		return false; 
	}
	const TArray<int32>& ConstrainLoop = Loops[LongestLoopIndex].Vertices;
	int32 LoopNum = ConstrainLoop.Num();

	// Potentially used in the non-spectral case
	TOptional<TPair<int32, FVector2f>> SinglePinnedElement;

	if (Options.bUseSpectral) 
	{
		Solver = UE::MeshDeformation::ConstructSpectralConformalParamSolver(Submesh, Options.bPreserveIrregularity);
		
		for (int32 Idx = 0; Idx < LoopNum; ++Idx)
		{
			// It doesnt matter what the uv values or weights are, we are only interested in the indicies of the
			// boundary vertices.
			Solver->AddConstraint(ConstrainLoop[Idx], 0.0, FVector2d(0.0, 0.0), false);
		}
	}
	else 
	{
		Solver = UE::MeshDeformation::ConstructNaturalConformalParamSolver(Submesh);

		// There are three options for constraints.
		// 1. No pinned elements: the standard thing is to constrain the two furthest boundary vertices (this is supposed
		//   to be a geodesic distance, but for now we just do euclidean).
		// 2. One pinned element: we can do the same thing as 1, except translate afterward to put the pinned element 
		//   in the desired coordinate.
		// 3. More than one pinned element: constrain all the pinned elements.

		int32 PinnedElementCount = 0;

		if (Options.PinnedElementIDs)
		{
			for (int32 ElementID : *Options.PinnedElementIDs)
			{
				if (!UVOverlay->IsElement(ElementID))
				{
					continue;
				}
				int32 BaseVert = bUseExistingUVTopology ? ElementID : UVOverlay->GetParentVertex(ElementID);
				int32* SubmeshVert = BaseToSubmeshV.Find(BaseVert);
				if (!SubmeshVert)
				{
					continue;
				}

				Solver->AddConstraint(*SubmeshVert, 1.0, FVector2d(UVOverlay->GetElement(ElementID)), false);

				++PinnedElementCount;

				if (!SinglePinnedElement.IsSet())
				{
					// We'll clear this later if we have more than one element
					SinglePinnedElement.Emplace(*SubmeshVert, UVOverlay->GetElement(ElementID));
				}
			}
		}

		if (PinnedElementCount > 1)
		{
			// We don't want to trigger our whole island translation code further below
			SinglePinnedElement.Reset();
		}

		// Pick our constraints if we have fewer than 2.
		if (PinnedElementCount < 2)
		{
			// In case we had one pinned element, we'll constrain it with our own translation afterward
			Solver->ClearConstraints(); 
			
			// Find a pair of vertices to constrain. The standard procedure is to find the two furthest-apart vertices 
			// on the largest boundary loop. 
			FIndex2i MaxDistPair = FIndex2i::Invalid();
			double MaxDistSqr = 0;
			for (int32 Idx = 0; Idx < LoopNum; ++Idx)
			{
				for (int32 NextIdx = Idx + 1; NextIdx < LoopNum; ++NextIdx)
				{
					const double DistSqr = DistanceSquared(Submesh.GetVertex(ConstrainLoop[Idx]), 
														   Submesh.GetVertex(ConstrainLoop[NextIdx]));
					if (DistSqr > MaxDistSqr)
					{
						MaxDistSqr = DistSqr;
						MaxDistPair = FIndex2i(ConstrainLoop[Idx], ConstrainLoop[NextIdx]);
					}
				}
			}

			if (ensure(MaxDistPair != FIndex2i::Invalid()) == false)
			{
				return false;
			}

			// pin those vertices
			Solver->AddConstraint(MaxDistPair.A, 1.0, FVector2d(0.0, 0.5), false);
			Solver->AddConstraint(MaxDistPair.B, 1.0, FVector2d(1.0, 0.5), false);
		}
	}

	// solve for UVs
	TArray<FVector2d> UVBuffer;
	if (Solver->SolveUVs(&Submesh, UVBuffer) == false)
	{
		return false;
	}

	// Handle the single-constrained-element case for the natural conformal solver
	if (SinglePinnedElement.IsSet())
	{
		FVector2d DeltaToApply = FVector2d(SinglePinnedElement->Value) - UVBuffer[SinglePinnedElement->Key];
		if (!DeltaToApply.IsZero())
		{
			int32 NumSubVerts = SubmeshToBaseV.Num();
			for (int32 k = 0; k < NumSubVerts; ++k)
			{
				UVBuffer[k] += DeltaToApply;
			}
		}
	}

	int32 NumFailed = 0;
	if (bUseExistingUVTopology)
	{
		// only need to copy elements
		int32 NumSubVerts = SubmeshToBaseV.Num();
		for (int32 k = 0; k < NumSubVerts; ++k )
		{
			FVector2d NewUV = UVBuffer[k];
			int32 ElemID = SubmeshToBaseV[k];
			UVOverlay->SetElement(ElemID, (FVector2f)NewUV);
		}

		if (Result != nullptr)
		{
			Result->NewUVElements = MoveTemp(SubmeshToBaseV);
		}
	}
	else
	{
		// copy back to target UVOverlay
		TArray<int32> VtxElementIDs;
		TArray<int32> NewElementIDs;
		VtxElementIDs.Init(IndexConstants::InvalidID, Submesh.MaxVertexID());
		for (int32 vid : Submesh.VertexIndicesItr())
		{
			VtxElementIDs[vid] = UVOverlay->AppendElement((FVector2f)UVBuffer[vid]);
			NewElementIDs.Add(VtxElementIDs[vid]);
		}

		// set triangles
		for (int32 tid : Submesh.TriangleIndicesItr())
		{
			FIndex3i SubTri = Submesh.GetTriangle(tid);
			FIndex3i UVTri = { VtxElementIDs[SubTri.A], VtxElementIDs[SubTri.B], VtxElementIDs[SubTri.C] };
			if (ensure(UVTri.A != IndexConstants::InvalidID && UVTri.B != IndexConstants::InvalidID && UVTri.C != IndexConstants::InvalidID) == false)
			{
				NumFailed++;
				continue;
			}
			int32 BaseTID = SubmeshToBaseT[tid];
			UVOverlay->SetTriangle(BaseTID, UVTri);
		}

		if (Result != nullptr)
		{
			Result->NewUVElements = MoveTemp(NewElementIDs);
		}
	}

	return (NumFailed == 0);
}







void UE::Geometry::FDynamicMeshUVEditor::MakeSureUVsAreSet(const TSet<int32>& Triangles, 
	FUVEditResult* Result, TSet<int32>* ChangedTrianglesOut)
{
	if (!ensure(Mesh && UVOverlay))
	{
		return;
	}

	TMap<int32, int32> VidToElement;
	for (int32 Tid : Triangles)
	{
		if (UVOverlay->IsSetTriangle(Tid))
		{
			continue;
		}

		FIndex3i ElementsToSet;
		FIndex3i TriVids = Mesh->GetTriangle(Tid);
		for (int i = 0; i < 3; ++i)
		{
			if (int32* ExistingElement = VidToElement.Find(TriVids[i]))
			{
				ElementsToSet[i] = *ExistingElement;
			}
			else
			{
				int32 Element = UVOverlay->AppendElement(FVector2f::Zero());
				ElementsToSet[i] = Element;
				VidToElement.Add(TriVids[i], Element);

				if (Result)
				{
					Result->NewUVElements.Add(Element);
				}
			}
		}
		UVOverlay->SetTriangle(Tid, ElementsToSet);

		if (ChangedTrianglesOut)
		{
			ChangedTrianglesOut->Add(Tid);
		}
	}
}


bool FDynamicMeshUVEditor::RemoveSeamsAtEdges(const TSet<int32>& EidsToRemoveAsSeams)
{
	return FDynamicMeshEditor::RemoveSeamsAtEdges(EidsToRemoveAsSeams, UVOverlay);
}

bool FDynamicMeshUVEditor::CreateSeamsAtEdges(const TSet<int32>& EidsToMakeIntoSeams, FUVEditResult* Result)
{
	return FDynamicMeshEditor::CreateSeamsAtEdges(EidsToMakeIntoSeams, UVOverlay, Result ? &Result->NewUVElements : nullptr);
}



bool UE::Geometry::FDynamicMeshUVEditor::MakeIsland(const TSet<int32>& TidsToMakeIntoIsland, FUVEditResult* Result, TSet<int32>* ChangedTrianglesOut)
{
	using namespace FDynamicMeshUVEditorLocals;

	if (!ensure(UVOverlay && Mesh))
	{
		return false;
	}
	
	// We may add new elements during either initialization or seam insertion. However, upon welding,
	//  we might end up destroying them. So we'll accumulate them and then filter them out at the end.
	ON_SCOPE_EXIT
	{
		if (Result)
		{
			Result->NewUVElements.RemoveAllSwap([this](int32 Element) { return !UVOverlay->IsElement(Element); });
		}
	};

	// First make sure that all the relevant triangles have UVs set.
	MakeSureUVsAreSet(TidsToMakeIntoIsland, Result, ChangedTrianglesOut);

	// Gather the edges we need to edit
	TSet<int32> EidsToMakeSeams;
	TSet<int32> EidsToJoin;
	TSet<int32> TouchedVids;

	TSet<int32> ProcessedEids;
	for (int32 Tid : TidsToMakeIntoIsland)
	{
		FIndex3i TriEids = Mesh->GetTriEdges(Tid);
		for (int i = 0; i < 3; ++i)
		{
			int32 Eid = TriEids[i];
			bool bAlreadyProcessed = false;
			ProcessedEids.Add(Eid, &bAlreadyProcessed);
			if (bAlreadyProcessed)
			{
				continue;
			}

			FDynamicMesh3::FEdge Edge = Mesh->GetEdge(Eid);
			if (Edge.Tri.B == IndexConstants::InvalidID)
			{
				// Don't need to do anything for edges that are on the mesh boundary
				continue;
			}

			bool bIsCurrentlySeam = UVOverlay->IsSeamEdge(Eid);
			bool bShouldBeSeam = !TidsToMakeIntoIsland.Contains(Edge.Tri.A == Tid ? Edge.Tri.B : Edge.Tri.A);
			
			if (bIsCurrentlySeam != bShouldBeSeam)
			{
				TouchedVids.Add(Edge.Vert.A);
				TouchedVids.Add(Edge.Vert.B);

				if (bShouldBeSeam)
				{
					EidsToMakeSeams.Add(Eid);
				}
				else
				{
					EidsToJoin.Add(Eid);
				}
			}
		}
	}//end gathering edges to edit

	if (EidsToJoin.IsEmpty() && EidsToMakeSeams.IsEmpty())
	{
		// There must not have been anything to change
		return true;
	}

	// We need to do seam insertion first so that we don't move neighboring triangles unnecessarily while welding
	//  seams inside the island. This is minorly inconvenient since we'll end up having to filter newly created
	//  elements after the subsequent join operation, but we have to do that for any newly initialized UVs anyway.
	UE::Geometry::FUVEditResult AddSeamResult;
	bool bSuccess = CreateSeamsAtEdges(EidsToMakeSeams, &AddSeamResult);
	if (Result)
	{
		Result->NewUVElements.Append(AddSeamResult.NewUVElements);
		// These get filtered on exit.
	}

	bSuccess = RemoveSeamsAtEdges(EidsToJoin) && bSuccess;

	if (ChangedTrianglesOut)
	{
		// Some of these didn't actually get changed (if they kept their original element), but this is the easiest
		//  way to make sure we mark anything whose connectivity might have changed. The ideal thing would have been
		//  to make CreateSeamsAtEdges and RemoveSeamsAtEdges output changed tids instead.
		for (int32 Vid : TouchedVids)
		{
			FDynamicMesh3::FLocalIntArray Tids;
			Mesh->GetVtxTriangles(Vid, Tids);
			ChangedTrianglesOut->Append(Tids);
		}
	}
	return bSuccess;
}


void FDynamicMeshUVEditor::SetTriangleUVsFromBoxProjection(
	const TArray<int32>& Triangles, 
	TFunctionRef<FVector3d(const FVector3d&)> PointTransform, 
	const FFrame3d& BoxFrame, 
	const FVector3d& BoxDimensions, 
	int32 MinIslandTriCount,
	FUVEditResult* Result)
{
	if (ensure(UVOverlay) == false) return;
	int32 NumTriangles = Triangles.Num();
	if (!NumTriangles) return;

	ResetUVs(Triangles);

	const int Minor1s[3] = { 1, 0, 0 };
	const int Minor2s[3] = { 2, 2, 1 };
	const int Minor1Flip[3] = { -1, 1, 1 };
	const int Minor2Flip[3] = { -1, -1, 1 };

	auto GetTriNormal = [this, &PointTransform](int32 tid) -> FVector3d
	{
		FVector3d A, B, C;
		Mesh->GetTriVertices(tid, A, B, C);
		return VectorUtil::Normal(PointTransform(A), PointTransform(B), PointTransform(C));
	};

	double ScaleX = (FMathd::Abs(BoxDimensions.X) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.X) : 1.0;
	double ScaleY = (FMathd::Abs(BoxDimensions.Y) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.Y) : 1.0;
	double ScaleZ = (FMathd::Abs(BoxDimensions.Z) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.Z) : 1.0;
	FVector3d Scale(ScaleX, ScaleY, ScaleZ);

	// compute assignments to the available planes based on face normals
	TArray<FVector3d> TriNormals;
	TArray<FIndex2i> TriangleBoxPlaneAssignments;
	TriNormals.SetNum(NumTriangles);
	TriangleBoxPlaneAssignments.SetNum(NumTriangles);
	TArray<int32> IndexMap;
	IndexMap.SetNum(Mesh->MaxTriangleID());
	ParallelFor(NumTriangles, [&](int32 i)
	{
		int32 tid = Triangles[i];
		TriNormals[i] = GetTriNormal(tid);
		FVector3d ScaledNormal = BoxFrame.ToFrameVector(TriNormals[i]);
		ScaledNormal *= Scale;
		FVector3d NAbs(FMathd::Abs(ScaledNormal.X), FMathd::Abs(ScaledNormal.Y), FMathd::Abs(ScaledNormal.Z));
		int MajorAxis = NAbs[0] > NAbs[1] ? (NAbs[0] > NAbs[2] ? 0 : 2) : (NAbs[1] > NAbs[2] ? 1 : 2);
		double MajorAxisSign = FMathd::Sign(ScaledNormal[MajorAxis]);
		int Bucket = (MajorAxisSign > 0) ? (MajorAxis+3) : MajorAxis;
		TriangleBoxPlaneAssignments[i] = FIndex2i(MajorAxis, Bucket);
		IndexMap[tid] = i;
	});


	// Optimize face assignments. Small regions are grouped with larger neighbour regions.
	if (MinIslandTriCount > 1)
	{
		FMeshConnectedComponents Components(Mesh);
		Components.FindConnectedTriangles(Triangles, [&](int32 t1, int32 t2) { return TriangleBoxPlaneAssignments[IndexMap[t1]] == TriangleBoxPlaneAssignments[IndexMap[t2]]; });
		FMeshRegionGraph RegionGraph;
		RegionGraph.BuildFromComponents(*Mesh, Components, [&](int32 ComponentIdx) { int32 tid = Components[ComponentIdx].Indices[0]; return TriangleBoxPlaneAssignments[IndexMap[tid]].A; });
		// todo: similarity measure should probably take normals into account
		bool bMerged = RegionGraph.MergeSmallRegions(MinIslandTriCount-1, 
			[&](int32 A, int32 B) { return RegionGraph.GetRegionTriCount(A) > RegionGraph.GetRegionTriCount(B); });
		bool bSwapped = RegionGraph.OptimizeBorders();
		if (bMerged || bSwapped)
		{
			int32 N = RegionGraph.MaxRegionIndex();
			for (int32 k = 0; k < N; ++k)
			{
				if (RegionGraph.IsRegion(k))
				{
					int32 MajorAxis = RegionGraph.GetExternalID(k);
					const TArray<int32>& Tris = RegionGraph.GetRegionTris(k);
					for (int32 tid : Tris)
					{
						int32 i = IndexMap[tid];
						FVector3d ScaledNormal = BoxFrame.ToFrameVector(TriNormals[i]) * Scale;
						double MajorAxisSign = FMathd::Sign(ScaledNormal[MajorAxis]);
						int Bucket = (MajorAxisSign > 0) ? (MajorAxis + 3) : MajorAxis;
						TriangleBoxPlaneAssignments[i] = FIndex2i(MajorAxis, Bucket);
					}
				}
			}
		}
	}


	auto ProjAxis = [](const FVector3d& P, int Axis1, int Axis2, float Axis1Scale, float Axis2Scale)
	{
		return FVector2f(float(P[Axis1]) * Axis1Scale, float(P[Axis2]) * Axis2Scale);
	};

	TMap<FIndex2i, int32> BaseToOverlayVIDMap;
	TArray<int32> NewUVIndices;

	for ( int32 i = 0; i < NumTriangles; ++i )
	{
		int32 tid = Triangles[i];
		FIndex3i BaseTri = Mesh->GetTriangle(tid);
		FIndex2i TriBoxInfo = TriangleBoxPlaneAssignments[i];
		FVector3d N = BoxFrame.ToFrameVector(TriNormals[i]);

		int MajorAxis = TriBoxInfo.A;
		int Bucket = TriBoxInfo.B;
		int MajorAxisSign =  (N[MajorAxis] > 0.0) ? 1 : ( (N[MajorAxis] < 0.0) ? -1 : 0 );
		
		FMathd::Sign(N[MajorAxis]);
		int Minor1 = Minor1s[MajorAxis];
		int Minor2 = Minor2s[MajorAxis];

		FIndex3i ElemTri;
		for (int32 j = 0; j < 3; ++j)
		{
			FIndex2i ElementKey(BaseTri[j], Bucket);
			const int32* FoundElementID = BaseToOverlayVIDMap.Find(ElementKey);
			if (FoundElementID == nullptr)
			{
				FVector3d Pos = Mesh->GetVertex(BaseTri[j]);
				FVector3d TransformPos = PointTransform(Pos);
				FVector3d BoxPos = BoxFrame.ToFramePoint(TransformPos);
				BoxPos *= Scale;

				FVector2f UV = ProjAxis(BoxPos, Minor1, Minor2, float(MajorAxisSign * Minor1Flip[MajorAxis] ), (float)Minor2Flip[MajorAxis]);

				ElemTri[j] = UVOverlay->AppendElement(UV);
				NewUVIndices.Add(ElemTri[j]);
				BaseToOverlayVIDMap.Add(ElementKey, ElemTri[j]);
			}
			else
			{
				ElemTri[j] = *FoundElementID;
			}
		}
		UVOverlay->SetTriangle(tid, ElemTri);
	}

	// Above process can introduce bowties, so we split any bowties on new element IDs
	SplitBowtiesOnUVElements(NewUVIndices, true);

	if (Result != nullptr)
	{
		Result->NewUVElements = MoveTemp(NewUVIndices);
	}
}



void FDynamicMeshUVEditor::SplitBowtiesOnUVElements(TArray<int32>& UVElementIDs, bool bAddNewElementsToInputArray)
{
	if (!ensure(UVOverlay)) return;

	const int32 InitialNumElements = UVElementIDs.Num();
	for (int32 Idx = 0; Idx < InitialNumElements; ++Idx)
	{
		int32 ParentVID = UVOverlay->GetParentVertex(UVElementIDs[Idx]);
		if (UVOverlay->IsBowtieInOverlay(ParentVID))
		{
			UVOverlay->SplitBowtiesAtVertex(ParentVID, bAddNewElementsToInputArray ? &UVElementIDs : nullptr);
		}
	}
}


void FDynamicMeshUVEditor::SetTriangleUVsFromCylinderProjection(
	const TArray<int32>& Triangles, 
	TFunctionRef<FVector3d(const FVector3d&)> PointTransform, 
	const FFrame3d& BoxFrame, 
	const FVector3d& BoxDimensions, 
	float CylinderSplitAngle,
	FUVEditResult* Result)
{
	if (ensure(UVOverlay) == false) return;
	int32 NumTriangles = Triangles.Num();
	if (!NumTriangles) return;

	ResetUVs(Triangles);

	const int Minor1s[3] = { 1, 0, 0 };
	const int Minor2s[3] = { 2, 2, 1 };
	const int Minor1Flip[3] = { -1, 1, 1 };
	const int Minor2Flip[3] = { -1, -1, 1 };

	auto GetTriNormalCentroid = [this, &PointTransform](int32 tid) -> TPair<FVector3d,FVector3d>
	{
		FVector3d A, B, C;
		Mesh->GetTriVertices(tid, A, B, C);
		A = PointTransform(A); B = PointTransform(B); C = PointTransform(C);
		return TPair<FVector3d, FVector3d>{ VectorUtil::Normal(A, B, C), (A + B + C) / 3.0};
	};

	double ScaleX = (FMathd::Abs(BoxDimensions.X) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.X) : 1.0;
	double ScaleY = (FMathd::Abs(BoxDimensions.Y) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.Y) : 1.0;
	double ScaleZ = (FMathd::Abs(BoxDimensions.Z) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.Z) : 1.0;
	FVector3d Scale(ScaleX, ScaleY, ScaleZ);

	double DotThresholdRejectFromPlane = FMathd::Cos(CylinderSplitAngle * FMathf::DegToRad);

	// sort triangles into buckets based on normal. 1/0 is +/-Z, and 3/4 is negative/positive angle around the cylinder,
	// where angles range from [-180,180]. Currently we split at 0 so the 3=[-180,0] and 4=[0,180] spans get their own UV islands
	TArray<FVector3d> TriNormals;
	TArray<FIndex2i> TriangleCylinderAssignments;
	TriNormals.SetNum(NumTriangles);
	TriangleCylinderAssignments.SetNum(NumTriangles);
	ParallelFor(NumTriangles, [&](int32 i)
	{
		int32 tid = Triangles[i];
		TPair<FVector3d, FVector3d> NormalCentroid = GetTriNormalCentroid(tid);
		TriNormals[i] = NormalCentroid.Key;
		FVector3d N = BoxFrame.ToFrameVector(TriNormals[i]);
		N = Normalized(N * Scale);

		if (FMathd::Abs(N.Z) > DotThresholdRejectFromPlane)
		{
			int MajorAxis = 2;
			double MajorAxisSign = FMathd::Sign(N[MajorAxis]);
			// project to +/- Z
			int Bucket = MajorAxisSign > 0 ? 1 : 0;

			TriangleCylinderAssignments[i] = FIndex2i(MajorAxis, Bucket);
		}
		else
		{
			FVector3d Centroid = BoxFrame.ToFramePoint(NormalCentroid.Value);
			double CentroidAngle = FMathd::Atan2(Centroid.Y, Centroid.X);
			int Bucket = (CentroidAngle < 0) ? 3 : 4;
			TriangleCylinderAssignments[i] = FIndex2i(-1, Bucket);
		}
	});

	auto ProjAxis = [](const FVector3d& P, int Axis1, int Axis2, float Axis1Scale, float Axis2Scale)
	{
		return FVector2f(float(P[Axis1]) * Axis1Scale, float(P[Axis2]) * Axis2Scale);
	};

	TMap<FIndex2i, int32> BaseToOverlayVIDMap;
	TArray<int32> NewUVIndices;

	for ( int32 i = 0; i < NumTriangles; ++i )
	{
		int32 tid = Triangles[i];
		FIndex3i BaseTri = Mesh->GetTriangle(tid);
		FIndex2i TriBoxInfo = TriangleCylinderAssignments[i];
		FVector3d N = BoxFrame.ToFrameVector(TriNormals[i]);

		int MajorAxis = TriBoxInfo.A;
		int Bucket = TriBoxInfo.B;

		FIndex3i ElemTri;
		for (int32 j = 0; j < 3; ++j)
		{
			FIndex2i ElementKey(BaseTri[j], Bucket);
			const int32* FoundElementID = BaseToOverlayVIDMap.Find(ElementKey);
			if (FoundElementID == nullptr)
			{
				FVector3d TransPos = PointTransform(Mesh->GetVertex(BaseTri[j]));
				FVector3d BoxPos = Scale * BoxFrame.ToFramePoint(TransPos);

				FVector2f UV = FVector2f::Zero();
				if (Bucket <= 2)
				{
					int MajorAxisSign =  (N[MajorAxis] > 0.0) ? 1 : ( (N[MajorAxis] < 0.0) ? -1 : 0 );
					UV = ProjAxis(BoxPos, 0, 1, float( MajorAxisSign * Minor1Flip[MajorAxis] ), (float)Minor2Flip[MajorAxis]);
				}
				else
				{
					double VAngle = FMathd::Atan2(BoxPos.Y, BoxPos.X);
					if (Bucket == 4 && VAngle < -FMathd::HalfPi)				// 4 = [0, 180]
					{
						VAngle += FMathd::TwoPi;
					}
					else if (Bucket == 3 && VAngle > FMathd::HalfPi)		// 3=[-180,0]
					{
						VAngle -= FMathd::TwoPi;
					}
					UV = FVector2f( -(float(VAngle) * FMathf::InvPi - 1.0f), -float(BoxPos.Z));
				}

				ElemTri[j] = UVOverlay->AppendElement(UV);
				NewUVIndices.Add(ElemTri[j]);
				BaseToOverlayVIDMap.Add(ElementKey, ElemTri[j]);
			}
			else
			{
				ElemTri[j] = *FoundElementID;
			}
		}

		UVOverlay->SetTriangle(tid, ElemTri);
	}

	// Above process can introduce bowties, so we split any bowties on new element IDs
	SplitBowtiesOnUVElements(NewUVIndices, true);

	if (Result != nullptr)
	{
		Result->NewUVElements = MoveTemp(NewUVIndices);
	}
}




bool FDynamicMeshUVEditor::ScaleUVAreaTo3DArea(const TArray<int32>& Triangles, bool bRecenterAtOrigin, float ScaleFactor)
{
	double Area3D = TMeshQueries<FDynamicMesh3>::GetVolumeArea(*Mesh, Triangles).Y;
	if (FMathd::Abs(Area3D) < FMathf::Epsilon || FMathd::IsFinite(Area3D) == false)
	{
		return false;
	}

	TSet<int32> Elements;
	FAxisAlignedBox2f UVBounds;
	double Area2D = DetermineAreaFromUVs(*UVOverlay, Triangles, &UVBounds);

	for (int32 tid : Triangles)
	{
		if (UVOverlay->IsSetTriangle(tid))
		{
			FIndex3i UVTri = UVOverlay->GetTriangle(tid);
			Elements.Add(UVTri.A); Elements.Add(UVTri.B); Elements.Add(UVTri.C);
		}
	}

	if (Elements.Num() == 0 || FMathd::Abs(Area2D) < FMathf::Epsilon ||  FMathd::IsFinite(Area2D) == false )
	{
		return false;
	}

	double UVScale = ScaleFactor * FMathd::Sqrt(Area3D) / FMathd::Sqrt(Area2D);
	if (!FMathd::IsFinite(UVScale))
	{
		return false;
	}

	FVector2f ScaleOrigin = UVBounds.Center();
	FVector2f Translation = (bRecenterAtOrigin) ? FVector2f::Zero() : ScaleOrigin;
	for (int32 eid : Elements)
	{
		FVector2f UV = UVOverlay->GetElement(eid);
		UV = (UV - ScaleOrigin) * float(UVScale) + Translation;
		UVOverlay->SetElement(eid, UV);
	}

	return true;
}

bool FDynamicMeshUVEditor::ScaleUVAreaToBoundingBox(const TArray<int32>& Triangles, const FAxisAlignedBox2f& BoundingBox, bool bPreserveAspectRatio, bool bRecenterAtBoundingBox)
{
	if (FMathd::Abs(BoundingBox.Area()) < FMathf::Epsilon || FMathd::IsFinite(BoundingBox.Area()) == false)
	{
		return false;
	}

	TSet<int32> Elements;
	FAxisAlignedBox2f UVBounds;
	double Area2D = DetermineAreaFromUVs(*UVOverlay, Triangles, &UVBounds);

	for (int32 tid : Triangles)
	{
		if (UVOverlay->IsSetTriangle(tid))
		{
			FIndex3i UVTri = UVOverlay->GetTriangle(tid);
			Elements.Add(UVTri.A); Elements.Add(UVTri.B); Elements.Add(UVTri.C);
		}
	}
	

	if (Elements.Num() == 0 || FMathd::Abs(Area2D) < FMathf::Epsilon || FMathd::IsFinite(Area2D) == false)
	{
		return false;
	}

	float WidthScale = BoundingBox.Width() / UVBounds.Width();
	float HeightScale = BoundingBox.Height() / UVBounds.Height();

	if (bPreserveAspectRatio)
	{
		WidthScale = FMath::Min(WidthScale, HeightScale);
		HeightScale = WidthScale;
	}

	if ( !FMathd::IsFinite(WidthScale) || !FMathd::IsFinite(HeightScale))
	{
		return false;
	}

	FVector2f ScaleOrigin = UVBounds.Center();
	FVector2f Translation = (bRecenterAtBoundingBox) ? BoundingBox.Center() : ScaleOrigin;
	TransformUVElements(Elements.Array(), [ScaleOrigin, WidthScale, HeightScale, Translation](const FVector2f& UV){
		FVector2f UVTransformed = UV;
		UVTransformed = (UVTransformed - ScaleOrigin);
		UVTransformed[0] = UVTransformed[0] * WidthScale;
		UVTransformed[1] = UVTransformed[1] * HeightScale;
		UVTransformed = UVTransformed + Translation;
		return UVTransformed;
	});

	return true;
}




bool FDynamicMeshUVEditor::AutoOrientUVArea(const TArray<int32>& Triangles)
{
	TSet<int32> Elements;
	TArray<FVector2f> UVs;
	FAxisAlignedBox2f UVBounds = FAxisAlignedBox2f::Empty();
	for (int32 tid : Triangles)
	{
		if (UVOverlay->IsSetTriangle(tid))
		{
			FIndex3i UVTri = UVOverlay->GetTriangle(tid);
			for (int32 j = 0; j < 3; ++j)
			{
				bool bIsAlreadyInSet = false;
				Elements.Add(UVTri[j], &bIsAlreadyInSet);
				if (bIsAlreadyInSet == false)
				{
					FVector2f UV = UVOverlay->GetElement(UVTri[j]);
					UVs.Add(UV);
					UVBounds.Contain(UV);
				}
			}
		}
	}
	int32 N = UVs.Num();
	if (N == 0)
	{
		return false;
	}

	// shift to origin so we can skip subtract below
	FVector2f BoxCenter = UVBounds.Center();
	for ( int32 k = 0; k < N; ++k )
	{
		UVs[k] -= BoxCenter;
	}

	FConvexHull2f ConvexHull;
	if (ConvexHull.Solve(UVs) == false)
	{
		return false;
	}
	TArray<int32> const& HullPointIndices = ConvexHull.GetPolygonIndices();
	check(HullPointIndices[0] != HullPointIndices.Last());

	TArray<FVector2f> HullPoints;
	HullPoints.Reserve(HullPointIndices.Num());
	for (int32 k = 0; k < HullPointIndices.Num(); ++k)
	{
		HullPoints.Add(UVs[HullPointIndices[k]]);
	}
	int32 NV = HullPoints.Num();


	// This is basically a brute-force rotating-calipers implementation. Probably should be moved to a CompGeom class (and implemented more efficiently)
	double MinBoxArea = UVBounds.Area();
	FVector2f MinAxisDirection = FVector2f::UnitX();
	bool bFoundSmallerBox = false;
	for (int32 j = 0; j <= NV; ++j)
	{
		FVector2f A = HullPoints[j%NV], B = HullPoints[(j+1)%NV], C = HullPoints[(j+NV/2)%NV]; //-V609
		FVector2f Axis0 = B - A; 
		float Dimension0 = Normalize(Axis0);
		FVector2f Axis1 = PerpCW(Axis0);
		float Dimension1 = (C - A).Dot(Axis1);

		FInterval1f Interval0 = FInterval1f::Empty();
		FInterval1f Interval1 = FInterval1f::Empty();

		bool bAbort = false;
		// Use modulo iteration here to try to grow the box more quickly, which (hopefully) hits the early-out faster
		// (would be worth profiling to see if this is a good idea, due to cache coherency)
		FModuloIteration Iter(NV);
		uint32 Index;
		while (Iter.GetNextIndex(Index))
		{
			Interval0.Contain(Axis0.Dot(HullPoints[Index]));
			Dimension0 = FMathf::Max(Dimension0, Interval0.Length());
			Interval1.Contain(Axis1.Dot(HullPoints[Index]));
			Dimension1 = FMathf::Max(Dimension1, Interval1.Length());
			if (Dimension0 * Dimension1 > MinBoxArea)
			{
				bAbort = true;
				break;
			}
		}
		if (bAbort == false && (Dimension0 * Dimension1 < MinBoxArea) )
		{
			MinBoxArea = Dimension0 * Dimension1;
			MinAxisDirection = Axis0;
			bFoundSmallerBox = true;
		}
	}

	if (bFoundSmallerBox)
	{
		float RotationAngle = FMathf::Atan2(MinAxisDirection.Y, MinAxisDirection.X);
		FMatrix2f RotMatrix = FMatrix2f::RotationRad(-RotationAngle);
		for (int32 eid : Elements)
		{
			FVector2f UV = UVOverlay->GetElement(eid);
			UV = RotMatrix * (UV - BoxCenter) + BoxCenter;
			UVOverlay->SetElement(eid, UV);
		}
	}

	return true;
}


bool FDynamicMeshUVEditor::QuickPack(int32 TargetTextureResolution, float GutterSize)
{
	// always split bowties before packing
	UVOverlay->SplitBowties();

	FDynamicMeshUVPacker Packer(UVOverlay);
	Packer.TextureResolution = TargetTextureResolution;
	Packer.GutterSize = GutterSize;
	Packer.bAllowFlips = false;

	bool bOK = Packer.StandardPack();

	return bOK;
}

bool FDynamicMeshUVEditor::UDIMPack(int32 TargetTextureResolution, float GutterSize, const FVector2i& UDIMCoordsIn, const TArray<int32>* Triangles)
{
	TUniquePtr<TArray<int32>> TileTids;
	if (Triangles)
	{
		TileTids = MakeUnique<TArray<int32>>(*Triangles);
	}
	else
	{
		// Add all set UV triangles
		TileTids = MakeUnique<TArray<int32>>();
		TileTids->Reserve(Mesh->TriangleCount());
		for (int32 TriangleID : Mesh->TriangleIndicesItr())
		{
			TileTids->Add(TriangleID);
		}
	}

	// Do this first, so we don't need to keep the TileTids around after moving it into the packer.
	TSet<int32> ElementsToMove;
	ElementsToMove.Reserve(TileTids->Num() * 3);
	for (int Tid : *TileTids)
	{
		// If the triangle is unset, we will skip it here, before moving on.
		if (UVOverlay->IsSetTriangle(Tid))
		{
			FIndex3i Elements = UVOverlay->GetTriangle(Tid);
			ElementsToMove.Add(Elements[0]);
			ElementsToMove.Add(Elements[1]);
			ElementsToMove.Add(Elements[2]);
		}
	}
	// Final check to make sure we didn't let any invalid element IDs from sneaking through.
	// This should never happen, since we are filtering potentially unset UV triangles above
	check(!ElementsToMove.Contains(IndexConstants::InvalidID));

	// TODO: There is a second connected components call inside the packer that might be unnessessary. Could be a future optimization.
	FDynamicMeshUVPacker Packer(UVOverlay, MoveTemp(TileTids));
	Packer.TextureResolution = TargetTextureResolution;
	Packer.GutterSize = GutterSize;
	Packer.bAllowFlips = false;
	bool bOK = Packer.StandardPack();

	// Transform this to match the internal UV storage layout of negative Y
	FVector2i TransformedUDIMCoords(UDIMCoordsIn.X, -UDIMCoordsIn.Y);

	for (int32 Element : ElementsToMove)
	{
		FVector2f UV = UVOverlay->GetElement(Element);
		UV = (UV)+(FVector2f)(TransformedUDIMCoords);
		UVOverlay->SetElement(Element, UV);
	}

	return bOK;
}


double FDynamicMeshUVEditor::DetermineAreaFromUVs(const FDynamicMeshUVOverlay& UVOverlay, const TArray<int32>& Triangles, FAxisAlignedBox2f* BoundingBox)
{
	if (BoundingBox)
	{
		(*BoundingBox) = FAxisAlignedBox2f::Empty();
	}
	double Area2D = 0.0;
	for (int32 tid : Triangles)
	{
		if (UVOverlay.IsSetTriangle(tid))
		{
			FIndex3i UVTri = UVOverlay.GetTriangle(tid);			
			FVector2f U = UVOverlay.GetElement(UVTri.A);
			FVector2f V = UVOverlay.GetElement(UVTri.B);
			FVector2f W = UVOverlay.GetElement(UVTri.C);
			if (BoundingBox)
			{
				BoundingBox->Contain(U); BoundingBox->Contain(V); BoundingBox->Contain(W);
			}
			Area2D += (double)VectorUtil::Area(U, V, W);
		}
	}

	return Area2D;
}

void FDynamicMeshUVEditor::TransformTriangleSelectionUVs(FDynamicMeshUVOverlay& UVOverlay, TConstArrayView<int32> Triangles, TFunctionRef<FVector2f(const FVector2f&)> TransformFunc)
{
	TSet<int32> Elements;
	for (int32 TID : Triangles)
	{
		FIndex3i Tri = UVOverlay.GetTriangle(TID);
		if (Tri.A != INDEX_NONE)
		{
			Elements.Add(Tri.A);
			Elements.Add(Tri.B);
			Elements.Add(Tri.C);
		}
	}
	for (int32 EID : Elements)
	{
		FVector2f UV = UVOverlay.GetElement(EID);
		UVOverlay.SetElement(EID, TransformFunc(UV));
	}
}

