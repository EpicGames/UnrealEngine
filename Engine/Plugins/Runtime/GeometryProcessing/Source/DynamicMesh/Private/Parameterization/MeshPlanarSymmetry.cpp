// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/MeshPlanarSymmetry.h"
#include "Async/ParallelFor.h"

#include "CompGeom/ConvexHull3.h" // for TExtremePoints3
#include "BoxTypes.h"
#include "MeshCurvature.h"
#include "DynamicMesh/MeshNormals.h"
#include "Spatial/PointHashGrid3.h"
#include "Spatial/PointHashGrid2.h"
#include "VertexConnectedComponents.h"


using namespace UE::Geometry;

void FMeshPlanarSymmetry::ComputeMeshInfo(const FDynamicMesh3* Mesh, const FAxisAlignedBox3d& Bounds, TArray<FVector3d>& InvariantFeaturesOut, FVector3d& MeshCentroidOut)
{
	// Compute centroid with an online sum to avoid messing up accuracy if the number of vertices is very large
	MeshCentroidOut = FVector::ZeroVector;
	int32 Idx = 0;
	FVector3d Center = Bounds.Center();
	for (FVector3d V : Mesh->VerticesItr())
	{
		MeshCentroidOut += (V - Center - MeshCentroidOut) / double(++Idx);
	}
	MeshCentroidOut += Center;

	// To group vertices by symmetry-pair candidates, we map the vertices to a feature space of:
	// (Vertex Angle Sum / 2pi, cos(Angle between normal and direction from centroid), Distance from Vertex Centroid / Half Max Bounds Dim)
	// Note these values are scaled so that a single distance tolerance can be used to compare features.
	FMeshNormals MeshNormals(Mesh);
	MeshNormals.ComputeVertexNormals(false, true); // Note: only weighting by angle, b/c it is less sensitive to tesselation than area
	InvariantFeaturesOut.SetNumUninitialized(Mesh->MaxVertexID());
	double InvHalfMaxDim = 2.0 / Bounds.MaxDim();
	ParallelFor(Mesh->MaxVertexID(), [&](int32 VID)
		{
			if (!Mesh->IsVertex(VID))
			{
				return;
			}

			double AngleSum = 0;
			for (int TID : Mesh->VtxTrianglesItr(VID))
			{
				FIndex3i Triangle = Mesh->GetTriangle(TID);
				int IndexInTriangle = IndexUtil::FindTriIndex(VID, Triangle);
				AngleSum += Mesh->GetTriInternalAngleR(TID, IndexInTriangle);
			}
			FVector3d VertexNormal = MeshNormals[VID];
			FVector3d Pos = Mesh->GetVertex(VID);
			double CentroidDist = FVector3d::Distance(MeshCentroidOut, Pos);
			FVector3d FromCentroid = Pos - MeshCentroidOut;
			if (CentroidDist > FMathd::ZeroTolerance)
			{
				FromCentroid /= CentroidDist;
			}
			double NormalAngleCos = FromCentroid.Dot(VertexNormal);
			// Note: Downscale the angle and normal features because they tend to have more error / tesselation sensitivity
			InvariantFeaturesOut[VID] = FVector3d(AngleSum * FMathd::InvTwoPi * .01, NormalAngleCos * .001, CentroidDist * InvHalfMaxDim);
		});

}

bool FMeshPlanarSymmetry::Validate(const FDynamicMesh3* Mesh)
{
	if (!Mesh)
	{
		return false;
	}
	if (Mesh->VertexCount() == 0)
	{
		return false;
	}
	return true;
}

void FMeshPlanarSymmetry::SetErrorScale(const FAxisAlignedBox3d& Bounds)
{
	ErrorScale = FMath::Max(1, Bounds.MaxDim()); // Error tolerances scale up with larger meshes
}

bool FMeshPlanarSymmetry::FindPlaneAndInitialize(FDynamicMesh3* Mesh, const FAxisAlignedBox3d& Bounds, FFrame3d& SymmetryFrameOut, TArrayView<const FVector3d> PreferredNormals)
{
	this->TargetMesh = Mesh;
	if (!Validate(Mesh))
	{
		return false;
	}

	SetErrorScale(Bounds);
	double OnPlaneTolerance = OnPlaneToleranceFactor * ErrorScale;
	double MatchVertexTolerance = MatchVertexToleranceFactor * ErrorScale;
	double VertexHashCellSize = VertexHashCellSizeFactor * ErrorScale;

	// If this mesh is a tiny mesh, then all points are on the symmetry plane ...
	double BoundsMaxDim = Bounds.MaxDim();
	if (BoundsMaxDim < OnPlaneTolerance)
	{
		if (bCanIgnoreDegenerateSymmetries)
		{
			return false;
		}
		else
		{
			// Just use a default/preferred plane
			FVector3d UseNormal = PreferredNormals.Num() ? PreferredNormals[0] : FVector3d::UnitX();
			SymmetryFrameOut = FFrame3d(Bounds.Center(), UseNormal);
			return Initialize(Mesh, Bounds, SymmetryFrameOut);
		}
	}

	// Compute vertex hash and features
	FVector3d MeshCentroid;
	TArray<FVector3d> Features;
	ComputeMeshInfo(Mesh, Bounds, Features, MeshCentroid);
	TPointHashGrid3d<int32> VertexHash(VertexHashCellSize, INDEX_NONE);
	VertexHash.Reserve(Mesh->VertexCount());
	for (int32 VID : Mesh->VertexIndicesItr())
	{
		VertexHash.InsertPointUnsafe(VID, Mesh->GetVertex(VID));
	}

	// Find groups of vertices that could by symmetric pairs, based on similar distance from the MeshCentroid
	FVertexConnectedComponents PairCandidateComponents(Mesh->MaxVertexID());

	// To quickly match in dist-to-centroid space, we hash the points by their distances
	// In the worst case, all the vertices may have the same distance (a sphere) so we create a hash entry covering a 'range'
	// and when we overlap an existing range, we just expand that range rather than create a new one
	struct FRange
	{
		FInterval1d Range;
		int32 ID;
	};
	// Because ranges cannot overlap within a bucket, there can only be a fixed number of ranges per bucket
	// A single-point range covers an interval of size 2*Tolerance, so a 3*Tolerance-wide bucket
	// should only have 3 separate ranges before any new range entered in the bucket will instead expand an existing range.
	// (3 because ranges could partially touch the bucket on each side.)
	// For safety, we allocate to allow up to 4 ...
	constexpr int MaxRangesPerBucket = 4;
	using FRangesBucket = TArray<FRange, TFixedAllocator<MaxRangesPerBucket>>;
	TMap<int32, FRangesBucket> Ranges;
	double RangeCellWidth = MatchVertexTolerance * (MaxRangesPerBucket - 1);
	double ToCellIdx = 1.0 / RangeCellWidth;
	for (int32 VID : Mesh->VertexIndicesItr())
	{
		double CentroidDistance = FVector::Distance(Mesh->GetVertex(VID), MeshCentroid);
		FInterval1d Range(CentroidDistance - MatchVertexTolerance, CentroidDistance + MatchVertexTolerance);
		for (int32 CellIdx = FMath::FloorToInt32(Range.Min * ToCellIdx); CellIdx <= FMath::FloorToInt32(Range.Max * ToCellIdx); ++CellIdx)
		{
			FRangesBucket& Bucket = Ranges.FindOrAdd(CellIdx);
			bool bPlacedInExisting = false;
			for (FRange& Existing : Bucket)
			{
				if (Existing.Range.Overlaps(Range))
				{
					bPlacedInExisting = true;
					PairCandidateComponents.ConnectVertices(VID, Existing.ID);
					Existing.Range.Contain(Range);
				}
			}
			if (!bPlacedInExisting)
			{
				check(Bucket.Num() < MaxRangesPerBucket);
				Bucket.Add(FRange{Range, VID});
			}
		}
	}

	// Get an ordered array of feature components and compute their centroids
	TArray<int32> PairCandidateComponentsArray = PairCandidateComponents.MakeContiguousComponentsArray(Mesh->MaxVertexID());
	TArray<FVector3d> AllCentroids;
	AllCentroids.Add(MeshCentroid);
	FVector3d BoundsCenter = Bounds.Center();
	PairCandidateComponents.EnumerateContiguousComponentsFromArray(PairCandidateComponentsArray, [&](int32 ComponentID, TArrayView<const int32> VertexIDs) -> bool
		{
			if (!Mesh->IsVertex(ComponentID))
			{
				return true;
			}

			// Compute centroid with an online sum to avoid messing up accuracy if there is a very large component
			FVector3d Centroid(0, 0, 0);
			int32 Idx = 0;
			for (int32 VID : VertexIDs)
			{
				const FVector3d CenteredPos = Mesh->GetVertex(VID) - BoundsCenter;
				Centroid += (CenteredPos - Centroid) / double(++Idx);
			}
			Centroid += BoundsCenter;
			AllCentroids.Add(Centroid);
			return true;
		});
	
	// Test a plane through MeshCentroid with the given Normal for symmetry.  While we're at it, accumulate an improved normal estimate from the matches.
	// Returns false at the first failure to find a match, so often very fast to reject an incorrect plane
	auto TestPlane = [Mesh, MeshCentroid, &VertexHash, this, OnPlaneTolerance, MatchVertexTolerance](FVector3d Normal, FVector3d& RefinedNormalOut) -> bool
	{
		FVector3d SumOfPtToMirrorDeltas(0, 0, 0);
		bool bAllOnPlane = true;
		RefinedNormalOut = Normal;
		for (int32 VID : Mesh->VertexIndicesItr())
		{
			FVector3d Position = Mesh->GetVertex(VID);
			double SignedPlaneDist = Normal.Dot(Position - MeshCentroid);
			if (SignedPlaneDist < OnPlaneTolerance)
			{
				continue;
			}
			bAllOnPlane = false;
			FVector3d MirrorPosition = Position - 2 * Normal * SignedPlaneDist;
			double NearestDistSqr = 0;
			TPair<int32, double> Nearest = VertexHash.FindNearestInRadius(MirrorPosition, MatchVertexTolerance, [MirrorPosition, &Mesh](const int32& FoundVID)
				{
					return FVector3d::DistSquared(MirrorPosition, Mesh->GetVertex(FoundVID));
				});
			int32 NearestVID = Nearest.Key;
			if (NearestVID == INDEX_NONE)
			{
				return false;
			}
			FVector3d FoundMirrorPosition = Mesh->GetVertex(NearestVID);

			FVector3d Delta = (Position - FoundMirrorPosition);
			SumOfPtToMirrorDeltas += Delta;
		}
		if (!bAllOnPlane && SumOfPtToMirrorDeltas.Normalize())
		{
			RefinedNormalOut = SumOfPtToMirrorDeltas;
		}
		return true;
	};

	// Check whether the centroids already span a plane; if so, we already have the mirror plane candidate
	// Note that the ExtremePoints algorithm internally scales its tolerance by the scale of the input,
	// so we shouldn't use our ErrorScale-scaled tolerances with it.
	constexpr double ExtremePointsTolerance = UE_DOUBLE_KINDA_SMALL_NUMBER;
	FExtremePoints3d CentroidsExtremePoints(AllCentroids.Num(), [&AllCentroids](int32 Idx) {return AllCentroids[Idx];}, [](int32) {return true;}, ExtremePointsTolerance);
	if (CentroidsExtremePoints.Dimension > 1)
	{
		FVector3d Normal = CentroidsExtremePoints.Basis[2];
		if (CentroidsExtremePoints.Dimension == 3)
		{
			// The centroids spanned 3 dimensions -> no possible symmetry plane
			return false;
		}
		// The plane the centroids spanned must be the symmetry plane
		SymmetryFrameOut = FFrame3d(MeshCentroid, Normal);
		return AssignMatches(Mesh, VertexHash, Features, SymmetryFrameOut);
	}

	// Track whether we at least have a line that the symmetry plane must include
	// Note: Often this will be the line of intersection of two (or more) symmetry planes, but
	// that is not guaranteed due to the possibility that a symmetry cluster can merge multiple
	// near-in-feature-space groups of points. (And AddPointOnPlane() can later set this vector
	// in a way that definitely won't correspond to a symmetry plane intersection line.)
	bool bHasVectorOnPlane = false;
	FVector3d VectorOnPlane(0, 0, 0);
	if (CentroidsExtremePoints.Dimension == 1)
	{
		bHasVectorOnPlane = true;
		VectorOnPlane = CentroidsExtremePoints.Basis[0];
	}

	// If the caller passed in preferred plane normals, try those first
	for (FVector3d PreferNormal : PreferredNormals)
	{
		// If we know a vector must be on the plane, the normal must be perpendicular to that
		if (!bHasVectorOnPlane || FMathd::Abs(PreferNormal.Dot(VectorOnPlane)) < UE_DOUBLE_KINDA_SMALL_NUMBER)
		{
			FVector3d RefinedNormal;
			if (TestPlane(PreferNormal, RefinedNormal))
			{
				// Note: Ignore RefinedNormal, since we already have an exact value preferred by the caller
				SymmetryFrameOut = FFrame3d(MeshCentroid, PreferNormal);
				return AssignMatches(Mesh, VertexHash, Features, SymmetryFrameOut);
			}
		}
	}

	// Test for the degenerate case of a completely collinear mesh (should be fast, usually, since we break out at the first off-axis point)
	if (bHasVectorOnPlane)
	{
		bool bAllOnAxis = true;
		for (int32 VID : Mesh->VertexIndicesItr())
		{
			FVector3d FromCentroid = Mesh->GetVertex(VID) - MeshCentroid;
			FVector3d OnAxis = FromCentroid.ProjectOnToNormal(VectorOnPlane);
			if (FVector3d::DistSquared(OnAxis, FromCentroid) > OnPlaneTolerance)
			{
				bAllOnAxis = false;
				break;
			}
		}
		if (bAllOnAxis)
		{
			if (bCanIgnoreDegenerateSymmetries)
			{
				return false;
			}
			else
			{
				FVector3d Normal = VectorUtil::MakePerpVector(VectorOnPlane);
				SymmetryFrameOut = FFrame3d(MeshCentroid, Normal);
				return AssignMatches(Mesh, VertexHash, Features, SymmetryFrameOut);
			}
		}
	}

	// Get a list of clusters sorted by size, so we can start w/ the smallest cluster when finding symmetry
	struct FClusterInfo
	{
		int32 StartIdx = -1, Size = 0;
	};
	TArray<FClusterInfo> SortedClusters;
	for (int32 ContigStart = 0, NextStart = -1; ContigStart < PairCandidateComponentsArray.Num(); ContigStart = NextStart)
	{
		int32 ComponentID = PairCandidateComponents.GetComponent(PairCandidateComponentsArray[ContigStart]);
		int32 ComponentSize = PairCandidateComponents.GetComponentSize(ComponentID);
		NextStart = ContigStart + ComponentSize;
		SortedClusters.Add(FClusterInfo{ ContigStart, ComponentSize });
	}
	SortedClusters.Sort([&](const FClusterInfo& A, const FClusterInfo& B) -> bool
		{
			return A.Size < B.Size;
		});

	// Helper to update our symmetry axis and plane estimate, when we have a new point that must by on the plane.
	// Returns true if a plane normal has been successfully estimated using the point, false if we still don't have a plane estimate.
	// (Note: We want to use a tolerance similar to ExtremePoints, but internally ExtremePoints scales by magnitude, so we scale by BoundsMaxDim)
	const double AddPointDistanceToleranceSq = ExtremePointsTolerance * ExtremePointsTolerance * BoundsMaxDim * BoundsMaxDim;
	auto AddPointOnPlane = [&bHasVectorOnPlane, &VectorOnPlane, MeshCentroid, AddPointDistanceToleranceSq](FVector3d Point, FVector3d& NormalOut) -> bool
	{
		FVector3d ToPoint = Point - MeshCentroid;
		double ToPointLenSq = ToPoint.SquaredLength();
		if (ToPointLenSq < AddPointDistanceToleranceSq)
		{
			return false;
		}
		if (bHasVectorOnPlane)
		{
			double DistSqToAxis = FVector3d::DistSquared(ToPoint.ProjectOnToNormal(VectorOnPlane), ToPoint);
			if (DistSqToAxis > AddPointDistanceToleranceSq)
			{
				NormalOut = ToPoint.Cross(VectorOnPlane);
				return NormalOut.Normalize();
			}
		}
		else
		{
			VectorOnPlane = ToPoint;
			bHasVectorOnPlane = ToPoint.Normalize();
		}
		return false;
	};

	// Process the clusters of possible symmetry pairs from smallest to largest
	for (const FClusterInfo& Cluster : SortedClusters)
	{
		int32 FirstVID = PairCandidateComponentsArray[Cluster.StartIdx];
		if (!Mesh->IsVertex(FirstVID))
		{
			continue;
		}
		// clusters of size 1 are their own centroid, and we already processed the centroids,
		// so we can't get any new information from them
		if (Cluster.Size == 1)
		{
			continue;
		}
		// Use ExtremePoints to find reference points to look for mirror symmetry; if there are any points in the cluster
		// that are not on the mirror symmetry plane, at least one of them should be in ExtremePoints
		FExtremePoints3d ExtremePoints(Cluster.Size, [&](int32 WithinClusterIdx) -> FVector3d
			{
				int32 VID = PairCandidateComponentsArray[Cluster.StartIdx + WithinClusterIdx];
				return Mesh->GetVertex(VID);
			}, [](int32 Idx) { return true; }, ExtremePointsTolerance);
		// helper to avoid repeating evaluation of extreme points
		auto IsPointExtreme = [ExtremePoints](int32 WithinClusterIdx, int32 NumExtremeToConsider)
		{
			for (int32 ExtremeIdx = 0; ExtremeIdx < NumExtremeToConsider; ++ExtremeIdx)
			{
				if (WithinClusterIdx == ExtremePoints.Extreme[ExtremeIdx])
				{
					return true;
				}
			}
			return false;
		};
		for (int32 ExtremePtIdx = 0; ExtremePtIdx < ExtremePoints.Dimension + 1; ++ExtremePtIdx)
		{
			int32 RefVID = PairCandidateComponentsArray[Cluster.StartIdx + ExtremePoints.Extreme[ExtremePtIdx]];
			FVector3d RefPos = Mesh->GetVertex(RefVID);
			FVector3d RefFeature = Features[RefVID];
			// Try pairing the extreme point with every other vertex in the cluster, and checking if that creates a mirror plane
			// Note this sounds O(ClusterSize*MeshSize) but in practice most bad matches can be rejected by just testing a few points,
			// so it's often closer to just ~ ClusterSize work (or ClusterSize+MeshSize if it finds a symmetry plane)
			for (int32 WithinClusterIdx = 0; WithinClusterIdx < Cluster.Size; ++WithinClusterIdx)
			{
				// Skip the point if it's the current or a previously-considered extreme point
				if (IsPointExtreme(WithinClusterIdx, ExtremePtIdx + 1))
				{
					continue;
				}
				int32 VID = PairCandidateComponentsArray[Cluster.StartIdx + WithinClusterIdx];
				checkSlow(VID != RefVID);
				FVector3d Pos = Mesh->GetVertex(VID);
				FVector3d Normal = Pos - RefPos;
				// Reject the plane if we can't compute its normal because the points are too close
				if (!Normal.Normalize())
				{
					continue;
				}
				// If we know a vector on the plane is known, sample points should project to the same position w.r.t. that
				// (i.e., plane normal should be perpendicular to the vector on the plane)
				if (bHasVectorOnPlane && FMath::IsNearlyEqual(VectorOnPlane.Dot(Pos), VectorOnPlane.Dot(RefPos), MatchVertexTolerance))
				{
					continue;
				}
				// Reject the plane if the points do not have matching feature vectors
				if (FVector3d::DistSquared(RefFeature, Features[VID]) > MatchFeaturesTolerance * MatchFeaturesTolerance)
				{
					FVector3d Diff = Features[VID] - RefFeature;
					continue;
				}
				FVector3d RefinedNormal;
				if (TestPlane(Normal, RefinedNormal))
				{
					SymmetryFrameOut = FFrame3d(MeshCentroid, RefinedNormal);
					return AssignMatches(Mesh, VertexHash, Features, SymmetryFrameOut);
				}
			}
			
			// Failed to find a symmetry plane from the matches in this cluster, so if the mesh is symmetric the point must
			// be on the symmetry plane.  Update what we know about the symmetry plane w/ this point, and early-out if it has given us the plane
			FVector3d Normal;
			if (AddPointOnPlane(Mesh->GetVertex(FirstVID), Normal))
			{
				SymmetryFrameOut = FFrame3d(MeshCentroid, Normal);
				return AssignMatches(Mesh, VertexHash, Features, SymmetryFrameOut);
			}
		}
		// Failed to find a symmetry plane within the whole cluster; all of its point must be on the plane
		for (int32 WithinClusterIdx = 0; WithinClusterIdx < Cluster.Size; ++WithinClusterIdx)
		{
			if (IsPointExtreme(WithinClusterIdx, ExtremePoints.Dimension + 1))
			{
				continue; // already added the extreme points in the above loop
			}
			int32 VID = PairCandidateComponentsArray[Cluster.StartIdx + WithinClusterIdx];
			FVector3d Normal;
			if (AddPointOnPlane(Mesh->GetVertex(FirstVID), Normal))
			{
				SymmetryFrameOut = FFrame3d(MeshCentroid, Normal);
				return AssignMatches(Mesh, VertexHash, Features, SymmetryFrameOut);
			}
		}
	}
	return false;
}

bool FMeshPlanarSymmetry::AssignMatches(const FDynamicMesh3* Mesh, const TPointHashGrid3d<int32>& VertexHash, const TArray<FVector3d>& InvariantFeatures, FFrame3d SymmetryFrameIn)
{
	this->SymmetryFrame = SymmetryFrameIn;
	this->CachedSymmetryAxis = SymmetryFrameIn.Z();
	Vertices.Init(FSymmetryVertex(), TargetMesh->MaxVertexID());

	TArray<int32> PositiveVerts;
	PositiveVerts.Reserve(Vertices.Num());

	for (int32 VID : Mesh->VertexIndicesItr())
	{
		FVector3d Position = Mesh->GetVertex(VID);
		Vertices[VID].PlaneSignedDistance = (Position - SymmetryFrame.Origin).Dot(CachedSymmetryAxis);
		if (Vertices[VID].PlaneSignedDistance > FMathd::ZeroTolerance)
		{
			PositiveVerts.Add(VID);
			Vertices[VID].bIsSourceVertex = true;
		}
		else
		{
			Vertices[VID].bIsSourceVertex = false;
		}
	}

	// Whether to reattempt matches w/out feature-based filtering, if a match-failure would break symmetry overall
	constexpr bool bReattemptMatchesOnFailure = true;

	double OnPlaneTolerance = OnPlaneToleranceFactor * ErrorScale;
	double MatchVertexTolerance = MatchVertexToleranceFactor * ErrorScale;

	TArray<TArray<int32, TInlineAllocator<4>>> CandidatesPerPositiveVertex;
	CandidatesPerPositiveVertex.SetNum(PositiveVerts.Num());
	
	// Compute unique matching vertices on mirror side (negative side of plane) for positive-side vertices
	ParallelFor(PositiveVerts.Num(), [&](int32 k)
		{
			int32 VertexID = PositiveVerts[k];

			FVector3d Position = Mesh->GetVertex(VertexID);
			FVector3d MirrorPosition = GetMirroredPosition(Position);

			// find vertex that is closest to mirrored position
			FVector3d Feature = InvariantFeatures[VertexID];
			double MatchFeaturesToleranceSqr = MatchFeaturesTolerance * MatchFeaturesTolerance;

			TArray<int32> Candidates;
			VertexHash.FindPointsInBall(MirrorPosition, MatchVertexTolerance, [MirrorPosition, Mesh](const int32& OtherVID)
			{
				return FVector3d::DistSquared(MirrorPosition, Mesh->GetVertex(OtherVID));
			}, 
			Candidates,
			[Feature, VertexID, this, &InvariantFeatures, MatchFeaturesToleranceSqr](const int32& OtherVID) -> bool // Ignore filter fn returns true for vertices we want to skip
			{
				return OtherVID == VertexID // don't match to self
					|| Vertices[OtherVID].bIsSourceVertex // don't match to other positive vertices
					|| FVector3d::DistSquared(InvariantFeatures[OtherVID], Feature) > MatchFeaturesToleranceSqr; // don't match if the features don't match
			}
			);

			if (Candidates.IsEmpty())
			{
				// If a failed match would break symmetry overall, we can retry without feature-based filtering
				if (bReattemptMatchesOnFailure && Vertices[VertexID].PlaneSignedDistance >= OnPlaneTolerance)
				{
					VertexHash.FindPointsInBall(MirrorPosition, MatchVertexTolerance, [MirrorPosition, Mesh](const int32& OtherVID)
					{
						return FVector3d::DistSquared(MirrorPosition, Mesh->GetVertex(OtherVID));
					}, 
					Candidates,
					[Feature, VertexID, this, &InvariantFeatures, MatchFeaturesToleranceSqr](const int32& OtherVID) -> bool // Ignore filter fn returns true for vertices we want to skip
					{
						return OtherVID == VertexID // don't match to self
							|| Vertices[OtherVID].bIsSourceVertex; // don't match to other positive vertices
					}
					);	
				}
			}

			// Rank all candidates first by Euclidean distance and then by feature distance
			Algo::Sort(Candidates,
				[&MirrorPosition, Mesh, &InvariantFeatures, Feature](const int32& A, const int32& B)
				{
					double DistanceA = FVector3d::DistSquared(MirrorPosition, Mesh->GetVertex(A));
					double FeatureDistanceA = FVector3d::DistSquared(InvariantFeatures[A], Feature);
				
					double DistanceB = FVector3d::DistSquared(MirrorPosition, Mesh->GetVertex(B));
					double FeatureDistanceB = FVector3d::DistSquared(InvariantFeatures[B], Feature);
				
					if (DistanceA < DistanceB)
					{
						return true;
					}
					
					if (DistanceA == DistanceB)
					{
						if (FeatureDistanceA < FeatureDistanceB)
						{
							return true;
						}
					}

					return false;
				});
			
			CandidatesPerPositiveVertex[k] = MoveTemp(Candidates); 
		});
	
	for (int32 k = 0; k < PositiveVerts.Num(); k++)
	{
		int32 VertexID = PositiveVerts[k];
		
		for (int32 Candidate : CandidatesPerPositiveVertex[k])
		{
			// Avoid many to one matching
			if (Vertices[Candidate].PairedVertex != INDEX_NONE)
			{
				continue;
			}

			Vertices[Candidate].PairedVertex = VertexID;
			Vertices[VertexID].PairedVertex = Candidate;
			break;
		}
	}

	// If a vertex has no match and is within the symmetry-plane tolerance band, consider it a plane vertex.
	// Otherwise it is a failed match
	int32 NumFailedMatches = 0;
	for (int32 VertexID : Mesh->VertexIndicesItr())
	{
		if (Vertices[VertexID].PairedVertex < 0)
		{
			if (FMathd::Abs(Vertices[VertexID].PlaneSignedDistance) < OnPlaneTolerance)
			{
				Vertices[VertexID].bIsSourceVertex = false;
				Vertices[VertexID].bOnPlane = true;
			}
			else
			{
				NumFailedMatches++;
			}
		}
	}

	return (NumFailedMatches == 0);
}


bool FMeshPlanarSymmetry::Initialize(FDynamicMesh3* Mesh, FDynamicMeshAABBTree3* Spatial, FFrame3d SymmetryFrameIn)
{
	return Initialize(Mesh, Spatial->GetBoundingBox(), SymmetryFrameIn);
}

bool FMeshPlanarSymmetry::Initialize(FDynamicMesh3* Mesh, const FAxisAlignedBox3d& Bounds, FFrame3d SymmetryFrameIn)
{
	constexpr bool bForceComputeVertexPairs = false;
	return Initialize(Mesh, Bounds, SymmetryFrameIn, bForceComputeVertexPairs);
}

bool FMeshPlanarSymmetry::Initialize(FDynamicMesh3* Mesh, const FAxisAlignedBox3d& Bounds, FFrame3d SymmetryFrameIn, bool bForceComputeVertexPairs)
{
	this->TargetMesh = Mesh;
	if (!Validate(Mesh))
	{
		return false;
	}

	SetErrorScale(Bounds);

	// Compute vertex hash and features
	FVector3d MeshCentroid;
	TArray<FVector3d> Features;
	ComputeMeshInfo(Mesh, Bounds, Features, MeshCentroid);

	if (!bForceComputeVertexPairs)
	{
		// Quick reject test: MeshCentroid must be on desired symmetry plane
		double CentroidPlaneSignedDist = (MeshCentroid - SymmetryFrameIn.Origin).Dot(SymmetryFrameIn.Z());
		if (FMath::Abs(CentroidPlaneSignedDist) > OnPlaneToleranceFactor * ErrorScale)
		{
			return false;
		}
	}

	// Hash the vertices.
	// TODO: For this case where we only test one plane, we could just hash the vertices w/ plane signed dist <= FMathd::ZeroTolerance ...
	// (but that would be better done inside AssignMatches, where it's clearer exactly how the hash will be used, so code would need some restructuring)
	TPointHashGrid3d<int32> VertexHash(VertexHashCellSizeFactor * ErrorScale, INDEX_NONE);
	VertexHash.Reserve(Mesh->VertexCount());
	for (int32 VID : Mesh->VertexIndicesItr())
	{
		VertexHash.InsertPointUnsafe(VID, Mesh->GetVertex(VID));
	}

	return AssignMatches(Mesh, VertexHash, Features, SymmetryFrameIn);
}


FFrame3d FMeshPlanarSymmetry::GetPositiveSideFrame(FFrame3d FromFrame) const
{
	FVector3d DeltaVec = (FromFrame.Origin - SymmetryFrame.Origin);
	double SignedDistance = DeltaVec.Dot(CachedSymmetryAxis);
	if (SignedDistance < 0)
	{
		FromFrame.Origin = GetMirroredPosition(FromFrame.Origin);
		FromFrame.Rotation = GetMirroredOrientation(FromFrame.Rotation);
	}
	return FromFrame;
}



FVector3d FMeshPlanarSymmetry::GetMirroredPosition(const FVector3d& Position) const
{
	FVector3d DeltaVec = (Position - SymmetryFrame.Origin);
	double SignedDistance = DeltaVec.Dot(CachedSymmetryAxis);
	FVector3d MirrorPosition = Position - (2 * SignedDistance * CachedSymmetryAxis);
	return MirrorPosition;
}

FVector3d FMeshPlanarSymmetry::GetMirroredAxis(const FVector3d& Axis) const
{
	double SignedDistance = Axis.Dot(CachedSymmetryAxis);
	FVector3d MirrorAxis = Axis - (2 * SignedDistance * CachedSymmetryAxis);
	return MirrorAxis;
}

FQuaterniond FMeshPlanarSymmetry::GetMirroredOrientation(const FQuaterniond& Orientation) const
{
	FVector3d Axis(Orientation.X, Orientation.Y, Orientation.Z);
	Axis = GetMirroredAxis(Axis);
	return FQuaterniond(Axis.X, Axis.Y, Axis.Z, -Orientation.W);
}


void FMeshPlanarSymmetry::GetMirrorVertexROI(const TArray<int>& VertexROI, TArray<int>& MirrorVertexROI, bool bForceSameSizeWithGaps, TFunctionRef<bool(int32 VID)> ShouldExcludeVertex) const
{
	MirrorVertexROI.Reset();
	if (bForceSameSizeWithGaps)
	{
		MirrorVertexROI.Reserve(VertexROI.Num());
	}

	for (int32 VertexID : VertexROI)
	{
		const FSymmetryVertex& Vertex = Vertices[VertexID];
		int32 MirrorVID = INDEX_NONE;
		if (Vertex.bIsSourceVertex)
		{
			MirrorVID = Vertex.PairedVertex;
		}
		else if (Vertex.bIsSourceVertex == false && Vertex.bOnPlane == false && Vertex.PairedVertex >= 0)
		{
			// The ROI already contains the mirror-vertex, we will keep it so that it can be updated later
			MirrorVID = VertexID;
		}
		if (MirrorVID == INDEX_NONE || ShouldExcludeVertex(MirrorVID))
		{
			if (bForceSameSizeWithGaps)
			{
				MirrorVertexROI.Add(INDEX_NONE);
			}
		}
		else
		{
			MirrorVertexROI.Add(MirrorVID);
		}
	}
}



void FMeshPlanarSymmetry::ApplySymmetryPlaneConstraints(const TArray<int>& VertexIndices, TArray<FVector3d>& VertexPositions) const
{
	int32 N = VertexIndices.Num();
	check(N == VertexPositions.Num());
	for (int32 k = 0; k < N; ++k)
	{
		int32 VertexID = VertexIndices[k];
		const FSymmetryVertex& Vertex = Vertices[VertexID];
		if (Vertex.bOnPlane)
		{
			VertexPositions[k] = SymmetryFrame.ToPlane(VertexPositions[k]);
		}
	}
}


namespace MeshPlanarSymmetryLocals
{
	template<bool bSkipNonSourceVertices>
	inline bool ValidateVertexSymmetry(const FDynamicMesh3& TestMesh, int32 Idx, 
		const bool bOnPlane, const bool bIsSourceVertex, const int32 PairedVertex,
		const FVector3d& Origin, const FVector3d& Normal, double OnPlaneTolerance, double MatchVertexToleranceSq)
	{
		FVector3d Pos = TestMesh.GetVertex(Idx);
		if (bOnPlane)
		{
			const double SignedDistance = (Pos - Origin).Dot(Normal);
			if (FMath::Abs(SignedDistance) > OnPlaneTolerance)
			{
				return false;
			}
		}
		else 
		{
			if constexpr (bSkipNonSourceVertices)
			{
				if (!bIsSourceVertex)
				{
					return true;
				}
			}
			const int32 PairedVertexIdx = PairedVertex;
			if (!TestMesh.IsVertex(PairedVertexIdx))
			{
				return false;
			}
			FVector3d PairedPos = TestMesh.GetVertex(PairedVertexIdx);
			const double SignedDistance = (Pos - Origin).Dot(Normal);
			FVector3d MirrorPos = Pos - (2 * SignedDistance * Normal);
			if (FVector3d::DistSquared(MirrorPos, PairedPos) > MatchVertexToleranceSq)
			{
				return false;
			}
		}
		return true;
	}
}



bool FMeshPlanarSymmetry::ValidateSymmetry(const FDynamicMesh3& TestMesh) const
{
	if (TestMesh.MaxVertexID() != Vertices.Num())
	{
		return false;
	}

	const double OnPlaneTolerance = OnPlaneToleranceFactor * ErrorScale;
	const double MatchVertexToleranceSq = (MatchVertexToleranceFactor * ErrorScale) * (MatchVertexToleranceFactor * ErrorScale);

	for (int32 Idx = 0; Idx < Vertices.Num(); ++Idx)
	{
		if (!TestMesh.IsVertex(Idx))
		{
			continue;
		}
		const FSymmetryVertex& SV = Vertices[Idx];
		if (!MeshPlanarSymmetryLocals::ValidateVertexSymmetry<true>(TestMesh, Idx, SV.bOnPlane, SV.bIsSourceVertex, SV.PairedVertex, 
			SymmetryFrame.Origin, CachedSymmetryAxis, OnPlaneTolerance, MatchVertexToleranceSq))
		{
			return false;
		}
	}
	return true;
}


bool FMeshPlanarSymmetry::ValidateSymmetry(const FDynamicMesh3& TestMesh, TConstArrayView<int32> VertexROI) const
{
	const double OnPlaneTolerance = OnPlaneToleranceFactor * ErrorScale;
	const double MatchVertexToleranceSq = (MatchVertexToleranceFactor * ErrorScale) * (MatchVertexToleranceFactor * ErrorScale);

	for (int32 Idx : VertexROI)
	{
		// All ROI vertices must be valid
		if (!TestMesh.IsVertex(Idx) || !Vertices.IsValidIndex(Idx))
		{
			return false;
		}
		const FSymmetryVertex& SV = Vertices[Idx];
		if (!MeshPlanarSymmetryLocals::ValidateVertexSymmetry<false>(TestMesh, Idx, SV.bOnPlane, SV.bIsSourceVertex, SV.PairedVertex, 
			SymmetryFrame.Origin, CachedSymmetryAxis, OnPlaneTolerance, MatchVertexToleranceSq))
		{
			return false;
		}
	}
	return true;
}


void FMeshPlanarSymmetry::ComputeSymmetryConstrainedPositions(
	const TArray<int>& SourceVertexROI,
	const TArray<int>& MirrorVertexROI,
	const TArray<FVector3d>& SourceVertexPositions,
	TArray<FVector3d>& MirrorVertexPositionsOut) const
{
	int32 NumV = SourceVertexROI.Num();
	check(MirrorVertexROI.Num() == NumV);
	check(SourceVertexPositions.Num() == NumV);

	MirrorVertexPositionsOut.SetNum(NumV, EAllowShrinking::No);

	for (int32 k = 0; k < NumV; ++k)
	{
		int32 MirrorVertexID = MirrorVertexROI[k];
		if (MirrorVertexID >= 0)
		{
			int32 SourceVertexID = SourceVertexROI[k];
			if (SourceVertexID == MirrorVertexID)
			{
				SourceVertexID = Vertices[MirrorVertexID].PairedVertex;
				MirrorVertexPositionsOut[k] = GetMirroredPosition(TargetMesh->GetVertex(SourceVertexID));
			}
			else
			{
				check(Vertices[SourceVertexID].PairedVertex == MirrorVertexID);
				MirrorVertexPositionsOut[k] = GetMirroredPosition(SourceVertexPositions[k]);
			}
		}
	}
}


void FMeshPlanarSymmetry::FullSymmetryUpdate()
{
	for (int32 vid : TargetMesh->VertexIndicesItr())
	{
		if (Vertices[vid].bIsSourceVertex)
		{
			UpdateSourceVertex(vid);
		}
		else if (Vertices[vid].bOnPlane)
		{
			UpdatePlaneVertex(vid);
		}
	}
}


void FMeshPlanarSymmetry::UpdateSourceVertex(int32 VertexID)
{
	FVector3d CurPosition = TargetMesh->GetVertex(VertexID);
	int32 MirrorVID = Vertices[VertexID].PairedVertex;
	check(TargetMesh->IsVertex(MirrorVID));
	FVector3d MirrorPosition = GetMirroredPosition(CurPosition);
	TargetMesh->SetVertex(MirrorVID, MirrorPosition);
}

void FMeshPlanarSymmetry::UpdatePlaneVertex(int32 VertexID)
{
	FVector3d CurPosition = TargetMesh->GetVertex(VertexID);
	FVector3d PlanePos = SymmetryFrame.ToPlane(CurPosition);
	TargetMesh->SetVertex(VertexID, PlanePos);
}


