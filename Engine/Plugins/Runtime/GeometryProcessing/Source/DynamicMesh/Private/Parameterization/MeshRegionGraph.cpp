// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/MeshRegionGraph.h"

#include "Async/ParallelFor.h"

using namespace UE::Geometry;


void FMeshRegionGraph::BuildFromComponents(
	const FDynamicMesh3& Mesh, 
	const FMeshConnectedComponents& Components, 
	TFunctionRef<int32(int32)> ExternalIDFunc,
	TFunctionRef<bool(int32, int32)> TrisConnectedPredicate)
{
	int32 N = Components.Num();
	Regions.SetNum(N);
	TriangleToRegionMap.Init(INDEX_NONE, Mesh.MaxTriangleID());
	TriangleNbrTris.Init(FIndex3i(INDEX_NONE, INDEX_NONE, INDEX_NONE), Mesh.MaxTriangleID());

	for (int32 k = 0; k < N; ++k)
	{
		Regions[k].Triangles = Components[k].Indices;
		Regions[k].ExternalID = ExternalIDFunc(k);

		for (int32 tid : Components[k].Indices)
		{
			TriangleToRegionMap[tid] = k;
			FIndex3i NbrTris = Mesh.GetTriNeighbourTris(tid);;
			for (int j = 0; j < 3; ++j)
			{
				if (NbrTris[j] >= 0 && TrisConnectedPredicate(tid, NbrTris[j]) == false)
				{
					NbrTris[j] = INDEX_NONE;
				}
			}
			TriangleNbrTris[tid] = NbrTris;
		}
	}

	for (int32 k = 0; k < N; ++k)
	{
		BuildNeigbours(k);
	}
}


void FMeshRegionGraph::BuildFromTriangleSets(const FDynamicMesh3& Mesh,
	const TArray<TArray<int32>>& TriangleSets,
	TFunctionRef<int32(int32)> ExternalIDFunc,
	TFunctionRef<bool(int32, int32)> TrisConnectedPredicate)
{
	int32 N = TriangleSets.Num();
	Regions.SetNum(N);
	TriangleToRegionMap.Init(INDEX_NONE, Mesh.MaxTriangleID());
	TriangleNbrTris.Init(FIndex3i(INDEX_NONE, INDEX_NONE, INDEX_NONE), Mesh.MaxTriangleID());

	for (int32 k = 0; k < N; ++k)
	{
		Regions[k].Triangles = TriangleSets[k];
		Regions[k].ExternalID = ExternalIDFunc(k);

		for (int32 tid : TriangleSets[k])
		{
			TriangleToRegionMap[tid] = k;
			FIndex3i NbrTris = Mesh.GetTriNeighbourTris(tid);
			for (int j = 0; j < 3; ++j)
			{
				if (NbrTris[j] >= 0 && TrisConnectedPredicate(tid, NbrTris[j]) == false)
				{
					NbrTris[j] = INDEX_NONE;
				}
			}
			TriangleNbrTris[tid] = NbrTris;
		}
	}

	for (int32 k = 0; k < N; ++k)
	{
		BuildNeigbours(k);
	}
}



TArray<int32> FMeshRegionGraph::GetNeighbours(int32 RegionIdx) const
{
	TArray<int32> Nbrs;
	if (IsRegion(RegionIdx))
	{
		for (const FNeighbour& Nbr : Regions[RegionIdx].Neighbours)
		{
			Nbrs.Add(Nbr.RegionIndex);
		}
	}
	return Nbrs;
}


bool FMeshRegionGraph::AreRegionsConnected(int32 RegionAIndex, int32 RegionBIndex) const
{
	if (IsRegion(RegionAIndex) == false || IsRegion(RegionBIndex) == false) return false;

	for (const FNeighbour& Nbr : Regions[RegionAIndex].Neighbours)
	{
		if (Nbr.RegionIndex == RegionBIndex)
		{
			return true;
		}
	}
	return false;
}



bool FMeshRegionGraph::MergeSmallRegions(int32 SmallThreshold,
										 TFunctionRef<float(int32 SmallRgnIdx, int32 NbrRgnIdx)> RegionSimilarityFunc)
{
	bool bMergedAny = false;
	int32 UseSmallThreshold = 1;
	while (UseSmallThreshold < SmallThreshold || MergeSmallRegionsPass(UseSmallThreshold, RegionSimilarityFunc))
	{
		bMergedAny = true;
		UseSmallThreshold = FMath::Min(UseSmallThreshold + 1, SmallThreshold);
	}
	return bMergedAny;
}



bool FMeshRegionGraph::MergeSmallRegionsPass(int32 SmallThreshold,
										 TFunctionRef<float(int32 SmallRgnIdx, int32 NbrRgnIdx)> RegionSimilarityFunc)
{
	bool bMergedAny = false;

	TArray<int32> SmallRegions;
	for (int32 k = 0; k < MaxRegionIndex(); ++k)
	{
		if (IsRegion(k) && Regions[k].Triangles.Num() <= SmallThreshold)
		{
			SmallRegions.Add(k);
		}
	}
	if (SmallRegions.Num() == 0)
	{
		return false;
	}

	SmallRegions.Sort([this](int a, int b) { return Regions[a].Triangles.Num() < Regions[b].Triangles.Num(); });

	struct FMatch 
	{
		float Score = 0.0f;
		int Index = INDEX_NONE;
	};

	for (int32 j = 0; j < SmallRegions.Num(); ++j)
	{
		int32 SmallRegionIdx = SmallRegions[j];
		if (IsRegion(SmallRegionIdx) == false) continue;
		const FRegion& SmallRegion = Regions[SmallRegionIdx];
		if (SmallRegion.Triangles.Num() > SmallThreshold)
		{
			continue;
		}

		TArray<FMatch> Matches;
		for (const FNeighbour& Nbr : SmallRegion.Neighbours)
		{
			float Score = RegionSimilarityFunc(SmallRegionIdx, Nbr.RegionIndex);
			Matches.Add(FMatch{ Score, Nbr.RegionIndex });
		}
		if (Matches.Num() == 0)
		{
			continue;
		}

		Matches.Sort([](const FMatch& A, const FMatch& B) { return A.Score > B.Score; });

		if (MergeRegion(SmallRegionIdx, Matches[0].Index))
		{
			bMergedAny = true;
		}
	}

	return bMergedAny;
}




bool FMeshRegionGraph::MergeRegion(int32 RemoveRgnIdx, int32 MergeToRgnIdx)
{
	if (AreRegionsConnected(RemoveRgnIdx, MergeToRgnIdx) == false)
	{
		ensure(false);
		return false;
	}
	return MergeRegionUnsafe(RemoveRgnIdx, MergeToRgnIdx);
}


bool FMeshRegionGraph::MergeRegionUnsafe(int32 RemoveRgnIdx, int32 MergeToRgnIdx)
{
	FRegion& RemoveRegion = Regions[RemoveRgnIdx];
	FRegion& AppendToRegion = Regions[MergeToRgnIdx];

	// append triangles to the new region
	AppendToRegion.Triangles.Reserve(AppendToRegion.Triangles.Num() + RemoveRegion.Triangles.Num());
	for (int32 tid : RemoveRegion.Triangles)
	{
		AppendToRegion.Triangles.Add(tid);
		TriangleToRegionMap[tid] = MergeToRgnIdx;
	}
	RemoveRegion.Triangles.Reset();
	RemoveRegion.bValid = false;

	// transfer boundary flags
	if (RemoveRegion.bIsOnMeshBoundary)
	{
		AppendToRegion.bIsOnMeshBoundary = true;
	}
	if (RemoveRegion.bIsOnROIBoundary)
	{
		AppendToRegion.bIsOnROIBoundary = true;
	}
	// remove RemoveRgnIdx from the AppendToRegion's neighbors list
	AppendToRegion.Neighbours.RemoveAll([RemoveRgnIdx](const FNeighbour& Nbr)
		{
			return Nbr.RegionIndex == RemoveRgnIdx;
		});

	// Mapping from RegionIndex -> Index of Region in AppendToRegion.Neighbors
	TMap<int32, int32> AppendToRegionNbrTransferMap;
	// Populate transfer map with relevant region keys -- the removed region neighbors, which will need updating
	for (FNeighbour& Nbr : RemoveRegion.Neighbours)
	{
		if (Nbr.RegionIndex != MergeToRgnIdx)
		{
			AppendToRegionNbrTransferMap.Add(Nbr.RegionIndex, INDEX_NONE);
		}
	}
	// Linear pass to get indices for neighbors that are already in the merge-to-region's neighbor array
	for (int32 AppendNbrIdx = 0; AppendNbrIdx < AppendToRegion.Neighbours.Num(); ++AppendNbrIdx)
	{
		int32 RegionIdx = AppendToRegion.Neighbours[AppendNbrIdx].RegionIndex;
		if (int32* FoundTransferIdx = AppendToRegionNbrTransferMap.Find(RegionIdx))
		{
			*FoundTransferIdx = AppendNbrIdx;
		}
	}

	// update neighbor info for all removed neighbors -- passing counts to the merged-to region
	for (FNeighbour& Nbr : RemoveRegion.Neighbours)
	{
		// if this is the merged-to region, skip it (we already removed the back-link for it above)
		if (Nbr.RegionIndex == MergeToRgnIdx)
		{
			continue;
		}

		FRegion& NbrRegion = Regions[Nbr.RegionIndex];
		TArray<FNeighbour>& NbrNbrs = NbrRegion.Neighbours;

		// Find the removed and kept regions in the neighbor list
		int32 NbrsRemoveIdx = INDEX_NONE;
		int32 NbrsKeepIdx = INDEX_NONE;
		for (int32 Idx = 0; Idx < NbrNbrs.Num(); ++Idx)
		{
			if (NbrNbrs[Idx].RegionIndex == RemoveRgnIdx)
			{
				NbrsRemoveIdx = Idx;
			}
			else if (NbrNbrs[Idx].RegionIndex == MergeToRgnIdx)
			{
				NbrsKeepIdx = Idx;
			}
		}
		if (!ensure(NbrsRemoveIdx != INDEX_NONE)) // we should always find the removed idx, by symmetry
		{
			continue;
		}

		// make sure the neighbor is connected to the merged-to region & transfer counts
		const FNeighbour& Removed = NbrNbrs[NbrsRemoveIdx];
		// Find the relevant entry in the merged-to region's neighbors array, to symmetrically update
		int32 NbrInMergeRegionIdx = AppendToRegionNbrTransferMap[Nbr.RegionIndex];
		// ... or create it if needed
		if (NbrInMergeRegionIdx == INDEX_NONE)
		{
			NbrInMergeRegionIdx = AppendToRegion.Neighbours.Add(
				FNeighbour
				{
					.RegionIndex = Nbr.RegionIndex,
					.Count = Removed.Count
				}
			);
		}
		else
		{
			AppendToRegion.Neighbours[NbrInMergeRegionIdx].Count += Removed.Count;
		}
		if (NbrsKeepIdx == INDEX_NONE)
		{
			// it wasn't previously connected to the merged-to region; can directly transfer the neighbor info
			NbrNbrs[NbrsRemoveIdx].RegionIndex = MergeToRgnIdx;
		}
		else
		{
			// it was previously connected; can add increase the connection count by that of the removed region
			NbrNbrs[NbrsKeepIdx].Count += Removed.Count;
			NbrNbrs.RemoveAt(NbrsRemoveIdx);
		}
	}
	RemoveRegion.Neighbours.Empty();

	return true;
}



bool FMeshRegionGraph::OptimizeBorders(int32 MaxRounds)
{
	auto GetSwapNbrRegionIndex = [this](int32 tid) -> int32
	{
		int32 Index = TriangleToRegionMap[tid];
		FIndex3i NbrTris = TriangleNbrTris[tid];
		int32 NbrCount = 0;
		int32 Nbrs[3];
		for (int32 j = 0; j < 3; ++j)
		{
			if (NbrTris[j] >= 0)
			{
				int32 NbrRegionIndex = TriangleToRegionMap[NbrTris[j]];
				if (IsRegion(NbrRegionIndex) && NbrRegionIndex != Index)
				{
					Nbrs[NbrCount++] = NbrRegionIndex;
				}
			}
		}
		if ((NbrCount == 2 && Nbrs[0] == Nbrs[1]) || (NbrCount == 3 && Nbrs[0] == Nbrs[1] && Nbrs[0] == Nbrs[2]))
		{
			return Nbrs[0];
		}

		return INDEX_NONE;
	};

	bool bModified = false;

	bool bDone = false;
	int32 RoundCounter = 0;
	TArray<FIndex2i> PotentialSwaps;
	while (bDone == false && RoundCounter++ < MaxRounds)
	{
		PotentialSwaps.Reset();

		for (int32 tid = 0, Num = TriangleToRegionMap.Num(); tid < Num; ++tid)
		{
			if (TriangleToRegionMap[tid] != INDEX_NONE)
			{
				const int32 SwapToNbrIndex = GetSwapNbrRegionIndex(tid);
				if (IsRegion(SwapToNbrIndex))
				{
					PotentialSwaps.Add(FIndex2i(tid, SwapToNbrIndex));
				}
			}
		}
		bDone = (PotentialSwaps.Num() == 0);

		for (const FIndex2i Swap : PotentialSwaps)
		{
			if (GetSwapNbrRegionIndex(Swap.A) == Swap.B)
			{
				TriangleToRegionMap[Swap.A] = Swap.B;
				bModified = true;
			}
		}
	}

	if (bModified)
	{
		// Rebuild region triangles

		for (FRegion& Region : Regions)
		{
			Region.Triangles.Reset();
		}

		for (int32 tid = 0, Num = TriangleToRegionMap.Num(); tid < Num; ++tid)
		{
			if (TriangleToRegionMap[tid] != INDEX_NONE)
			{
				Regions[TriangleToRegionMap[tid]].Triangles.Add(tid);
			}
		}
	}

	return bModified;
}





void FMeshRegionGraph::BuildNeigbours(int32 RegionIdx)
{
	FRegion& Region = Regions[RegionIdx];
	Region.Neighbours.Reset();

	TMap<int32, int32> NeighborMap;

	for (int32 tid : Region.Triangles)
	{
		int32 RegionIndex = TriangleToRegionMap[tid];
		check(RegionIndex == RegionIdx);

		FIndex3i Nbrs = TriangleNbrTris[tid];
		for (int32 j = 0; j < 3; ++j)
		{
			if (Nbrs[j] == FDynamicMesh3::InvalidID)
			{
				Region.bIsOnMeshBoundary = true;
			}
			else
			{
				int32 NbrRegionIndex = TriangleToRegionMap[Nbrs[j]];
				if (NbrRegionIndex == INDEX_NONE)
				{
					Region.bIsOnROIBoundary = true;
				}
				else
				{
					if (NbrRegionIndex != RegionIndex)
					{
						int32* FoundNbrIdx = NeighborMap.Find(NbrRegionIndex);
						if (!FoundNbrIdx)
						{
							FNeighbour Nbr
							{
								.RegionIndex = NbrRegionIndex,
								.Count = 1
							};
							int32 NbrIdx = Region.Neighbours.Add(Nbr);
							NeighborMap.Add(NbrRegionIndex, NbrIdx);
						}
						else
						{
							Region.Neighbours[*FoundNbrIdx].Count++;
						}
					}
				}
			}
		}
	}
}