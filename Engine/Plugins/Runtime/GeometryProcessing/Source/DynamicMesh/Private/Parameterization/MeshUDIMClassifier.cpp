// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/MeshUDIMClassifier.h"
#include "Spatial/SparseDynamicOctree3.h"
#include "Selections/MeshConnectedComponents.h"

using namespace UE::Geometry;

namespace UDIMClassifierLocals
{
	FVector2f InternalUVToExternalUV(const FVector2f& UV)
	{
		return FVector2f(UV.X, 1 - UV.Y);
	}
}

FDynamicMeshUDIMClassifier::FDynamicMeshUDIMClassifier(const FDynamicMeshUVOverlay* UVOverlayIn, TOptional<TArray<int32>> SelectionIn)
{
	UVOverlay = UVOverlayIn;
	Selection = SelectionIn;
	ClassifyUDIMs();
}

TArray<FVector2i> FDynamicMeshUDIMClassifier::ActiveTiles() const
{
	TArray<FVector2i> Keys;
	UDIMs.GetKeys(Keys);
	return Keys;
}

TArray<int32> FDynamicMeshUDIMClassifier::TidsForTile(FVector2i TileIndexIn) const
{	
	if (ensure(UDIMs.Contains(TileIndexIn)))
	{
		return *(UDIMs.Find(TileIndexIn));
	}
	else
	{
		return TArray<int32>();
	}
}

FVector2i FDynamicMeshUDIMClassifier::ClassifyTrianglesToUDIM(const FDynamicMeshUVOverlay* UVOverlay, TArray<int32> Tids)
{
	auto UVTriangleToUDIM = [UVOverlay](int32 Tid)
	{
		FVector2i UDIM;
		FVector2f Vertex0, Vertex1, Vertex2;
		UVOverlay->GetTriElements(Tid, Vertex0, Vertex1, Vertex2);
		FVector2f BaryCenter = (Vertex0 + Vertex1 + Vertex2) / 3.0f;
		BaryCenter = UDIMClassifierLocals::InternalUVToExternalUV(BaryCenter);
		UDIM = ClassifyPointToUDIM(BaryCenter);		
		return UDIM;
	};

	TMap<FVector2i, int32> UDIMCount;
	for (int32 Tid : Tids)
	{
		FVector2i UDIM = UVTriangleToUDIM(Tid);
		UDIMCount.FindOrAdd(UDIM, 0)++;
	}
	FVector2i MaximumUDIM;
	int32 MaximumUDIMCount = 0;
	for (auto UDIMEntry : UDIMCount)
	{
		if (UDIMEntry.Value > MaximumUDIMCount)
		{
			MaximumUDIMCount = UDIMEntry.Value;
			MaximumUDIM = UDIMEntry.Key;
		}
	}
	return MaximumUDIM;
}

FVector2i FDynamicMeshUDIMClassifier::ClassifyBoundingBoxToUDIM(const FDynamicMeshUVOverlay* UVOverlay, const FAxisAlignedBox2d& BoundingBox)
{
	FVector2i MinUDIM = ClassifyPointToUDIM((FVector2f)BoundingBox.Min);
	double MaxOverlapArea = 0.0;
	FVector2i SelectedUDIM = MinUDIM;

	// We only really need to test the minimum UDIM tile and one greater in each dimension.
	// Either the BoundingBox will be fully overlapping in any or none of these, but
	// we don't need to find the largest indexed UDIM with full overlap, just the first.
	for (int32 UDIM_X = MinUDIM.X; UDIM_X <= MinUDIM.X+1; ++UDIM_X)
	{
		for (int32 UDIM_Y = MinUDIM.Y; UDIM_Y <= MinUDIM.Y+1; ++UDIM_Y)
		{
			FAxisAlignedBox2d TestUDIMBoundingBox(FVector2d(UDIM_X, UDIM_Y), FVector2d(UDIM_X + 1, UDIM_Y + 1));
			double OverlapArea = BoundingBox.Intersect(TestUDIMBoundingBox).Area();
			if (OverlapArea > MaxOverlapArea)
			{
				SelectedUDIM = FVector2i(UDIM_X, UDIM_Y);
				MaxOverlapArea = OverlapArea;
			}
		}
	}

	return SelectedUDIM;
}


FVector2i FDynamicMeshUDIMClassifier::ClassifyPointToUDIM(const FVector2f& UVPoint)
{
	FVector2i UDIM;
	UDIM.X = FMath::FloorToInt32(UVPoint.X);
	UDIM.Y = FMath::FloorToInt32(UVPoint.Y);
	return UDIM;
}

void FDynamicMeshUDIMClassifier::ClassifyUDIMs()
{
	int32 CurrentUDIMIndex = 0;
	const FDynamicMesh3* Mesh = UVOverlay->GetParentMesh();

	auto UVIslandPredicate = [this](int32 Triangle0, int32 Triangle1)
	{
		return UVOverlay->AreTrianglesConnected(Triangle0, Triangle1);
	};

	FMeshConnectedComponents UVComponents(Mesh);
	if (Selection.IsSet())
	{
		UVComponents.FindConnectedTriangles(Selection.GetValue(), UVIslandPredicate);
	}
	else
	{
		UVComponents.FindConnectedTriangles(UVIslandPredicate);
	}

	for (int32 Cid = 0; Cid < UVComponents.Num(); ++Cid)
	{
		FVector2i UDIM = ClassifyTrianglesToUDIM(UVOverlay, UVComponents.GetComponent(Cid).Indices);
		UDIMs.FindOrAdd(UDIM).Append(UVComponents.GetComponent(Cid).Indices);
	}

	return;

}
