// Copyright Epic Games, Inc. All Rights Reserved.


#include "Parameterization/MeshUVPacking.h"
#include "DynamicSubmesh3.h"
#include "Selections/MeshConnectedComponents.h"

// the generic GeometricObjects version
#include "Parameterization/UVPacking.h"

using namespace UE::Geometry;

namespace UE::Geometry
{
	struct FUVOverlayView : public FUVPacker::IUVMeshView
	{
		FDynamicMesh3* Mesh;
		FDynamicMeshUVOverlay* UVOverlay;

		FUVOverlayView(FDynamicMeshUVOverlay* UVOverlay) : UVOverlay(UVOverlay)
		{
			Mesh = UVOverlay->GetParentMesh();
		}

		virtual FIndex3i GetTriangle(int32 TID) const
		{
			return Mesh->GetTriangle(TID);
		}

		virtual FIndex3i GetUVTriangle(int32 TID) const
		{
			return UVOverlay->GetTriangle(TID);
		}

		virtual FVector3d GetVertex(int32 VID) const
		{
			return Mesh->GetVertex(VID);
		}

		virtual FVector2f GetUV(int32 EID) const
		{
			return UVOverlay->GetElement(EID);
		}

		virtual void SetUV(int32 EID, FVector2f UV)
		{
			return UVOverlay->SetElement(EID, UV);
		}
	};
}

FDynamicMeshUVPacker::FDynamicMeshUVPacker(FDynamicMeshUVOverlay* UVOverlayIn)
{
	UVOverlay = UVOverlayIn;
}

FDynamicMeshUVPacker::FDynamicMeshUVPacker(FDynamicMeshUVOverlay* UVOverlayIn, TUniquePtr<TArray<int32>>&& TidsToRepackIn)
{
	UVOverlay = UVOverlayIn;
	TidsToRepack = MoveTemp(TidsToRepackIn);
}

bool FDynamicMeshUVPacker::StandardPack()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshUVPacker_StandardPack);
	
	FUVPacker Packer;
	Packer.bPreserveScale = bPreserveScale;
	Packer.bPreserveRotation = bPreserveRotation;
	Packer.bAllowFlips = bAllowFlips;
	Packer.GutterSize = GutterSize;
	Packer.TextureResolution = TextureResolution;
	Packer.bScaleIslandsByWorldSpaceTexelRatio = bScaleIslandsByWorldSpaceTexelRatio;

	FUVOverlayView MeshView(UVOverlay);
	FMeshConnectedComponents UVComponents = CollectUVIslandsToPack(MeshView);

	return Packer.StandardPack(&MeshView, UVComponents.Num(), [&UVComponents](int Idx, TArray<int32>& Island)
		{
			Island = UVComponents.GetComponent(Idx).Indices;
		});
}

bool FDynamicMeshUVPacker::StackPack()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshUVPacker_StackPack);
	
	FUVPacker Packer;
	Packer.bPreserveScale = bPreserveScale;
	Packer.bPreserveRotation = bPreserveRotation;
	Packer.bAllowFlips = bAllowFlips;
	Packer.GutterSize = GutterSize;
	Packer.TextureResolution = TextureResolution;
	Packer.bScaleIslandsByWorldSpaceTexelRatio = bScaleIslandsByWorldSpaceTexelRatio;

	FUVOverlayView MeshView(UVOverlay);
	FMeshConnectedComponents UVComponents = CollectUVIslandsToPack(MeshView);

	return Packer.StackPack(&MeshView, UVComponents.Num(), [&UVComponents](int Idx, TArray<int32>& Island)
		{
			Island = UVComponents.GetComponent(Idx).Indices;
		});
}

FMeshConnectedComponents FDynamicMeshUVPacker::CollectUVIslandsToPack(const FUVOverlayView& MeshView)
{
	auto UVIslandPredicate = [&MeshView](int32 Triangle0, int32 Triangle1)
	{
		return MeshView.UVOverlay->AreTrianglesConnected(Triangle0, Triangle1);
	};

	FMeshConnectedComponents UVComponents(MeshView.Mesh);
	if (TidsToRepack)
	{
		UVComponents.FindConnectedTriangles(*TidsToRepack, UVIslandPredicate);
	}
	else
	{
		UVComponents.FindConnectedTriangles(UVIslandPredicate);
	}

	return UVComponents;
}