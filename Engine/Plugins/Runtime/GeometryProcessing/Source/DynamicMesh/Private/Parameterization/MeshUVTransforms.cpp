// Copyright Epic Games, Inc. All Rights Reserved.


#include "Parameterization/MeshUVTransforms.h"
#include "Math/RandomStream.h"

using namespace UE::Geometry;

void UE::MeshUVTransforms::RecenterScale(FDynamicMeshUVOverlay* UVOverlay, const TArray<int32>& UVElementIDs,
	EIslandPositionType NewPosition, double UVScale)
{
	FAxisAlignedBox2d UVBounds(FAxisAlignedBox2d::Empty());
	FVector2d InitialTranslation = FVector2d::Zero();
	if (NewPosition != EIslandPositionType::CurrentPosition)
	{
		for (int32 elemid : UVElementIDs)
		{
			FVector2d UV = (FVector2d)UVOverlay->GetElement(elemid);
			UVBounds.Contain(UV);
		}

		if (NewPosition == EIslandPositionType::MinBoxCornerToOrigin)
		{
			InitialTranslation = -UVBounds.Min;
		}
		else if (NewPosition == EIslandPositionType::CenterToOrigin)
		{
			InitialTranslation = -UVBounds.Center();
		}
	}

	for (int32 elemid : UVElementIDs)
	{
		FVector2d UV = (FVector2d)UVOverlay->GetElement(elemid);
		FVector2d NewUV = (UV + InitialTranslation) * UVScale;
		UVOverlay->SetElement(elemid, (FVector2f)NewUV);
	}
}





template<typename EnumerableType>
void FitToBox_Internal(FDynamicMeshUVOverlay* UVOverlay, EnumerableType UVElementIDs, const FAxisAlignedBox2d& TargetBox, bool bUniformScale)
{
	FAxisAlignedBox2d UVBounds(FAxisAlignedBox2d::Empty());
	for (int32 elemid : UVElementIDs)
	{
		FVector2d UV = (FVector2d)UVOverlay->GetElement(elemid);
		UVBounds.Contain(UV);
	}

	FVector2d CurCenter = UVBounds.Center();
	FVector2d TargetCenter = TargetBox.Center();

	double ScaleX = TargetBox.Width() / UVBounds.Width();
	double ScaleY = TargetBox.Height() / UVBounds.Height();
	if (bUniformScale)
	{
		double RatioX = (ScaleX < 1) ? (1.0 / ScaleX) : ScaleX;
		double RatioY = (ScaleY < 1) ? (1.0 / ScaleY) : ScaleY;
		if (RatioY > RatioX)
		{
			ScaleX = ScaleY;
		}
		else
		{
			ScaleY = ScaleX;
		}
	}


	for (int32 elemid : UVElementIDs)
	{
		FVector2d UV = (FVector2d)UVOverlay->GetElement(elemid);
		double NewX = (UV.X - CurCenter.X) * ScaleX + TargetCenter.X;
		double NewY = (UV.Y - CurCenter.Y) * ScaleY + TargetCenter.Y;
		UVOverlay->SetElement(elemid, FVector2f((float)NewX, (float)NewY));
	}
}



void UE::MeshUVTransforms::FitToBox(FDynamicMeshUVOverlay* UVOverlay, const TArray<int32>& UVElementIDs, const FAxisAlignedBox2d& TargetBox, bool bUniformScale)
{
	FitToBox_Internal(UVOverlay, UVElementIDs, TargetBox, bUniformScale);
}



void UE::MeshUVTransforms::FitToBox(FDynamicMeshUVOverlay* UVOverlay, const FAxisAlignedBox2d& Box, bool bUniformScale)
{
	FitToBox_Internal(UVOverlay, UVOverlay->ElementIndicesItr(), Box, bUniformScale);
}





void UE::MeshUVTransforms::MakeSeamsDisjoint(FDynamicMeshUVOverlay* UVOverlay)
{
	FRandomStream Random(31337);

	// return random 2D unit vector scaled by Jitterscale
	auto GetRandomUVJitter = [&Random](float JitterScale)
	{
		FVector2f Result;
		float Magnitude;
		do {
			Result.X = Random.GetFraction() * 2.0f - 1.0f;
			Result.Y = Random.GetFraction() * 2.0f - 1.0f;
			Magnitude = Result.SquaredLength();
		} while (Magnitude > 1.0f || Magnitude < FMathf::ZeroTolerance);
		Normalize(Result);
		return JitterScale * Result;
	};

	const FDynamicMesh3* Mesh = UVOverlay->GetParentMesh();

	// mapping from vertex ID to linear index, only defined for seam verts
	TArray<int32> SeamVerticesMap;
	SeamVerticesMap.Init(-1, Mesh->MaxVertexID());

	// make linear index for all seam vertices
	int32 SeamCounter = 0;
	for (int32 vid : Mesh->VertexIndicesItr())
	{
		if (UVOverlay->IsSeamVertex(vid))
		{
			SeamVerticesMap[vid] = SeamCounter++;
		}
	}

	// per-seam arrays for collecting unique element positions
	TArray<TArray<FVector2f, TInlineAllocator<8>>> SeamUVPositions;
	SeamUVPositions.SetNum(SeamCounter);

	// for each element at a seam vertex, we ensure it has a unique position by
	// adding random offsets.
	for (int32 elemid : UVOverlay->ElementIndicesItr())
	{
		int32 ParentVtxID = UVOverlay->GetParentVertex(elemid);
		if (ParentVtxID == FDynamicMesh3::InvalidID)
		{
			checkSlow(false);
			continue;
		}
		int32 SeamListIndex = SeamVerticesMap[ParentVtxID];
		if (SeamListIndex == -1)
		{
			continue;
		}

		// if current position exists, add increasing amounts of random offset until
		// the position becomes unique
		FVector2f UVPos = UVOverlay->GetElement(elemid);
		int32 TryCounter = 0;
		float JitterScale = 0.0001f;
		while (SeamUVPositions[SeamListIndex].Contains(UVPos) && TryCounter++ < 25)
		{
			UVPos = UVOverlay->GetElement(elemid) + GetRandomUVJitter(JitterScale);
			JitterScale *= 2.0f;
		}

		SeamUVPositions[SeamListIndex].Add(UVPos);
		UVOverlay->SetElement(elemid, UVPos);
	}
}