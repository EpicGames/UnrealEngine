// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/PatchBasedMeshUVGenerator.h"

#include "DynamicMesh/MeshNormals.h"
#include "DynamicSubmesh3.h"
#include "Sampling/NormalHistogram.h"

#include "Polygroups/PolygroupsGenerator.h"
#include "Parameterization/MeshLocalParam.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/MeshRegionGraph.h"
#include "Parameterization/MeshDijkstra.h"

#include "Async/ParallelFor.h"

using namespace UE::Geometry;


#define LOCTEXT_NAMESPACE "PatchBasedMeshUVGenerator"



FGeometryResult FPatchBasedMeshUVGenerator::AutoComputeUVs(
	FDynamicMesh3& TargetMesh,
	FDynamicMeshUVOverlay& TargetUVOverlay,
	FProgressCancel* Progress)
{
	FGeometryResult ResultInfo = FGeometryResult(EGeometryResultType::InProgress);

	// 
	// Step 1: decompose the input mesh into small patches
	//
	FMeshConnectedComponents InitialPatches(&TargetMesh);
	bool bPatchesOK = ComputeInitialMeshPatches(TargetMesh, InitialPatches, Progress);
	if (!bPatchesOK)
	{
		ResultInfo.SetFailed(LOCTEXT("InitialPatchesFailed", "Failed to compute initial patch set"));
		return ResultInfo;
	}
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return ResultInfo;
	}


	// 
	// Step 2: merge the small patches into bigger patches, which will become UV islands
	//
	TArray<TArray<int32>> UVIslandTriangleSets;
	bool bIslandsOK = ComputeIslandsByRegionMerging(TargetMesh, TargetUVOverlay, InitialPatches, UVIslandTriangleSets, Progress);
	if (!bIslandsOK)
	{
		ResultInfo.SetFailed(LOCTEXT("IslandMergingFailed", "Failed to merge UV islands"));
		return ResultInfo;
	}
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return ResultInfo;
	}


	// 
	// Step 3: compute UVs for the UV islands
	//
	TArray<bool> bIslandUVsValid;
	int32 NumSolvesFailed = ComputeUVsFromTriangleSets(TargetMesh, TargetUVOverlay, UVIslandTriangleSets, bIslandUVsValid, Progress);
	if (NumSolvesFailed > 0)
	{
		ResultInfo.AddWarning(FGeometryWarning(0, LOCTEXT("PartialSolvesFailed", "Failed to compute UVs for some UV islands")));
	}
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return ResultInfo;
	}


	FDynamicMeshUVEditor UVEditor(&TargetMesh, &TargetUVOverlay);

	// 
	// Step 4 (Optional): Rotate the UV islands to optimize their orientation, to (hopefully) improve packing
	//
	if (bAutoAlignPatches)
	{
		ParallelFor(UVIslandTriangleSets.Num(), [&](int32 k)
		{
			if (Progress && Progress->Cancelled())
			{
				return;
			}
			if (bIslandUVsValid[k])
			{
				UVEditor.AutoOrientUVArea(UVIslandTriangleSets[k]);
			}
		});
	}
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return ResultInfo;
	}


	// 
	// Step 5 (Optional): Automatically pack the UV islands into the unit UV rectangle
	//
	if (bAutoPack)
	{
		bool bPackingSuccess = UVEditor.QuickPack(PackingTextureResolution, PackingGutterWidth);
		if (!bPackingSuccess)
		{
			ResultInfo.AddWarning(FGeometryWarning(0, LOCTEXT("IslandPackingFailed", "Failed to pack UV islands")));
		}
	}
	if (ResultInfo.CheckAndSetCancelled(Progress))
	{
		return ResultInfo;
	}

	ResultInfo.SetSuccess();
	return ResultInfo;
}

static TAutoConsoleVariable<int32> CVarEnableAutoUVAreaDensitySampling(
	TEXT("modeling.DisableAutoUVAreaDensitySampling"),
	0,
	TEXT("If set, return the behavior of AutoUV's PatchBuilder algorithm to the legacy behavior."));

bool FPatchBasedMeshUVGenerator::ComputeInitialMeshPatches(
	FDynamicMesh3& TargetMesh,
	FMeshConnectedComponents& InitialComponentsOut,
	FProgressCancel* Progress)
{
	FPolygroupsGenerator Generator(&TargetMesh);
	Generator.bCopyToMesh = false;

	Generator.MinGroupSize = this->MinPatchSize;
	FPolygroupsGenerator::EWeightingType WeightType = (this->bNormalWeightedPatches) ?
		FPolygroupsGenerator::EWeightingType::NormalDeviation : FPolygroupsGenerator::EWeightingType::None;
	FVector3d GeneratorParams(this->PatchNormalWeight, 1.0, 1.0);

	bool bUseAreaDensitySampling = (CVarEnableAutoUVAreaDensitySampling.GetValueOnAnyThread() == 0);
	bool bOK = false;
	if (bUseAreaDensitySampling)
	{
		bOK = Generator.FindPolygroupsFromAreaDensityPointSampling(this->TargetPatchCount, WeightType, GeneratorParams, this->GroupConstraint);
	}
	else
	{
		bOK = Generator.FindPolygroupsFromFurthestPointSampling(this->TargetPatchCount, WeightType, GeneratorParams, this->GroupConstraint);
	}

	if (!bOK)
	{
		return false;
	}
	if (Progress && Progress->Cancelled())
	{
		return false;
	}
	InitialComponentsOut.InitializeFromTriangleComponents(Generator.FoundPolygroups, true, false);

	return true;
}






struct FUVAreaMetrics
{
	double AvgSqrAreaDistortion = 1.0;
	FVector3d AreaWeightedNormal = FVector3d::Zero();
	double NormalSpread = 1.0;

	FAxisAlignedBox2d BoundsUV = FAxisAlignedBox2d::Empty();
	FAxisAlignedBox3d Bounds3D = FAxisAlignedBox3d::Empty();

	double SurfAreaSum = 0.0;
	double UVAreaSum = 0.0;
	double UVPerimeter = 0.0;
	double Compactness = 1.0;

	static FUVAreaMetrics Invalid() {
		FUVAreaMetrics Tmp; Tmp.AvgSqrAreaDistortion = TNumericLimits<double>::Max(); return Tmp;
	}
};


static FUVAreaMetrics Compute_UV_Area_Metrics(const FDynamicMesh3* Mesh, const TArray<int32>& Triangles, int32 NormalSmoothingRounds, double NormalSmoothingAlpha)
{
	FDynamicSubmesh3 SubmeshCalc(Mesh, Triangles, (int)EMeshComponents::None, false);
	FDynamicMesh3& Submesh = SubmeshCalc.GetSubmesh();
	FMeshNormals::QuickComputeVertexNormals(Submesh);
	if (NormalSmoothingRounds > 0)
	{
		FMeshNormals::SmoothVertexNormals(Submesh, NormalSmoothingRounds, NormalSmoothingAlpha);
	}


	TArray<FVector2d> SeedPoints;
	TArray<bool> BoundaryVerts;
	BoundaryVerts.Init(false, Submesh.MaxVertexID());
	for (int32 eid : Submesh.EdgeIndicesItr())
	{
		if (Submesh.IsBoundaryEdge(eid))
		{
			FIndex2i BoundaryV = Submesh.GetEdgeV(eid);
			for (int32 j = 0; j < 2; ++j)
			{
				if (BoundaryVerts[BoundaryV[j]] == false)
				{
					BoundaryVerts[BoundaryV[j]] = true;
					SeedPoints.Add(FVector2d(BoundaryV[j], 0.0));
				}
			}
		}
	}

	FFrame3d SeedFrame;
	int32 SeedVID = *Submesh.VertexIndicesItr().begin();
	if (SeedPoints.Num() > 0)
	{
		TMeshDijkstra<FDynamicMesh3> Dijkstra(&Submesh);
		Dijkstra.ComputeToMaxDistance(SeedPoints, TNumericLimits<float>::Max());
		SeedVID = Dijkstra.GetMaxGraphDistancePointID();
		if (ensure(Submesh.IsVertex(SeedVID)) == false)
		{
			SeedVID = *Submesh.VertexIndicesItr().begin();
		}
	}

	FVector3d Normal = FMeshNormals::ComputeVertexNormal(Submesh, SeedVID);
	SeedFrame = Submesh.GetVertexFrame(SeedVID, false, &Normal);

	TMeshLocalParam<FDynamicMesh3> Param(&Submesh);
	Param.ParamMode = ELocalParamTypes::ExponentialMapUpwindAvg;
	Param.ComputeToMaxDistance(SeedVID, SeedFrame, TNumericLimits<float>::Max());

	double SumAreaDistortion = 0.0;
	double SumDistortionsSqr = 0.0;
	double Count = 0;
	double MaxAreaDistortion = 1.0;

	const double ClampAreaDistortionMetric = 100.0;

	FUVAreaMetrics Metrics;

	TNormalHistogram<double> NormalHistogram(128);

	// TODO:
	//   - compute bounding box/circle, compare to UV area (convexity metric - could also consider in 3D?)

	for (int32 tid : Submesh.TriangleIndicesItr())
	{
		FIndex3i Tri = Submesh.GetTriangle(tid);

		FVector3d TriNormal, Centroid; double Area3D;
		Submesh.GetTriInfo(tid, TriNormal, Area3D, Centroid);

		Metrics.AreaWeightedNormal += Area3D * TriNormal;
		Metrics.SurfAreaSum += Area3D;
		Metrics.Bounds3D.Contain(Submesh.GetTriBounds(tid));
		NormalHistogram.Count(TriNormal, Area3D);

		FVector2d UVTri[3];
		UVTri[0] = Param.GetUV(Tri.A);
		UVTri[1] = Param.GetUV(Tri.B);
		UVTri[2] = Param.GetUV(Tri.C);
		// TODO: this param seems to emit flipped areas ?!?
		double SignedArea2D = VectorUtil::SignedArea(UVTri[0], UVTri[1], UVTri[2]);
		double Area2D = VectorUtil::Area(UVTri[0], UVTri[1], UVTri[2]);

		// TODO: penalize UV flips?

		Metrics.UVAreaSum += Area2D;
		Metrics.BoundsUV.Contain(UVTri[0]);
		Metrics.BoundsUV.Contain(UVTri[1]);
		Metrics.BoundsUV.Contain(UVTri[2]);

		double Distortion = FMath::Max(Area3D, Area2D) / FMath::Min(Area3D, Area2D);

		Distortion = FMath::Min(Distortion, ClampAreaDistortionMetric);
		if (SignedArea2D > 0)
		{
			Distortion = ClampAreaDistortionMetric;
		}

		SumAreaDistortion += Area3D * Distortion;
		SumDistortionsSqr += Area3D * Distortion * Distortion;
		MaxAreaDistortion = FMath::Max(MaxAreaDistortion, Distortion);

		FIndex3i TriEdges = Submesh.GetTriEdges(tid);
		for (int j = 0; j < 3; ++j)
		{
			if (Submesh.IsBoundaryEdge(TriEdges[j]))
			{
				Metrics.UVPerimeter += Distance( UVTri[j], UVTri[(j + 1) % 3] );
			}
		}

		Count++;
	}


	double AvgAreaDistortion = SumAreaDistortion / Metrics.SurfAreaSum;
	double AvgSqrAreaDistortion = SumDistortionsSqr / Metrics.SurfAreaSum;
	double L2AreaDistortion = FMathd::Sqrt(SumDistortionsSqr);

	Metrics.AvgSqrAreaDistortion = AvgSqrAreaDistortion;

	Metrics.AreaWeightedNormal.Normalize();
	Metrics.NormalSpread = NormalHistogram.WeightedSpreadMetric();

	// things to try:
	//  - quick OBB fit to try to get rectangular bounds
	double IdealBoxArea = FMathd::Sqrt(Metrics.UVAreaSum);
	double RealBoxArea = Metrics.BoundsUV.Area();
	Metrics.Compactness = FMathd::Max(IdealBoxArea, RealBoxArea) / FMathd::Min(IdealBoxArea, RealBoxArea);
	Metrics.Compactness = FMathd::Sqrt(Metrics.Compactness);

	//double IdealUVSquareArea = (Metrics.UVPerimeter/4.0) * (Metrics.UVPerimeter/4.0);		// if patch was square, this would be it's area
	//double SquareCompactness = Metrics.UVAreaSum / IdealUVSquareArea;

	//double UVPerimeterCircleRadius = (Metrics.UVPerimeter / (2.0 * FMathd::Pi));
	//double UVCircleArea = FMathd::Pi * UVPerimeterCircleRadius * UVPerimeterCircleRadius;
	//Metrics.Compactness = UVCircleArea / Metrics.UVAreaSum;
	//ensure(Metrics.Compactness >= 1.0);

	return Metrics;
}





bool FPatchBasedMeshUVGenerator::ComputeIslandsByRegionMerging(
	FDynamicMesh3& TargetMesh,
	FDynamicMeshUVOverlay& TargetUVOverlay,
	const FMeshConnectedComponents& ConnectedComponents,
	TArray<TArray<int32>>& IslandsOut,
	FProgressCancel* Progress)
{
	return ComputeIslandsByRegionMerging(
		TargetMesh,
		TargetUVOverlay,
		ConnectedComponents,
		false,
		IslandsOut,
		Progress);
}


bool FPatchBasedMeshUVGenerator::ComputeIslandsByRegionMerging(
	FDynamicMesh3& TargetMesh,
	FDynamicMeshUVOverlay& TargetUVOverlay,
	const FMeshConnectedComponents& ConnectedComponents,
	bool bOnlyOutputMergedRegions,
	TArray<TArray<int32>>& IslandsOut,
	FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	FMeshRegionGraph RegionGraph;
	if (GroupConstraint)
	{
		RegionGraph.BuildFromComponents(TargetMesh, ConnectedComponents, [&](int32 k) { return k; },
			[this](int32 a, int32 b) { return GroupConstraint->GetGroup(a) == GroupConstraint->GetGroup(b); });
	}
	else
	{
		RegionGraph.BuildFromComponents(TargetMesh, ConnectedComponents, [&](int32 k) { return k; });
	}

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	TSet<int32> RemainingRegions;
	for (int32 k = 0; k < RegionGraph.MaxRegionIndex(); ++k)
	{
		if (RegionGraph.IsRegion(k))
		{
			RemainingRegions.Add(k);
		}
	}

	FDynamicMeshUVEditor::FExpMapOptions Options;
	Options.NormalSmoothingRounds = this->NormalSmoothingRounds;
	Options.NormalSmoothingAlpha = this->NormalSmoothingAlpha;

	// TODO: use constants instead of relative/multipliers
	//double MergeThreshold = 4.0;
	//double NeverMergeThreshold = 25.0;
	double MergeDistortionThreshold = this->MergingThreshold;
	double InvalidMaxDistortion = 4.0 * this->MergingThreshold;

	TArray<FUVAreaMetrics> RegionMetrics;
	RegionMetrics.SetNum(RegionGraph.MaxRegionIndex());
	ParallelFor(RegionGraph.MaxRegionIndex(), [&](int32 rid)
	{
		const TArray<int32>& RegionTris = RegionGraph.GetRegionTris(rid);
		RegionMetrics[rid] = Compute_UV_Area_Metrics(&TargetMesh, RegionTris, Options.NormalSmoothingRounds, Options.NormalSmoothingAlpha);

		if (Progress && Progress->Cancelled())
		{
			return;
		}
	});
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	TSet<TPair<int, int>> FailedMatches;

	struct FPotentialMatch
	{
		int RegionIdx;
		int NbrRegionIdx;
		FUVAreaMetrics Metrics;
	};


	struct FPatchJoinParams
	{
		double DistortionThreshold;
		double CompactnessThreshold;
		double AvgNormalAngleThreshold;
	};

	TArray<FPatchJoinParams> ParameterIncrements;
	ParameterIncrements.Add(FPatchJoinParams{ MergeDistortionThreshold, CompactnessThreshold, MaxNormalDeviationDeg/8.0 });
	ParameterIncrements.Add(FPatchJoinParams{ MergeDistortionThreshold, CompactnessThreshold, MaxNormalDeviationDeg/4.0 });
	ParameterIncrements.Add(FPatchJoinParams{ MergeDistortionThreshold, CompactnessThreshold, MaxNormalDeviationDeg/2.0 });
	ParameterIncrements.Add(FPatchJoinParams{ MergeDistortionThreshold, CompactnessThreshold, MaxNormalDeviationDeg });

	int32 CurFilterIndex = 0;
	int32 PassCount = 0;

	bool bDone = false;
	while (!bDone)
	{
		FPatchJoinParams ActiveJoinParams = ParameterIncrements[CurFilterIndex];

		bDone = true;
		int32 MergedInPass = 0;

		TArray<FPotentialMatch> Matches;

		// collect up all valid potential matches
		for (int32 rid : RemainingRegions)
		{
			TArray<int32> Nbrs = RegionGraph.GetNeighbours(rid);
			for (int32 nbrid : Nbrs)
			{
				if (rid < nbrid)
				{
					if (FailedMatches.Contains(TPair<int, int>(rid, nbrid)) == false)
					{
						Matches.Add(FPotentialMatch{ rid, nbrid, FUVAreaMetrics::Invalid() });
					}
				}
			}
		}

		// compute them all in parallel
		ParallelFor(Matches.Num(), [&](int32 i)
		{
			FPotentialMatch& Match = Matches[i];
			const TArray<int32>& RegionTris = RegionGraph.GetRegionTris(Match.RegionIdx);
			const TArray<int32>& NbrTris = RegionGraph.GetRegionTris(Match.NbrRegionIdx);
			TArray<int32> MergedTriangles;
			MergedTriangles.Append(RegionTris);
			MergedTriangles.Append(NbrTris);
			Match.Metrics = Compute_UV_Area_Metrics(&TargetMesh, MergedTriangles, Options.NormalSmoothingRounds, Options.NormalSmoothingAlpha);

			if (Progress && Progress->Cancelled())
			{
				return;
			}
		} /*, EParallelForFlags::ForceSingleThread*/ );

		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		auto SortScoreFunc = [&](const FPotentialMatch& Match)
		{
			double NormalAngleRad = AngleR(RegionMetrics[Match.RegionIdx].AreaWeightedNormal, RegionMetrics[Match.NbrRegionIdx].AreaWeightedNormal);
			NormalAngleRad *= 10.0;
			//NormalAngleRad *= 0.1;
			double DistortionDelta1 = FMathd::Abs(Match.Metrics.AvgSqrAreaDistortion - RegionMetrics[Match.RegionIdx].AvgSqrAreaDistortion);
			double DistortionDelta2 = FMathd::Abs(Match.Metrics.AvgSqrAreaDistortion - RegionMetrics[Match.NbrRegionIdx].AvgSqrAreaDistortion);

			// should we consider compactness for sorting?

			//double CompactnessDelta1 = FMathd::Abs(Match.Metrics.Compactness - RegionMetrics[Match.RegionIdx].Compactness);
			//double CompactnessDelta2 = FMathd::Abs(Match.Metrics.Compactness - RegionMetrics[Match.NbrRegionIdx].Compactness);

			return NormalAngleRad + FMath::Max(DistortionDelta1, DistortionDelta2);
			//return NormalAngleRad + FMath::Max(DistortionDelta1, DistortionDelta2) + FMath::Max(CompactnessDelta1, CompactnessDelta2);
		};

		// sort
		Matches.Sort([&](const FPotentialMatch& A, const FPotentialMatch& B) { return SortScoreFunc(A) < SortScoreFunc(B); });

		auto DistortionMeasureFunc = [](const FPotentialMatch& Match)
		{
			// TODO: compactness penalizes squares, does not seem right

			//return (Match.Metrics.AvgSqrAreaDistortion + FMathd::Sqrt(Match.Metrics.Compactness)) * 0.5;
			return Match.Metrics.AvgSqrAreaDistortion;
		};


		auto NormalsFilterFunc = [&](const FPotentialMatch& Match) -> bool
		{
			double NormalAngleDeg = AngleD(RegionMetrics[Match.RegionIdx].AreaWeightedNormal, RegionMetrics[Match.NbrRegionIdx].AreaWeightedNormal);
			return NormalAngleDeg < ActiveJoinParams.AvgNormalAngleThreshold;
		};
		auto CompactnessFilterFunc = [&](const FPotentialMatch& Match) -> bool
		{
			return Match.Metrics.Compactness < ActiveJoinParams.CompactnessThreshold;
		};


		TSet<int32> ModifiedRegions;
		for (int32 i = 0; i < Matches.Num(); ++i)
		{
			FPotentialMatch& Match = Matches[i];
			if (ModifiedRegions.Contains(Match.RegionIdx) || ModifiedRegions.Contains(Match.NbrRegionIdx))
			{
				continue;
			}

			double JoinedDistortionMeasure = DistortionMeasureFunc(Match);
			bool bNormalsAcceptable = NormalsFilterFunc(Match);
			bool bCompactnessAcceptable = CompactnessFilterFunc(Match);

			if (bNormalsAcceptable == false || bCompactnessAcceptable == false || JoinedDistortionMeasure > InvalidMaxDistortion)
			{
				FailedMatches.Add(TPair<int, int>(Match.RegionIdx, Match.NbrRegionIdx));
			}
			else if (JoinedDistortionMeasure < ActiveJoinParams.DistortionThreshold)
			{
				bool bMerged = RegionGraph.MergeRegion(Match.NbrRegionIdx, Match.RegionIdx);
				if (bMerged)
				{
					RegionMetrics[Match.RegionIdx] = Match.Metrics;
					RemainingRegions.Remove(Match.NbrRegionIdx);
					ModifiedRegions.Add(Match.RegionIdx);
					ModifiedRegions.Add(Match.NbrRegionIdx);
					bDone = false;
					MergedInPass++;
				}
				else
				{
					FailedMatches.Add(TPair<int, int>(Match.RegionIdx, Match.NbrRegionIdx));
				}
			}

			if (Progress && Progress->Cancelled())
			{
				return false;
			}
		}

		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		//UE_LOGF(LogGeometry, Warning, "ParamStep %d - PASS %d - Merged %d - Remaining %d", CurFilterIndex, PassCount, MergedInPass, RemainingRegions.Num());

		PassCount++;
		if (bDone && CurFilterIndex < (ParameterIncrements.Num()-1) )
		{
			CurFilterIndex++;
			FailedMatches.Reset();
			bDone = false;
		}
	}

	// accumulate remaining valid regions

	auto KeepRegion = [bOnlyOutputMergedRegions, &ConnectedComponents, &RegionGraph](int32 RID) -> bool
	{
		int32 KeepAboveTriCount = bOnlyOutputMergedRegions ? ConnectedComponents[RID].Indices.Num() : 0;
		return RegionGraph.GetRegionTriCount(RID) > KeepAboveTriCount;
	};

	int32 NumValidRegions = 0;
	for (int32 RID : RemainingRegions)
	{
		NumValidRegions += int32(KeepRegion(RID));
	}

	IslandsOut.Reserve(NumValidRegions);

	for (int32 RID : RemainingRegions)
	{
		if (KeepRegion(RID))
		{
			IslandsOut.Add(RegionGraph.MoveRegionTris(RID));
		}
		if (Progress && Progress->Cancelled())
		{
			return false;
		}
	}

	return true;
}



int32 FPatchBasedMeshUVGenerator::ComputeUVsFromTriangleSets(
	FDynamicMesh3& TargetMesh,
	FDynamicMeshUVOverlay& TargetUVOverlay,
	const TArray<TArray<int32>>& TriangleSets,
	TArray<bool>& bValidUVsComputedOut,
	FProgressCancel* Progress
)
{
	FDynamicMeshUVEditor::FExpMapOptions Options;
	Options.NormalSmoothingRounds = this->NormalSmoothingRounds;
	Options.NormalSmoothingAlpha = this->NormalSmoothingAlpha;

	FDynamicMeshUVEditor UVEditor(&TargetMesh, &TargetUVOverlay);

	int32 NumUVIslands = TriangleSets.Num();
	bValidUVsComputedOut.Init(false, NumUVIslands);

	int32 NumFailed = 0;
	for ( int32 k = 0; k < NumUVIslands; ++k)
	{
		const TArray<int32>& TriangleSet = TriangleSets[k];
		bValidUVsComputedOut[k] = UVEditor.SetTriangleUVsFromExpMap(TriangleSet, Options);
		if (bValidUVsComputedOut[k] == false)
		{
			NumFailed++;
		}

		if (Progress && Progress->Cancelled())
		{
			return -1;
		}
	}

	return NumFailed;
}



#undef LOCTEXT_NAMESPACE