// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/UVTransfer.h"

#include "Distance/DistLine3Segment3.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "EdgeSpan.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/MeshDijkstra.h"
#include "Selections/MeshConnectedComponents.h"
#include "Util/ProgressCancel.h"

UE::Geometry::FDynamicMeshUVTransfer::FDynamicMeshUVTransfer(
	const FDynamicMesh3* SourceMeshIn, FDynamicMesh3* DestinationMeshIn, int32 SourceUVLayerIndexIn, int32 DestUVLayerIndexIn)
: SourceMesh(SourceMeshIn)
, DestinationMesh(DestinationMeshIn)
, UVLayerIndex(FMath::Clamp(SourceUVLayerIndexIn, 0, 7))
, DestUVLayerIndex(FMath::Clamp(DestUVLayerIndexIn, 0, 7))
{}

bool UE::Geometry::FDynamicMeshUVTransfer::TransferSeams(FProgressCancel* Progress)
{
	if (!DestinationMesh || !SourceMesh
		|| UVLayerIndex < 0 || UVLayerIndex > 7
		|| !SourceMesh->HasAttributes()
		|| SourceMesh->Attributes()->NumUVLayers() <= UVLayerIndex
		|| DestUVLayerIndex < 0 || DestUVLayerIndex > 7
		|| !DestinationMesh->HasAttributes()
		|| DestinationMesh->Attributes()->NumUVLayers() <= DestUVLayerIndex)
	{
		return ensure(false);
	}

	SourceOverlay = SourceMesh->Attributes()->GetUVLayer(UVLayerIndex);
	if (!ensure(SourceOverlay))
	{
		return false;
	}

	DestOverlay = DestinationMesh->Attributes()->GetUVLayer(DestUVLayerIndex);
	if (!ensure(DestOverlay))
	{
		return false;
	}

	if (Progress && Progress->Cancelled()) { return false; }

	InitializeHashGrid();

	if (Progress && Progress->Cancelled()) { return false; }

	if (bClearExistingSeamsInDestination)
	{
		// Start with a UV topology that matches the mesh topology, i.e. remove any existing seams in the
		//  region of interest.
		ResetDestinationUVTopology(Progress);
	}

	if (Progress && Progress->Cancelled()) { return false; }

	return PerformSeamTransfer(Progress);
}

bool  UE::Geometry::FDynamicMeshUVTransfer::TransferSeamsAndUVs(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled()) { return false; }

	bool bSeamSuccess = TransferSeams(Progress);

	if (Progress && Progress->Cancelled()) { return false; }

	// Even if seam transfer wasn't fully successful, we'll try to transfer what we can.
	return PerformElementsTransfer(Progress) && bSeamSuccess;
}

void UE::Geometry::FDynamicMeshUVTransfer::InitializeHashGrid()
{
	HashGrid = MakeUnique<TPointHashGrid3<int32, double>>(
		FMath::Max(KINDA_SMALL_NUMBER, VertexSearchCellSize), IndexConstants::InvalidID);
	for (int32 Vid : DestinationMesh->VertexIndicesItr())
	{
		HashGrid->InsertPointUnsafe(Vid, DestinationMesh->GetVertex(Vid));
	}
}

int32 UE::Geometry::FDynamicMeshUVTransfer::GetCorrespondingDestVid(int32 SourceVid)
{
	int32* FoundDestVid = SourceVidToDestinationVid.Find(SourceVid);
	if (FoundDestVid)
	{
		return *FoundDestVid;
	}

	FVector3d VertPosition = SourceMesh->GetVertex(SourceVid);
	int32 DestVid = HashGrid->FindNearestInRadius(VertPosition, VertexSearchDistance, [this, &VertPosition](int32 DestVid)
	{
		return FVector3d::DistSquared(VertPosition, DestinationMesh->GetVertex(DestVid));
	}).Key;

	if (DestVid != IndexConstants::InvalidID)
	{
		SourceVidToDestinationVid.Add(SourceVid, DestVid);
	}

	return DestVid;
};

void UE::Geometry::FDynamicMeshUVTransfer::ResetDestinationUVTopology(FProgressCancel* Progress)
{
	// To remove any seams in the region of interest, we just make sure that every vid is
	//  only matched to a single element.
	TMap<int32, int32> MeshVidToElementID;

	auto ProcessTriangle = [this, &MeshVidToElementID](int32 Tid)
	{
		FIndex3i MeshTriangle = DestinationMesh->GetTriangle(Tid);
		FIndex3i CurrentUVTriangle = DestOverlay->GetTriangle(Tid);
		FIndex3i NewUVTriangle;
		for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
		{
			int32 Vid = MeshTriangle[SubIdx];
			int32* ExistingMatchedElement = MeshVidToElementID.Find(Vid);
			if (ExistingMatchedElement)
			{
				NewUVTriangle[SubIdx] = *ExistingMatchedElement;
			}
			else
			{
				int32 ElementID = CurrentUVTriangle[SubIdx] >= 0 ? CurrentUVTriangle[SubIdx]
					// The value we choose if we didn't have the UVs set doesn't matter, since we'll
					//  be changing it anyway.
					: DestOverlay->AppendElement(FVector2f::Zero());
				MeshVidToElementID.Add(Vid, ElementID);
				NewUVTriangle[SubIdx] = ElementID;
			}
		}
		DestOverlay->SetTriangle(Tid, NewUVTriangle);
	};

	// We'll check Progress->Cancelled every 2^10=1024 tris.
	uint32 Count = 0;
	uint32 IntervalCheckMask = (1 << 10) - 1;

	if (DestinationSelectionTids)
	{
		for (int32 Tid : *DestinationSelectionTids)
		{
			if (Progress && ((++Count & IntervalCheckMask) == 0) && Progress->Cancelled()) { return; }
			ProcessTriangle(Tid);
		}
	}
	else
	{
		for (int32 Tid : DestinationMesh->TriangleIndicesItr())
		{
			if (Progress && ((++Count & IntervalCheckMask) == 0) && Progress->Cancelled()) { return; }
			ProcessTriangle(Tid);
		}
	}
}//end ResetDestinationUVTopology

bool UE::Geometry::FDynamicMeshUVTransfer::PerformSeamTransfer(FProgressCancel* Progress)
{
	bool bAllSuccessful = true;

	if (!HashGrid.IsValid())
	{
		InitializeHashGrid();
	}

	TSet<int32> DestEidsToMakeSeams;
	SourceBoundaryElements.Reset();

	double MeshMaxDim = SourceMesh->GetBounds(true).MaxDim();

	auto ProcessSourceEdge = [this, &DestEidsToMakeSeams, MeshMaxDim](int32 SourceEid) -> bool
	{
		if (!SourceOverlay->IsSeamEdge(SourceEid))
		{
			return true;
		}

		FIndex2i SourceVids = SourceMesh->GetEdgeV(SourceEid);
		FIndex2i DestVids;
		DestVids.A = GetCorrespondingDestVid(SourceVids.A);
		if (DestVids.A == IndexConstants::InvalidID)
		{
			return false;
		}
		DestVids.B = GetCorrespondingDestVid(SourceVids.B);
		if (DestVids.B == IndexConstants::InvalidID)
		{
			return false;
		}

		if (SourceMesh->IsBoundaryEdge(SourceEid))
		{
			// We don't try to transfer boundary edges because we don't want to accidentally add a seam close
			//  to (but not exactly on) the destination mesh boundary just because the simplified mesh boundary
			//  passed here. However, we do need to mark the source boundary elements at some point because that
			//  knowledge is useful for tranferring any seams that touch the boundary.
			int32 Tid = SourceMesh->GetEdgeT(SourceEid).A;
			FIndex3i TriVids = SourceMesh->GetTriangle(Tid);
			FIndex3i TriElements = SourceOverlay->GetTriangle(Tid);
			for (int i = 0; i < 2; ++i)
			{
				int SubIdx = TriVids.IndexOf(SourceVids[i]);
				if (ensure(SubIdx >= 0) && SourceOverlay->IsElement(TriElements[SubIdx]))
				{
					SourceBoundaryElements.Add(TriElements[SubIdx]);
				}
			}
			return true;
		}

		FVector SourceEdgeStart = SourceMesh->GetVertex(SourceVids.A);
		FVector SourceEdgeVector = SourceMesh->GetVertex(SourceVids.B) - SourceEdgeStart;
		double SourceEdgeLength = 0;
		FVector SourceEdgeDirection = FVector::Zero();
		SourceEdgeVector.ToDirectionAndLength(SourceEdgeDirection, SourceEdgeLength);

		UE::Geometry::TMeshDijkstra<FDynamicMesh3> PathFinder(DestinationMesh);
		TMeshDijkstra<FDynamicMesh3>::FSeedPoint SeedPoint = { DestVids.A, DestVids.A, 0 };
		
		double MaxDistance = TNumericLimits<double>::Max();
		if (PathLengthToleranceMultiplier > 1)
		{
			MaxDistance = PathLengthToleranceMultiplier * SourceEdgeLength;
			if (PathSimilarityWeight > 0)
			{
				MaxDistance *= (1 + PathSimilarityWeight);
			}
			// Arbitrary clamping to prevent failures to find vertices when source edge is tiny
			MaxDistance = FMath::Max(MinimalPathSearchDistance, MaxDistance);
		}
		
		if (PathSimilarityWeight > 0 && !SourceEdgeDirection.IsZero())
		{
			PathFinder.bEnableDistanceWeighting = true;

			// We need some similarity metric. The current approach is to integrate the squared distance of points along
			//  the new edge to the line of the source edge.
			PathFinder.GetWeightedDistanceFunc = [this, SourceEdgeStart, SourceEdgeDirection, MeshMaxDim]
				(int32 FromVid, int32 ToVid, int32 SeedVid, double EuclideanDistance) 
			{
				double SimilarityMetric = SquaredDistanceFromLineIntegratedAlongSegment(
					FLine3d(SourceEdgeStart, SourceEdgeDirection),
					FSegment3d(DestinationMesh->GetVertex(FromVid), DestinationMesh->GetVertex(ToVid)));

				// The problem is that we want to combine this integral with distance, and it's hard to know the relative
				//  importance of the two. Moreover, that balance changes based on mesh scaling, since the integral is
				//  cubic relative to scale, and distance is linear. To try to balance the two, we divide the integral
				//  by squared maximal dimension to try to make the integral linear with respect to scale.
				SimilarityMetric /= (MeshMaxDim*MeshMaxDim);
				
				return EuclideanDistance + PathSimilarityWeight * SimilarityMetric;
			};
		}
		
		if (!PathFinder.ComputeToTargetPoint({ SeedPoint }, DestVids.B, MaxDistance))
		{
			return false;
		}

		TArray<int32> DestVidPath;
		PathFinder.FindPathToNearestSeed(DestVids.B, DestVidPath); // Note that the output path comes out backwards here

		TArray<int32> DestEidPath;
		FEdgeSpan::VertexSpanToEdgeSpan(DestinationMesh, DestVidPath, DestEidPath);
		if (!ensure(DestEidPath.Num() > 0))
		{
			return false;
		}

		if (!DestinationSelectionTids)
		{
			DestEidsToMakeSeams.Append(DestEidPath);
		}
		else
		{
			for (int32 Eid : DestEidPath)
			{
				FIndex2i EdgeTids = DestinationMesh->GetEdgeT(Eid);
				if (DestinationSelectionTids->Contains(EdgeTids.A)
					|| (EdgeTids.B != IndexConstants::InvalidID && DestinationSelectionTids->Contains(EdgeTids.B)))
				{
					DestEidsToMakeSeams.Add(Eid);
				}
			}
		}

		SourceEidToDestinationEndpointEidsVids.Add(SourceEid, TPair<FIndex2i, FIndex2i>(
			FIndex2i(DestEidPath.Last(), DestEidPath[0]),
			FIndex2i(DestVidPath.Last(), DestVidPath[0])));

		return true;
	};//end ProcessSourceEdge 

	// We'll check Progress->Cancelled every 256 source edges.
	uint32 Count = 0;
	uint32 IntervalCheckMask = 0xFF;

	if (SourceSelectionTids)
	{
		// Iterate through all three edges of each triangle, and use a set to avoid redoing an edge
		TSet<int32> ProcessedEids;
		for (int32 Tid : *SourceSelectionTids)
		{
			FIndex3i TriEids = SourceMesh->GetTriEdges(Tid);
			for (int i = 0; i < 3; ++i)
			{
				if (Progress && ((++Count & IntervalCheckMask) == 0) && Progress->Cancelled()) { return false; }

				int32 Eid = TriEids[i];
				bool bAlreadyProcessed = false;
				ProcessedEids.Add(Eid, &bAlreadyProcessed);
				if (!bAlreadyProcessed)
				{
					bAllSuccessful = ProcessSourceEdge(Eid) && bAllSuccessful;
				}
			}
		}
	}
	else
	{
		for (int32 Eid : SourceMesh->EdgeIndicesItr())
		{
			if (Progress && ((++Count & IntervalCheckMask) == 0) && Progress->Cancelled()) { return false; }

			bAllSuccessful = ProcessSourceEdge(Eid) && bAllSuccessful;
		}
	}

	FDynamicMeshUVEditor UVEditor(DestinationMesh, UVLayerIndex, true);
	bAllSuccessful = UVEditor.CreateSeamsAtEdges(DestEidsToMakeSeams) && bAllSuccessful;

	return bAllSuccessful;
}

bool UE::Geometry::FDynamicMeshUVTransfer::PerformElementsTransfer(FProgressCancel* Progress)
{
	if (!SourceOverlay || !DestOverlay)
	{
		// Must have had invalid parameters
		return false;
	}

	// We'll check Progress->Cancelled every 512 elements.
	uint32 Count = 0;
	uint32 IntervalCheckMask = (1 << 9) - 1;

	bool bAllSuccessful = true;
	TSet<int32> ProcessedSourceElementIDs;

	// Transferring elements on seams is tricky. We have information about the destination edges
	//  that correspond to the starts and ends of the source edges, and this information can be
	//  used to match source seam elements to destination elements. Unfortunately, it is possible
	//  for multiple source edges to map to the same destination edges if the shortest destination
	//  paths partially overlap. This typically looks like a "pinching" or collapse of a corner 
	//  of an island, and it causes confusion about which source elements should map to the 
	//  destination elements at those locations, because some source elements are collapsed away
	//  here.
	// The way we resolve this relies on the following: each source element that lies on a seam
	//  has two adjoining seam edges that can try to bind it to the destination element that is on
	//  that side of their corresponding destination edge. That element will be the same if the
	//  two edges didn't collapse, and different if it did. So, by counting the number of times we try
	//  to make the same binding of elements, we can keep the the right ones.
	// Seam ends are a special case where the same edge tries to do the same mapping twice, so they
	//  don't need special handling. Elements at the boundary where an internal seam meets the boundary,
	//  however, require special handling because we don't happen to iterate through source mesh boundary
	//  (because we don't try to transfer that). The way we handle that is as follows: if the boundary
	//  element doesn't show up in the mappings at all, then its source triangle must have been collapsed
	//  by path transfer, or else it's not at a meeting point of interior seam, in which case it doesn't
	//  need special handling. If it does show up, then an adjoining seam try to set up that mapping, and
	//  since we can assume the mesh boundary to still exist, we should accept that mapping. This requires
	//  us to know when source elements are on the boundary.

	// Key is (DestinationElementID, SourceElementID), value is true if mapping was repeated more than once. 
	TMap<FIndex2i, bool> Mappings;
	for (TPair<int32, TPair<FIndex2i, FIndex2i>> EidAndDestEidsVids : SourceEidToDestinationEndpointEidsVids)
	{
		if (Progress && ((++Count & IntervalCheckMask) == 0) && Progress->Cancelled()) { return false; }

		const FDynamicMesh3::FEdge Edge = SourceMesh->GetEdge(EidAndDestEidsVids.Key);
		const FIndex2i& SourceTids = Edge.Tri;
		const FIndex2i& SourceVids = Edge.Vert;
		const FIndex2i& DestEids = EidAndDestEidsVids.Value.Key;
		const FIndex2i& DestVids = EidAndDestEidsVids.Value.Value;

		// Get an "orientation indicator" for source Tid A, which will be either 1 or 2 depending on 
		//  the ordering of vert A and B in the triangle. This will let us pick the matching tids in
		//  the destination.
		FIndex3i SourceTriA = SourceMesh->GetTriangle(SourceTids.A);
		int SourceTriAOrientationIndicator = (SourceTriA.IndexOf(SourceVids.A) + 3 - SourceTriA.IndexOf(SourceVids.B)) % 3;

		for (int EndpointIndex = 0; EndpointIndex < 2; ++EndpointIndex) // for each endpoint of edge
		{
			int32 SourceVid = SourceVids[EndpointIndex];
			int32 DestVid = DestVids[EndpointIndex];
			int32 DestOtherVid = DestinationMesh->GetEdgeV(DestEids[EndpointIndex]).OtherElement(DestVid);
			FIndex2i VertsForOrientation = EndpointIndex == 0 ? FIndex2i(DestVid, DestOtherVid) : FIndex2i(DestOtherVid, DestVid);
			
			FIndex2i DestTids = DestinationMesh->GetEdgeT(DestEids[EndpointIndex]);
			FIndex3i DestTriA = DestinationMesh->GetTriangle(DestTids.A);
			if (SourceTriAOrientationIndicator != (DestTriA.IndexOf(VertsForOrientation.A) + 3 - DestTriA.IndexOf(VertsForOrientation.B)) % 3)
			{
				Swap(DestTids.A, DestTids.B);
			}

			for (int SideIndex = 0; SideIndex < 2; ++SideIndex)
			{
				int32 SourceTid = SourceTids[SideIndex];
				if (SourceTid == IndexConstants::InvalidID)
				{
					continue;
				}

				FIndex3i SourceTriangle = SourceMesh->GetTriangle(SourceTid);
				int SourceSubIdx = SourceTriangle.IndexOf(SourceVid);
				FIndex3i SourceElements = SourceOverlay->GetTriangle(SourceTid);
				int32 SourceElementID = SourceElements[SourceSubIdx];

				int32 DestTid = DestTids[SideIndex];
				if (DestTid == IndexConstants::InvalidID)
				{
					continue;
				}
				FIndex3i DestTriangle = DestinationMesh->GetTriangle(DestTid);
				int DestSubIdx = DestTriangle.IndexOf(DestVid);
				FIndex3i DestElements = DestOverlay->GetTriangle(DestTid);
				int32 DestElementID = DestElements[DestSubIdx];

				if (bool* ExistingValue = Mappings.Find(FIndex2i(DestElementID, SourceElementID)))
				{
					*ExistingValue = true;
				}
				else
				{
					Mappings.Add(FIndex2i(DestElementID, SourceElementID), false);
				}

				// Note that we don't early out on having processed before because we specifically
				//  need to touch the same element multiple times to count it. The set is just to
				//  avoid doing this work for the other element transfer loop further below.
				ProcessedSourceElementIDs.Add(SourceElementID);
			}
		}
	}

	TSet<int32> PinnedElementIDs;

	for (TPair<FIndex2i, bool> Mapping : Mappings)
	{
		// See the big comment block above for our approach to which mappings we keep
		if (Mapping.Value || SourceBoundaryElements.Contains(Mapping.Key.B))
		{
			DestOverlay->SetElement(Mapping.Key.A, SourceOverlay->GetElement(Mapping.Key.B));
			PinnedElementIDs.Add(Mapping.Key.A);
		}
	}

	// Now carry over the values for elements that were not on seams. 

	// Returns false if we fail to find a correspondence or fail to get value
	auto ProcessElement = [this, &ProcessedSourceElementIDs, &PinnedElementIDs](int32 SourceElementID) -> bool
	{
		if (SourceElementID == IndexConstants::InvalidID)
		{
			return true;
		}

		bool bAlreadyProcessed = false;
		ProcessedSourceElementIDs.Add(SourceElementID, &bAlreadyProcessed);
		if (bAlreadyProcessed)
		{
			return true;
		}

		int32 ParentSourceVid = SourceOverlay->GetParentVertex(SourceElementID);
		if (!ensure(ParentSourceVid != IndexConstants::InvalidID))
		{
			return false;
		}
		int32 DestVid = GetCorrespondingDestVid(ParentSourceVid);
		if (DestVid == IndexConstants::InvalidID)
		{
			return false;
		}

		// Unfortunately it is possible for a vertex that was not on a seam before to become a seam if
		//  a nearby path runs through it. With sufficient pain, it might be possible to figure out which
		//  side of the seam to place the source element value, but that seems not worth it. 
		TArray<int32> DestElements;
		DestOverlay->GetVertexElements(DestVid, DestElements);
		if (DestElements.Num() > 1)
		{
			return true;
		}

		DestOverlay->SetElement(DestElements[0], SourceOverlay->GetElement(SourceElementID));
		PinnedElementIDs.Add(DestElements[0]);
		return true;
	};

	if (SourceSelectionTids)
	{
		for (int32 Tid : *SourceSelectionTids)
		{
			FIndex3i TriangleElements = SourceOverlay->GetTriangle(Tid);
			for (int i = 0; i < 3; ++i)
			{
				if (Progress && ((++Count & IntervalCheckMask) == 0) && Progress->Cancelled()) { return false; }

				bAllSuccessful = ProcessElement(TriangleElements[i]) && bAllSuccessful;
			}
		}
	}
	else
	{
		for (int32 ElementID : SourceOverlay->ElementIndicesItr())
		{
			if (Progress && ((++Count & IntervalCheckMask) == 0) && Progress->Cancelled()) { return false; }

			bAllSuccessful = ProcessElement(ElementID) && bAllSuccessful;
		}
	}

	FMeshConnectedComponents ConnectedComponents(DestinationMesh);
	if (DestinationSelectionTids)
	{
		ConnectedComponents.FindConnectedTriangles(DestinationSelectionTids->Array(), [this](int32 Tid1, int32 Tid2)
		{
			return DestOverlay->AreTrianglesConnected(Tid1, Tid2);
		});
	}
	else
	{
		ConnectedComponents.FindConnectedTriangles([this](int32 Tid1, int32 Tid2)
		{
			return DestOverlay->AreTrianglesConnected(Tid1, Tid2);
		});
	}

	FDynamicMeshUVEditor UVEditor(DestinationMesh, UVLayerIndex, true);
	int32 NumComponents = ConnectedComponents.Num();
	for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
	{
		if (Progress && Progress->Cancelled()) { return false; }

		bAllSuccessful = UVEditor.SetTriangleUVsFromFreeBoundaryConformal(
			ConnectedComponents[ComponentIndex].Indices, PinnedElementIDs)
			&& bAllSuccessful;
	}

	return bAllSuccessful;
}
