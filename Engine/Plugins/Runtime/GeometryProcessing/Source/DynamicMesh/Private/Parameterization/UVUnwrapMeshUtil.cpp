// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/UVUnwrapMeshUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Util/UniqueIndexSet.h"
#include "Algo/Unique.h"

using namespace UE::Geometry;

namespace UVUnwrapMeshUtilLocals
{
	/**
	 * Helper function to update the triangles in an unwrap mesh based on an overlay.
	 *
	 * When updating a particular set of triangles, RemovalTriIterator and InsertionTriIterator
	 * should be the same (the triangles to update). When updating all triangles, RemovalTriIterator
	 * should go over all of the triangles in the destination (which may include ones not in the source)
	 * and InsertionTriIterator should go over all of the triangles in the source (which may include ones
	 * not in the destination).
	 */
	template <typename TriIteratorType>
	void UpdateUnwrapTriangles(const FDynamicMeshUVOverlay& UVOverlayIn, const TriIteratorType& RemovalTriIterator, int32 MaxRemoveCount,
		const TriIteratorType& InsertionTriIterator, FDynamicMesh3& UnwrapMeshOut)
	{
		// Updating triangles is a little messy. To be able to handle arbitrary remeshing, we
		// have to delete all tris first before adding any, so that we don't fail in SetTriangle
		// due to (temporary) non-manifold edges. We have to do this without removing verts as we
		// go along, which means we later need to check for isolated verts to remove any that are
		// isolated. Finally, because we don't have a way to avoid removing temporarily isolated 
		// UV elements, we need to reattach the UV elements for verts that were temporarily left 
		// but ended up not isolated.

		// Remove tris and keep track of potentially isolated elements
		TArray<int32> PotentiallyIsolatedElements;
		PotentiallyIsolatedElements.Reserve(MaxRemoveCount * 3);
		for (int32 Tid : RemovalTriIterator)
		{
			if (!UnwrapMeshOut.IsTriangle(Tid))
			{
				// This triangle was previously unset, or else we had a duplicate tri in the triangles we were passed.
				// The latter probably indicates an error on the caller's part; we catch this in certain cases further
				// down where it would cause us issues (to avoid reinserting a triangle twice)
				continue;
			}

			FIndex3i PrevTriangle = UnwrapMeshOut.GetTriangle(Tid);
			for (int i = 0; i < 3; ++i)
			{
				PotentiallyIsolatedElements.Add(PrevTriangle[i]);
			}
			UnwrapMeshOut.RemoveTriangle(Tid, false);
		}		
		PotentiallyIsolatedElements.Sort();
		PotentiallyIsolatedElements.SetNum(Algo::Unique(PotentiallyIsolatedElements));

		// Reinsert new tris
		UnwrapMeshOut.BeginUnsafeTrianglesInsert();
		for (int32 Tid : InsertionTriIterator)
		{
			if (!UVOverlayIn.IsSetTriangle(Tid))
			{
				// This triangle was changed to be unset
				continue;
			}

			if (UnwrapMeshOut.IsTriangle(Tid))
			{
				// If we find ourselves here, then TriIterator had a duplicate triangle, which we probably
				// didn't mean to do, and the caller probably has an error somewhere. 
				ensure(false);
				continue;
			}

			FIndex3i NewTriangle = UVOverlayIn.GetTriangle(Tid);
			UE::Geometry::EMeshResult Result = UnwrapMeshOut.InsertTriangle(Tid, NewTriangle, 0, true);
			
			// One way you could hit this ensure is if the corresponding vertices didn't exist in the unwrap. That
			// means that somewhere, newly added elements were excluded from the list of changed elements.
			ensure(Result == UE::Geometry::EMeshResult::Ok);
		}
		UnwrapMeshOut.EndUnsafeTrianglesInsert();

		// Deal with isolated and non-isolated verts
		FDynamicMeshUVOverlay* UnwrapMeshUVOverlay = UnwrapMeshOut.Attributes()->PrimaryUV();
		UnwrapMeshUVOverlay->BeginUnsafeElementsInsert();
		for (int32 ElementID : PotentiallyIsolatedElements)
		{
			if (!UnwrapMeshOut.IsReferencedVertex(ElementID))
			{
				UnwrapMeshOut.RemoveVertex(ElementID);
			}
			else if (!UnwrapMeshUVOverlay->IsElement(ElementID))
			{
				// This is a referenced vert without a UV element (because it got removed
				// during temporary isolation), so reinstate the element.
				FVector2f ElementValue = UVOverlayIn.GetElement(ElementID);
				UnwrapMeshUVOverlay->InsertElement(ElementID, &ElementValue.X, true);
			}
		}
		UnwrapMeshUVOverlay->EndUnsafeElementsInsert();

		// Update overlay tris now that we know that the elements exist.
		for (int32 Tid : InsertionTriIterator)
		{
			if (UVOverlayIn.IsSetTriangle(Tid))
			{
				UnwrapMeshUVOverlay->SetTriangle(Tid, UVOverlayIn.GetTriangle(Tid));
			}
			// We don't need to explicitly unset triangles that were unset since
			// that should have happened on triangle removal above.
		}
	}
}

void UVUnwrapMeshUtil::GenerateUVUnwrapMesh(
	const FDynamicMeshUVOverlay& UVOverlay, FDynamicMesh3& UnwrapMeshOut,
	TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition)
{
	UnwrapMeshOut.Clear();

	// The unwrap mesh will have an overlay on top of it with the corresponding UVs,
	// in case we want to draw the texture on it, etc. However note that we can't
	// just do a Copy() call using the source overlay because the parent vertices will differ.
	UnwrapMeshOut.EnableAttributes(); // Makes one UV layer
	FDynamicMeshUVOverlay* UnwrapMeshUVOverlay = UnwrapMeshOut.Attributes()->PrimaryUV();

	// Create a vert for each uv overlay element
	UnwrapMeshOut.BeginUnsafeVerticesInsert();
	UnwrapMeshUVOverlay->BeginUnsafeElementsInsert();
	for (int32 ElementID : UVOverlay.ElementIndicesItr())
	{
		FVector2f UVElement = UVOverlay.GetElement(ElementID);
		UnwrapMeshOut.InsertVertex(ElementID, UVToVertPosition(UVElement), true);
		UnwrapMeshUVOverlay->InsertElement(ElementID, &UVElement.X, true);
	}
	UnwrapMeshOut.EndUnsafeVerticesInsert();
	UnwrapMeshUVOverlay->EndUnsafeElementsInsert();

	// Insert a tri connecting the same vids as elements in the overlay.
	const FDynamicMesh3* OverlayParentMesh = UVOverlay.GetParentMesh();
	UnwrapMeshOut.BeginUnsafeTrianglesInsert();
	for (int32 Tid : OverlayParentMesh->TriangleIndicesItr())
	{
		if (UVOverlay.IsSetTriangle(Tid))
		{
			FIndex3i UVTri = UVOverlay.GetTriangle(Tid);
			UnwrapMeshOut.InsertTriangle(Tid, UVTri, 0, true);
			UnwrapMeshUVOverlay->SetTriangle(Tid, UVTri);
		}
	}
	UnwrapMeshOut.EndUnsafeTrianglesInsert();
}

void UVUnwrapMeshUtil::UpdateUVUnwrapMesh(const FDynamicMeshUVOverlay& UVOverlayIn,
	FDynamicMesh3& UnwrapMeshOut, TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition,
	const TArray<int32>* ChangedElementIDs, const TArray<int32>* ChangedTids)
{
	using namespace UVUnwrapMeshUtilLocals;

	// Note that we don't want to use GenerateUVUnwrapMesh even when doing a full update
	// because that clears the mesh and rebuilds it, and that resets the attributes pointer.
	// That would prevent us from using a dynamic mesh change tracker across an update, as
	// it would lose its attribute pointer.

	FDynamicMeshUVOverlay* UnwrapMeshUVOverlay = UnwrapMeshOut.Attributes()->PrimaryUV();

	auto UpdateVertPositions = [&UVOverlayIn, &UnwrapMeshOut, UVToVertPosition, UnwrapMeshUVOverlay](const auto& ElementIterator)
	{
		UnwrapMeshUVOverlay->BeginUnsafeElementsInsert();
		UnwrapMeshOut.BeginUnsafeVerticesInsert();
		for (int32 ElementID : ElementIterator)
		{
			if (!ensure(UVOverlayIn.IsElement(ElementID)))
			{
				// [ELEMENT_NOT_IN_SOURCE]
				// If you ended up here, then you asked to update an element that wasn't in the source mesh.
				// Perhaps you gathered the changing elements pre-change, and that element was deleted. You 
				// shouldn't gather pre-change because you risk not including any added elements, and because 
				// deleted elements should be captured by changed tri connectivity.
				continue;
			}

			FVector2f ElementValue = UVOverlayIn.GetElement(ElementID);

			// Update the actual unwrap mesh
			if (UnwrapMeshOut.IsVertex(ElementID))
			{
				UnwrapMeshOut.SetVertex(ElementID, UVToVertPosition(ElementValue));
			}
			else
			{
				UnwrapMeshOut.InsertVertex(ElementID, UVToVertPosition(ElementValue), true);
			}

			// Update the unwrap overlay.
			if (UnwrapMeshUVOverlay->IsElement(ElementID))
			{
				UnwrapMeshUVOverlay->SetElement(ElementID, ElementValue);
			}
			else
			{
				UnwrapMeshUVOverlay->InsertElement(ElementID, &ElementValue.X, true);
			}
		}
		UnwrapMeshUVOverlay->EndUnsafeElementsInsert();
		UnwrapMeshOut.EndUnsafeVerticesInsert();
	};

	if (ChangedElementIDs)
	{
		UpdateVertPositions(*ChangedElementIDs);
	}
	else
	{
		UpdateVertPositions(UVOverlayIn.ElementIndicesItr());
	}

	if (ChangedTids)
	{
		UpdateUnwrapTriangles(UVOverlayIn, *ChangedTids, ChangedTids->Num(), *ChangedTids, UnwrapMeshOut);
	}
	else
	{
		UpdateUnwrapTriangles(UVOverlayIn, UnwrapMeshOut.TriangleIndicesItr(), UnwrapMeshOut.TriangleCount(),
			UVOverlayIn.GetParentMesh()->TriangleIndicesItr(), UnwrapMeshOut);
	}
}

void UVUnwrapMeshUtil::UpdateUVUnwrapMesh(const FDynamicMesh3& SourceUnwrapMesh, FDynamicMesh3& DestUnwrapMesh,
	const TArray<int32>* ChangedVids, const TArray<int32>* ChangedConnectivityTids)
{
	using namespace UVUnwrapMeshUtilLocals;

	if (!ChangedVids && !ChangedConnectivityTids)
	{
		DestUnwrapMesh.Copy(SourceUnwrapMesh, false, false, false, true); // Copy positions and UVs
		return;
	}

	const FDynamicMeshUVOverlay* SourceOverlay = SourceUnwrapMesh.Attributes()->PrimaryUV();
	FDynamicMeshUVOverlay* DestOverlay = DestUnwrapMesh.Attributes()->PrimaryUV();

	auto UpdateVerts = [&SourceUnwrapMesh, &DestUnwrapMesh, SourceOverlay, DestOverlay](const auto& VidIterator)
	{
		DestOverlay->BeginUnsafeElementsInsert();
		DestUnwrapMesh.BeginUnsafeVerticesInsert();
		for (int32 Vid : VidIterator)
		{
			if (!ensure(SourceUnwrapMesh.IsVertex(Vid)))
			{
				// See comment labeled [ELEMENT_NOT_IN_SOURCE] above.
				continue;
			}

			if (DestUnwrapMesh.IsVertex(Vid))
			{
				DestUnwrapMesh.SetVertex(Vid, SourceUnwrapMesh.GetVertex(Vid));
				DestOverlay->SetElement(Vid, SourceOverlay->GetElement(Vid));
			}
			else
			{
				DestUnwrapMesh.InsertVertex(Vid, SourceUnwrapMesh.GetVertex(Vid), true);
				FVector2f ElementValue = SourceOverlay->GetElement(Vid);
				DestOverlay->InsertElement(Vid, &ElementValue.X, true);
			}
		}
		DestUnwrapMesh.EndUnsafeVerticesInsert();
		DestOverlay->EndUnsafeElementsInsert();
	};
	if (ChangedVids)
	{
		UpdateVerts(*ChangedVids);
	}
	else
	{
		UpdateVerts(SourceUnwrapMesh.VertexIndicesItr());
	}

	if (ChangedConnectivityTids)
	{
		UpdateUnwrapTriangles(*SourceOverlay, *ChangedConnectivityTids, ChangedConnectivityTids->Num(),
			*ChangedConnectivityTids, DestUnwrapMesh);
	}
	else
	{
		UpdateUnwrapTriangles(*SourceOverlay, DestUnwrapMesh.TriangleIndicesItr(), DestUnwrapMesh.TriangleCount(),
			SourceUnwrapMesh.TriangleIndicesItr(), DestUnwrapMesh);
	}
}

void UVUnwrapMeshUtil::UpdateUVOverlayFromUnwrapMesh(
	const FDynamicMesh3& UnwrapMeshIn, FDynamicMeshUVOverlay& UVOverlayOut,
	TFunctionRef<FVector2f(const FVector3d&)> VertPositionToUV,
	const TArray<int32>* ChangedVids, const TArray<int32>* ChangedTids)
{
	if (!ensure(UVOverlayOut.GetParentMesh()->MaxTriangleID() == UnwrapMeshIn.MaxTriangleID()))
	{
		return;
	}

	auto UpdateElements = [&UnwrapMeshIn, &UVOverlayOut, VertPositionToUV](const auto& VidIterator)
	{
		UVOverlayOut.BeginUnsafeElementsInsert();
		for (int32 Vid : VidIterator)
		{
			if (!ensure(UnwrapMeshIn.IsVertex(Vid)))
			{
				// See comment labeled [ELEMENT_NOT_IN_SOURCE] above.
				continue;
			}

			FVector2f UV = VertPositionToUV(UnwrapMeshIn.GetVertex(Vid));

			if (UVOverlayOut.IsElement(Vid))
			{
				UVOverlayOut.SetElement(Vid, UV);
			}
			else
			{
				UVOverlayOut.InsertElement(Vid, &UV.X, true);
			}
		}
		UVOverlayOut.EndUnsafeElementsInsert();
	};
	if (ChangedVids)
	{
		UpdateElements(*ChangedVids);
	}
	else
	{
		UpdateElements(UnwrapMeshIn.VertexIndicesItr());
	}

	auto UpdateTriangles = [&UnwrapMeshIn, &UVOverlayOut](const auto& UnsetTriIterator, const auto& InsertTriIterator)
	{
		TSet<int32> PotentiallyFreedElements;

		// Gather up potentially freed elements and unset any triangles if needed
		for (int32 Tid : UnsetTriIterator)
		{
			bool bTriangleWillBeSet = UnwrapMeshIn.IsTriangle(Tid);
			FIndex3i NewTriangle = bTriangleWillBeSet ? UnwrapMeshIn.GetTriangle(Tid) : FIndex3i();

			// Gather up potentially freed elements
			if (UVOverlayOut.IsSetTriangle(Tid))
			{
				FIndex3i PrevTriangle = UVOverlayOut.GetTriangle(Tid);
				for (int i = 0; i < 3; ++i)
				{
					if (!bTriangleWillBeSet || NewTriangle[i] != PrevTriangle[i])
					{
						PotentiallyFreedElements.Add(PrevTriangle[i]);
					}
				}

				// Go ahead and unset if needed
				if (!bTriangleWillBeSet)
				{
					UVOverlayOut.UnsetTriangle(Tid, false);
				}
			}
		}

		for (int32 Tid : InsertTriIterator)
		{
			// Update the triangle
			if (UnwrapMeshIn.IsTriangle(Tid))
			{
				FIndex3i NewElementTri = UnwrapMeshIn.GetTriangle(Tid);

				// Force the parent pointers if necessary
				FIndex3i NewParentTriInOutput = UVOverlayOut.GetParentMesh()->GetTriangle(Tid);
				for (int i = 0; i < 3; ++i)
				{
					if (UVOverlayOut.GetParentVertex(NewElementTri[i]) != NewParentTriInOutput[i])
					{
						UVOverlayOut.SetParentVertex(NewElementTri[i], NewParentTriInOutput[i]);
					}
				}

				UVOverlayOut.SetTriangle(Tid, NewElementTri, false);
			}
		}

		UVOverlayOut.FreeUnusedElements(&PotentiallyFreedElements);
	};
	if (ChangedTids)
	{
		UpdateTriangles(*ChangedTids, *ChangedTids);
	}
	else
	{
		UpdateTriangles(UVOverlayOut.GetParentMesh()->TriangleIndicesItr(), UnwrapMeshIn.TriangleIndicesItr());
	}
}

void UVUnwrapMeshUtil::UpdateOverlayFromOverlay(
	const FDynamicMeshUVOverlay& OverlayIn, FDynamicMeshUVOverlay& OverlayOut,
	bool bMeshesHaveSameTopology, const TArray<int32>* ChangedElements,
	const TArray<int32>* ChangedConnectivityTids)
{
	if (!ChangedElements && !ChangedConnectivityTids && bMeshesHaveSameTopology)
	{
		OverlayOut.Copy(OverlayIn);
		return;
	}

	auto UpdateElements = [&OverlayIn, &OverlayOut](const auto& ElementIterator)
	{
		OverlayOut.BeginUnsafeElementsInsert();
		for (int32 ElementID : ElementIterator)
		{
			if (!ensure(OverlayIn.IsElement(ElementID)))
			{
				// See comment labeled [ELEMENT_NOT_IN_SOURCE] above.
				continue;
			}

			FVector2f ElementValue = OverlayIn.GetElement(ElementID);
			if (OverlayOut.IsElement(ElementID))
			{
				OverlayOut.SetElement(ElementID, ElementValue);
			}
			else
			{
				OverlayOut.InsertElement(ElementID, &ElementValue.X, true);
			}
		}
		OverlayOut.EndUnsafeElementsInsert();
	};

	if (ChangedElements)
	{
		UpdateElements(*ChangedElements);
	}
	else
	{
		UpdateElements(OverlayIn.ElementIndicesItr());
	}

	auto UpdateTriangles = [&OverlayIn, &OverlayOut](const auto& TriIterator)
	{
		// To handle arbitrary remeshing in the UV overlay, not only do we need to
		// check for freed elements only after finishing the updates, but we may 
		// also need to forcefully change the parent pointer of elements (imagine
		// a mesh of two disconnected triangles whose element mappings changed)
		TSet<int32> PotentiallyFreedElements;
		for (int32 Tid : TriIterator)
		{
			bool bTriWasSet = OverlayOut.IsSetTriangle(Tid);
			bool bTriWillBeSet = OverlayIn.IsSetTriangle(Tid);
			FIndex3i NewElementTri = bTriWillBeSet ? OverlayIn.GetTriangle(Tid) : FIndex3i();

			if (bTriWasSet)
			{
				FIndex3i OldElementTri = bTriWasSet ? OverlayOut.GetTriangle(Tid) : FIndex3i();

				// Gather up current elements
				for (int i = 0; i < 3; ++i)
				{
					if (!bTriWillBeSet || NewElementTri[i] != OldElementTri[i])
					{
						PotentiallyFreedElements.Add(OldElementTri[i]);
					}
				}

				if (!bTriWillBeSet)
				{
					OverlayOut.UnsetTriangle(Tid, false);
				}
			}

			if (bTriWillBeSet)
			{
				// Force the parent pointer if necessary
				FIndex3i NewParentTriInOutput = OverlayOut.GetParentMesh()->GetTriangle(Tid);
				for (int i = 0; i < 3; ++i)
				{
					if (OverlayOut.GetParentVertex(NewElementTri[i]) != NewParentTriInOutput[i])
					{
						OverlayOut.SetParentVertex(NewElementTri[i], NewParentTriInOutput[i]);
					}
				}
				OverlayOut.SetTriangle(Tid, NewElementTri, false);
			}
		}

		OverlayOut.FreeUnusedElements(&PotentiallyFreedElements);
	};

	if (ChangedConnectivityTids)
	{
		UpdateTriangles(*ChangedConnectivityTids);
	}
	else
	{
		UpdateTriangles(OverlayIn.GetParentMesh()->TriangleIndicesItr());
	}
}

template <EValidityCheckFailMode FailMode>
bool UVUnwrapMeshUtil::DoesUnwrapMatchOverlay(const FDynamicMeshUVOverlay& Overlay, const FDynamicMesh3& UnwrapMesh,
	TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition, double Tolerance)
{
	// A macro that checks the condition and if it's false, returns false. Depending on the fail mode, it may
	// also do a check(false) or ensure(false) to make it easier to figure out what is broken when debugging
	// (uses compile-time branches for that)
#define UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(Condition) if (!(Condition)){ \
	if constexpr (FailMode == EValidityCheckFailMode::Check) { check(false); } \
	else if constexpr (FailMode == EValidityCheckFailMode::Ensure) { ensure(false); } \
	return false; }

	const FDynamicMesh3* ParentMesh = Overlay.GetParentMesh();
	UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(ParentMesh);

	// Do the easy tests. Unfortunately we can't just match up triangle counts because we don't
	// know how many of the overlay triangles are set. However we know that it can't be more
	// than triangles in the parent mesh.
	UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(Overlay.ElementCount() == UnwrapMesh.VertexCount());
	UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(UnwrapMesh.TriangleCount() <= ParentMesh->TriangleCount());

	// Verify that each element of the overlay has a vertex with the same Vid as ElementID, and a
	// corresponding position.
	for (int32 ElementID : Overlay.ElementIndicesItr())
	{
		UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(UnwrapMesh.IsVertex(ElementID));

		FVector3d ExpectedPosition = UVToVertPosition(Overlay.GetElement(ElementID));
		FVector3d ActualPosition = UnwrapMesh.GetVertex(ElementID);
		UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(ExpectedPosition.Equals(ActualPosition, Tolerance));
	}
	// Since the element count equaled the vertex count, at this point we know that there aren't
	// any vertices that don't have a corresponding element (i.e. we don't need to check the reverse).


	// Verify that each set triangle in the overlay has a matching triangle in the unwrap
	for (int32 Tid : ParentMesh->TriangleIndicesItr())
	{
		if (Overlay.IsSetTriangle(Tid))
		{
			UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(UnwrapMesh.IsTriangle(Tid));
			UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(UnwrapMesh.GetTriangle(Tid) == Overlay.GetTriangle(Tid));
		}
	}
	// Verify the reverse- that every triangle in the unwrap appears in the overlay
	for (int32 Tid : UnwrapMesh.TriangleIndicesItr())
	{
		UVEDITOR_CHECK_AND_RETURN_ON_FAILURE(Overlay.IsSetTriangle(Tid));
	}

	// If we got to here, things match up
	return true;
}

#undef UVEDITOR_CHECK_AND_RETURN_ON_FAILURE

// Explicit instantiations
template DYNAMICMESH_API bool UVUnwrapMeshUtil::DoesUnwrapMatchOverlay<EValidityCheckFailMode::ReturnOnly>
(const FDynamicMeshUVOverlay& Overlay, const FDynamicMesh3& UnwrapMesh,
	TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition, double Tolerance);
template DYNAMICMESH_API bool UVUnwrapMeshUtil::DoesUnwrapMatchOverlay<EValidityCheckFailMode::Check>
(const FDynamicMeshUVOverlay& Overlay, const FDynamicMesh3& UnwrapMesh,
	TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition, double Tolerance);
template DYNAMICMESH_API bool UVUnwrapMeshUtil::DoesUnwrapMatchOverlay<EValidityCheckFailMode::Ensure>
(const FDynamicMeshUVOverlay& Overlay, const FDynamicMesh3& UnwrapMesh,
	TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition, double Tolerance);
