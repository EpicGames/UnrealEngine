// Copyright Epic Games, Inc. All Rights Reserved.

#include "Polygroups/GroupSetAdapter.h"

using namespace UE::Geometry;

ITriangleGroupSetAdapter::~ITriangleGroupSetAdapter() = default;

ITriangleGroupSetAdapter::ITriangleGroupSetAdapter(const FDynamicMesh3* InMesh)
	: Mesh(InMesh)
{
	check(InMesh);
}

void ITriangleGroupSetAdapter::SetGroup(const int32 TriangleID, const int32 GroupID, FDynamicMesh3& WritableMesh)
{}

int32 ITriangleGroupSetAdapter::GetMaxGroupID() const
{
	return MaxGroupID;
}

const FDynamicMesh3* ITriangleGroupSetAdapter::GetMesh() const
{
	return Mesh;	
}

FPolygroupLayerAdapter::FPolygroupLayerAdapter(const FDynamicMesh3* InMesh, const FDynamicMeshPolygroupAttribute* Attribute)
	: ITriangleGroupSetAdapter(InMesh)
	, PolygroupSet(InMesh, Attribute)
{}

int32 FPolygroupLayerAdapter::GetGroup(const int32 TriangleID) const
{
	return PolygroupSet.GetGroup(TriangleID);
}

void FPolygroupLayerAdapter::SetGroup(const int32 TriangleID, const int32 NewGroupID, FDynamicMesh3& WritableMesh)
{
	return PolygroupSet.SetGroup(TriangleID, NewGroupID, WritableMesh);
}

int32 FPolygroupLayerAdapter::GetLayerIndex() const
{
	return PolygroupSet.GroupLayerIndex;
}

FTriangleLabelAdapter::FTriangleLabelAdapter(const FDynamicMesh3* InMesh, TriangleAttributeType* InAttribute, const FName InBaseLabel)
	: FTriangleAttributeAdapter(InMesh, InAttribute)
	, BaseLabel(InBaseLabel)
{
	bool AttributeFound = false;

	if (const FDynamicMeshAttributeSet* Attributes = InMesh && InAttribute ? InMesh->Attributes() : nullptr)
	{
		for (const auto&[Name, Attribute]: Attributes->GetTriangleLabelAttributes())
		{
			if (Attribute.Get() == InAttribute)
			{
				LayerName = Name;
				AttributeFound = true;
				break;
			}
		}
	}
	
	check(AttributeFound);
}

FTriangleLabelAdapter::FTriangleLabelAdapter(const FTriangleLabelAdapter* InCopy)
	: FTriangleAttributeAdapter(InCopy)
	, LayerName(InCopy->LayerName)
{}

FName FTriangleLabelAdapter::GetLayerName() const
{
	 return LayerName;
}

FName FTriangleLabelAdapter::GetUniqueValue() const
{
	int32 NumberCounter = MaxGroupID;
	FName NextValue(BaseLabel);
	NextValue.SetNumber(NumberCounter++);
	while (ValueToID.Contains(NextValue))
	{
		NextValue.SetNumber(NumberCounter++);
	}
	return NextValue;
}

FTriangleReadOnlyLabelAdapter::FTriangleReadOnlyLabelAdapter(const FDynamicMesh3* InMesh, const TriangleAttributeType* InAttribute)
	: FTriangleReadOnlyAttributeAdapter(InMesh, InAttribute)
{
	auto AttributeFound = [InMesh, InAttribute]()
	{
		if(const FDynamicMeshAttributeSet* Attributes = InMesh && InAttribute ? InMesh->Attributes() : nullptr)
		{
			for (const auto&[Name, Attribute]: Attributes->GetTriangleLabelAttributes())
			{
				if (Attribute.Get() == InAttribute)
				{
					return true;
				}
			}
		}
		return false;
	};
	check(AttributeFound());
}