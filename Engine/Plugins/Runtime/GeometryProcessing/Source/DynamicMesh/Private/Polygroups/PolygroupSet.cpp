// Copyright Epic Games, Inc. All Rights Reserved.

#include "Polygroups/PolygroupSet.h"
#include "Polygroups/PolygroupUtil.h"

using namespace UE::Geometry;


bool FPolygroupLayer::CheckExists(const FDynamicMesh3* Mesh) const
{
	if (Mesh)
	{
		if (bIsDefaultLayer)
		{
			if (Mesh->HasTriangleGroups())
			{
				return true;
			}
		}
		else
		{
			if (LayerIndex >= 0 && Mesh->HasAttributes() && LayerIndex < Mesh->Attributes()->NumPolygroupLayers())
			{
				return true;
			}
		}
	}
	return false;
}


void FPolygroupLayer::EnableOnMesh(FDynamicMesh3& Mesh) const
{
	if (bIsDefaultLayer)
	{
		if (!Mesh.HasTriangleGroups())
		{
			Mesh.EnableTriangleGroups();
		}
	}
	else
	{
		if (!Mesh.HasAttributes())
		{
			Mesh.EnableAttributes();
		}
		if (Mesh.Attributes()->NumPolygroupLayers() <= LayerIndex)
		{
			Mesh.Attributes()->SetNumPolygroupLayers(LayerIndex + 1);
		}
	}
}


FPolygroupSet::FPolygroupSet(const FPolygroupSet* CopyIn)
{
	Mesh = CopyIn->Mesh;
	PolygroupAttrib = CopyIn->PolygroupAttrib;
	GroupLayerIndex = CopyIn->GroupLayerIndex;
	MaxGroupID = CopyIn->MaxGroupID;
}

FPolygroupSet::FPolygroupSet(const FDynamicMesh3* MeshIn)
{
	Mesh = MeshIn;
	GroupLayerIndex = -1;
	RecalculateMaxGroupID();
}

/** Initialize a PolygroupSet for the given Mesh, and standard triangle group layer */
FPolygroupSet::FPolygroupSet(const FDynamicMesh3* MeshIn, FPolygroupLayer GroupLayer)
{
	Mesh = MeshIn;
	GroupLayerIndex = -1;
	if (! GroupLayer.bIsDefaultLayer )
	{
		if (ensure(Mesh->Attributes()))
		{
			if (GroupLayer.LayerIndex < Mesh->Attributes()->NumPolygroupLayers())
			{
				PolygroupAttrib = Mesh->Attributes()->GetPolygroupLayer(GroupLayer.LayerIndex);
				GroupLayerIndex = GroupLayer.LayerIndex;
			}
		}
		if (GroupLayerIndex == -1)
		{
			ensureMsgf(false, TEXT("FPolygroupSet: Attribute index missing!"));
		}
	}
	RecalculateMaxGroupID();
}


FPolygroupSet::FPolygroupSet(const FDynamicMesh3* MeshIn, const FDynamicMeshPolygroupAttribute* PolygroupAttribIn)
{
	Mesh = MeshIn;
	PolygroupAttrib = PolygroupAttribIn;
	GroupLayerIndex = UE::Geometry::FindPolygroupLayerIndex(*MeshIn, PolygroupAttrib);
	RecalculateMaxGroupID();
}

FPolygroupSet::FPolygroupSet(const FDynamicMesh3* MeshIn, int32 PolygroupLayerIndex)
{
	Mesh = MeshIn;
	if (ensure(Mesh->Attributes()))
	{
		if (PolygroupLayerIndex < Mesh->Attributes()->NumPolygroupLayers())
		{
			PolygroupAttrib = Mesh->Attributes()->GetPolygroupLayer(PolygroupLayerIndex);
			GroupLayerIndex = PolygroupLayerIndex;
			return;
		}
	}
	RecalculateMaxGroupID();
	ensureMsgf(false, TEXT("FPolygroupSet: Attribute index missing!"));
}


FPolygroupSet::FPolygroupSet(const FDynamicMesh3* MeshIn, FName AttribName)
{
	Mesh = MeshIn;
	PolygroupAttrib = UE::Geometry::FindPolygroupLayerByName(*MeshIn, AttribName);
	GroupLayerIndex = UE::Geometry::FindPolygroupLayerIndex(*MeshIn, PolygroupAttrib);
	RecalculateMaxGroupID();
	ensureMsgf(PolygroupAttrib != nullptr, TEXT("FPolygroupSet: Attribute set missing!"));
}


void FPolygroupSet::RecalculateMaxGroupID()
{
	MaxGroupID = 0;
	if (PolygroupAttrib)
	{
		for (int32 tid : Mesh->TriangleIndicesItr())
		{
			MaxGroupID = FMath::Max(MaxGroupID, PolygroupAttrib->GetValue(tid) + 1);
		}
	}
	else
	{
		for (int32 tid : Mesh->TriangleIndicesItr())
		{
			MaxGroupID = FMath::Max(MaxGroupID, Mesh->GetTriangleGroup(tid) + 1);
		}
	}
}