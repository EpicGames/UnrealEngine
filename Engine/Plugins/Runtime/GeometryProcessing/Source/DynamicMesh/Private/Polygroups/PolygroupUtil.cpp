// Copyright Epic Games, Inc. All Rights Reserved.

#include "Polygroups/PolygroupUtil.h"

using namespace UE::Geometry;

FDynamicMeshPolygroupAttribute* UE::Geometry::FindPolygroupLayerByName(FDynamicMesh3& Mesh, FName Name)
{
	FDynamicMeshAttributeSet* AttributeSet = Mesh.Attributes();
	if (AttributeSet == nullptr) return nullptr;
	int32 NumPolygroupLayers = AttributeSet->NumPolygroupLayers();
	for (int32 k = 0; k < NumPolygroupLayers; ++k)
	{
		if (AttributeSet->GetPolygroupLayer(k)->GetName() == Name)
		{
			return AttributeSet->GetPolygroupLayer(k);
		}
	}
	return nullptr;
}

const FDynamicMeshPolygroupAttribute* UE::Geometry::FindPolygroupLayerByName(const FDynamicMesh3& Mesh, FName Name)
{
	const FDynamicMeshAttributeSet* AttributeSet = Mesh.Attributes();
	if (AttributeSet == nullptr) return nullptr;
	int32 NumPolygroupLayers = AttributeSet->NumPolygroupLayers();
	for (int32 k = 0; k < NumPolygroupLayers; ++k)
	{
		if (AttributeSet->GetPolygroupLayer(k)->GetName() == Name)
		{
			return AttributeSet->GetPolygroupLayer(k);
		}
	}
	return nullptr;
}

FDynamicMeshTriangleLabelAttribute* UE::Geometry::FindTriangleLabelLayerByName(FDynamicMesh3& Mesh, const FName Name)
{
	FDynamicMeshAttributeSet* AttributeSet = Mesh.Attributes();
	return AttributeSet && Name != NAME_None ? AttributeSet->FindTriangleLabelAttribute(Name) : nullptr;
}

const FDynamicMeshTriangleLabelAttribute* UE::Geometry::FindTriangleLabelLayerByName(const FDynamicMesh3& Mesh, FName Name)
{
	const FDynamicMeshAttributeSet* AttributeSet = Mesh.Attributes();
	return AttributeSet && Name != NAME_None ? AttributeSet->FindTriangleLabelAttribute(Name) : nullptr;
}

int32 UE::Geometry::FindPolygroupLayerIndexByName(const FDynamicMesh3& Mesh, FName Name)
{
	const FDynamicMeshAttributeSet* AttributeSet = Mesh.Attributes();
	if (AttributeSet == nullptr) return -1;
	int32 NumPolygroupLayers = AttributeSet->NumPolygroupLayers();
	for (int32 k = 0; k < NumPolygroupLayers; ++k)
	{
		if (AttributeSet->GetPolygroupLayer(k)->GetName() == Name)
		{
			return k;
		}
	}
	return -1;
}


int32 UE::Geometry::FindPolygroupLayerIndex(const FDynamicMesh3& Mesh, const FDynamicMeshPolygroupAttribute* Layer)
{
	const FDynamicMeshAttributeSet* AttributeSet = Mesh.Attributes();
	if (AttributeSet == nullptr) return -1;
	int32 NumPolygroupLayers = AttributeSet->NumPolygroupLayers();
	for (int32 k = 0; k < NumPolygroupLayers; ++k)
	{
		if (AttributeSet->GetPolygroupLayer(k) == Layer)
		{
			return k;
		}
	}
	return -1;
}


int32 UE::Geometry::ComputeGroupIDBound(const FDynamicMesh3& Mesh, const FDynamicMeshPolygroupAttribute* Layer)
{
	int Bound = 0;
	for (int TID : Mesh.TriangleIndicesItr())
	{
		Bound = FMath::Max(Bound, Layer->GetValue(TID) + 1);
	}
	return Bound;
}


FString UE::Geometry::MakeUniqueGroupLayerName(const FDynamicMesh3& Mesh, FString BaseName)
{
	if (BaseName.Len() == 0)
	{
		BaseName = TEXT("group");
	}

	if (Mesh.HasAttributes() == false)
	{
		return BaseName;
	}
	const FDynamicMeshAttributeSet* AttribSet = Mesh.Attributes();
	int32 NumGroupLayers = AttribSet->NumPolygroupLayers();
	if (NumGroupLayers == 0)
	{
		return BaseName;
	}

	// TODO: would be nice to detect if BaseName has a numeric suffix in either basename_0 or basename0 style
	// and increment the number. But we should have a general utility function for that because it is so common...

	FString UniqueName = BaseName;
	int32 NumberCounter = 0;
	while (true)
	{
		bool bFoundDuplicate = false;
		for (int32 k = 0; k < NumGroupLayers; ++k)
		{
			if ( AttribSet->GetPolygroupLayer(k)->GetName() == FName(UniqueName) )
			{
				bFoundDuplicate = true;
			}
		}
		if (!bFoundDuplicate)
		{
			return UniqueName;
		}
		UniqueName = FString::Printf(TEXT("%s_%d"), *BaseName, NumberCounter++);
	}
	ensureMsgf(false, TEXT("Failed to create unique name, returning base name"));
	return BaseName;

}