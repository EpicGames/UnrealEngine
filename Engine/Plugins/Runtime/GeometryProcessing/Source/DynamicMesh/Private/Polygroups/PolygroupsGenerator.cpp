// Copyright Epic Games, Inc. All Rights Reserved.

#include "Polygroups/PolygroupsGenerator.h"

#include "Clustering/FaceNormalClustering.h"
#include "Containers/StaticArray.h"
#include "DynamicMesh/MeshNormals.h"
#include "Selections/MeshConnectedComponents.h"
#include "Util/IndexUtil.h"
#include "Parameterization/MeshDijkstra.h"
#include "Parameterization/IncrementalMeshDijkstra.h"
#include "Parameterization/MeshRegionGraph.h"
#include "Async/ParallelFor.h"
#include "Util/IndexPriorityQueue.h"
#include "Selections/MeshFaceSelection.h"

#include "Curve/DynamicGraph3.h"

using namespace UE::Geometry;

FPolygroupsGenerator::FPolygroupsGenerator(FDynamicMesh3* MeshIn)
{
	Mesh = MeshIn;
}



bool FPolygroupsGenerator::FindPolygroupsFromUVIslands(int32 UVLayer)
{
	const FDynamicMeshUVOverlay* UV = Mesh->Attributes()->GetUVLayer(UVLayer);
	if (!UV)
	{
		return false;
	}

	return FindPolygroupsFromConnectedTris([&UV](int32 TriIdx0, int32 TriIdx1)
	{
		return UV->AreTrianglesConnected(TriIdx0, TriIdx1);
	});
}


bool FPolygroupsGenerator::FindPolygroupsFromHardNormalSeams()
{
	const FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->GetNormalLayer(0);
	if (!Normals)
	{
		return false;
	}

	return FindPolygroupsFromConnectedTris([Normals](int32 TriIdx0, int32 TriIdx1)
	{
		return Normals->AreTrianglesConnected(TriIdx0, TriIdx1);
	});
}

bool FPolygroupsGenerator::FindPolygroupsFromMaterialIDs()
{
	const FDynamicMeshMaterialAttribute* Materials = Mesh->Attributes()->GetMaterialID();
	if (!Materials)
	{
		return false;
	}

	return FindPolygroupsFromConnectedTris([Materials](int32 TriIdx0, int32 TriIdx1)
	{
		return Materials->GetValue(TriIdx0) == Materials->GetValue(TriIdx1);
	});
}

bool FPolygroupsGenerator::FindPolygroupsFromConnectedTris(TFunction<bool(int32, int32)> AreTrisConnected)
{
	FMeshConnectedComponents Components(Mesh);
	Components.FindConnectedTriangles(AreTrisConnected);

	int32 NumComponents = Components.Num();
	for (int32 ci = 0; ci < NumComponents; ++ci)
	{
		FoundPolygroups.Add(Components.GetComponent(ci).Indices);
	}

	if (bApplyPostProcessing)
	{
		PostProcessPolygroups(false);
	}
	if (bCopyToMesh)
	{
		CopyPolygroupsToMesh();
	}

	return (FoundPolygroups.Num() > 0);
}






void FPolygroupsGenerator::GetSeamConstraintEdges(bool bUVSeams, bool bNormalSeams, TSet<int32>& ConstraintEdgesOut) const
{
	if (bUVSeams && Mesh->HasAttributes() && Mesh->Attributes()->PrimaryUV() != nullptr )
	{
		FDynamicMeshUVOverlay* UVOverlay = Mesh->Attributes()->PrimaryUV();
		for (int32 eid : Mesh->EdgeIndicesItr())
		{
			if (UVOverlay->IsSeamEdge(eid))
			{
				ConstraintEdgesOut.Add(eid);
			}
		}
	}
	if (bNormalSeams && Mesh->HasAttributes() && Mesh->Attributes()->PrimaryNormals() != nullptr )
	{
		FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->PrimaryNormals();
		for (int32 eid : Mesh->EdgeIndicesItr())
		{
			if (NormalOverlay->IsSeamEdge(eid))
			{
				ConstraintEdgesOut.Add(eid);
			}
		}
	}
}


bool FPolygroupsGenerator::FindSourceMeshPolygonPolygroups(
	bool bRespectUVSeams,
	bool bRespectNormalSeams,
	double QuadMetricClamp,
	double QuadAdjacencyWeight,
	int MaxSearchRounds)
{
	// precompute triangle normals
	int32 MaxTriangleID = Mesh->MaxTriangleID();
	TArray<FVector3d> FaceNormals;
	FaceNormals.SetNum(MaxTriangleID);
	TArray<double> FaceAreas;
	FaceAreas.SetNum(MaxTriangleID);
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		FVector3d Centroid;
		Mesh->GetTriInfo(tid, FaceNormals[tid], FaceAreas[tid], Centroid);
	}

	// if we are respecting seams or hard normals, find all those edges
	TSet<int32> InvalidEdges;
	if (bRespectUVSeams || bRespectNormalSeams)
	{
		GetSeamConstraintEdges(bRespectUVSeams, bRespectNormalSeams, InvalidEdges);
	}

	// FTriPairQuad is a quad formed by a pair of triangles in the mesh
	struct FTriPairQuad
	{
		FIndex2i Triangles = FIndex2i::Invalid();
		int32 MidEdgeID = -1;	// edge through middle of quad

		FVector2d TriAreas = FVector2d::Zero();
		FIndex4i VertexLoop = FIndex4i::Invalid();
		FIndex4i QuadEdges = FIndex4i::Invalid();
		FIndex4i QuadEdgeOtherTris = FIndex4i::Invalid();

		int32 Index = -1;		// index into sorted Squareness list

		bool IsValid() const { return Triangles.A >= 0 && Triangles.B >= 0; }

		bool IsAdjacent(const FTriPairQuad& OtherQuad) const
		{
			return OtherQuad.QuadEdgeOtherTris.Contains(Triangles.A) || OtherQuad.QuadEdgeOtherTris.Contains(Triangles.B);
		}
		bool IsAdjacent(int32 TriangleID) const
		{
			return QuadEdgeOtherTris.Contains(TriangleID);
		}
	};

	// For each triangle, there are 3 possible adjacent triangles that form valid quads. 
	// TriPotentialQuads is an [N][3] array of those potential quads, stored as FTriPairQuad
	TArray<TStaticArray<FTriPairQuad, 3>> TriPotentialQuads;
	TriPotentialQuads.SetNum(MaxTriangleID);
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		FIndex3i TriEdges = Mesh->GetTriEdges(tid);
		FIndex3i TriNbrTris = Mesh->GetTriNeighbourTris(tid);
		for (int32 j = 0; j < 3; ++j)
		{
			FTriPairQuad NbrQuad;
			int32 nbr_tid = TriNbrTris[j];
			if (nbr_tid >= 0 && (InvalidEdges.Contains(TriEdges[j]) == false) )
			{
				NbrQuad.Triangles = FIndex2i(tid, nbr_tid);
				NbrQuad.MidEdgeID = TriEdges[j];

				FIndex2i EdgeV = Mesh->GetEdgeV(NbrQuad.MidEdgeID);
				FIndex2i OtherEdgeV = Mesh->GetEdgeOpposingV(NbrQuad.MidEdgeID);
				NbrQuad.VertexLoop = FIndex4i(EdgeV.A, OtherEdgeV.A, EdgeV.B, OtherEdgeV.B);

				NbrQuad.TriAreas[0] = Mesh->GetTriArea(NbrQuad.Triangles[0]);
				NbrQuad.TriAreas[1] = Mesh->GetTriArea(NbrQuad.Triangles[1]);

				for (int32 k = 0; k < 4; ++k)
				{
					NbrQuad.QuadEdges[k] = Mesh->FindEdge(NbrQuad.VertexLoop[k], NbrQuad.VertexLoop[(k+1)%4]);

					int32 eid = NbrQuad.QuadEdges[k];
					FIndex2i EdgeT = Mesh->GetEdgeT(eid);
					if (EdgeT.B != IndexConstants::InvalidID)
					{
						int IsA = (EdgeT.A == tid || EdgeT.A == nbr_tid) ? 1 : 0;
						int IsB = (EdgeT.B == tid || EdgeT.B == nbr_tid) ? 1 : 0;
						check( (IsA ^ IsB) == 1 );

						int32 other_tid = (EdgeT.A == tid || EdgeT.A == nbr_tid) ? EdgeT.B : EdgeT.A;
						NbrQuad.QuadEdgeOtherTris[k] = other_tid;
					}
				}
			}
			TriPotentialQuads[tid][j] = NbrQuad;
		}
	}

	// QuadnessMetric measures the likelihood of an individual Quad, with 0 being 
	// extremely quad (ie a planar square) and larger numbers being less-good. 
	// The metric is ultimately sqrt( sum-of-squares ) where each term is in
	// range [0,1]

	// to measure:
	//   - internal angles, closer to 90 == better
	//   - length similarity of edges (0,2) and (1<3)
	//   - option to prevent quads from crossing UV seams, hard normal seams

	// TODO: does it make sense to provide ability to modulate the face-planarity weight?
	// It is mostly necessary on hard-surface meshes, on organic shapes it's will potentially
	// prevent finding non-planar quads...
	// TODO: should we consider edge length metric, comparing mid-edge w/ opposite mid-edge length?
	auto QuadnessMetric = [this, &FaceNormals](const FTriPairQuad& Quad)
	{
		double Metric = 0;

		double FacesDot = Dot( FaceNormals[Quad.Triangles.A], FaceNormals[Quad.Triangles.B] );

		// prefer quads that are planar, ie angle between normals is closer to 0
		double FacesAngle = AngleD( FaceNormals[Quad.Triangles.A], FaceNormals[Quad.Triangles.B] );
		double MappedUnitFacesAngle = FMathd::Abs( FMathd::Abs(FacesAngle) / 180.0 );
		Metric += MappedUnitFacesAngle*MappedUnitFacesAngle;

		// prefer quads where opening angles are ~90 degrees
		double OpeningAnglesDeviationMetric = 0;
		if (FacesDot > 0.001)
		{
			FVector3d AvgNormal = Normalized(FaceNormals[Quad.Triangles.A] + FaceNormals[Quad.Triangles.B]);
	
			for (int32 j = 0; j < 4; ++j)
			{
				FVector3d C = Mesh->GetVertex(Quad.VertexLoop[j]);
				FVector3d Corner = Mesh->GetVertex(Quad.VertexLoop[(j+1)%4]);
				FVector3d D = Mesh->GetVertex(Quad.VertexLoop[(j+2)%4]);
				FFrame3d AvgPlane(Corner, AvgNormal);
				FVector2d ProjC = Normalized( AvgPlane.ToPlaneUV( C ) );
				FVector2d ProjD = Normalized( AvgPlane.ToPlaneUV( D ) );

				double OpeningAngle = AngleD(ProjC, ProjD);		// [-180, 180]
				double MappedUnitOpeningAngle = FMathd::Abs( (FMathd::Abs(OpeningAngle)-90) / 90.0 );
				OpeningAnglesDeviationMetric += MappedUnitOpeningAngle;
			}
		}
		else
		{
			OpeningAnglesDeviationMetric = 4;		// quad opening angle is >= 90 deg, cannot reliably measure 2D opening angle and 3D is unreliable...
		}
		OpeningAnglesDeviationMetric /= 4.0;
		Metric += OpeningAnglesDeviationMetric * OpeningAnglesDeviationMetric;

		// prefer quads where both tris are the same area
		double MinArea = FMathd::Min(Quad.TriAreas.X, Quad.TriAreas.Y);
		double AreaMetric = FMathd::Max(Quad.TriAreas.X, Quad.TriAreas.Y) / FMathd::Max(MinArea, 0.0000001);
		// min area metric should be 1...
		AreaMetric = FMathd::Clamp(AreaMetric, 1.0, 5.0);
		AreaMetric = (AreaMetric - 1.0) / 4.0;
		check(AreaMetric >= 0 && AreaMetric <= 1.0);
		Metric += AreaMetric * AreaMetric;

		return FMathd::Sqrt(Metric);
	};
	double CurrentMaxMetric = FMathd::Sqrt(3);		// each term is in range [0,1] so this is sqrt(# of terms)
	auto IsViableQuad = [CurrentMaxMetric, QuadMetricClamp](double Metric)
	{
		return (Metric / CurrentMaxMetric) <= QuadMetricClamp;
	};


	// FQuadWithMetric 
	struct FQuadWithMetric
	{
		FIndex2i QuadIndex;			// (tri,j) index into TriPotentialQuads
		double QuadMetric;			// initial value of QuadnessMetric()
		int32 FixedQuadNbrs = 0;	// number of known quad neighbours

		double GetMetric(double AdjacencyWeight) const
		{
			if (FixedQuadNbrs == 0)
			{
				return QuadMetric;
			}
			else
			{
				return QuadMetric / FMathd::Pow((double)(FixedQuadNbrs + 1), AdjacencyWeight);
			}
		}
	};
	TArray<FQuadWithMetric> AllQuadsMetricList;

	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		for (int32 j = 0; j < 3; ++j)
		{
			if (TriPotentialQuads[tid][j].IsValid())
			{
				AllQuadsMetricList.Add(FQuadWithMetric{ FIndex2i(tid,j), QuadnessMetric(TriPotentialQuads[tid][j]) });
			}
		}
	}
	AllQuadsMetricList.Sort( [](const FQuadWithMetric& A, const FQuadWithMetric& B) { return A.QuadMetric < B.QuadMetric; } );

	// returns ref to the FTriPairQuad for the given index into AllQuadsMetricList
	auto GetQuadForIndex = [&AllQuadsMetricList, &TriPotentialQuads](int32 QMIndex) -> FTriPairQuad&
	{
		FQuadWithMetric& MetricQuad = AllQuadsMetricList[QMIndex];
		FIndex2i TriNbrIdx = MetricQuad.QuadIndex;
		return TriPotentialQuads[TriNbrIdx.A][TriNbrIdx.B];
	};

	bool bDone = false;
	int Iters = 0;
	int32 LastTris = Mesh->TriangleCount(), LastQuads = 0;
	TArray<FTriPairQuad> FoundQuads;
	TArray<FTriPairQuad> LastRoundQuads;
	TArray<bool> DoneTris;
	while (!bDone && Iters++ < MaxSearchRounds)
	{
		DoneTris.Init(false, MaxTriangleID);
		// returns true if either triangle of the quad has been used up in another quad
		auto IsDoneQuad = [&DoneTris](const FTriPairQuad& Quad)
		{
			return DoneTris[Quad.Triangles.A] == true || DoneTris[Quad.Triangles.B] == true;
		};

		// reset index/nbrcount
		for (int32 k = 0; k < AllQuadsMetricList.Num(); ++k)
		{
			GetQuadForIndex(k).Index = k;
			AllQuadsMetricList[k].FixedQuadNbrs = 0;
		}

		int32 RemainingTriangles = Mesh->TriangleCount();

		// list of final quads extracted by the search below
		FoundQuads.Reset();

		// If we are in a round > 1, we are generally fine-tuning the existing solution.
		// Currently this is done by just deleting some quad-areas and letting them re-populate
		// with the new neighbour set. So re-add all the "known" quads from the previous round.
		if (LastRoundQuads.Num() > 0)
		{
			for (FTriPairQuad PrevQuad : LastRoundQuads)
			{
				FoundQuads.Add(PrevQuad);
				DoneTris[PrevQuad.Triangles.A] = true;
				DoneTris[PrevQuad.Triangles.B] = true;
				RemainingTriangles -= 2;
			}
			LastRoundQuads.Reset();

			// initialize neighbour info (technically only need this for unfinished quads?)
			for (int32 Index = 0; Index < AllQuadsMetricList.Num(); ++Index)
			{
				FTriPairQuad& Quad = GetQuadForIndex(Index);
				for (int32 j = 0; j < 4; ++j)
				{
					if (Quad.QuadEdgeOtherTris[j] >= 0 && DoneTris[Quad.QuadEdgeOtherTris[j]])
					{
						AllQuadsMetricList[Index].FixedQuadNbrs++;
						check(AllQuadsMetricList[Index].FixedQuadNbrs >= 0 && AllQuadsMetricList[Index].FixedQuadNbrs <= 4);
					}
				}
			}
		}


		// insert all potential quads into priority queue
		FIndexPriorityQueue Queue;
		Queue.Initialize(AllQuadsMetricList.Num());
		for (int32 k = 0; k < AllQuadsMetricList.Num(); ++k)
		{
			if (IsDoneQuad(GetQuadForIndex(k)) == false)
			{
				double Metric = AllQuadsMetricList[k].GetMetric(QuadAdjacencyWeight);
				if (IsViableQuad(Metric))
				{
					Queue.Insert(k, float(Metric));
				}
			}
		}

		// Repeatedly extract quads until the priority queue is empty.
		TArray<int32> Reinsertions, OtherNbrTris;
		while (Queue.GetCount() > 0)
		{
			int32 CurIndex = Queue.Dequeue();
			FTriPairQuad& Quad = GetQuadForIndex(CurIndex);
			if (IsDoneQuad(Quad))
			{
				continue;
			}

			FoundQuads.Add(Quad);
			DoneTris[Quad.Triangles.A] = true;
			DoneTris[Quad.Triangles.B] = true;
			RemainingTriangles -= 2;

			// After each quad is selected, it's adjacent quad-neighbours become more likely 
			// to also be quads. Find and reinsert those neighbours with updated metrics.
			if ( QuadAdjacencyWeight > 0 )
			{
				Reinsertions.Reset();
				OtherNbrTris.Reset();
				for (int32 j = 0; j < 4; ++j)
				{
					int32 adjacent_tid = Quad.QuadEdgeOtherTris[j];
					if (adjacent_tid != -1)
					{
						for (int32 k = 0; k < 3; ++k)
						{
							FTriPairQuad& NbrQuad = TriPotentialQuads[adjacent_tid][k];
							if (NbrQuad.IsValid() && IsDoneQuad(NbrQuad) == false && NbrQuad.IsAdjacent(Quad) )
							{
								Reinsertions.AddUnique(NbrQuad.Index);
								OtherNbrTris.AddUnique( NbrQuad.Triangles.OtherElement(adjacent_tid) );
							}
						}
					}
				}

				// gross need second pass here because each quad appears twice in TriPotentialQuads list, 
				// and so the 'far' neighbour of each quad needs to be updated too..., but it's first-tris
				// are not in the adjacency list of current Quad
				for (int32 far_adjacent_tid : OtherNbrTris)
				{
					for (int32 k = 0; k < 3; ++k)
					{
						FTriPairQuad& NbrQuad = TriPotentialQuads[far_adjacent_tid][k];
						if (NbrQuad.IsValid() && IsDoneQuad(NbrQuad) == false && NbrQuad.IsAdjacent(Quad) )
						{
							Reinsertions.AddUnique(NbrQuad.Index);
						}
					}
				}

				// for each nbr quad, increment it's nbr count, compute a new metric weighted
				// by number of known neighbours, and reinsert into the queue
				for (int32 Index : Reinsertions)
				{
					if (Queue.Contains(Index))		// may not be in queue due to metric limit or other filtering
					{
						FQuadWithMetric& UpdateQM = AllQuadsMetricList[Index];
						UpdateQM.FixedQuadNbrs++;
						check(UpdateQM.FixedQuadNbrs >= 0 && UpdateQM.FixedQuadNbrs <= 4);

						// more quad nbrs == much more likely. Possibly just divide is not enough....
						double ModifiedMetric = UpdateQM.GetMetric(QuadAdjacencyWeight);
						Queue.Remove(Index);
						Queue.Insert(Index, float(ModifiedMetric));
					}
				}
			}
		}

#if 1
		// In first round we do purely greedy search. In later rounds we use isolated triangles
		// as the seeds for "failure areas", and remove those areas from the solution, and then
		// re-run it, with the idea that the adjacency metric may correct the result. Note that
		// this is only possible if a large enough area is removed, otherwise the solution can be
		// "locked" by the surrounding quads. So, the 'removal' region grows each round, and the
		// neighbour weight increases
		if ((MaxSearchRounds - Iters) > 0)
		{
			FMeshFaceSelection RecomputeRegion(Mesh);

			for (int32 tid : Mesh->TriangleIndicesItr())
			{
				if (DoneTris[tid] == false)
				{
					FIndex3i TriNbrTris = Mesh->GetTriNeighbourTris(tid);
					int32 QuadNbrs = 0;
					QuadNbrs += (TriNbrTris.A >= 0 && DoneTris[TriNbrTris.A]) ? 1 : 0;
					QuadNbrs += (TriNbrTris.B >= 0 && DoneTris[TriNbrTris.B]) ? 1 : 0;
					QuadNbrs += (TriNbrTris.C >= 0 && DoneTris[TriNbrTris.C]) ? 1 : 0;
					if (QuadNbrs == 3)
					{
						RecomputeRegion.Select(tid);
					}
				}
			}
			int32 NumIsolated = RecomputeRegion.Num();
			RecomputeRegion.ExpandToFaceNeighbours(1+Iters);

			if (NumIsolated > 0)
			{
				for (FTriPairQuad Quad : FoundQuads)
				{
					bool bIsFilteredQuad = ( RecomputeRegion.Contains(Quad.Triangles.A) ||
						RecomputeRegion.Contains(Quad.Triangles.B) );
					if (bIsFilteredQuad == false)
					{
						LastRoundQuads.Add(Quad);
					}
				}
			}

			QuadAdjacencyWeight *= 1.1;

			//UE_LOGF(LogTemp, Warning, "Found %d quads, have %d tris remaining, %d isolated tris, %d isolated-adjacent quads", FoundQuads.Num(), RemainingTriangles, NumIsolated, (FoundQuads.Num() - LastRoundQuads.Num()));
		}
#endif
	}

	// in some cases we clearly have planar polygons or tip-fans...how can we extract those?

	// create group for each quad
	FoundPolygroups.SetNum(FoundQuads.Num());
	for (int32 QuadIdx = 0; QuadIdx < FoundQuads.Num(); ++QuadIdx)
	{
		FTriPairQuad Quad = FoundQuads[QuadIdx];
		FoundPolygroups[QuadIdx].Add(Quad.Triangles.A);
		FoundPolygroups[QuadIdx].Add(Quad.Triangles.B);
	}

	// add remaining triangles as individual groups
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		if (DoneTris[tid] == false)
		{
			TArray<int> Group;
			Group.Add(tid);
			FoundPolygroups.Add(MoveTemp(Group));
		}
	}

	if (bCopyToMesh)
	{
		CopyPolygroupsToMesh();
	}

	return (FoundQuads.Num() > 0);
}



bool FPolygroupsGenerator::FindPolygroupsFromFaceNormals(
	double OneMinusCosAngleTolerance,
	bool bRespectUVSeams,
	bool bRespectNormalSeams,
	bool bUseAveragePolygroupNormals)
{
	// if we are respecting seams or hard normals, find all those edges
	TSet<int32> InvalidEdges;
	if (bRespectUVSeams || bRespectNormalSeams)
	{
		GetSeamConstraintEdges(bRespectUVSeams, bRespectNormalSeams, InvalidEdges);
	}

	if (bUseAveragePolygroupNormals)
	{
		FaceNormalClustering::FClusterOptions Options;
		Options.NormalOneMinusCosTolerance = OneMinusCosAngleTolerance;
		Options.bApplyNormalToleranceToClusters = true;
		FaceNormalClustering::ComputeMeshPolyGroupsFromClusters(*Mesh, FoundPolygroups, Options, &InvalidEdges);
	}
	else
	{
		// Transform into normal-dot-product tolerance
		const double DotTolerance = 1.0 - OneMinusCosAngleTolerance;

		// compute face normals
		FMeshNormals Normals(Mesh);
		Normals.ComputeTriangleNormals();

		TArray<bool> DoneTriangle;
		DoneTriangle.SetNum(Mesh->MaxTriangleID());

		TArray<int> Stack;

		// grow outward from vertices until we have no more left
		for (int TriID : Mesh->TriangleIndicesItr())
		{
			if (DoneTriangle[TriID] == true)
			{
				continue;
			}

			TArray<int> Polygroup;
			Polygroup.Add(TriID);
			DoneTriangle[TriID] = true;

			Stack.Reset();
			Stack.Add(TriID);
			while (Stack.Num() > 0)
			{
				int CurTri = Stack.Pop(EAllowShrinking::No);
				FIndex3i NbrTris = Mesh->GetTriNeighbourTris(CurTri);
				FIndex3i NbrEdges = Mesh->GetTriEdges(CurTri);
				for (int j = 0; j < 3; ++j)
				{
					if (InvalidEdges.Contains(NbrEdges[j]))
					{
						continue;
					}

					if (NbrTris[j] >= 0
						&& DoneTriangle[NbrTris[j]] == false)
					{
						double Dot = Normals[CurTri].Dot(Normals[NbrTris[j]]);
						if (Dot > DotTolerance)
						{
							Polygroup.Add(NbrTris[j]);
							Stack.Add(NbrTris[j]);
							DoneTriangle[NbrTris[j]] = true;
						}
					}
				}
			}

			FoundPolygroups.Add(Polygroup);
		}
	}

	if (bApplyPostProcessing)
	{
		PostProcessPolygroups(true);
	}
	if (bCopyToMesh)
	{
		CopyPolygroupsToMesh();
	}

	return (FoundPolygroups.Num() > 0);
}



/**
 * Dual graph of mesh faces, ie graph of edges across faces between face centers.
 * Normals and Areas are tracked for each point.
 */
class FMeshFaceDualGraph : public FDynamicGraph3d
{
public:
	TDynamicVectorN<double, 3> Normals;
	TDynamicVector<double> Areas;

	int AppendVertex(const FVector3d& Centroid, const FVector3d& Normal, double Area)
	{
		int vid = FDynamicGraph3d::AppendVertex(Centroid);
		Normals.InsertAt({ {Normal.X, Normal.Y, Normal.Z} }, vid);
		Areas.InsertAt(Area, vid);
		return vid;
	}

	void InsertVertex(int32 Vid, const FVector3d& Centroid, const FVector3d& Normal, double Area)
	{
		FDynamicGraph3d::InsertVertex(Vid, Centroid);
		Normals.InsertAt({ {Normal.X, Normal.Y, Normal.Z} }, Vid);
		Areas.InsertAt(Area, Vid);
	}

	FVector3d GetNormal(int32 vid) const
	{
		return Normals.AsVector3(vid);
	}

	/** Build a Face Dual Graph for a triangle mesh */
	static void MakeFaceDualGraphForMesh(
		FDynamicMesh3* Mesh, 
		FMeshFaceDualGraph& FaceGraph,
		TFunctionRef<bool(int,int)> TrisConnectedPredicate )
	{
		bool bIsCompact = Mesh->IsCompactT();
		for (int32 tid : Mesh->TriangleIndicesItr())
		{
			FVector3d Normal, Centroid;
			double Area;
			Mesh->GetTriInfo(tid, Normal, Area, Centroid);

			if (bIsCompact)
			{
				int32 newid = FaceGraph.AppendVertex(Centroid, Normal, Area);
				checkSlow(newid == tid);
			}
			else
			{
				FaceGraph.InsertVertex(tid, Centroid, Normal, Area);
			}
		}

		for (int32 tid : Mesh->TriangleIndicesItr())
		{
			FIndex3i NbrT = Mesh->GetTriNeighbourTris(tid);
			for (int32 j = 0; j < 3; ++j)
			{
				if (Mesh->IsTriangle(NbrT[j]) && TrisConnectedPredicate(tid, NbrT[j]) )
				{
					FaceGraph.AppendEdge(tid, NbrT[j]);
				}
			}
		}
	}

};






bool FPolygroupsGenerator::FindPolygroupsFromFurthestPointSampling(
	int32 NumPoints, 
	EWeightingType WeightingType, 
	FVector3d WeightingCoeffs,
	FPolygroupSet* StartingGroups)
{
	NumPoints = FMath::Min(NumPoints, Mesh->VertexCount());

	// cannot seem to use auto or TUniqueFunction here...
	TFunction<bool(int32, int32)> TrisConnectedPredicate;
	TrisConnectedPredicate = [](int, int) -> bool { return true; };
	if (StartingGroups != nullptr)
	{
		TrisConnectedPredicate = [StartingGroups](int a, int b) -> bool { return StartingGroups->GetGroup(a) == StartingGroups->GetGroup(b) ? true : false; };
	}

	FMeshFaceDualGraph FaceGraph;
	FMeshFaceDualGraph::MakeFaceDualGraphForMesh(Mesh, FaceGraph, TrisConnectedPredicate);

	TIncrementalMeshDijkstra<FMeshFaceDualGraph> FurthestPoints(&FaceGraph);

	TArray<int32> SeedIndices;

	// need to add at least one seed point for each mesh connected component, so that all triangles are assigned a group
	// TODO: two seed points for components that have no boundary?
	FMeshConnectedComponents Components(Mesh);
	Components.FindConnectedTriangles(TrisConnectedPredicate);
	int32 NumConnected = Components.Num();
	for (int32 k = 0; k < NumConnected; ++k)
	{
		SeedIndices.Add(Components[k].Indices[0]);
	}

	// initial incremental update from per-component points
	TArray<TIncrementalMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint> ComponentSeeds;
	for (int32 vid : SeedIndices)
	{
		ComponentSeeds.Add(TIncrementalMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint{ vid, vid, 0.0 });
	}
	FurthestPoints.AddSeedPoints(ComponentSeeds);

	// TODO: can (approximately) bound the size of a region based on mesh area. Then can pass that in
	// with seed point as upper distance bound. This will change the result as it will initially grow on
	// the 'front' (and possibly no guarantee that mesh is covered?)
	//    (initial furthest-points would be any with an invalid value...then we can ensure coverage)
	// FindMaxGraphDistancePointID() seems like it might be somewhat expensive...

	// incrementally add furthest-points
	while ( SeedIndices.Num() < NumPoints)
	{
		int32 NextPointID = FurthestPoints.FindMaxGraphDistancePointID();
		if (NextPointID >= 0)
		{
			TIncrementalMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint NextSeedPoint{ NextPointID, NextPointID, 0.0 };
			FurthestPoints.AddSeedPoints({ NextSeedPoint });
			SeedIndices.Add(NextPointID);
		}
		else
		{
			break;
		}
	}


	// Now that we have furthest point set, recompute a Dijkstra propagation with optional weighting
	// (unweighted version should be the same as the FurthestPoints dijkstra though...could re-use?)

	TMeshDijkstra<FMeshFaceDualGraph> SuperPixels(&FaceGraph);
	
	if (WeightingType == EWeightingType::NormalDeviation)
	{
		SuperPixels.bEnableDistanceWeighting = true;
		SuperPixels.GetWeightedDistanceFunc = [this, WeightingCoeffs, &FaceGraph](int32 FromVID, int32 ToVID, int32 SeedVID, double Distance)
		{
			FVector3d NA = FaceGraph.GetNormal(ToVID);
			FVector3d NB = FaceGraph.GetNormal(SeedVID);
			double Dot = NA.Dot(NB);
			if (WeightingCoeffs.X > 0.001)
			{
				Dot = FMathd::Pow(Dot, WeightingCoeffs.X);
			}
			double W = FMathd::Clamp(1.0 - Dot * Dot, 0.0, 1.0);
			W = W * W * W;
			double Weight = FMathd::Clamp(W, 0.001, 1.0);
			return Weight * Distance;
		};
	}

	TArray<TMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint> SuperPixelSeeds;
	for (int32 k = 0; k < SeedIndices.Num(); ++k)
	{
		SuperPixelSeeds.Add(TMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint{ k, SeedIndices[k], 0.0 });
	}
	SuperPixels.ComputeToMaxDistance(SuperPixelSeeds, TNumericLimits<double>::Max());

	TArray<TArray<int32>> TriSets;
	TriSets.SetNum(SeedIndices.Num());
	TArray<int32> FailedSet;

	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		int32 SeedID = SuperPixels.GetSeedExternalIDForPointSetID(tid);
		if (SeedID >= 0)
		{
			TriSets[SeedID].Add(tid);
		}
		else
		{
			FailedSet.Add(tid);
		}
	}


	for (int32 k = 0; k < TriSets.Num(); ++k)
	{
		if (TriSets[k].Num() > 0)
		{
			FoundPolygroups.Add(MoveTemp(TriSets[k]));
		}
	}
	
	if (FailedSet.Num() > 0)
	{
		FMeshConnectedComponents FailedComponents(Mesh);
		FailedComponents.FindConnectedTriangles(FailedSet);
		for (FMeshConnectedComponents::FComponent& Component : FailedComponents)
		{
			FoundPolygroups.Add(Component.Indices);
		}
	}

	if (bApplyPostProcessing)
	{
		PostProcessPolygroups(true, TrisConnectedPredicate);
	}
	if (bCopyToMesh)
	{
		CopyPolygroupsToMesh();
	}

	return (FoundPolygroups.Num() > 0);
}


bool FPolygroupsGenerator::FindPolygroupsFromAreaDensityPointSampling(
	int32 NumPoints,
	EWeightingType WeightingType,
	FVector3d WeightingCoeffs,
	FPolygroupSet* StartingGroups)
{
	NumPoints = FMath::Min(NumPoints, Mesh->VertexCount());

	// cannot seem to use auto or TUniqueFunction here...
	TFunction<bool(int32, int32)> TrisConnectedPredicate;
	TrisConnectedPredicate = [](int, int) -> bool { return true; };
	if (StartingGroups != nullptr)
	{
		TrisConnectedPredicate = [StartingGroups](int a, int b) -> bool { return StartingGroups->GetGroup(a) == StartingGroups->GetGroup(b) ? true : false; };
	}

	double MeshTotalArea = 0.0;
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		MeshTotalArea += Mesh->GetTriArea(tid);
	}
	double ExpectedAreaPerSeed = MeshTotalArea / NumPoints;

	FMeshFaceDualGraph FaceGraph;
	FMeshFaceDualGraph::MakeFaceDualGraphForMesh(Mesh, FaceGraph, TrisConnectedPredicate);

	TArray<int32> SeedIndices;

	// need to add at least one seed point for each mesh connected component, so that all triangles are assigned a group
	// TODO: two seed points for components that have no boundary?
	FMeshConnectedComponents Components(Mesh);
	Components.FindConnectedTriangles(TrisConnectedPredicate);
	int32 NumConnected = Components.Num();

	struct FComponentContext
	{
		TArray< int32 > Seeds;
	};

	TArray<FComponentContext> Contexts;
	Contexts.SetNum(NumConnected);
	TArrayView<FComponentContext> ContextView = TArrayView<FComponentContext>(Contexts);

	ParallelForWithExistingTaskContext(ContextView, NumConnected, 1,
		[this, ExpectedAreaPerSeed, &FaceGraph, &Components](FComponentContext &Context, int32 k) {

			TArray<int32> ComponentSeeds;

			double ComponentTotalArea = 0.0;
			for (int32 tid : Components[k].Indices)
			{
				ComponentTotalArea += Mesh->GetTriArea(tid);
			}
			int32 DesiredSeeds = FMath::Max(1, FMath::CeilToInt32(ComponentTotalArea / ExpectedAreaPerSeed));
			
			ComponentSeeds.Add(Components[k].Indices[0]);

			// We're going to bail early if we only need one seed for this component.
			// No need to build the expensive Dijkstra weights if we're already done.
			if (DesiredSeeds == 1)
			{
				Context.Seeds.Append(ComponentSeeds);
				return;
			}

			TIncrementalMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint InitialSeedPoint{ ComponentSeeds[0], ComponentSeeds[0], 0.0 };
			TIncrementalMeshDijkstra<FMeshFaceDualGraph> FurthestPoints(&FaceGraph);
			FurthestPoints.AddSeedPoints({ InitialSeedPoint });

			while (ComponentSeeds.Num() < DesiredSeeds)
			{
				int32 NextPointID = FurthestPoints.FindMaxGraphDistancePointID();
				if (NextPointID >= 0)
				{
					TIncrementalMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint NextSeedPoint{ NextPointID, NextPointID, 0.0 };
					FurthestPoints.AddSeedPoints({ NextSeedPoint });
					ComponentSeeds.Add(NextPointID);
				}
				else
				{
					break;
				}
			}

			Context.Seeds.Append(ComponentSeeds);
		}
	);

	for (int32 k = 0; k < NumConnected; ++k)
	{
		SeedIndices.Append(Contexts[k].Seeds);
	}

	// Now that we have furthest point set, recompute a Dijkstra propagation with optional weighting
	// (unweighted version should be the same as the FurthestPoints dijkstra though...could re-use?)

	TMeshDijkstra<FMeshFaceDualGraph> SuperPixels(&FaceGraph);

	if (WeightingType == EWeightingType::NormalDeviation)
	{
		SuperPixels.bEnableDistanceWeighting = true;
		SuperPixels.GetWeightedDistanceFunc = [this, WeightingCoeffs, &FaceGraph](int32 FromVID, int32 ToVID, int32 SeedVID, double Distance)
		{
			FVector3d NA = FaceGraph.GetNormal(ToVID);
			FVector3d NB = FaceGraph.GetNormal(SeedVID);
			double Dot = NA.Dot(NB);
			if (WeightingCoeffs.X > 0.001)
			{
				Dot = FMathd::Pow(Dot, WeightingCoeffs.X);
			}
			double W = FMathd::Clamp(1.0 - Dot * Dot, 0.0, 1.0);
			W = W * W * W;
			double Weight = FMathd::Clamp(W, 0.001, 1.0);
			return Weight * Distance;
		};
	}

	TArray<TMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint> SuperPixelSeeds;
	for (int32 k = 0; k < SeedIndices.Num(); ++k)
	{
		SuperPixelSeeds.Add(TMeshDijkstra<FMeshFaceDualGraph>::FSeedPoint{ k, SeedIndices[k], 0.0 });
	}
	SuperPixels.ComputeToMaxDistance(SuperPixelSeeds, TNumericLimits<double>::Max());

	TArray<TArray<int32>> TriSets;
	TriSets.SetNum(SeedIndices.Num());
	TArray<int32> FailedSet;

	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		int32 SeedID = SuperPixels.GetSeedExternalIDForPointSetID(tid);
		if (SeedID >= 0)
		{
			TriSets[SeedID].Add(tid);
		}
		else
		{
			FailedSet.Add(tid);
		}
	}


	for (int32 k = 0; k < TriSets.Num(); ++k)
	{
		if (TriSets[k].Num() > 0)
		{
			FoundPolygroups.Add(MoveTemp(TriSets[k]));
		}
	}

	if (FailedSet.Num() > 0)
	{
		FMeshConnectedComponents FailedComponents(Mesh);
		FailedComponents.FindConnectedTriangles(FailedSet);
		for (FMeshConnectedComponents::FComponent& Component : FailedComponents)
		{
			FoundPolygroups.Add(Component.Indices);
		}
	}

	if (bApplyPostProcessing)
	{
		PostProcessPolygroups(true, TrisConnectedPredicate);
	}
	if (bCopyToMesh)
	{
		CopyPolygroupsToMesh();
	}

	return (FoundPolygroups.Num() > 0);
}

void FPolygroupsGenerator::PostProcessPolygroups(bool bApplyMerging, TFunctionRef<bool(int32, int32)> TrisConnectedPredicate)
{
	if (bApplyMerging && MinGroupSize > 1)
	{
		OptimizePolygroups(TrisConnectedPredicate);
	}
}


void FPolygroupsGenerator::OptimizePolygroups(TFunctionRef<bool(int32, int32)> TrisConnectedPredicate)
{
	FMeshRegionGraph RegionGraph;
	RegionGraph.BuildFromTriangleSets(*Mesh, FoundPolygroups, [&](int32 SetIdx) { return SetIdx; },
		                                                      TrisConnectedPredicate);		                                                      
	bool bMerged = RegionGraph.MergeSmallRegions(MinGroupSize-1, [&](int32 A, int32 B) { return RegionGraph.GetRegionTriCount(A) > RegionGraph.GetRegionTriCount(B); });
	bool bSwapped = RegionGraph.OptimizeBorders();
	if (bMerged || bSwapped)
	{
		FoundPolygroups.Reset();

		int32 N = RegionGraph.MaxRegionIndex();
		for (int32 k = 0; k < N; ++k)
		{
			if (RegionGraph.IsRegion(k))
			{
				const TArray<int32>& Tris = RegionGraph.GetRegionTris(k);
				FoundPolygroups.Add(Tris);
			}
		}
	}
}




void FPolygroupsGenerator::CopyPolygroupsToMesh()
{
	Mesh->EnableTriangleGroups(0);

	// set groups from Polygroups
	int NumPolygroups = FoundPolygroups.Num();

	// can be parallel for
	for (int PolyIdx = 0; PolyIdx < NumPolygroups; PolyIdx++)
	{
		const TArray<int>& Polygroup = FoundPolygroups[PolyIdx];
		int NumTriangles = Polygroup.Num();
		for (int k = 0; k < NumTriangles; ++k)
		{
			Mesh->SetTriangleGroup(Polygroup[k], InitialGroupID + (PolyIdx + 1));
		}
	}
}

void FPolygroupsGenerator::CopyPolygroupsToPolygroupSet(FPolygroupSet& Polygroups, FDynamicMesh3& TargetMesh)
{
	// set groups from Polygroups
	int NumPolygroups = FoundPolygroups.Num();

	// can be parallel for
	for (int PolyIdx = 0; PolyIdx < NumPolygroups; PolyIdx++)
	{
		const TArray<int>& Polygroup = FoundPolygroups[PolyIdx];
		int NumTriangles = Polygroup.Num();
		for (int k = 0; k < NumTriangles; ++k)
		{
			Polygroups.SetGroup(Polygroup[k], InitialGroupID + (PolyIdx + 1), TargetMesh);
		}
	}
}



bool FPolygroupsGenerator::FindPolygroupEdges()
{
	PolygroupEdges.Reset();

	for (int eid : Mesh->EdgeIndicesItr())
	{
		if (Mesh->IsGroupBoundaryEdge(eid))
		{
			PolygroupEdges.Add(eid);
		}
	}
	return (PolygroupEdges.Num() > 0);
}
