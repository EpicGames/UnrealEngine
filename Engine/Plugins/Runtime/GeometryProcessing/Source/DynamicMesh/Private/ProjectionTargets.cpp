// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectionTargets.h"
#include "Distance/DistPoint3Triangle3.h"

using namespace UE::Geometry;

FVector3d FMeshProjectionTarget::Project(const FVector3d& Point, int Identifier)
{
	double fDistSqr;
	int tNearestID = Spatial->FindNearestTriangle(Point, fDistSqr);
	if (tNearestID < 0)
	{
		return Point;
	}
	FTriangle3d Triangle;
	Mesh->GetTriVertices(tNearestID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);

	FDistPoint3Triangle3d DistanceQuery(Point, Triangle);
	DistanceQuery.GetSquared();
	if (VectorUtil::IsFinite(DistanceQuery.ClosestTrianglePoint))
	{
		return DistanceQuery.ClosestTrianglePoint;
	}
	else
	{
		return Point;
	}
}

/**
 * @return Projection of Point onto this target, and set ProjectNormalOut to the triangle normal at the returned point (*not* interpolated vertex normal)
 */
FVector3d FMeshProjectionTarget::Project(const FVector3d& Point, FVector3d& ProjectNormalOut, int Identifier)
{
	double fDistSqr;
	int tNearestID = Spatial->FindNearestTriangle(Point, fDistSqr);
	if (tNearestID < 0)
	{
		return Point;
	}
	FTriangle3d Triangle;
	Mesh->GetTriVertices(tNearestID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);

	ProjectNormalOut = Triangle.Normal();

	FDistPoint3Triangle3d DistanceQuery(Point, Triangle);
	DistanceQuery.GetSquared();
	if (VectorUtil::IsFinite(DistanceQuery.ClosestTrianglePoint))
	{
		return DistanceQuery.ClosestTrianglePoint;
	}
	else
	{
		return Point;
	}
}