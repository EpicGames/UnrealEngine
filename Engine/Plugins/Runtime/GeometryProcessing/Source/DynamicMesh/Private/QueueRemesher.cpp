// Copyright Epic Games, Inc. All Rights Reserved.

#include "QueueRemesher.h"
#include <iostream>
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

void FQueueRemesher::FastestRemesh()
{
	if (Mesh->HasAttributes() && GetConstraints().IsSet() == false)
	{
		ensureMsgf(false, TEXT("Input Mesh has Attribute overlays but no Constraints are configured. Use FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams() to create a Constraint Set for Attribute seams."));
	}

	ResetQueue();

	// First we do fast splits to hit edge length target

	for (int k = 0; k < MaxFastSplitIterations; ++k)
	{
		if (Cancelled())
		{
			return;
		}

		int nSplits = FastSplitIteration();
		ModifiedEdgesLastPass = nSplits;

		if ((double)nSplits / (double)Mesh->EdgeCount() < 0.01)
		{
			// Call it converged
			break;
		}
	}

	ResetQueue();

	// Now do Remesh iterations. Disable projection every other iteration to improve speed

	ETargetProjectionMode OriginalProjectionMode = ProjectionMode;
	for (int k = 0; k < MaxRemeshIterations - 1; ++k)
	{
		if (Cancelled())
		{
			break;
		}

		// if fraction of modified edges falls below threshold, consider the result converged and terminate
		auto EarlyTerminationCheck = [k, this]() {
			double ModifiedEdgeFraction = (double)ModifiedEdgesLastPass / (double)Mesh->EdgeCount();
			return (k > 10 && ModifiedEdgeFraction < MinActiveEdgeFraction);
		};

		ProjectionMode = (k % 2 == 0) ? ETargetProjectionMode::NoProjection : OriginalProjectionMode;
		RemeshIteration(EarlyTerminationCheck);

		// If fraction of active edges falls below threshold, terminate early
		if (EarlyTerminationCheck())
		{
			break;
		}
	}

	if (Cancelled())
	{
		return;
	}

	// Final pass with full projection
	ProjectionMode = OriginalProjectionMode;
	RemeshIteration( []() { return false; } );
}

int FQueueRemesher::FastSplitIteration()
{
	if (Mesh->TriangleCount() == 0)
	{
		return 0;
	}

	if (Mesh->HasAttributes() && GetConstraints().IsSet() == false)
	{
		ensureMsgf(false, TEXT("Input Mesh has Attribute overlays but no Constraints are configured. Use FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams() to create a Constraint Set for Attribute seams."));
	}

	// Temporarily turn off all operations except edge splits
	PushState();
	bEnableFlips = bEnableCollapses = bEnableSmoothing = false;
	ProjectionMode = ETargetProjectionMode::NoProjection;

	ProfileBeginPass();
	ProfileBeginOps();

	// When we split an edge, we need to check it and the adjacent ones we added.
	// Because of overhead in ProcessEdge, it is worth it to do a distance-check here

	double MaxEdgeLengthSqr = MaxEdgeLength * MaxEdgeLength;

	// Callback after an edge is split: enqueue new edges whose lengths are long enough
	PostEdgeSplitFunction = [this, &MaxEdgeLengthSqr](int EdgeID, int VertexA, int VertexB, int NewVertex) //-V1047 - This lambda is cleared before routine exit
	{
		FVector3d NewVertexPosition = Mesh->GetVertex(NewVertex);
		for (auto eid : Mesh->VtxEdgesItr(NewVertex))
		{
			FIndex2i EdgeVertices = Mesh->GetEdgeV(eid);
			int OtherVertex = (EdgeVertices.A == NewVertex) ? EdgeVertices.B : EdgeVertices.A;
			if (DistanceSquared(Mesh->GetVertex(OtherVertex), NewVertexPosition) > MaxEdgeLengthSqr)
			{
				QueueEdge(eid);
			}
		}
	};

	int TotalNumSplitEdges = 0;

	// Generic lambda to traverse a range of edges and process them. Below we will call this either with iterators
	// from the entire Mesh, or iterators over the set of Modified edges.
	auto DoSplits = [this, &TotalNumSplitEdges](const auto& Begin, const auto& End)
	{
		for (auto EdgeIter = Begin; EdgeIter != End; ++EdgeIter)
		{
			int CurrentEdgeID = *EdgeIter;

			if (Cancelled())
			{
				break;
			}

			if (Mesh->IsEdge(CurrentEdgeID))
			{
				EProcessResult result = ProcessEdge(CurrentEdgeID);

				if (result == EProcessResult::Ok_Split)
				{
					// new edges queued by PostSplitFunction
					++ModifiedEdgesLastPass;
					++TotalNumSplitEdges;
				}
			}
		}
	};

	if (!ModifiedEdges)
	{
		// FIRST ITERATION
		// If the set of ModifiedEdges doesn't exist, this must be the first time through this iteration. In this
		// case, iterate over the full set of Mesh edges.

		ModifiedEdges = TSet<int>(); // But first, create the ModifiedEdges set to write into.

		DoSplits(Mesh->EdgeIndicesItr().begin(), Mesh->EdgeIndicesItr().end());
	}
	else
	{
		// SUBSEQUENT ITERATIONS
		// If the set of ModifiedEdges *does* exist, iterate over (a copy of) that set.

		EdgeBuffer = MoveTemp(*ModifiedEdges);
		ModifiedEdges->Reset();

		DoSplits(EdgeBuffer.begin(), EdgeBuffer.end());
	}

	ProfileEndOps();

	PostEdgeSplitFunction = nullptr;

	// Turn back on pre-existing operations
	PopState();

	ProfileEndPass();

	return TotalNumSplitEdges;
}


void FQueueRemesher::RemeshIteration(TFunctionRef<bool(void)> EarlyTerminationCheck)
{
	ModifiedEdgesLastPass = 0;

	if (Mesh->TriangleCount() == 0)
	{
		return;
	}

	ProfileBeginPass();
	ProfileBeginOps();

	// Attempt to process an edge, and queue the resulting vertex neighborhoods
	auto ProcessCurrentEdge = [this](int CurrentEdgeID)
	{
		FIndex2i EdgeVertices = Mesh->GetEdgeV(CurrentEdgeID);
		FIndex2i OpposingEdgeVertices = Mesh->GetEdgeOpposingV(CurrentEdgeID);

		EProcessResult result = ProcessEdge(CurrentEdgeID);
		if (result == EProcessResult::Ok_Collapsed || result == EProcessResult::Ok_Flipped || result == EProcessResult::Ok_Split)
		{
			QueueOneRing(EdgeVertices.A);
			QueueOneRing(EdgeVertices.B);
			QueueOneRing(OpposingEdgeVertices.A);
			QueueOneRing(OpposingEdgeVertices.B);

			++ModifiedEdgesLastPass;
		}
	};

	int32 ProcessedEdges = 0;
	if (!ModifiedEdges)
	{
		// FIRST ITERATION
		// If the set of ModifiedEdges doesn't exist, this must be the first time through this iteration. In this
		// case, iterate over the full set of Mesh edges.

		ModifiedEdges = TSet<int>(); // But first, create the ModifiedEdges set to write into.

		int CurrentEdgeID = StartEdges();
		bool done = false;
		while (done == false)
		{
			if (Cancelled())
			{
				break;
			}

			if (Mesh->IsEdge(CurrentEdgeID))
			{
				ProcessCurrentEdge(CurrentEdgeID);
			}

			CurrentEdgeID = GetNextEdge(CurrentEdgeID, done);
		}
	}
	else
	{
		// SUBSEQUENT ITERATIONS
		// If the set of ModifiedEdges *does* exist, iterate over (a copy of) that set.

		EdgeBuffer = MoveTemp(*ModifiedEdges);
		ModifiedEdges->Reset();

		for (int CurrentEdgeID : EdgeBuffer)
		{
			if (Cancelled())
			{
				break;
			}

			if (!Mesh->IsEdge(CurrentEdgeID))
			{
				continue;
			}

			ProcessCurrentEdge(CurrentEdgeID);
		}
	}

	ProfileEndOps();

	if (EarlyTerminationCheck())
	{
		return;
	}
	if (Cancelled())
	{
		return;
	}

	ProfileBeginSmooth();
	if (bEnableSmoothing && SmoothSpeedT > 0)
	{
		TrackedFullSmoothPass_Buffer(bEnableParallelSmooth);
		DoDebugChecks();
	}
	ProfileEndSmooth();

	if (Cancelled())
	{
		return;
	}

	ProfileBeginProject();
	if (ProjTarget != nullptr && ProjectionMode == ETargetProjectionMode::AfterRefinement)
	{
		TrackedFullProjectionPass(bEnableParallelProjection);
		DoDebugChecks();
	}
	ProfileEndProject();

	ProfileEndPass();
}


void FQueueRemesher::InitializeVertexBufferForPass()
{
	FRemesher::InitializeVertexBufferForPass();

	// This is not done in FRemesher::InitializeVertexBufferForPass 
	for (size_t i = 0; i < TempPosBuffer.Num(); ++i)
	{
		TempPosBuffer[i] = FVector3d(0.0, 0.0, 0.0);
	}

	if (EdgeShouldBeQueuedBuffer.Num() < Mesh->MaxEdgeID())
	{
		EdgeShouldBeQueuedBuffer.SetNum(2 * Mesh->MaxEdgeID());
	}
	EdgeShouldBeQueuedBuffer.Init(false, EdgeShouldBeQueuedBuffer.Num());
}

void FQueueRemesher::TrackedMoveVerticesParallel(TFunction<FVector3d(int, bool&)> NewVertexPosition)
{
	// This is done in several passes:
	//
	// 1. Looping over all vertices, compute new vertex positions and put them in a buffer. Simultaneously fill a 
	//    parallel buffer of bools indicating which vertices were given new positions.
	// 2. Loop over all edges and determine if their new lengths would make them a candidate for splitting or collapsing.
	//    We maintain a temporary buffer of bools, one for each edge, which can be updated in this pass without 
	//    requiring a lock.
	// 3. Add all edges identified in step 2 to the queue of modified edges. This is done in serial, reading from the
	//    buffer of bools from step 2 and writing into ModifiedEdges.
	// 4. Copy all vertex positions saved in step 1 from the buffer into the Mesh data structure. Update the timestamps.
	
	// Note: Updating the queue of modified edges as the vertices change (using a mutex around ModifiedEdges) is 
	// significantly slower than this approach in my testing. The upside of the mutex approach is that it doesn't require
	// the auxiliary EdgeShouldBeQueuedBuffer data structure, so it requires less temporary memory.

	InitializeVertexBufferForPass();

	check(TempFlagBuffer.Num() >= Mesh->MaxVertexID());
	check(static_cast<int>(TempPosBuffer.Num()) >= Mesh->MaxVertexID());

	// First compute all vertex displacements and put them into a buffer
	ParallelFor(Mesh->MaxVertexID(), [this, NewVertexPosition](int32 VertexID)
	{
		if (!Mesh->IsVertex(VertexID)) { return; }

		const FVector3d CurrentPosition = Mesh->GetVertex(VertexID);
		bool bModified = false;
		const FVector3d NewPosition = NewVertexPosition(VertexID, bModified);

		if (bModified)
		{
			TempFlagBuffer[VertexID] = true;
			TempPosBuffer[VertexID] = NewPosition;
		}
	}, false);

	check(EdgeShouldBeQueuedBuffer.Num() >= Mesh->MaxEdgeID());

	// Now check all edges to see if they need to be added to the queue (they do if one of their end points moved.)
	ParallelFor(Mesh->MaxEdgeID(), [this](int32 EdgeID)
	{
		if (!Mesh->IsEdge(EdgeID)) { return; }

		const FIndex2i EdgeVertices = Mesh->GetEdgeV(EdgeID);
		auto VertexA = EdgeVertices[0]; auto VertexB = EdgeVertices[1];

		if (TempFlagBuffer[VertexA])
		{
			const FVector3d& SmoothedPosition = TempPosBuffer[VertexA];
			const FVector3d OtherVertexPosition = Mesh->GetVertex(VertexB);
			const double NewEdgeLengthSqr = DistanceSquared(SmoothedPosition, OtherVertexPosition);
			if (NewEdgeLengthSqr < MinEdgeLength*MinEdgeLength || NewEdgeLengthSqr > MaxEdgeLength*MaxEdgeLength)
			{
				EdgeShouldBeQueuedBuffer[EdgeID] = true;
			}
		}

		if (TempFlagBuffer[VertexB] && !EdgeShouldBeQueuedBuffer[EdgeID])
		{
			const FVector3d& SmoothedPosition = TempPosBuffer[VertexB];
			const FVector3d OtherVertexPosition = Mesh->GetVertex(VertexA);
			const double NewEdgeLengthSqr = DistanceSquared(SmoothedPosition, OtherVertexPosition);
			if (NewEdgeLengthSqr < MinEdgeLength * MinEdgeLength || NewEdgeLengthSqr > MaxEdgeLength* MaxEdgeLength)
			{
				EdgeShouldBeQueuedBuffer[EdgeID] = true;
			}
		}

	}, false);

	// Now add all flagged edges to the queue (serial loop due to the shared data structure ModifiedEdges)
	for (auto EdgeID : Mesh->EdgeIndicesItr())
	{
		if (!Mesh->IsEdge(EdgeID)) { continue; }

		if (EdgeShouldBeQueuedBuffer[EdgeID])
		{
			QueueEdge(EdgeID);
		}
	}

	// Finally move the vertex positions according to the buffer
	ApplyVertexBuffer(true);


	// Needed for determinism, to get the same results as serial mode.
	// TODO: Should we have a switch for deterministic mode?
	ModifiedEdges->Sort(std::less<int>());
}

void FQueueRemesher::TrackedFullSmoothPass_Buffer(bool bParallel)
{
	if (bParallel) 
	{
		TFunction<FVector3d(const FDynamicMesh3&, int, double)> UseSmoothFunc = GetSmoothFunction();
		auto VertexMoveFunction = [&UseSmoothFunc, this](int VertexID, bool& bModified)
		{
			return ComputeSmoothedVertexPos(VertexID, UseSmoothFunc, bModified);
		};
		TrackedMoveVerticesParallel(VertexMoveFunction);
		return;
	}

	// Serial:

	InitializeVertexBufferForPass();

	TFunction<FVector3d(const FDynamicMesh3&, int, double)> UseSmoothFunc = GetSmoothFunction();

	auto SmoothAndUpdateFunc = [this, &UseSmoothFunc](int VertexID)
	{
		if (!Mesh->IsVertex(VertexID))
		{
			return;
		}

		FVector3d CurrentPosition = Mesh->GetVertex(VertexID);
		bool bModified = false;
		FVector3d SmoothedPosition = ComputeSmoothedVertexPos(VertexID, UseSmoothFunc, bModified);

		if (bModified)
		{
			TempFlagBuffer[VertexID] = true;
			TempPosBuffer[VertexID] = SmoothedPosition;

			for (int EdgeID : Mesh->VtxEdgesItr(VertexID))
			{
				FIndex2i EdgeVertices = Mesh->GetEdgeV(EdgeID);
				int OtherVertexID = (EdgeVertices.A == VertexID) ? EdgeVertices.B : EdgeVertices.A;
				FVector3d OtherVertexPosition = Mesh->GetVertex(OtherVertexID);

				double NewEdgeLength = Distance(SmoothedPosition, OtherVertexPosition);
				if (NewEdgeLength < MinEdgeLength || NewEdgeLength > MaxEdgeLength)
				{
					QueueEdge(EdgeID);
				}
			}
		}
	};

	check(TempFlagBuffer.Num() >= Mesh->MaxVertexID());
	check(static_cast<int>(TempPosBuffer.Num()) >= Mesh->MaxVertexID());
	check(EdgeShouldBeQueuedBuffer.Num() >= Mesh->MaxEdgeID());

	ApplyToSmoothVertices(SmoothAndUpdateFunc);

	ApplyVertexBuffer(bParallel);

	// Needed for determinism, to get the same results as parallel mode.
	// TODO: Should we have a switch for deterministic mode?
	ModifiedEdges->Sort(std::less<int>());
}

void FQueueRemesher::TrackedFullProjectionPass(bool bParallel)
{
	if (bParallel)
	{
		auto VertexMoveFunction = [this](int VertexID, bool& bModified)
		{
			bModified = false;
			FVector3d CurrentPosition = Mesh->GetVertex(VertexID);

			if (IsVertexPositionConstrained(VertexID))
			{
				return CurrentPosition;
			}
			if (VertexControlF != nullptr && ((int)VertexControlF(VertexID) & (int)EVertexControl::NoProject) != 0)
			{
				return CurrentPosition;
			}

			FVector3d ProjectedPosition = ProjTarget->Project(CurrentPosition, VertexID);
			bModified = !VectorUtil::EpsilonEqual(CurrentPosition, ProjectedPosition, FMathd::ZeroTolerance);

			return ProjectedPosition;
		};

		TrackedMoveVerticesParallel(VertexMoveFunction);
		return;
	}

	// Serial:

	InitializeVertexBufferForPass();

	ensure(ProjTarget != nullptr);

	auto UseProjectionFunc = [this](int VertexID)
	{
		if (IsVertexPositionConstrained(VertexID))
		{
			return;
		}

		if (VertexControlF != nullptr && ((int)VertexControlF(VertexID) & (int)EVertexControl::NoProject) != 0)
		{
			return;
		}

		FVector3d CurrentPosition = Mesh->GetVertex(VertexID);
		FVector3d ProjectedPosition = ProjTarget->Project(CurrentPosition, VertexID);

		bool bModified = !VectorUtil::EpsilonEqual(CurrentPosition, ProjectedPosition, FMathd::ZeroTolerance);

		if (bModified)
		{
			TempFlagBuffer[VertexID] = true;
			TempPosBuffer[VertexID] = ProjectedPosition;

			for (int EdgeID : Mesh->VtxEdgesItr(VertexID))
			{
				FIndex2i EdgeVertices = Mesh->GetEdgeV(EdgeID);
				int OtherVertexID = (EdgeVertices.A == VertexID) ? EdgeVertices.B : EdgeVertices.A;
				FVector3d OtherVertexPosition = Mesh->GetVertex(OtherVertexID);

				double NewEdgeLength = Distance(ProjectedPosition, OtherVertexPosition);
				if (NewEdgeLength < MinEdgeLength || NewEdgeLength > MaxEdgeLength)
				{
					QueueEdge(EdgeID);
				}
			}
		}
	};

	ApplyToProjectVertices(UseProjectionFunc);

	ApplyVertexBuffer(bParallel);

	// Needed for determinism, to get the same results as parallel mode.
	// TODO: Should we have a switch for deterministic mode?
	ModifiedEdges->Sort(std::less<int>());
}

