// Copyright Epic Games, Inc. All Rights Reserved.

#include "Remesher.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshWeights.h"

#include "Async/ParallelFor.h"

using namespace UE::Geometry;

void FRemesher::SetTargetEdgeLength(double fLength)
{
	// from Botsch paper
	//MinEdgeLength = fLength * (4.0/5.0);
	//MaxEdgeLength = fLength * (4.0/3.0);
	// much nicer!! makes sense as when we split, edges are both > min !
	MinEdgeLength = fLength * 0.66;
	MaxEdgeLength = fLength * 1.33;
}



void FRemesher::Precompute()
{
	// if we know Mesh is closed, we can skip is-boundary checks, which makes
	// the flip-valence tests much faster!
	bMeshIsClosed = true;
	for (int eid : Mesh->EdgeIndicesItr()) 
	{
		if (Mesh->IsBoundaryEdge(eid)) 
		{
			bMeshIsClosed = false;
			break;
		}
	}
}





void FRemesher::BasicRemeshPass() 
{
	if (Mesh->TriangleCount() == 0)    // badness if we don't catch this...
	{
		return;
	}

	if (Mesh->HasAttributes() && GetConstraints().IsSet() == false)
	{
		ensureMsgf(false, TEXT("Input Mesh has Attribute overlays but no Constraints are configured. Use FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams() to create a Constraint Set for Attribute seams."));
	}

	ProfileBeginPass();

	// Iterate over all edges in the mesh at start of pass.
	// Some may be removed, so we skip those.
	// However, some old eid's may also be re-used, so we will touch
	// some new edges. Can't see how we could efficiently prevent this.
	//
	ProfileBeginOps();
	ModifiedEdgesLastPass = 0;
	{
		int cur_eid = StartEdges();
		bool done = false;
		int IterationCount = 0;
		TRACE_CPUPROFILER_EVENT_SCOPE(Remesher_EdgesPass);
		do
		{
			if (Mesh->IsEdge(cur_eid))
			{
				EProcessResult result = ProcessEdge(cur_eid);
				if (result == EProcessResult::Ok_Collapsed || result == EProcessResult::Ok_Flipped || result == EProcessResult::Ok_Split)
				{
					ModifiedEdgesLastPass++;
				}
			}
			if (IterationCount++ % 1000 == 0 && Cancelled())        // expensive to check every iter?
			{
				return;
			}
			cur_eid = GetNextEdge(cur_eid, done);
		} while (done == false);
	}
	ProfileEndOps();

	if (Cancelled())
	{
		return;
	}

	ProfileBeginSmooth();
	if (bEnableSmoothing && (SmoothSpeedT > 0 || CustomSmoothSpeedF) )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Remesher_SmoothingPass);
		if (bEnableSmoothInPlace) 
		{
			//FullSmoothPass_InPlace(EnableParallelSmooth);
			check(false);
		}
		else
		{
			FullSmoothPass_Buffer(bEnableParallelSmooth);
		}
		DoDebugChecks();
	}
	ProfileEndSmooth();

	if (Cancelled())
	{
		return;
	}

	ProfileBeginProject();
	if (ProjTarget != nullptr && ProjectionMode == ETargetProjectionMode::AfterRefinement)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Remesher_ProjectionPass);
		FullProjectionPass(bEnableParallelProjection);
		DoDebugChecks();
	}
	ProfileEndProject();

	DoDebugChecks(true);

	if (Cancelled())
	{
		return;
	}

	ProfileEndPass();
}





FRemesher::EProcessResult FRemesher::ProcessEdge(int edgeID)
{
	RuntimeDebugCheck(edgeID);

	FEdgeConstraint constraint =
		(!Constraints) ? FEdgeConstraint::Unconstrained() : Constraints->GetEdgeConstraint(edgeID);
	if (constraint.NoModifications())
	{
		return EProcessResult::Ignored_EdgeIsFullyConstrained;
	}

	// look up verts and tris for this edge
	FDynamicMesh3::FEdge Edge(Mesh->GetEdge(edgeID));
	int a = Edge.Vert[0], b = Edge.Vert[1];
	int t0 = Edge.Tri[0], t1 = Edge.Tri[1];
	bool bIsBoundaryEdge = (t1 == IndexConstants::InvalidID);
	FVector3d vA(Mesh->GetVertex(a));
	FVector3d vB(Mesh->GetVertex(b));
	double edge_len_sqr = DistanceSquared(vA, vB);

	if (CustomEdgeLengthScaleF)
	{
		const double EdgeLengthScale = CustomEdgeLengthScaleF(*Mesh, a, b);
		edge_len_sqr = (EdgeLengthScale * EdgeLengthScale) * edge_len_sqr;
	}

	// check if we should collapse, and also find which vertex we should collapse to,
	// in cases where we have constraints/etc
	int collapse_to = -1;
	bool bCanCollapse = bEnableCollapses
		&& constraint.CanCollapse()
		&& edge_len_sqr < MinEdgeLength* MinEdgeLength;

	// if edge length is too short, we want to collapse it
	bool bTriedCollapse = false;
	if (bCanCollapse) 
	{
		// look up 'other' verts c (from t0) and d (from t1, if it exists)
		FIndex2i ov = Mesh->GetEdgeOpposingV(edgeID);
		int c = ov[0], d = ov[1];
		if (CanCollapseEdge(edgeID, a, b, c, d, t0, t1, collapse_to) == false)
		{
			goto abort_collapse;
		}
		// optimization: if edge cd exists, we cannot collapse or flip. look that up here?
		//  funcs will do it internally...
		//  (or maybe we can collapse if cd exists? edge-collapse doesn't check for it explicitly...)

		int iKeep;
		int iCollapse;
		double collapse_t;		// need to know t-value along edge to update lerpable attributes properly
		FVector3d vNewPos;

		// If either vtx is fixed, make that vertex iKeep and set collapse_t to 0 so vNewPos corresponds to the kept vtx
		// Otherwise set collapse_t and vNewPos using the projected edge midpoint and measure collapse_t from iKeep
		if (collapse_to == b)
		{
			iKeep = b;
			iCollapse = a;
			collapse_t = 0;
			vNewPos = vB;
		}
		else if (collapse_to == a)
		{
			iKeep = a;
			iCollapse = b;
			collapse_t = 0;
			vNewPos = vA;
		}
		else
		{
			iKeep = b;
			iCollapse = a;
			vNewPos = GetProjectedCollapsePosition(iKeep, (vA + vB) * 0.5);
			double div = Distance(vA, vB);
			collapse_t = (div < FMathd::ZeroTolerance) ? 0.5 : (Distance(vNewPos, Mesh->GetVertex(iKeep))) / div;
			collapse_t = VectorUtil::Clamp(collapse_t, 0.0, 1.0);
		}

		// if new position would flip normal of one of the existing triangles
		// either one-ring, don't allow it
		if (bPreventNormalFlips) 
		{
			if (CheckIfCollapseCreatesFlipOrInvalid(a, b, vNewPos, t0, t1) || CheckIfCollapseCreatesFlipOrInvalid(b, a, vNewPos, t0, t1)) 
			{
				goto abort_collapse;
			}
		}

		if (bPreventTinyTriangles)
		{
			if (CheckIfCollapseCreatesTinyTriangle(a, b, vNewPos, t0, t1) || CheckIfCollapseCreatesTinyTriangle(b, a, vNewPos, t0, t1))
			{
				goto abort_collapse;
			}
		}


		// lots of cases where we cannot collapse, but we should just let
		// mesh sort that out, right?
		SaveEdgeBeforeModify(edgeID);
		COUNT_COLLAPSES++;
		FDynamicMesh3::FEdgeCollapseInfo collapseInfo;
		EMeshResult result = Mesh->CollapseEdge(iKeep, iCollapse, collapse_t, collapseInfo);
		if (result == EMeshResult::Ok) 
		{
			Mesh->SetVertex(iKeep, vNewPos);
			if (Constraints)
			{
				Constraints->ClearEdgeConstraint(edgeID);
				Constraints->ClearEdgeConstraint(collapseInfo.RemovedEdges.A);
				if (collapseInfo.RemovedEdges.B != IndexConstants::InvalidID)
				{
					Constraints->ClearEdgeConstraint(collapseInfo.RemovedEdges.B);
				}
				Constraints->ClearVertexConstraint(iCollapse);
			}
			OnEdgeCollapse(edgeID, iKeep, iCollapse, collapseInfo);
			DoDebugChecks();

			return EProcessResult::Ok_Collapsed;
		}
		else
		{
			bTriedCollapse = true;
		}
	}
abort_collapse:

	// if this is not a boundary edge, maybe we want to flip
	bool bTriedFlip = false;
	if (bEnableFlips && constraint.CanFlip() && bIsBoundaryEdge == false) 
	{
		// look up 'other' verts c (from t0) and d (from t1, if it exists)
		FIndex2i ov = Mesh->GetEdgeOpposingV(edgeID);
		int c = ov[0], d = ov[1];

		// can we do this more efficiently somehow?
		bool bTryFlip = false;
		if (FlipMetric == EFlipMetric::OptimalValence)
		{
			bool a_is_boundary_vtx = (bMeshIsClosed) ? false : (bIsBoundaryEdge || Mesh->IsBoundaryVertex(a));
			bool b_is_boundary_vtx = (bMeshIsClosed) ? false : (bIsBoundaryEdge || Mesh->IsBoundaryVertex(b));
			bool c_is_boundary_vtx = (bMeshIsClosed) ? false : Mesh->IsBoundaryVertex(c);
			bool d_is_boundary_vtx = (bMeshIsClosed) ? false : Mesh->IsBoundaryVertex(d);
			int valence_a = Mesh->GetVtxEdgeCount(a), valence_b = Mesh->GetVtxEdgeCount(b);
			int valence_c = Mesh->GetVtxEdgeCount(c), valence_d = Mesh->GetVtxEdgeCount(d);
			int valence_a_target = (a_is_boundary_vtx) ? valence_a : 6;
			int valence_b_target = (b_is_boundary_vtx) ? valence_b : 6;
			int valence_c_target = (c_is_boundary_vtx) ? valence_c : 6;
			int valence_d_target = (d_is_boundary_vtx) ? valence_d : 6;

			// if total valence error improves by flip, we want to do it
			int curr_err = abs(valence_a - valence_a_target) + abs(valence_b - valence_b_target)
				+ abs(valence_c - valence_c_target) + abs(valence_d - valence_d_target);
			int flip_err = abs((valence_a - 1) - valence_a_target) + abs((valence_b - 1) - valence_b_target)
				+ abs((valence_c + 1) - valence_c_target) + abs((valence_d + 1) - valence_d_target);

			bTryFlip = flip_err < curr_err;
		}
		else
		{
			double CurDistSqr = DistanceSquared(Mesh->GetVertex(a), Mesh->GetVertex(b));
			double FlipDistSqr = DistanceSquared(Mesh->GetVertex(c), Mesh->GetVertex(d));
			bTryFlip = (FlipDistSqr < MinLengthFlipThresh*MinLengthFlipThresh * CurDistSqr);
		}

		if (bTryFlip && bPreventNormalFlips && CheckIfFlipInvertsNormals(a, b, c, d, t0))
		{
			bTryFlip = false;
		}

		if (bTryFlip && bPreventTinyTriangles && CheckIfFlipCreatesTinyTriangle(a, b, c, d, t0))
		{
			bTryFlip = false;
		}

		if (bTryFlip) 
		{
			SaveEdgeBeforeModify(edgeID);
			FDynamicMesh3::FEdgeFlipInfo flipInfo;
			COUNT_FLIPS++;
			EMeshResult result = Mesh->FlipEdge(edgeID, flipInfo);
			if (result == EMeshResult::Ok) 
			{
				OnEdgeFlip(edgeID, flipInfo);
				DoDebugChecks();
				return EProcessResult::Ok_Flipped;
			}
			else
			{
				bTriedFlip = true;
			}

		}
	}

	// if edge length is too long, we want to split it
	bool bTriedSplit = false;
	if (bEnableSplits && constraint.CanSplit() && edge_len_sqr > MaxEdgeLength*MaxEdgeLength) 
	{
		SaveEdgeBeforeModify(edgeID);
		FDynamicMesh3::FEdgeSplitInfo SplitInfo;
		COUNT_SPLITS++;
		EMeshResult result = Mesh->SplitEdge(edgeID, SplitInfo);
		if (result == EMeshResult::Ok) 
		{
			UpdateAfterSplit(edgeID, a, b, SplitInfo);
			OnEdgeSplit(edgeID, a, b, SplitInfo);
			DoDebugChecks();
			return EProcessResult::Ok_Split;
		}
		else
		{
			bTriedSplit = true;
		}
	}

	if (bTriedFlip || bTriedSplit || bTriedCollapse)
	{
		return EProcessResult::Failed_OpNotSuccessful;
	}
	else
	{
		return EProcessResult::Ignored_EdgeIsFine;
	}
}






void FRemesher::UpdateAfterSplit(int edgeID, int va, int vb, const FDynamicMesh3::FEdgeSplitInfo& SplitInfo)
{
	bool bPositionFixed = false;
	if (Constraints && Constraints->HasEdgeConstraint(edgeID))
	{
		// inherit edge constraint
		Constraints->SetOrUpdateEdgeConstraint(SplitInfo.NewEdges.A, Constraints->GetEdgeConstraint(edgeID));

		// Update vertex constraints. Note that there is some ambiguity here.
		//   Both verts being constrained doesn't inherently mean that the edge is on
		//   a constraint, that's why these checks are only applied if edge is constrained.
		//   But constrained edge doesn't necessarily mean we want to inherit vert constraints!!
		//
		//   although, pretty safe to assume that we would at least disable flips
		//   if both vertices are constrained to same line/curve. So, maybe this makes sense...
		//
		//   (TODO: perhaps edge constraint should be explicitly tagged to resolve this ambiguity??)

		// vert inherits Fixed if both orig edge verts Fixed, and both tagged with same SetID
		FVertexConstraint ca = Constraints->GetVertexConstraint(va);
		FVertexConstraint cb = Constraints->GetVertexConstraint(vb);
		if (ca.bCannotDelete && cb.bCannotDelete)
		{
			int nSetID = (ca.FixedSetID > 0 && ca.FixedSetID == cb.FixedSetID) ?
				ca.FixedSetID : FVertexConstraint::InvalidSetID;
			bool bMovable = ca.bCanMove && cb.bCanMove;
			Constraints->SetOrUpdateVertexConstraint(SplitInfo.NewVertex,
				FVertexConstraint(true, bMovable, nSetID));
			bPositionFixed = true;
		}

		// vert inherits Target if:
		//  1) both source verts and edge have same Target, and is same as edge target, or
		//  2) either vert has same target as edge, and other vert can't move
		if (ca.Target != nullptr || cb.Target != nullptr) 
		{
			IProjectionTarget* edge_target = Constraints->GetEdgeConstraint(edgeID).Target;
			IProjectionTarget* set_target = nullptr;
			if (ca.Target == cb.Target && ca.Target == edge_target)
			{
				set_target = edge_target;
			}
			else if (ca.Target == edge_target && !cb.bCanMove)
			{
				set_target = edge_target;
			}
			else if (cb.Target == edge_target && !ca.bCanMove)
			{
				set_target = edge_target;
			}

			if (set_target != nullptr) 
			{
				Constraints->SetOrUpdateVertexConstraint(SplitInfo.NewVertex,
					FVertexConstraint(set_target));
				ProjectVertex(SplitInfo.NewVertex, set_target);
				bPositionFixed = true;
			}
		}
	}

	if (EnableInlineProjection() && bPositionFixed == false && ProjTarget != nullptr) 
	{
		ProjectVertex(SplitInfo.NewVertex, ProjTarget);
	}
}



void FRemesher::ProjectVertex(int VertexID, IProjectionTarget* UseTarget)
{
	FVector3d curpos = Mesh->GetVertex(VertexID);
	FVector3d projected = UseTarget->Project(curpos, VertexID);
	Mesh->SetVertex(VertexID, projected);
}

// used by collapse-edge to get projected position for new vertex
FVector3d FRemesher::GetProjectedCollapsePosition(int vid, const FVector3d& vNewPos)
{
	if (Constraints)
	{
		FVertexConstraint vc = Constraints->GetVertexConstraint(vid);
		if (vc.Target != nullptr)
		{
			return vc.Target->Project(vNewPos, vid);
		}
		if (vc.bCanMove == false)
		{
			return vNewPos;
		}
	}
	// no constraint applied, so if we have a target surface, project to that
	if (EnableInlineProjection() && ProjTarget != nullptr) 
	{
		if (VertexControlF == nullptr || ((int)VertexControlF(vid) & (int)EVertexControl::NoProject) == 0)
		{
			return ProjTarget->Project(vNewPos, vid);
		}
	}
	return vNewPos;
}




static FVector3d UniformSmooth(const FDynamicMesh3& mesh, int vID, double t)
{
	FVector3d v = mesh.GetVertex(vID);
	FVector3d c;
	mesh.GetVtxOneRingCentroid(vID, c);
	return (1.0 - t)*v + (t)*c;
}


static FVector3d MeanValueSmooth(const FDynamicMesh3& mesh, int vID, double t)
{
	FVector3d v = mesh.GetVertex(vID);
	FVector3d c = FMeshWeights::MeanValueCentroid(mesh, vID);
	return (1.0 - t)*v + (t)*c;
}

static FVector3d CotanSmooth(const FDynamicMesh3& mesh, int vID, double t)
{
	FVector3d v = mesh.GetVertex(vID);
	FVector3d c = FMeshWeights::CotanCentroid(mesh, vID);
	return (1.0 - t)*v + (t)*c;
}

TFunction<FVector3d(const FDynamicMesh3&, int, double)> FRemesher::GetSmoothFunction()
{
	if (CustomSmoothF != nullptr)
	{
		return CustomSmoothF;
	} 
	else if (SmoothType == ESmoothTypes::MeanValue)
	{
		return MeanValueSmooth;
	}
	else if (SmoothType == ESmoothTypes::Cotan)
	{
		return CotanSmooth;
	}
	return UniformSmooth;
}


void FRemesher::FullSmoothPass_Buffer(bool bParallel)
{
	TFunction<FVector3d(const FDynamicMesh3&, int, double)> UseSmoothFunc = GetSmoothFunction();

	if (bParallel)
	{
		auto VertexMoveFunction = [&UseSmoothFunc, this](int VertexID, bool& bModified)
		{
			return ComputeSmoothedVertexPos(VertexID, UseSmoothFunc, bModified);
		};

		MoveVerticesParallel(VertexMoveFunction);
	}
	else
	{
		// Serial

		InitializeVertexBufferForPass();

		auto SmoothAndUpdateFunc = [this, UseSmoothFunc](int vID)
		{
			bool bModified = false;
			FVector3d vSmoothed = ComputeSmoothedVertexPos(vID, UseSmoothFunc, bModified);
			if (bModified)
			{
				TempFlagBuffer[vID] = true;
				TempPosBuffer[vID] = vSmoothed;
			}
		};

		ApplyToSmoothVertices(SmoothAndUpdateFunc);

		ApplyVertexBuffer(false);
	}
}






void FRemesher::InitializeVertexBufferForPass()
{
	if ((int)TempPosBuffer.GetLength() < Mesh->MaxVertexID())
	{
		TempPosBuffer.Resize(Mesh->MaxVertexID() + Mesh->MaxVertexID() / 5);
	}
	if (TempFlagBuffer.Num() < Mesh->MaxVertexID())
	{
		TempFlagBuffer.SetNum(2 * Mesh->MaxVertexID());
	}
	TempFlagBuffer.Init(false, TempFlagBuffer.Num());
}


void FRemesher::ApplyVertexBuffer(bool bParallel)
{
	check(TempFlagBuffer.Num() >= Mesh->MaxVertexID());
	check(static_cast<int>(TempPosBuffer.Num()) >= Mesh->MaxVertexID());

	if (!bParallel)
	{
		// Serial
		for (int vid : Mesh->VertexIndicesItr())
		{
			if (TempFlagBuffer[vid])
			{
				Mesh->SetVertex(vid, TempPosBuffer[vid], false);
			}
		}
		Mesh->UpdateChangeStamps(true, false);
	}
	else
	{
		// TODO: verify that this batching is still necessary, now that timestamps/locking situation has been improved
		const int BatchSize = 1000;
		const int NumBatches = FMath::CeilToInt32(static_cast<float>(Mesh->MaxVertexID()) / static_cast<float>(BatchSize));
		ParallelFor(NumBatches, [this, BatchSize](int32 BatchID)
		{
			for (int VertexID = BatchID * BatchSize; VertexID < (BatchID + 1) * BatchSize; ++VertexID)
			{
				if (VertexID < TempFlagBuffer.Num() && TempFlagBuffer[VertexID] && Mesh->IsVertex(VertexID))
				{
					Mesh->SetVertex(VertexID, TempPosBuffer[VertexID], false);
				}
			}
		}, false);
		Mesh->UpdateChangeStamps(true, false);
	}
}

FVector3d FRemesher::ComputeSmoothedVertexPos(int vID,
	TFunction<FVector3d(const FDynamicMesh3&, int, double)> smoothFunc, bool& bModified)
{
	bModified = false;
	FVertexConstraint vConstraint = FVertexConstraint::Unconstrained();
	GetVertexConstraint(vID, vConstraint);
	if (vConstraint.bCanMove == false)
	{
		return Mesh->GetVertex(vID);
	}
	EVertexControl vControl = (VertexControlF == nullptr) ? EVertexControl::AllowAll : VertexControlF(vID);
	if (((int)vControl & (int)EVertexControl::NoSmooth) != 0)
	{
		return Mesh->GetVertex(vID);
	}

	double UseSmoothSpeed = (CustomSmoothSpeedF) ? CustomSmoothSpeedF(*Mesh, vID) : SmoothSpeedT;
	if (UseSmoothSpeed <= 0)
	{
		return Mesh->GetVertex(vID);
	}

	FVector3d vSmoothed = smoothFunc(*Mesh, vID, SmoothSpeedT);
	// @todo we should probably make sure that vertex does not move too far here...
	checkSlow(VectorUtil::IsFinite(vSmoothed));
	if (VectorUtil::IsFinite(vSmoothed) == false)
	{
		return Mesh->GetVertex(vID);
	}

	// project onto either vtx constraint target, or surface target
	if (vConstraint.Target != nullptr) 
	{
		vSmoothed = vConstraint.Target->Project(vSmoothed, vID);
	}
	else if (EnableInlineProjection() && ProjTarget != nullptr) 
	{
		if (((int)vControl & (int)EVertexControl::NoProject) == 0)
		{
			vSmoothed = ProjTarget->Project(vSmoothed, vID);
		}
	}

	bModified = true;
	return vSmoothed;
}


void FRemesher::ApplyToSmoothVertices(const TFunction<void(int)>& VertexSmoothFunc)
{
	for (int vid : Mesh->VertexIndicesItr())
	{
		VertexSmoothFunc(vid);
	}
}




// Project vertices onto projection target. 
void FRemesher::FullProjectionPass(bool bParallel)
{
	if (bParallel)
	{
		auto VertexMoveFunction = [this](int VertexID, bool& bModified)
		{
			bModified = false;
			FVector3d CurrentPosition = Mesh->GetVertex(VertexID);

			if (IsVertexPositionConstrained(VertexID))
			{
				return CurrentPosition;
			}
			if (VertexControlF != nullptr && ((int)VertexControlF(VertexID) & (int)EVertexControl::NoProject) != 0)
			{
				return CurrentPosition;
			}

			FVector3d ProjectedPosition = ProjTarget->Project(CurrentPosition, VertexID);
			bModified = !VectorUtil::EpsilonEqual(CurrentPosition, ProjectedPosition, FMathd::ZeroTolerance);

			return ProjectedPosition;
		};

		MoveVerticesParallel(VertexMoveFunction);
	}
	else
	{
		// Serial

		auto UseProjectionFunc = [this](int vID)
		{
			if (IsVertexPositionConstrained(vID))
			{
				return;
			}
			if (VertexControlF != nullptr && ((int)VertexControlF(vID) & (int)EVertexControl::NoProject) != 0)
			{
				return;
			}
			FVector3d curpos = Mesh->GetVertex(vID);
			FVector3d projected = ProjTarget->Project(curpos, vID);
			Mesh->SetVertex(vID, projected);
		};

		ApplyToProjectVertices(UseProjectionFunc);
	}

}




void FRemesher::ApplyToProjectVertices(const TFunction<void(int)>& VertexProjectFunc)
{
	for (int vid : Mesh->VertexIndicesItr())
	{
		VertexProjectFunc(vid);
	}
}



void FRemesher::MoveVerticesParallel(TFunction<FVector3d(int, bool&)> NewVertexPosition)
{
	// This is done in two passes:
	// 1. Looping over all vertices, compute new vertex positions and put them in a buffer. Simultaneously fill a 
	//    parallel buffer of bools indicating which vertices were given new positions.
	// 2. Copy all vertex positions saved in step 1 from the buffer into the Mesh data structure. Update the timestamps.

	InitializeVertexBufferForPass();

	check(TempFlagBuffer.Num() >= Mesh->MaxVertexID());
	check(static_cast<int>(TempPosBuffer.Num()) >= Mesh->MaxVertexID());

	// First compute all vertex displacements and put them into a buffer
	ParallelFor(Mesh->MaxVertexID(), [this, NewVertexPosition](int32 VertexID)
	{
		if (!Mesh->IsVertex(VertexID)) { return; }

		const FVector3d CurrentPosition = Mesh->GetVertex(VertexID);
		bool bModified = false;
		const FVector3d NewPosition = NewVertexPosition(VertexID, bModified);

		if (bModified)
		{
			TempFlagBuffer[VertexID] = true;
			TempPosBuffer[VertexID] = NewPosition;
		}
	}, false);

	// Finally move the vertex positions according to the buffer
	ApplyVertexBuffer(true);

}
