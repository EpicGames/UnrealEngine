// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshBakerCommon.h"
#include "Sampling/MeshBaseBaker.h"
#include "MeshQueries.h"

using namespace UE::Geometry;


/**
 * Find point on Detail mesh that corresponds to point on Base mesh.
 * Strategy is:
 *    1) cast a ray inwards along -Normal from BasePoint + Thickness*Normal
 *    2) cast a ray outwards along Normal from BasePoint
 *    3) cast a ray inwards along -Normal from BasePoint
 * We take (1) preferentially, and then (2), and then (3)
 *
 * If all of those fail, if bFailToNearestPoint is true we fall back to nearest-point,
 *
 * If all the above fail, return nullptr
 */
const void* UE::Geometry::GetDetailMeshTrianglePoint_Raycast(
	const IMeshBakerDetailSampler* DetailSampler,
	const FVector3d& BasePoint,
	const FVector3d& BaseNormal,
	int32& DetailTriangleOut,
	FVector3d& DetailTriBaryCoords,
	double Thickness,
	bool bFailToNearestPoint
)
{
	// TODO: should we check normals here? inverse normal should probably not be considered valid

	// shoot rays forwards and backwards
	const FRay3d InwardRay(BasePoint + Thickness * BaseNormal, -BaseNormal);
	const FRay3d ForwardRay(BasePoint, BaseNormal);
	const FRay3d BackwardRay(BasePoint, -BaseNormal);
	int32 ForwardHitTID = IndexConstants::InvalidID, InwardHitTID = IndexConstants::InvalidID, BackwardHitTID = IndexConstants::InvalidID;
	double ForwardHitDist, InwardHitDist, BackwardHitDist = TNumericLimits<float>::Max();

	IMeshSpatial::FQueryOptions Options;
	Options.MaxDistance = Thickness;

	int32 HitTID = IndexConstants::InvalidID;
	FVector3d HitBaryCoords;

	// Inward hit test
	const void* HitMesh = nullptr;
	if (const void* InwardMesh = DetailSampler->FindNearestHitTriangle(InwardRay, InwardHitDist, InwardHitTID, HitBaryCoords, Options))
	{
		HitMesh = InwardMesh;
		HitTID = InwardHitTID;
	}
	// Forward hit test
	else if (const void* ForwardMesh = DetailSampler->FindNearestHitTriangle(ForwardRay, ForwardHitDist, ForwardHitTID, HitBaryCoords, Options))
	{
		HitMesh = ForwardMesh;
		HitTID = ForwardHitTID;
	}
	// Backward hit test
	else if (const void* BackwardMesh = DetailSampler->FindNearestHitTriangle(BackwardRay, BackwardHitDist, BackwardHitTID, HitBaryCoords, Options))
	{
		HitMesh = BackwardMesh;
		HitTID = BackwardHitTID;
	}

	// if we got a valid ray hit, use it
	if (HitMesh && DetailSampler->IsTriangle(HitMesh, HitTID))
	{
		DetailTriangleOut = HitTID;
		DetailTriBaryCoords = HitBaryCoords;
	}
	else
	{
		// if we did not find any hits, try nearest-point
		IMeshSpatial::FQueryOptions OnSurfQueryOptions;
		OnSurfQueryOptions.MaxDistance = Thickness;
		double NearDistSqr = 0;
		int32 NearestTriID = IndexConstants::InvalidID;
		
		// if we are using absolute nearest point as a fallback, then ignore max distance
		HitMesh = bFailToNearestPoint ? 
			DetailSampler->FindNearestTriangle(BasePoint, NearDistSqr, NearestTriID, HitBaryCoords) :
			DetailSampler->FindNearestTriangle(BasePoint, NearDistSqr, NearestTriID, HitBaryCoords, OnSurfQueryOptions);
		if (HitMesh && DetailSampler->IsTriangle(HitMesh, NearestTriID))
		{
			DetailTriangleOut = NearestTriID;
			DetailTriBaryCoords = HitBaryCoords;
		}
	}

	return HitMesh;
}


/**
 * Find point on Detail mesh that corresponds to point on Base mesh using minimum distance
 */
const void* UE::Geometry::GetDetailMeshTrianglePoint_Nearest(
	const IMeshBakerDetailSampler* DetailSampler,
	const FVector3d& BasePoint,
	int32& DetailTriangleOut,
	FVector3d& DetailTriBaryCoords)
{
	double NearDistSqr = TNumericLimits<float>::Max();
	int32 NearestTriID = IndexConstants::InvalidID;
	FVector3d NearestBaryCoords = FVector3d::Zero();
	const void* NearestMesh = DetailSampler->FindNearestTriangle(BasePoint, NearDistSqr, NearestTriID, NearestBaryCoords);
	if (NearestMesh && DetailSampler->IsTriangle(NearestMesh, NearestTriID))
	{
		DetailTriangleOut = NearestTriID;
		DetailTriBaryCoords = NearestBaryCoords;
		return NearestMesh;
	}

	return nullptr;
}


