// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshConstantMapEvaluator.h"
#include "Sampling/MeshMapBaker.h"

using namespace UE::Geometry;


FMeshConstantMapEvaluator::FMeshConstantMapEvaluator(float ValueIn)
	: Value(ValueIn)
{
}

void FMeshConstantMapEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	Context.Evaluate = &EvaluateSample;
	Context.EvaluateDefault = &EvaluateDefault;
	Context.EvaluateColor = &EvaluateColor;
	Context.EvaluateChannel = &EvaluateChannel;
	Context.EvalData = this;
	Context.AccumulateMode = EAccumulateMode::Overwrite;
	Context.DataLayout = DataLayout();
}

const TArray<FMeshMapEvaluator::EComponents>& FMeshConstantMapEvaluator::DataLayout() const
{
	static const TArray<EComponents> Layout{ EComponents::Float1 };
	return Layout;
}

void FMeshConstantMapEvaluator::EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	FMeshConstantMapEvaluator* Eval = static_cast<FMeshConstantMapEvaluator*>(EvalData);
	WriteToBuffer(Out, Eval->Value);
}

void FMeshConstantMapEvaluator::EvaluateDefault(float*& Out, void* EvalData)
{
	FMeshConstantMapEvaluator* Eval = static_cast<FMeshConstantMapEvaluator*>(EvalData);
	WriteToBuffer(Out, Eval->Value);
}

void FMeshConstantMapEvaluator::EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
{
	Out = FVector4f(In[0], In[0], In[0], 1.0f);
	In += 1;
}

void FMeshConstantMapEvaluator::EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData)
{
	Out = *In;
	In += 1;
}

