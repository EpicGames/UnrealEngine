// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshCurvatureMapBaker.h"
#include "Sampling/SphericalFibonacci.h"
#include "Sampling/Gaussians.h"
#include "MeshCurvature.h"
#include "MeshWeights.h"
#include "DynamicMesh/MeshNormals.h"

#include "Math/RandomStream.h"

using namespace UE::Geometry;

void FMeshCurvatureMapBaker::CacheDetailCurvatures(const FDynamicMesh3* DetailMesh)
{
	if (! Curvatures)
	{
		Curvatures = MakeShared<FMeshVertexCurvatureCache>();
		Curvatures->BuildAll(*DetailMesh);
	}
	check(Curvatures->Num() == DetailMesh->MaxVertexID());

}


void FMeshCurvatureMapBaker::Bake()
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	check(BakeCache);
	const FDynamicMesh3* DetailMesh = BakeCache->GetDetailMesh();

	CacheDetailCurvatures(DetailMesh);

	ResultBuilder = MakeUnique<TImageBuilder<FVector3f>>();
	ResultBuilder->SetDimensions(BakeCache->GetDimensions());
	ResultBuilder->Clear(FVector3f::Zero());


	Bake_Single();


	const FImageOccupancyMap& Occupancy = *BakeCache->GetOccupancyMap();

	for (int64 k = 0; k < Occupancy.GutterTexels.Num(); k++)
	{
		TPair<int64, int64> GutterTexel = Occupancy.GutterTexels[k];
		ResultBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
	}

	if (BlurRadius > 0.01)
	{
		TDiscreteKernel2f BlurKernel2d;
		TGaussian2f::MakeKernelFromRadius((float)BlurRadius, BlurKernel2d);
		TArray<FVector3f> AOBlurBuffer;
		Occupancy.ParallelProcessingPass<FVector3f>(
			[&](int64 Index) { return FVector3f::Zero(); },
			[&](int64 LinearIdx, float Weight, FVector3f& CurValue) { CurValue += Weight * ResultBuilder->GetPixel(LinearIdx); },
			[&](int64 LinearIdx, float WeightSum, FVector3f& CurValue) { CurValue /= WeightSum; },
			[&](int64 LinearIdx, FVector3f& CurValue) { ResultBuilder->SetPixel(LinearIdx, CurValue); },
			[&](const FVector2i& TexelOffset) { return BlurKernel2d.EvaluateFromOffset(TexelOffset); },
			BlurKernel2d.IntRadius,
			AOBlurBuffer);
	}

}


void FMeshCurvatureMapBaker::Bake_Single()
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	const FDynamicMesh3* DetailMesh = BakeCache->GetDetailMesh();
	const FMeshVertexCurvatureCache& Cache = *Curvatures;


	double MinPreClamp = -TNumericLimits<double>::Max();
	double MaxPreClamp = TNumericLimits<double>::Max();
	if (UseClampMode == EClampMode::Positive)
	{
		MinPreClamp = 0;
	}
	else if (UseClampMode == EClampMode::Negative)
	{
		MaxPreClamp = 0;
	}



	auto GetCurvature = [&](int32 vid) {
		switch (UseCurvatureType)
		{
		default:
		case ECurvatureType::Mean:
			return FMathd::Clamp(Cache[vid].Mean, MinPreClamp, MaxPreClamp);
		case ECurvatureType::Gaussian:
			return FMathd::Clamp(Cache[vid].Gaussian, MinPreClamp, MaxPreClamp);
		case ECurvatureType::MaxPrincipal:
			return FMathd::Clamp(Cache[vid].MaxPrincipal, MinPreClamp, MaxPreClamp);
		case ECurvatureType::MinPrincipal:
			return FMathd::Clamp(Cache[vid].MinPrincipal, MinPreClamp, MaxPreClamp);
		}
	};

	FSampleSetStatisticsd CurvatureStats = Cache.MeanStats;
	switch (UseCurvatureType)
	{
	default:
	case ECurvatureType::Mean:
		CurvatureStats = Cache.MeanStats;
		break;
	case ECurvatureType::Gaussian:
		CurvatureStats = Cache.GaussianStats;
		break;
	case ECurvatureType::MaxPrincipal:
		CurvatureStats = Cache.MaxPrincipalStats;
		break;
	case ECurvatureType::MinPrincipal:
		CurvatureStats = Cache.MinPrincipalStats;
		break;
	}

	double ClampMax = RangeScale * (CurvatureStats.Mean + CurvatureStats.StandardDeviation);
	if (bOverrideCurvatureRange)
	{
		ClampMax = RangeScale * OverrideRangeMax;
	}
	double MinClamp = MinRangeScale * ClampMax;
	FInterval1d ClampRange(MinClamp, ClampMax);

	// sample curvature of detail surface in tangent-space of base surface
	auto CurvatureSampleFunction = [&](const FMeshImageBakingCache::FCorrespondenceSample& SampleData)
	{
		int32 DetailTriID = SampleData.DetailTriID;
		if (DetailMesh->IsTriangle(DetailTriID))
		{
			FIndex3i DetailTri = DetailMesh->GetTriangle(DetailTriID);
			double CurvatureA = GetCurvature(DetailTri.A);
			double CurvatureB = GetCurvature(DetailTri.B);
			double CurvatureC = GetCurvature(DetailTri.C);
			double InterpCurvature = SampleData.DetailBaryCoords.X * CurvatureA
				+ SampleData.DetailBaryCoords.Y * CurvatureB
				+ SampleData.DetailBaryCoords.Z * CurvatureC;

			return InterpCurvature;
		}
		return 0.0;
	};



	FVector3f NegativeColor, ZeroColor, PositiveColor;
	GetColorMapRange(NegativeColor, ZeroColor, PositiveColor);

	BakeCache->EvaluateSamples([&](const FVector2i& Coords, const FMeshImageBakingCache::FCorrespondenceSample& Sample)
	{
		double Curvature = CurvatureSampleFunction(Sample);

		double Sign = FMathd::Sign(Curvature);
		float T = (float)ClampRange.GetT(FMathd::Abs(Curvature));

		FVector3f CurvatureColor = (Sign < 0) ?
			Lerp(ZeroColor, NegativeColor, T) : Lerp(ZeroColor, PositiveColor, T);

		ResultBuilder->SetPixel(Coords, CurvatureColor);
	});
}


void FMeshCurvatureMapBaker::Bake_Multi()
{

}


void FMeshCurvatureMapBaker::GetColorMapRange(FVector3f& NegativeColor, FVector3f& ZeroColor, FVector3f& PositiveColor)
{
	switch (UseColorMode)
	{
	default:
	case EColorMode::BlackGrayWhite:
		NegativeColor = FVector3f(0, 0, 0);
		ZeroColor = FVector3f(0.5, 0.5, 0.5);
		PositiveColor = FVector3f(1, 1, 1);
		break;
	case EColorMode::RedGreenBlue:
		NegativeColor = FVector3f(1, 0, 0);
		ZeroColor = FVector3f(0, 1, 0);
		PositiveColor = FVector3f(0, 0, 1);
		break;
	case EColorMode::RedBlue:
		NegativeColor = FVector3f(1, 0, 0);
		ZeroColor = FVector3f(0, 0, 0);
		PositiveColor = FVector3f(0, 0, 1);
		break;
	}
}