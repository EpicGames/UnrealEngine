// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshCurvatureMapEvaluator.h"
#include "Sampling/MeshMapBaker.h"
#include "MeshCurvature.h"

using namespace UE::Geometry;

void FMeshCurvatureMapEvaluator::CacheDetailCurvatures()
{
	// TODO: Generalize this evaluator for N meshes.
	if (!Curvatures)
	{
		Curvatures = MakeShared<FMeshVertexCurvatureCache>();
		auto GetCurvature = [this](const void* Mesh)
		{
			DetailSampler->GetCurvature(Mesh, *Curvatures);
		};
		DetailSampler->ProcessMeshes(GetCurvature);
	}
}

void FMeshCurvatureMapEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	Context.Evaluate = &EvaluateSample;
	Context.EvaluateDefault = &EvaluateDefault;
	Context.EvaluateColor = &EvaluateColor;
	Context.EvaluateChannel = &EvaluateChannel;
	Context.EvalData = this;
	Context.AccumulateMode = EAccumulateMode::Add;
	Context.DataLayout = DataLayout();

	// Cache data from the baker
	DetailSampler = Baker.GetDetailSampler();
	CacheDetailCurvatures();

	MinPreClamp = -TNumericLimits<double>::Max();
	MaxPreClamp = TNumericLimits<double>::Max();
	if (UseClampMode == EClampMode::Positive)
	{
		MinPreClamp = 0;
	}
	else if (UseClampMode == EClampMode::Negative)
	{
		MaxPreClamp = 0;
	}

	const FMeshVertexCurvatureCache& Cache = *Curvatures;
	FSampleSetStatisticsd CurvatureStats = Cache.MeanStats;
	switch (UseCurvatureType)
	{
	default:
	case ECurvatureType::Mean:
		CurvatureStats = Cache.MeanStats;
		break;
	case ECurvatureType::Gaussian:
		CurvatureStats = Cache.GaussianStats;
		break;
	case ECurvatureType::MaxPrincipal:
		CurvatureStats = Cache.MaxPrincipalStats;
		break;
	case ECurvatureType::MinPrincipal:
		CurvatureStats = Cache.MinPrincipalStats;
		break;
	}

	double ClampMax = RangeScale * (CurvatureStats.Mean + CurvatureStats.StandardDeviation);
	if (bOverrideCurvatureRange)
	{
		ClampMax = RangeScale * OverrideRangeMax;
	}
	const double MinClamp = MinRangeScale * ClampMax;
	ClampRange = FInterval1d(MinClamp, ClampMax);

	GetColorMapRange(NegativeColor, ZeroColor, PositiveColor);
}

const TArray<FMeshMapEvaluator::EComponents>& FMeshCurvatureMapEvaluator::DataLayout() const
{
	static const TArray<EComponents> Layout{ EComponents::Float1 };
	return Layout;
}

void FMeshCurvatureMapEvaluator::EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	const FMeshCurvatureMapEvaluator* Eval = static_cast<FMeshCurvatureMapEvaluator*>(EvalData);
	const float Curvature = (float) Eval->SampleFunction(Sample);
	WriteToBuffer(Out, Curvature);
}

void FMeshCurvatureMapEvaluator::EvaluateDefault(float*& Out, void* EvalData)
{
	WriteToBuffer(Out, 0.0f);
}

void FMeshCurvatureMapEvaluator::EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
{
	const FMeshCurvatureMapEvaluator* Eval = static_cast<FMeshCurvatureMapEvaluator*>(EvalData);
	const float Curvature = In[0];
	const float Sign = FMathf::Sign(Curvature);
	const float T = (float)Eval->ClampRange.GetT(FMathf::Abs(Curvature));
	FVector3f CurvatureColor = (Sign < 0) ?
		Lerp(Eval->ZeroColor, Eval->NegativeColor, T) : Lerp(Eval->ZeroColor, Eval->PositiveColor, T);
	Out = FVector4f(CurvatureColor[0], CurvatureColor[1], CurvatureColor[2], 1.0f);
	In += 1;
}


void FMeshCurvatureMapEvaluator::EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData)
{
	const FMeshCurvatureMapEvaluator* Eval = static_cast<FMeshCurvatureMapEvaluator*>(EvalData);
	const float Curvature = In[0];
	const float Sign = FMathf::Sign(Curvature);
	const float T = (float)Eval->ClampRange.GetT(FMathf::Abs(Curvature));
	
	constexpr float ZeroValue = 0.5f;
	constexpr float NegativeValue = 0.0f;
	constexpr float PositiveValue = 1.0f;
	Out = (Sign < 0) ? FMath::Lerp(ZeroValue, NegativeValue, T) : FMath::Lerp(ZeroValue, PositiveValue, T);

	In += 1;
}

double FMeshCurvatureMapEvaluator::SampleFunction(const FCorrespondenceSample& SampleData) const
{
	const void* DetailMesh = SampleData.DetailMesh;
	const int32 DetailTriID = SampleData.DetailTriID;
	const FIndex3i DetailTri = DetailSampler->GetTriangle(DetailMesh, DetailTriID);
	const double CurvatureA = GetCurvature(DetailTri.A);
	const double CurvatureB = GetCurvature(DetailTri.B);
	const double CurvatureC = GetCurvature(DetailTri.C);
	const double InterpCurvature = SampleData.DetailBaryCoords.X * CurvatureA
		+ SampleData.DetailBaryCoords.Y * CurvatureB
		+ SampleData.DetailBaryCoords.Z * CurvatureC;
	return InterpCurvature;
}

double FMeshCurvatureMapEvaluator::GetCurvature(const int32 vid) const
{
	const FMeshVertexCurvatureCache& Cache = *Curvatures;
	switch (UseCurvatureType)
	{
	default:
	case ECurvatureType::Mean:
		return FMathd::Clamp(Cache[vid].Mean, MinPreClamp, MaxPreClamp);
	case ECurvatureType::Gaussian:
		return FMathd::Clamp(Cache[vid].Gaussian, MinPreClamp, MaxPreClamp);
	case ECurvatureType::MaxPrincipal:
		return FMathd::Clamp(Cache[vid].MaxPrincipal, MinPreClamp, MaxPreClamp);
	case ECurvatureType::MinPrincipal:
		return FMathd::Clamp(Cache[vid].MinPrincipal, MinPreClamp, MaxPreClamp);
	}
}

void FMeshCurvatureMapEvaluator::GetColorMapRange(FVector3f& NegativeColorOut, FVector3f& ZeroColorOut, FVector3f& PositiveColorOut) const
{
	switch (UseColorMode)
	{
	default:
	case EColorMode::BlackGrayWhite:
		NegativeColorOut = FVector3f(0, 0, 0);
		ZeroColorOut = FVector3f(0.5, 0.5, 0.5);
		PositiveColorOut = FVector3f(1, 1, 1);
		break;
	case EColorMode::RedGreenBlue:
		NegativeColorOut = FVector3f(1, 0, 0);
		ZeroColorOut = FVector3f(0, 1, 0);
		PositiveColorOut = FVector3f(0, 0, 1);
		break;
	case EColorMode::RedBlue:
		NegativeColorOut = FVector3f(1, 0, 0);
		ZeroColorOut = FVector3f(0, 0, 0);
		PositiveColorOut = FVector3f(0, 0, 1);
		break;
	}
}
