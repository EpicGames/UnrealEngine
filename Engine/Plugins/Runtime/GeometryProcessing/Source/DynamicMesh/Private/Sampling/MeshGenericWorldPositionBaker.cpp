// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshGenericWorldPositionBaker.h"

using namespace UE::Geometry;

void FMeshGenericWorldPositionColorBaker::Bake()
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	check(BakeCache);

	ResultBuilder = MakeUnique<TImageBuilder<FVector4f>>();
	ResultBuilder->SetDimensions(BakeCache->GetDimensions());
	ResultBuilder->Clear(DefaultColor);

	BakeCache->EvaluateSamples([this](const FVector2i& Coords, const FMeshImageBakingCache::FCorrespondenceSample& Sample)
	{
		FVector3d Position = Sample.BaseSample.SurfacePoint;
		FVector3d Normal = Sample.BaseNormal;
		FVector4f Color = ColorSampleFunction(Position, Normal);

		ResultBuilder->SetPixel(Coords, Color);
	});

	const FImageOccupancyMap& Occupancy = *BakeCache->GetOccupancyMap();
	for (int64 k = 0; k < Occupancy.GutterTexels.Num(); k++)
	{
		TPair<int64, int64> GutterTexel = Occupancy.GutterTexels[k];
		ResultBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
		//ResultBuilder->SetPixel(GutterTexel.Key, FVector4f(0, 1, 0, 1) );
	}
}






void FMeshGenericWorldPositionNormalBaker::Bake()
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	check(BakeCache);

	// make sure we have per-triangle tangents computed so that GetInterpolatedTriangleTangent() below works
	check(BaseMeshTangents->GetTangents().Num() >= 3 * BakeCache->GetBakeTargetMesh()->MaxTriangleID());

	NormalsBuilder = MakeUnique<TImageBuilder<FVector3f>>();
	NormalsBuilder->SetDimensions(BakeCache->GetDimensions());
	FVector3f DefaultMapNormal = (DefaultNormal + FVector3f::One()) * 0.5f;
	NormalsBuilder->Clear(DefaultMapNormal);

	BakeCache->EvaluateSamples([this](const FVector2i& Coords, const FMeshImageBakingCache::FCorrespondenceSample& Sample)
	{
		// Compute tangent space normal
		FVector3f RelativeDetailNormal;
		{
			FVector3d BasePosition = Sample.BaseSample.SurfacePoint;
			FVector3d BaseNormal = Sample.BaseNormal;
			int32 TriangleID = Sample.BaseSample.TriangleIndex;
			FVector3d BaryCoords = Sample.BaseSample.BaryCoords;

			// get tangents on base mesh
			FVector3d BaseTangentX, BaseTangentY;
			BaseMeshTangents->GetInterpolatedTriangleTangent(TriangleID, BaryCoords, BaseTangentX, BaseTangentY);

			// sample world normal at world position
			FVector3f DetailNormal = NormalSampleFunction(BasePosition, BaseNormal);
			Normalize(DetailNormal);

			// compute normal in tangent space
			float dx = DetailNormal.Dot((FVector3f)BaseTangentX);
			float dy = DetailNormal.Dot((FVector3f)BaseTangentY);
			float dz = DetailNormal.Dot((FVector3f)BaseNormal);
			RelativeDetailNormal = FVector3f(dx, dy, dz);
		}

		// Remap normal components [-1,1] to color components [0,1]
		FVector3f MapNormal = (RelativeDetailNormal + FVector3f::One()) * 0.5f;
		NormalsBuilder->SetPixel(Coords, MapNormal);
	});

	const FImageOccupancyMap& Occupancy = *BakeCache->GetOccupancyMap();
	for (int64 k = 0; k < Occupancy.GutterTexels.Num(); k++)
	{
		TPair<int64, int64> GutterTexel = Occupancy.GutterTexels[k];
		NormalsBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
	}
}