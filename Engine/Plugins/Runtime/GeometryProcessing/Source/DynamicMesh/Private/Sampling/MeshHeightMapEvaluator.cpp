// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshHeightMapEvaluator.h"
#include "Sampling/MeshMapBaker.h"
#include "Util/ColorConstants.h"

using namespace UE::Geometry;

void FMeshHeightMapEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	// Cache data from the baker
	DetailSampler = Baker.GetDetailSampler();

	Context.Evaluate = &EvaluateSample;
	Context.EvaluateDefault = &EvaluateDefault;
	Context.EvaluateColor = &EvaluateColor;
	Context.EvaluateChannel = &EvaluateChannel;
	Context.EvalData = this;
	Context.AccumulateMode = EAccumulateMode::Add;
	Context.DataLayout = DataLayout();

	FAxisAlignedBox3d Bounds = DetailSampler->GetBounds();
	const float MaxDim = static_cast<float>(FMath::Max(Bounds.MaxDim(), FMathf::ZeroTolerance));
	CachedRange = RangeMode == EHeightRangeMode::RelativeBounds ? Range * MaxDim : Range;
}

const TArray<FMeshMapEvaluator::EComponents>& FMeshHeightMapEvaluator::DataLayout() const
{
	static const TArray<EComponents> Layout{ EComponents::Float1 };
	return Layout;
}

void FMeshHeightMapEvaluator::EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	const FMeshHeightMapEvaluator* Eval = static_cast<FMeshHeightMapEvaluator*>(EvalData);
	const float SampleResult = Eval->SampleFunction(Sample);
	WriteToBuffer(Out, SampleResult);
}

void FMeshHeightMapEvaluator::EvaluateDefault(float*& Out, void* EvalData)
{
	WriteToBuffer(Out, 0.0f);
}

void FMeshHeightMapEvaluator::EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
{
	const FMeshHeightMapEvaluator* Eval = static_cast<FMeshHeightMapEvaluator*>(EvalData);
	const float RangeT = Eval->CachedRange.GetT(In[0]);
	Out = FVector4f(RangeT, RangeT, RangeT);
	In += 1;
}

void FMeshHeightMapEvaluator::EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData)
{
	const FMeshHeightMapEvaluator* Eval = static_cast<FMeshHeightMapEvaluator*>(EvalData);
	const float RangeT = Eval->CachedRange.GetT(In[0]);
	Out = RangeT;
	In += 1;
}

float FMeshHeightMapEvaluator::SampleFunction(const FCorrespondenceSample& SampleData) const
{
	const FVector3d& BasePosition = SampleData.BaseSample.SurfacePoint;
	
	const void* DetailMesh = SampleData.DetailMesh;
	const FVector3d& DetailBaryCoords = SampleData.DetailBaryCoords;
	const int32 DetailTriID = SampleData.DetailTriID;

	const FVector3d DetailPosition = DetailSampler->TriBaryInterpolatePoint(DetailMesh, DetailTriID, DetailBaryCoords);

	const FVector3d HeightVector = DetailPosition - BasePosition; 
	return static_cast<float>(SampleData.BaseNormal.Dot(HeightVector));
}


