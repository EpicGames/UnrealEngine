// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshImageBakingCache.h"

using namespace UE::Geometry;

void FMeshImageBakingCache::SetDetailMesh(const FDynamicMesh3* Mesh, const FDynamicMeshAABBTree3* Spatial)
{
	check(Mesh);
	DetailMesh = Mesh;
	check(Spatial);
	DetailSpatial = Spatial;
	InvalidateSamples();
	InvalidateOccupancy();
}

void FMeshImageBakingCache::SetBakeTargetMesh(const FDynamicMesh3* Mesh)
{
	check(Mesh);
	TargetMesh = Mesh;
	InvalidateSamples();
	InvalidateOccupancy();
}


void FMeshImageBakingCache::SetDimensions(FImageDimensions DimensionsIn)
{
	if (Dimensions != DimensionsIn)
	{
		Dimensions = DimensionsIn;
		InvalidateSamples();
		InvalidateOccupancy();
	}
}


void FMeshImageBakingCache::SetGutterSize(int32 GutterSizeIn)
{
	if (GutterSize != GutterSizeIn)
	{
		GutterSize = GutterSizeIn;
		InvalidateOccupancy();
	}
}


void FMeshImageBakingCache::SetUVLayer(int32 UVLayerIn)
{
	if (UVLayer != UVLayerIn)
	{
		UVLayer = UVLayerIn;
		InvalidateSamples();
		InvalidateOccupancy();
	}
}


void FMeshImageBakingCache::SetThickness(double ThicknessIn)
{
	if (Thickness != ThicknessIn)
	{
		Thickness = ThicknessIn;
		InvalidateSamples();
		InvalidateOccupancy();   // do we need to do this?
	}
}


void FMeshImageBakingCache::SetCorrespondenceStrategy(ECorrespondenceStrategy Strategy)
{
	if (CorrespondenceStrategy != Strategy)
	{
		CorrespondenceStrategy = Strategy;
		InvalidateSamples();
		InvalidateOccupancy();   // do we need to do this?
	}
}



const FDynamicMeshNormalOverlay* FMeshImageBakingCache::GetDetailNormals() const
{
	check(DetailMesh && DetailMesh->HasAttributes());
	return DetailMesh->Attributes()->PrimaryNormals();
}


const FDynamicMeshUVOverlay* FMeshImageBakingCache::GetBakeTargetUVs() const
{
	check(TargetMesh && TargetMesh->HasAttributes() && UVLayer < TargetMesh->Attributes()->NumUVLayers());
	return TargetMesh->Attributes()->GetUVLayer(UVLayer);
}

const FDynamicMeshNormalOverlay* FMeshImageBakingCache::GetBakeTargetNormals() const
{
	check(TargetMesh && TargetMesh->HasAttributes());
	return TargetMesh->Attributes()->PrimaryNormals();
}

const FImageOccupancyMap* FMeshImageBakingCache::GetOccupancyMap() const
{
	check(IsCacheValid());
	return OccupancyMap.Get();
}

void FMeshImageBakingCache::InvalidateSamples()
{
	bSamplesValid = false;
}

void FMeshImageBakingCache::InvalidateOccupancy()
{
	bOccupancyValid = false;
}




namespace UE
{
namespace Geometry
{


/**
 * Find point on Detail mesh that corresponds to point on Base mesh.
 * Strategy is:
 *    1) cast a ray inwards along -Normal from BasePoint + Thickness*Normal
 *    2) cast a ray outwards along Normal from BasePoint 
 *    3) cast a ray inwards along -Normal from BasePoint
 * We take (1) preferentially, and then (2), and then (3)
 * 
 * If all of those fail, if bFailToNearestPoint is true we fall back to nearest-point, 
 * 
 * If all the above fail, return false
 */
static bool GetDetailTrianglePoint_Raycast(
	const FDynamicMesh3& DetailMesh,
	const FDynamicMeshAABBTree3& DetailSpatial,
	const FVector3d& BasePoint,
	const FVector3d& BaseNormal,
	int32& DetailTriangleOut,
	FVector3d& DetailTriBaryCoords,
	double Thickness,
	bool bFailToNearestPoint
)
{
	// TODO: should we check normals here? inverse normal should probably not be considered valid

	// shoot rays forwards and backwards
	FRay3d InwardRay = FRay3d(BasePoint + Thickness * BaseNormal, -BaseNormal);
	FRay3d ForwardRay(BasePoint, BaseNormal);
	FRay3d BackwardRay(BasePoint, -BaseNormal);
	int32 ForwardHitTID = IndexConstants::InvalidID, InwardHitTID = IndexConstants::InvalidID, BackwardHitTID = IndexConstants::InvalidID;
	double ForwardHitDist, InwardHitDist, BackwardHitDist;

	IMeshSpatial::FQueryOptions Options;
	Options.MaxDistance = Thickness;
	bool bHitInward = DetailSpatial.FindNearestHitTriangle(InwardRay, InwardHitDist, InwardHitTID, Options);
	bool bHitForward = DetailSpatial.FindNearestHitTriangle(ForwardRay, ForwardHitDist, ForwardHitTID, Options);
	bool bHitBackward = DetailSpatial.FindNearestHitTriangle(BackwardRay, BackwardHitDist, BackwardHitTID, Options);

	FRay3d HitRay;
	int32 HitTID = IndexConstants::InvalidID;
	double HitDist = TNumericLimits<double>::Max();

	if (bHitInward)
	{
		HitRay = InwardRay;
		HitTID = InwardHitTID;
		HitDist = InwardHitDist;
	}
	else if (bHitForward)
	{
		HitRay = ForwardRay;
		HitTID = ForwardHitTID;
		HitDist = ForwardHitDist;
	}
	else if (bHitBackward)
	{
		HitRay = BackwardRay;
		HitTID = BackwardHitTID;
		HitDist = BackwardHitDist;
	}

	// if we did not find any hits, try nearest-point
	if (DetailMesh.IsTriangle(HitTID) == false)
	{
		IMeshSpatial::FQueryOptions OnSurfQueryOptions;
		OnSurfQueryOptions.MaxDistance = Thickness;
		double NearDistSqr = 0;
		int32 NearestTriID = DetailSpatial.FindNearestTriangle(BasePoint, NearDistSqr, OnSurfQueryOptions);
		if (DetailMesh.IsTriangle(NearestTriID))
		{
			DetailTriangleOut = NearestTriID;
			FDistPoint3Triangle3d DistQuery = TMeshQueries<FDynamicMesh3>::TriangleDistance(DetailMesh, NearestTriID, BasePoint);
			DetailTriBaryCoords = DistQuery.TriangleBaryCoords;
			return true;
		}
	}

	// if we got a valid ray hit, use it
	if (DetailMesh.IsTriangle(HitTID))
	{
		DetailTriangleOut = HitTID;
		FIntrRay3Triangle3d IntrQuery = TMeshQueries<FDynamicMesh3>::TriangleIntersection(DetailMesh, HitTID, HitRay);
		DetailTriBaryCoords = IntrQuery.TriangleBaryCoords;
		return true;
	}

	// if we get this far, all rays missed or were too far, so use absolute nearest point regardless of distance
	if (bFailToNearestPoint)
	{
		double NearDistSqr = 0;
		int32 NearestTriID = DetailSpatial.FindNearestTriangle(BasePoint, NearDistSqr);
		if (DetailMesh.IsTriangle(NearestTriID))
		{
			DetailTriangleOut = NearestTriID;
			FDistPoint3Triangle3d DistQuery = TMeshQueries<FDynamicMesh3>::TriangleDistance(DetailMesh, NearestTriID, BasePoint);
			DetailTriBaryCoords = DistQuery.TriangleBaryCoords;
			return true;
		}
	}

	return false;
}





/**
 * Find point on Detail mesh that corresponds to point on Base mesh using minimum distance
 */
static bool GetDetailTrianglePoint_Nearest(
	const FDynamicMesh3& DetailMesh,
	const FDynamicMeshAABBTree3& DetailSpatial,
	const FVector3d& BasePoint,
	int32& DetailTriangleOut,
	FVector3d& DetailTriBaryCoords)
{
	double NearDistSqr = 0;
	int32 NearestTriID = DetailSpatial.FindNearestTriangle(BasePoint, NearDistSqr);
	if (DetailMesh.IsTriangle(NearestTriID))
	{
		DetailTriangleOut = NearestTriID;
		FDistPoint3Triangle3d DistQuery = TMeshQueries<FDynamicMesh3>::TriangleDistance(DetailMesh, NearestTriID, BasePoint);
		DetailTriBaryCoords = DistQuery.TriangleBaryCoords;
		return true;
	}

	return false;
}


}
}


bool FMeshImageBakingCache::ValidateCache()
{
	check(TargetMesh && DetailMesh && DetailSpatial);
	check(Dimensions.GetWidth() > 0 && Dimensions.GetHeight() > 0);

	const FDynamicMesh3* Mesh = TargetMesh;
	const FDynamicMeshUVOverlay* UVOverlay = GetBakeTargetUVs();
	const FDynamicMeshNormalOverlay* NormalOverlay = GetBakeTargetNormals();

	// make UV-space version of mesh
	if (bOccupancyValid == false)
	{
		FDynamicMesh3 FlatMesh(EMeshComponents::FaceGroups);
		for (int32 tid : Mesh->TriangleIndicesItr())
		{
			if (UVOverlay->IsSetTriangle(tid))
			{
				FVector2f A, B, C;
				UVOverlay->GetTriElements(tid, A, B, C);
				int32 VertA = FlatMesh.AppendVertex(FVector3d(A.X, A.Y, 0));
				int32 VertB = FlatMesh.AppendVertex(FVector3d(B.X, B.Y, 0));
				int32 VertC = FlatMesh.AppendVertex(FVector3d(C.X, C.Y, 0));
				int32 NewTriID = FlatMesh.AppendTriangle(VertA, VertB, VertC, tid);
			}
		}

		// calculate occupancy map
		OccupancyMap = MakeUnique<FImageOccupancyMap>();
		OccupancyMap->GutterSize = GutterSize;
		OccupancyMap->Initialize(Dimensions);
		OccupancyMap->ComputeFromUVSpaceMesh(FlatMesh, [&](int32 TriangleID) { return FlatMesh.GetTriangleGroup(TriangleID); });

		bOccupancyValid = true;
	}


	if (bSamplesValid == false)
	{
		ECorrespondenceStrategy UseStrategy = this->CorrespondenceStrategy;
		if (UseStrategy == ECorrespondenceStrategy::Identity && ensure(DetailMesh == Mesh) == false)
		{
			// Identity strategy requires mesh to be the same. Could potentially have two copies, in which
			// case this ensure is too conservative, but for now we will assume this
			UseStrategy = ECorrespondenceStrategy::NearestPoint;
		}

		// this sampler finds the correspondence between base surface and detail surface
		TMeshSurfaceUVSampler<FCorrespondenceSample> DetailMeshSampler;
		DetailMeshSampler.Initialize(Mesh, UVOverlay, EMeshSurfaceSamplerQueryType::TriangleAndUV, FCorrespondenceSample(),
			[Mesh, NormalOverlay, UseStrategy, this](const FMeshUVSampleInfo& SampleInfo, FCorrespondenceSample& ValueOut)
		{
			NormalOverlay->GetTriBaryInterpolate<double>(SampleInfo.TriangleIndex, &SampleInfo.BaryCoords.X, &ValueOut.BaseNormal.X);
			Normalize(ValueOut.BaseNormal);
			FVector3d RayDir = ValueOut.BaseNormal;

			ValueOut.BaseSample = SampleInfo;
			ValueOut.DetailTriID = FDynamicMesh3::InvalidID;

			if (UseStrategy == ECorrespondenceStrategy::Identity)
			{
				ValueOut.DetailTriID = SampleInfo.TriangleIndex;
				ValueOut.DetailBaryCoords = SampleInfo.BaryCoords;
			}
			else if (UseStrategy == ECorrespondenceStrategy::NearestPoint)
			{
				bool bFoundTri = GetDetailTrianglePoint_Nearest(*DetailMesh, *DetailSpatial, SampleInfo.SurfacePoint,
					ValueOut.DetailTriID, ValueOut.DetailBaryCoords);
			}
			else	// fall back to raycast strategy
			{
				double SampleThickness = this->GetThickness();		// could modulate w/ a map here...

				// find detail mesh triangle point
				bool bFoundTri = GetDetailTrianglePoint_Raycast(*DetailMesh, *DetailSpatial, SampleInfo.SurfacePoint, RayDir,
					ValueOut.DetailTriID, ValueOut.DetailBaryCoords, SampleThickness, 
					(UseStrategy == ECorrespondenceStrategy::RaycastStandardThenNearest) );
			}

		});


		SampleMap.Resize(Dimensions.GetWidth(), Dimensions.GetHeight());

		// calculate interior texels
		ParallelFor(Dimensions.GetHeight(), [this, &DetailMeshSampler](int32 ImgY)
		{
			if (CancelF())
			{
				return;
			}

			for (int32 ImgX = 0; ImgX < Dimensions.GetWidth(); ImgX++)
			{
				int64 LinearIdx = Dimensions.GetIndex(ImgX, ImgY);

				if (OccupancyMap->IsInterior(LinearIdx) == false)
				{
					continue;
				}

				FVector2d UVPosition = (FVector2d)OccupancyMap->TexelQueryUV[LinearIdx];
				int32 UVTriangleID = OccupancyMap->TexelQueryTriangle[LinearIdx];

				FCorrespondenceSample Sample;
				DetailMeshSampler.SampleUV(UVTriangleID, UVPosition, Sample);

				SampleMap[LinearIdx] = Sample;
			}
		});

		bSamplesValid = true;
	}

	return IsCacheValid();
}



void FMeshImageBakingCache::EvaluateSamples(
	TFunctionRef<void(const FVector2i&, const FCorrespondenceSample&)> SampleFunction,
	bool bParallel) const
{
	check(IsCacheValid());

	ParallelFor(Dimensions.GetHeight(), [this, &SampleFunction](int32 ImgY)
	{
		if (CancelF())
		{
			return;
		}

		for (int32 ImgX = 0; ImgX < Dimensions.GetWidth(); ImgX++)
		{
			int64 LinearIdx = Dimensions.GetIndex(ImgX, ImgY);
			if (OccupancyMap->IsInterior(LinearIdx) == false)
			{
				continue;
			}
			FVector2i Coords(ImgX, ImgY);

			const FCorrespondenceSample& Sample = SampleMap[LinearIdx];

			SampleFunction(Coords, Sample);
		}

	}, !bParallel ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);
}




void FMeshImageBakingCache::FindSamplingHoles(
	TFunctionRef<bool(const FVector2i&)> IsInvalidSampleFunction,
	TArray<FVector2i>& HolePixelsOut,
	bool bParallel) const
{
	check(IsCacheValid());

	FCriticalSection HoleLock;

	ParallelFor(Dimensions.GetHeight(), [this, &IsInvalidSampleFunction, &HoleLock, &HolePixelsOut](int32 ImgY)
	{
		if (CancelF())
		{
			return;
		}
		for (int32 ImgX = 0; ImgX < Dimensions.GetWidth(); ImgX++)
		{
			int64 LinearIdx = Dimensions.GetIndex(ImgX, ImgY);
			if (OccupancyMap->IsInterior(LinearIdx) == false)
			{
				continue;
			}
			FVector2i Coords(ImgX, ImgY);

			const FCorrespondenceSample& Sample = SampleMap[LinearIdx];
			if (IsInvalidSampleFunction(Coords))
			{
				HoleLock.Lock();
				HolePixelsOut.Add(Coords);
				HoleLock.Unlock();
			}
		}

	}, !bParallel ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);
}