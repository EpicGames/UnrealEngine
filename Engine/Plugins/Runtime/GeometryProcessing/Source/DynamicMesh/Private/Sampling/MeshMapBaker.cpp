// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshMapBaker.h"
#include "Sampling/MeshBakerCommon.h"
#include "Sampling/MeshMapBakerQueue.h"
#include "Image/ImageOccupancyMap.h"
#include "Image/ImageTile.h"
#include "Selections/MeshConnectedComponents.h"
#include "ProfilingDebugging/ScopedTimers.h"

using namespace UE::Geometry;

//
// FMeshMapBaker
//

static constexpr float BoxFilterRadius = 0.5f;
static constexpr float BCFilterRadius = 0.769f;

FBoxFilter FMeshMapBaker::BoxFilter(BoxFilterRadius);
FBSplineFilter FMeshMapBaker::BSplineFilter(BCFilterRadius);
FMitchellNetravaliFilter FMeshMapBaker::MitchellNetravaliFilter(BCFilterRadius);

namespace
{

void ComputeGutterTexelsUsingFilterKernelCoverage(
	FImageOccupancyMap& OccupancyMap,
	FImageTile Tile,
	FImageDimensions Dimensions,
	int32 FilterKernelSize,
	bool(*IsInFilterRegionEval)(const FVector2d& Dist))
{
	const int64 TexelsPerTile = Tile.Num();
	const int64 TexelsPerPaddedTile = OccupancyMap.Tile.Num();
	const int32 SamplesPerTexel = OccupancyMap.PixelSampler.Num();

	TArray64<int8> TexelType;
	TexelType.Init(FImageOccupancyMap::GutterTexel, Tile.Num());

	// First pass: Identify Gutter texels as the texels whose centers are not covered by any filter kernels
	// placed at interior/border sample points of texels in the PaddedTile.
	for (int64 TexelIndexInPaddedTile = 0; TexelIndexInPaddedTile < TexelsPerPaddedTile; ++TexelIndexInPaddedTile)
	{
		const FVector2i TexelCoordsInImage = OccupancyMap.Tile.GetSourceCoords(TexelIndexInPaddedTile);

		// Check filter kernel coverage for each interior sample in the image texels
		for (int32 SampleIndexInTexel = 0; SampleIndexInTexel < SamplesPerTexel; ++SampleIndexInTexel)
		{
			const int64 SampleIndexInPaddedTile = TexelIndexInPaddedTile * SamplesPerTexel + SampleIndexInTexel;
			if (!OccupancyMap.IsInterior(SampleIndexInPaddedTile))
			{
				continue; // Not an interior sample, do nothing
			}

			const FVector2i KernelStartCoordsInImage(
				FMath::Clamp(TexelCoordsInImage.X - FilterKernelSize, 0, Dimensions.GetWidth()),
				FMath::Clamp(TexelCoordsInImage.Y - FilterKernelSize, 0, Dimensions.GetHeight())
				);
			const FVector2i KernelEndCoordsInImage(
				FMath::Clamp(TexelCoordsInImage.X + FilterKernelSize + 1, 0, Dimensions.GetWidth()),
				FMath::Clamp(TexelCoordsInImage.Y + FilterKernelSize + 1, 0, Dimensions.GetHeight())
				);
			const FImageTile KernelTile(KernelStartCoordsInImage, KernelEndCoordsInImage);
			const int64 NbrTexelsPerKernelTile = KernelTile.Num();

			for (int64 NbrTexelIndexInKernel = 0; NbrTexelIndexInKernel < NbrTexelsPerKernelTile; NbrTexelIndexInKernel++)
			{
				const FVector2i NbrTexelCoordsInImage = KernelTile.GetSourceCoords(NbrTexelIndexInKernel);

				// Skip neighbour texels in the padding
				if (!Tile.Contains(NbrTexelCoordsInImage.X, NbrTexelCoordsInImage.Y))
				{
					continue;
				}

				const int64 NbrTexelIndexInPaddedTile = OccupancyMap.Tile.GetIndexFromSourceCoords(NbrTexelCoordsInImage);
				const int32 NbrTexelUVChart = OccupancyMap.TexelQueryUVChart[NbrTexelIndexInPaddedTile];
				// Note: The occupancy map assigned UV chart indices to texels so all samples use this value
				const int32 SampleUVChart = OccupancyMap.TexelQueryUVChart[TexelIndexInPaddedTile];

				// Compute the filter coverage based on the UV distance from the neighbor texel center to the sample position
				// Note: No contribution if the sample and neighbor texel are on different UV charts
				if (SampleUVChart == NbrTexelUVChart)
				{
					// See :GridAlignedFilterKernels
					FVector2d SampleUVInImage;
					{
						const FVector2d TexelSize = Dimensions.GetTexelSize();
						const int64 TexelIndexInImage = Dimensions.GetIndex(TexelCoordsInImage.X, TexelCoordsInImage.Y);
						const FVector2d TexelCenterUV = Dimensions.GetTexelUV(TexelIndexInImage);
						const FVector2d SampleUVInTexel = OccupancyMap.PixelSampler.Sample(SampleIndexInTexel);
						SampleUVInImage = TexelCenterUV - 0.5 * TexelSize + SampleUVInTexel * TexelSize;
					}

					const FVector2d NbrTexelCenterUV = Dimensions.GetTexelUV(NbrTexelCoordsInImage);

					const FVector2d TexelDistance = Dimensions.GetTexelDistance(NbrTexelCenterUV, SampleUVInImage);
					if (IsInFilterRegionEval(TexelDistance))
					{
						const int64 NbrTexelIndexInTile = Tile.GetIndexFromSourceCoords(NbrTexelCoordsInImage);
						TexelType[NbrTexelIndexInTile] = OccupancyMap.IsInterior(NbrTexelIndexInPaddedTile) ? FImageOccupancyMap::InteriorTexel : FImageOccupancyMap::BorderTexel;
					}
				}
			}
		} // end sample loop
	} // end texel loop

	// Second pass: Construct Gutter texel to Interior texel mapping
	for (int64 GutterTexelIndexInTile = 0; GutterTexelIndexInTile < TexelsPerTile; ++GutterTexelIndexInTile)
	{
		const FVector2i GutterTexelCoordsInImage = Tile.GetSourceCoords(GutterTexelIndexInTile);
		const int64 GutterTexelIndexInImage = Dimensions.GetIndex(GutterTexelCoordsInImage);
		const int8 GutterTexelType = TexelType[GutterTexelIndexInTile];
		if (GutterTexelType != FImageOccupancyMap::BorderTexel && GutterTexelType != FImageOccupancyMap::GutterTexel)
		{
			continue; // Not a Border/Gutter texel, do nothing
		}

		for (int32 SampleIndexInTexel = 0; SampleIndexInTexel < SamplesPerTexel; ++SampleIndexInTexel)
		{
			const int64 GutterTexelIndexInPaddedTile = OccupancyMap.Tile.GetIndexFromSourceCoords(GutterTexelCoordsInImage);
			const int64 SampleIndexInPaddedTile = GutterTexelIndexInPaddedTile * SamplesPerTexel + SampleIndexInTexel;
			if (OccupancyMap.TexelType[SampleIndexInPaddedTile] == OccupancyMap.GutterTexel)
			{
				// To avoid artifacts with mipmapping the gutter texels store the nearest interior texel and a
				// post pass copies those nearest interior texel values to each gutter pixel. The mapped texel
				// should be an interior texel since interior texels cover the UV mesh, so when we snap the
				// closest UV point to the nearest texel it should be an interior texel
				//
				// TODO There can be interior texels closer to the gutter the texel than the one we find here. The texel
				// found here is the interior texel that partially covers the UV mesh. Since we set interior texels from
				// kernel coverage there will be a nearer texel. We could fix this by using the TexelQueryUV to define a
				// search direction along which we can find the right texel, or better, use filter kernels to propogate
				// known data into the gutter. Search :NearestInteriorGutterTexel for a related problem in the OccupancyMap
				//
				const FVector2d SampleUV = static_cast<FVector2d>(OccupancyMap.TexelQueryUV[SampleIndexInPaddedTile]);
				const FVector2i NearestTexelCoordsInImage = Dimensions.UVToCoords(SampleUV);
				const int64 MappedTexelIndexInImage = Dimensions.GetIndex(NearestTexelCoordsInImage);
				const TTuple<int64, int64> GutterTexel(GutterTexelIndexInImage, MappedTexelIndexInImage);

				if (TexelType[GutterTexelIndexInTile] == FImageOccupancyMap::BorderTexel)
				{
					OccupancyMap.BorderTexels.Add(GutterTexel);
				}
				else if (TexelType[GutterTexelIndexInTile] == FImageOccupancyMap::GutterTexel)
				{
					OccupancyMap.GutterTexels.Add(GutterTexel);
				}

				break;
			}
		} // end sample loop
	} // end tile pixel loop
}

} // end anonymous namespace

void FMeshMapBaker::InitBake()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMapBaker::InitBake);
	
	// Retrieve evaluation contexts and cache:
	// - index lists of accumulation modes (BakeAccumulateLists)
	// - evaluator to bake result offsets (BakeOffsets)
	// - buffer size per sample (BakeSampleBufferSize)
	const int32 NumBakers = Bakers.Num();
	BakeContexts.SetNum(NumBakers);
	BakeOffsets.SetNumUninitialized(NumBakers + 1);
	BakeAccumulateLists.SetNum(static_cast<int32>(FMeshMapEvaluator::EAccumulateMode::Last));
	BakeSampleBufferSize = 0;
	int32 Offset = 0;
	for (int32 Idx = 0; Idx < NumBakers; ++Idx)
	{
		Bakers[Idx]->Setup(*this, BakeContexts[Idx]);
		checkSlow(BakeContexts[Idx].Evaluate != nullptr && BakeContexts[Idx].EvaluateDefault != nullptr);
		checkSlow(BakeContexts[Idx].DataLayout.Num() > 0);
		const int32 NumData = BakeContexts[Idx].DataLayout.Num();
		for (int32 DataIdx = 0; DataIdx < NumData; ++DataIdx)
		{
			BakeSampleBufferSize += static_cast<int32>(BakeContexts[Idx].DataLayout[DataIdx]);
		}
		BakeOffsets[Idx] = Offset;
		Offset += NumData;
		BakeAccumulateLists[static_cast<int32>(BakeContexts[Idx].AccumulateMode)].Add(Idx);
	}
	BakeOffsets[NumBakers] = Offset;

	// Initialize our BakeResults list and cache offsets into the sample buffer
	// per bake result
	const int32 NumResults = Offset;
	BakeResults.SetNum(NumResults);
	BakeSampleOffsets.SetNumUninitialized(NumResults + 1);
	int32 SampleOffset = 0;
	for (int32 Idx = 0; Idx < NumBakers; ++Idx)
	{
		const int32 NumData = BakeContexts[Idx].DataLayout.Num();
		for (int32 DataIdx = 0; DataIdx < NumData; ++DataIdx)
		{
			const int32 ResultIdx = BakeOffsets[Idx] + DataIdx;
			BakeResults[ResultIdx] = MakeUnique<TImageBuilder<FVector4f>>();
			BakeResults[ResultIdx]->SetDimensions(Dimensions);

			BakeSampleOffsets[ResultIdx] = SampleOffset;
			const int32 NumFloats = static_cast<int32>(BakeContexts[Idx].DataLayout[DataIdx]);
			SampleOffset += NumFloats;
		}
	}
	BakeSampleOffsets[NumResults] = SampleOffset;

	InitBakeDefaults();

	for (int32 Idx = 0; Idx < NumResults; ++Idx)
	{
		BakeResults[Idx]->Clear(BakeDefaultColors[Idx]);
	}

	InitFilter();

	// Compute UV charts if null or invalid.
	if (!TargetMeshUVCharts || !ensure(TargetMeshUVCharts->Num() == TargetMesh->TriangleCount()))
	{
		ComputeUVCharts(*TargetMesh, TargetMeshUVChartsLocal);
		TargetMeshUVCharts = &TargetMeshUVChartsLocal;
	}
}

void FMeshMapBaker::InitBakeDefaults()
{
	// Cache default float buffer and colors for each bake result.
	checkSlow(BakeSampleBufferSize > 0);
	BakeDefaults.SetNumUninitialized(BakeSampleBufferSize);
	float* Buffer = BakeDefaults.GetData();
	float* BufferPtr = Buffer;

	const int32 NumBakers = Bakers.Num();
	for (int32 Idx = 0; Idx < NumBakers; ++Idx)
	{
		BakeContexts[Idx].EvaluateDefault(BufferPtr, BakeContexts[Idx].EvalData);
	}
	checkSlow((BufferPtr - Buffer) == BakeSampleBufferSize);

	BufferPtr = Buffer;
	const int32 NumBakeResults = BakeResults.Num();
	BakeDefaultColors.SetNumUninitialized(NumBakeResults);
	for (int32 Idx = 0; Idx < NumBakers; ++Idx)
	{
		const FMeshMapEvaluator::FEvaluationContext& Context = BakeContexts[Idx];
		const int32 NumData = Context.DataLayout.Num();
		for (int32 DataIdx = 0; DataIdx < NumData; ++DataIdx)
		{
			const int32 ResultIdx = BakeOffsets[Idx] + DataIdx;
			Context.EvaluateColor(DataIdx, BufferPtr, BakeDefaultColors[ResultIdx], Context.EvalData);
		}
	}
	checkSlow((BufferPtr - Buffer) == BakeSampleBufferSize);
}

void FMeshMapBaker::Bake()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMapBaker::Bake);

	BakeAnalytics.Reset();
	FScopedDurationTimer TotalBakeTimer(BakeAnalytics.TotalBakeDuration);
	
	if (Bakers.IsEmpty() || !TargetMesh)
	{
		return;
	}

	InitBake();

	const FDynamicMesh3* Mesh = TargetMesh;
	const FDynamicMeshUVOverlay* UVOverlay = GetTargetMeshUVs();
	const FDynamicMeshNormalOverlay* NormalOverlay = GetTargetMeshNormals();

	{
		// Generate UV space mesh
		TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMapBaker::Bake_CreateUVMesh);
		
		FlatMesh = FDynamicMesh3(EMeshComponents::FaceGroups);
		for (const int32 TriId : Mesh->TriangleIndicesItr())
		{
			if (UVOverlay->IsSetTriangle(TriId))
			{
				FVector2f A, B, C;
				UVOverlay->GetTriElements(TriId, A, B, C);
				const int32 VertA = FlatMesh.AppendVertex(FVector3d(A.X, A.Y, 0));
				const int32 VertB = FlatMesh.AppendVertex(FVector3d(B.X, B.Y, 0));
				const int32 VertC = FlatMesh.AppendVertex(FVector3d(C.X, C.Y, 0));
				/*int32 NewTriID =*/ FlatMesh.AppendTriangle(VertA, VertB, VertC, TriId);
			}
		}
	}

	// Generate UV space mesh Spatial
	TMeshAABBTree3 FlatSpatialCache(&FlatMesh, true);

	ECorrespondenceStrategy UseStrategy = this->CorrespondenceStrategy;
	bool bIsIdentity = true;
	int NumDetailMeshes = 0;
	auto CheckIdentity = [this, Mesh, &bIsIdentity, &NumDetailMeshes](const void* DetailMesh)
	{
		// When the mesh pointers differ, loosely compare the meshes as a sanity check.
		// TODO: Expose additional comparison metrics on the detail sampler when the mesh pointers differ.
		bIsIdentity = bIsIdentity && (DetailMesh == Mesh || Mesh->TriangleCount() == DetailSampler->GetTriangleCount(DetailMesh));
		++NumDetailMeshes;
	};
	DetailSampler->ProcessMeshes(CheckIdentity);
	if (UseStrategy == ECorrespondenceStrategy::Identity && !ensure(bIsIdentity && NumDetailMeshes == 1))
	{
		// Identity strategy requires there to be only one mesh that is the same
		// as the target mesh. 
		UseStrategy = ECorrespondenceStrategy::NearestPoint;
	}

	// Computes the correspondence sample assuming the SampleInfo is valid
	// Returns true if the correspondence is valid and false otherwise
	auto ComputeCorrespondenceSample
		= [Mesh, NormalOverlay, UseStrategy, this](const FMeshUVSampleInfo& SampleInfo, FMeshMapEvaluator::FCorrespondenceSample& ValueOut)
	{
		NormalOverlay->GetTriBaryInterpolate<double>(SampleInfo.TriangleIndex, &SampleInfo.BaryCoords.X, &ValueOut.BaseNormal.X);
		Normalize(ValueOut.BaseNormal);

		ValueOut.BaseSample = SampleInfo;
		ValueOut.DetailMesh = nullptr;
		ValueOut.DetailTriID = FDynamicMesh3::InvalidID;

		if (UseStrategy == ECorrespondenceStrategy::Identity && DetailSampler->SupportsIdentityCorrespondence())
		{
			ValueOut.DetailMesh = Mesh;
			ValueOut.DetailTriID = SampleInfo.TriangleIndex;
			ValueOut.DetailBaryCoords = SampleInfo.BaryCoords;
		}
		else if (UseStrategy == ECorrespondenceStrategy::NearestPoint && DetailSampler->SupportsNearestPointCorrespondence())
		{
			ValueOut.DetailMesh = GetDetailMeshTrianglePoint_Nearest(DetailSampler, SampleInfo.SurfacePoint,
				ValueOut.DetailTriID, ValueOut.DetailBaryCoords);
		}
		else if (UseStrategy == ECorrespondenceStrategy::Custom && DetailSampler->SupportsCustomCorrespondence())
		{
			ValueOut.DetailMesh = DetailSampler->ComputeCustomCorrespondence(SampleInfo, ValueOut);
		}
		else	// Fall back to raycast strategy
		{
			checkSlow(DetailSampler->SupportsRaycastCorrespondence());
			
			const double SampleThickness = this->GetProjectionDistance();		// could modulate w/ a map here...

			// Find detail mesh triangle point
			const FVector3d RayDir = ValueOut.BaseNormal;
			ValueOut.DetailMesh = GetDetailMeshTrianglePoint_Raycast(DetailSampler, SampleInfo.SurfacePoint, RayDir,
				ValueOut.DetailTriID, ValueOut.DetailBaryCoords, SampleThickness,
				(UseStrategy == ECorrespondenceStrategy::RaycastStandardThenNearest));
		}

		return DetailSampler->IsValidCorrespondence(ValueOut);
	};

	// This computes a FMeshUVSampleInfo to pass to the ComputeCorrespondenceSample function, which will find the
	// correspondence between the target surface and detail surface.
	MeshUVSampler.Initialize(Mesh, UVOverlay, EMeshSurfaceSamplerQueryType::TriangleAndUV);

	// Create a temporary output float buffer for the full image dimensions.
	const FImageTile FullImageTile(FVector2i(0,0), FVector2i(Dimensions.GetWidth(), Dimensions.GetHeight()));
	FMeshMapTileBuffer FullImageTileBuffer(FullImageTile, BakeSampleBufferSize);

	// Tile the image
	FImageTiling Tiles(Dimensions, TileSize, TileSize);
	const int32 NumTiles = Tiles.Num();
	TArray<TArray64<TTuple<int64, int64>>> BorderTexelsPerTile;
	TArray<TArray64<TTuple<int64, int64>>> GutterTexelsPerTile;
	BorderTexelsPerTile.SetNum(NumTiles);
	GutterTexelsPerTile.SetNum(NumTiles);

	// WriteToOutputBuffer transfers local tile data (TileBuffer) to the image output buffer (FullImageTileBuffer).
	auto WriteToOutputBuffer = [this, &FullImageTileBuffer] (FMeshMapTileBuffer& TileBufferIn, const FImageTile& TargetTile, const TArray<int32>& EvaluatorIds, auto&& Op, auto&& WeightOp)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMapBaker::Bake_WriteToOutputBuffer);
		
		const int TargetTileWidth = TargetTile.GetWidth();
		const int TargetTileHeight = TargetTile.GetHeight();
		for (FVector2i TileCoords(0,0); TileCoords.Y < TargetTileHeight; ++TileCoords.Y)
		{
			for (TileCoords.X = 0; TileCoords.X < TargetTileWidth; ++TileCoords.X)
			{
				if (CancelF())
				{
					return; // WriteToOutputBuffer
				}

				const FVector2i ImageCoords = TargetTile.GetSourceCoords(TileCoords);

				const int64 ImageLinearIdx = Dimensions.GetIndex(ImageCoords);
				float& ImagePixelWeight = FullImageTileBuffer.GetPixelWeight(ImageLinearIdx);
				float* ImagePixelBuffer = FullImageTileBuffer.GetPixel(ImageLinearIdx);

				const FImageTile& BufferTile = TileBufferIn.GetTile();

				const int64 TilePixelLinearIdx = BufferTile.GetIndexFromSourceCoords(ImageCoords);
				const float& TilePixelWeight = TileBufferIn.GetPixelWeight(TilePixelLinearIdx);
				float* TilePixelBuffer = TileBufferIn.GetPixel(TilePixelLinearIdx);

				WeightOp(TilePixelWeight, ImagePixelWeight);
				for( int32 Idx : EvaluatorIds )
				{
					const FMeshMapEvaluator::FEvaluationContext& Context = BakeContexts[Idx];
					const int32 NumData = Context.DataLayout.Num();
					const int32 ResultOffset = BakeOffsets[Idx];
					for (int32 DataIdx = 0; DataIdx < NumData; ++DataIdx)
					{
						const int32 ResultIdx = ResultOffset + DataIdx;
						const int32 Offset = BakeSampleOffsets[ResultIdx];
						float* BufferPtr = &TilePixelBuffer[Offset];
						float* ImageBufferPtr = &ImagePixelBuffer[Offset];

						const int32 NumFloats = static_cast<int32>(Context.DataLayout[DataIdx]);
						for (int32 FloatIdx = 0; FloatIdx < NumFloats; ++FloatIdx)
						{
							Op(BufferPtr[FloatIdx], ImageBufferPtr[FloatIdx]);
						}
					}
				}
			}
		}
	};

	auto WriteToOutputBufferQueued = [this, &WriteToOutputBuffer](FMeshMapBakerQueue& Queue)
	{
		constexpr auto AddFn = [](const float& In, float& Out)
		{
			Out += In;
		};

		if (Queue.AcquireProcessLock())
		{
			void* OutputData = Queue.Process();
			while (OutputData)
			{
				FMeshMapTileBuffer* TileBufferPtr = static_cast<FMeshMapTileBuffer*>(OutputData);
				WriteToOutputBuffer(*TileBufferPtr, TileBufferPtr->GetTile(), EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Add), AddFn, AddFn);
				delete TileBufferPtr;
				OutputData = Queue.Process();
			}
			Queue.ReleaseProcessLock();
		}
	};

	FMeshMapBakerQueue OutputQueue(NumTiles);
	ParallelFor(NumTiles, [this, &Tiles, &BorderTexelsPerTile, &GutterTexelsPerTile, &OutputQueue, &WriteToOutputBuffer, &WriteToOutputBufferQueued, &ComputeCorrespondenceSample, &FlatSpatialCache](int32 TileIdx)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMapBaker::Bake_EvalTile);

		if (CancelF())
		{
			return; // ParallelFor
		}
		
		// Generate unpadded and padded tiles.
		const FImageTile Tile = Tiles.GetTile(TileIdx);	// Image area to sample
		const FImageTile PaddedTile = Tiles.GetTile(TileIdx, TilePadding); // Filtered image area

		FImageOccupancyMap OccupancyMap;
		OccupancyMap.GutterSize = GutterSize;
		OccupancyMap.Initialize(Dimensions, PaddedTile, SamplesPerPixel);
		const auto GetTriangleIDFunc = [this](int32 TriangleID) { return FlatMesh.GetTriangleGroup(TriangleID); };
		OccupancyMap.ClassifySamplesFromUVSpaceMesh(FlatMesh, FlatSpatialCache, GetTriangleIDFunc, TargetMeshUVCharts);
		ComputeGutterTexelsUsingFilterKernelCoverage(OccupancyMap, Tile, Dimensions, FilterKernelSize, IsInFilterRegionEval);
		BorderTexelsPerTile[TileIdx] = OccupancyMap.BorderTexels;
		GutterTexelsPerTile[TileIdx] = OccupancyMap.GutterTexels;

		const int64 NumTilePixels = Tile.Num();
		for (int64 TilePixelIdx = 0; TilePixelIdx < NumTilePixels; ++TilePixelIdx)
		{
			const FVector2i SourceCoords = Tile.GetSourceCoords(TilePixelIdx);
			const int64 OccupancyMapIdx = OccupancyMap.Tile.GetIndexFromSourceCoords(SourceCoords);
			BakeAnalytics.NumSamplePixels += OccupancyMap.TexelInteriorSamples[OccupancyMapIdx];; 
		}

		FMeshMapTileBuffer* TileBuffer = new FMeshMapTileBuffer(PaddedTile, BakeSampleBufferSize);

		{
			// Evaluate valid/interior samples
			TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMapBaker::Bake_EvalTileSamples);
			
			const int TileWidth = Tile.GetWidth();
			const int TileHeight = Tile.GetHeight();
			const int32 NumSamples = OccupancyMap.PixelSampler.Num();
			for (FVector2i TileCoords(0,0); TileCoords.Y < TileHeight; ++TileCoords.Y)
			{
				for (TileCoords.X = 0; TileCoords.X < TileWidth; ++TileCoords.X)
				{
					if (CancelF())
					{
						delete TileBuffer;
						return; // ParallelFor
					}

					const FVector2i ImageCoords = Tile.GetSourceCoords(TileCoords);
					const int64 OccupancyMapLinearIdx = OccupancyMap.Tile.GetIndexFromSourceCoords(ImageCoords);
					if (OccupancyMap.TexelNumSamples(OccupancyMapLinearIdx) == 0)
					{
						continue;
					}

					// Iterate over all the samples in the pixel
					for (int32 SampleIdx = 0; SampleIdx < NumSamples; ++SampleIdx)
					{
						const int64 LinearIdx = OccupancyMapLinearIdx * NumSamples + SampleIdx;
						if (OccupancyMap.IsInterior(LinearIdx))
						{
							// This block calls BakeSample on all interior/border samples of the OccupancyMap. In the code
							// below QueryUVInImage is the UV coordinate in the multisampled texel image where the
							// evaluators are evaluated. This coordinate is computed by the OccupancyMap and is the same
							// as the sample's UV coordinate in the sample grid implied by the multisampled image
							// (SampleUVInImage) if SampleUVInImage lies in the UV unwrap mesh. If SampleUVInImage is
							// not in the UV mesh this sample is a border sample and the samples texel will not fully
							// overlap with UV unwrap mesh. The evaluators must be evaluated at a UV position in the
							// unwrap mesh so we must use QueryUVInImage position. The filter weights which apply the
							// evaluated result to surrounding texels, however, could be could be placed at either of
							// these positions. Until UE 5.2 we used QueryUVInImage but this contributed to issue UE-169350.
							// QueryUVInImage points can be arbitrarily located relative to the texel centers which
							// meant we ended up with kernels that narrowly overlapped with texel centers resulting
							// in tiny filter weights and cases where the overall filter weight for the texel was zero
							// (persumably because the Mitchell-Netravali filter function becomes negative near the
							// edges so contributions from different samples can sum to zero). This resulted in speckles
							// (aka black texels surrounded by neighbours of a very different value) in the final image.
							// To fix this we now place filter kernels at SampleUVInImage which eliminates the
							// small/arbitrary overlap between the filter kernel and texel centers (in fact the kernel
							// overlaps are limited to a small number of possiblities determined by the sample positions
							// in the texel and the kernel radius ie independent of the position of the uv unwrap mesh).
							// This is also relevant for the code labeled :GridAlignedFilterKernels

							FVector2d SampleUVInImage;
							{
								const FVector2d TexelSize = Dimensions.GetTexelSize();
								const int64 TexelIndexInImage = Dimensions.GetIndex(ImageCoords.X, ImageCoords.Y);
								const FVector2d TexelCenterUV = Dimensions.GetTexelUV(TexelIndexInImage);
								const FVector2d SampleUVInTexel = OccupancyMap.PixelSampler.Sample(SampleIdx);
								SampleUVInImage = TexelCenterUV - 0.5 * TexelSize + SampleUVInTexel * TexelSize;
							}

							const FVector2d QueryUVInImage = (FVector2d)OccupancyMap.TexelQueryUV[LinearIdx];

							// Compute the per-sample correspondence data 
							// Note: Since we check LinearIdx is an interior sample above we know we'll get a valid
							// SampleInfo because interior samples all have valid UVTriangleIDs.
							FMeshUVSampleInfo SampleInfo;
							const int32 UVTriangleID = OccupancyMap.TexelQueryTriangle[LinearIdx];
							if (MeshUVSampler.QuerySampleInfo(UVTriangleID, QueryUVInImage, SampleInfo))
							{
								FMeshMapEvaluator::FCorrespondenceSample Sample;
								bool bSampleValid = ComputeCorrespondenceSample(SampleInfo, Sample);
								if (bSampleValid)
								{
									BakeSample(*TileBuffer, Sample, SampleUVInImage, ImageCoords, OccupancyMap);
								}
								InteriorSampleCallback(bSampleValid, Sample, SampleUVInImage, ImageCoords);
							}
						}
					}
				}
			}
		}

		constexpr auto NoopFn = [](const float& In, float& Out)
		{
		};

		constexpr auto OverwriteFn = [](const float& In, float& Out)
		{
			Out = In;
		};

		// Transfer 'Overwrite' float data to image tile buffer
		WriteToOutputBuffer(*TileBuffer, Tile, EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Overwrite), OverwriteFn, NoopFn);

		// Accumulate 'Add' float data to image tile buffer
		OutputQueue.Post(TileIdx, TileBuffer);
		WriteToOutputBufferQueued(OutputQueue);
	}, !bParallel ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);

	if (CancelF())
	{
		// If cancelled, delete any outstanding tile buffers in the queue.
		while (!OutputQueue.IsDone())
		{
			void* Data = OutputQueue.Process</*bFlush*/ true>();
			if (Data)
			{
				const FMeshMapTileBuffer* TileBuffer = static_cast<FMeshMapTileBuffer*>(Data);
				delete TileBuffer;
			}
		}

		return;
	}

	{
		// The queue only acquires the process lock if the next item in the queue
		// is ready. This could mean that there are potential leftovers in the queue
		// after the parallel for. Write them out now.
		WriteToOutputBufferQueued(OutputQueue);
	}
	
	if (CancelF())
	{
		return;
	}

	{
		FScopedDurationTimer WriteToImageTimer(BakeAnalytics.WriteToImageDuration);
		
		// Normalize and convert ImageTileBuffer data to color data.
		ParallelFor(NumTiles, [this, &Tiles, &FullImageTileBuffer](int32 TileIdx)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMapBaker::Bake_WriteToImageBuffer);
		
			const FImageTile Tile = Tiles.GetTile(TileIdx);
			const int TileWidth = Tile.GetWidth();
			const int TileHeight = Tile.GetHeight();
			for (FVector2i TileCoords(0,0); TileCoords.Y < TileHeight; ++TileCoords.Y)
			{
				for (TileCoords.X = 0; TileCoords.X < TileWidth; ++TileCoords.X)
				{
					if (CancelF())
					{
						return; // ParallelFor
					}

					const FVector2i ImageCoords = Tile.GetSourceCoords(TileCoords);

					const int64 ImageLinearIdx = Dimensions.GetIndex(ImageCoords);
					const float& PixelWeight = FullImageTileBuffer.GetPixelWeight(ImageLinearIdx);
					float* PixelBuffer = FullImageTileBuffer.GetPixel(ImageLinearIdx);

					auto WriteToPixel = [this, &PixelBuffer, &ImageLinearIdx](const TArray<int32>& EvaluatorIds, float OneOverWeight)
					{
						for (const int32 Idx : EvaluatorIds)
						{
							const FMeshMapEvaluator::FEvaluationContext& Context = BakeContexts[Idx];
							const int32 NumData = Context.DataLayout.Num();
							const int32 ResultOffset = BakeOffsets[Idx];
							for (int32 DataIdx = 0; DataIdx < NumData; ++DataIdx)
							{
								const int32 ResultIdx = ResultOffset + DataIdx;
								const int32 Offset = BakeSampleOffsets[ResultIdx];
								float* BufferPtr = &PixelBuffer[Offset];

								// Apply weight to raw float data.
								const int32 NumFloats = static_cast<int32>(Context.DataLayout[DataIdx]);
								for (int32 FloatIdx = 0; FloatIdx < NumFloats; ++FloatIdx)
								{
									BufferPtr[FloatIdx] *= OneOverWeight;
								}

								// Convert float data to color.
								FVector4f& Pixel = BakeResults[ResultIdx]->GetPixel(ImageLinearIdx);
								Context.EvaluateColor(DataIdx, BufferPtr, Pixel, Context.EvalData);
							}
						}
					};
				
					if (PixelWeight > 0.0)
					{
						WriteToPixel(EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Add), 1.0f / PixelWeight);
					}
					WriteToPixel(EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Overwrite), 1.0f);
				}
			}
		}, !bParallel ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);
	}

	if (CancelF())
	{
		return;
	}

	PostWriteToImageCallback(BakeResults);

	if (CancelF())
	{
		return;
	}

	// Gutter Texel processing
	if (bGutterEnabled)
	{
		FScopedDurationTimer WriteToGutterTimer(BakeAnalytics.WriteToGutterDuration);
		
		const int32 NumResults = BakeResults.Num();
		ParallelFor(NumTiles, [this, &NumResults, &BorderTexelsPerTile, &GutterTexelsPerTile](int32 TileIdx)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMapBaker::Bake_WriteGutterPixels);

			if (CancelF())
			{
				return; // ParallelFor
			}

			const int64 NumGutter = GutterTexelsPerTile[TileIdx].Num();
			for (int64 GutterIdx = 0; GutterIdx < NumGutter; ++GutterIdx)
			{
				int64 GutterPixelTo;
				int64 GutterPixelFrom;
				Tie(GutterPixelTo, GutterPixelFrom) = GutterTexelsPerTile[TileIdx][GutterIdx];
				for (int32 Idx = 0; Idx < NumResults; Idx++)
				{
					BakeResults[Idx]->CopyPixel(GutterPixelFrom, GutterPixelTo);
				}
			}

			// For EAccumulateMode::Overwrite evaluators, Border pixels are gutter pixels. 
			const int64 NumBorder = BorderTexelsPerTile[TileIdx].Num();
			const TArray<int32>& EvaluatorOverwriteIds = EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Overwrite);
			for (const int32 Idx : EvaluatorOverwriteIds)
			{
				const FMeshMapEvaluator::FEvaluationContext& Context = BakeContexts[Idx];
				const int32 NumData = Context.DataLayout.Num();
				const int32 ResultOffset = BakeOffsets[Idx];
				for (int32 DataIdx = 0; DataIdx < NumData; ++DataIdx)
				{
					const int32 ResultIdx = ResultOffset + DataIdx;
					for (int64 BorderIdx = 0; BorderIdx < NumBorder; ++BorderIdx)
					{
						int64 BorderPixelTo;
						int64 BorderPixelFrom;
						Tie(BorderPixelTo, BorderPixelFrom) = BorderTexelsPerTile[TileIdx][BorderIdx];
						BakeResults[ResultIdx]->CopyPixel(BorderPixelFrom, BorderPixelTo);
					}
				}
			}

			BakeAnalytics.NumGutterPixels += NumGutter;
		}, !bParallel ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);
	}
}

// Precondition: Must be passed a valid Sample
void FMeshMapBaker::BakeSample(
	FMeshMapTileBuffer& TileBuffer,
	const FMeshMapEvaluator::FCorrespondenceSample& Sample,
	const FVector2d& SampleFilterUVPosition,
	const FVector2i& ImageCoords,
	const FImageOccupancyMap& OccupancyMap)
{
	// Evaluate each baker into stack allocated float buffer
	float* Buffer = static_cast<float*>(FMemory_Alloca(sizeof(float) * BakeSampleBufferSize));
	float* BufferPtr = Buffer;
	const int32 NumEvaluators = Bakers.Num();
	for (int32 Idx = 0; Idx < NumEvaluators; ++Idx)
	{
		BakeContexts[Idx].Evaluate(BufferPtr, Sample, BakeContexts[Idx].EvalData);
	}
	checkSlow((BufferPtr - Buffer) == BakeSampleBufferSize);

	const FImageTile& Tile = TileBuffer.GetTile();

	const int64 OccupancyMapSampleIdx = OccupancyMap.Tile.GetIndexFromSourceCoords(ImageCoords);
	const int32 SampleUVChart = OccupancyMap.TexelQueryUVChart[OccupancyMapSampleIdx];

	auto AddFn = [this, &ImageCoords, &SampleFilterUVPosition, &Tile, &TileBuffer, &OccupancyMap, SampleUVChart]
		(const TArray<int32>& EvaluatorIds, const float* SourceBuffer, float Weight) -> void
	{
		const FVector2i BoxFilterStart(
			FMath::Clamp(ImageCoords.X - FilterKernelSize, 0, Dimensions.GetWidth()),
			FMath::Clamp(ImageCoords.Y - FilterKernelSize, 0, Dimensions.GetHeight())
			);
		const FVector2i BoxFilterEnd(
			FMath::Clamp(ImageCoords.X + FilterKernelSize + 1, 0, Dimensions.GetWidth()),
			FMath::Clamp(ImageCoords.Y + FilterKernelSize + 1, 0, Dimensions.GetHeight())
			);
		const FImageTile BoxFilterTile(BoxFilterStart, BoxFilterEnd);

		for (int64 FilterIdx = 0; FilterIdx < BoxFilterTile.Num(); FilterIdx++)
		{
			const FVector2i SourceCoords = BoxFilterTile.GetSourceCoords(FilterIdx);
			const int64 OccupancyMapFilterIdx = OccupancyMap.Tile.GetIndexFromSourceCoords(SourceCoords);
			const int32 BufferTilePixelUVChart = OccupancyMap.TexelQueryUVChart[OccupancyMapFilterIdx];

			// Get the weight and value buffers for this pixel
			const int64 BufferTilePixelLinearIdx = Tile.GetIndexFromSourceCoords(SourceCoords);
			float* PixelBuffer = TileBuffer.GetPixel(BufferTilePixelLinearIdx);
			float& PixelWeight = TileBuffer.GetPixelWeight(BufferTilePixelLinearIdx);

			// Compute the filter weight based on the UV distance from the pixel center to the sample position
			// Note: There will be no contribution if the sample and pixel are on different UV charts
			float FilterWeight = Weight * static_cast<float>(SampleUVChart == BufferTilePixelUVChart);
			{
				const FVector2d PixelCenterUVPosition = Dimensions.GetTexelUV(SourceCoords);
				const FVector2d TexelDistance = Dimensions.GetTexelDistance(PixelCenterUVPosition, SampleFilterUVPosition);
				FilterWeight *= TextureFilterEval(TexelDistance);
			}

			// Update the weight of this pixel
			PixelWeight += FilterWeight;

			// Update the value of this pixel for each evaluator
			for (const int32 Idx : EvaluatorIds)
			{
				const FMeshMapEvaluator::FEvaluationContext& Context = BakeContexts[Idx];
				const int32 NumData = Context.DataLayout.Num();
				const int32 ResultOffset = BakeOffsets[Idx];
				for (int32 DataIdx = 0; DataIdx < NumData; ++DataIdx)
				{
					const int32 ResultIdx = ResultOffset + DataIdx;
					const int32 Offset = BakeSampleOffsets[ResultIdx];

					const int32 NumFloats = static_cast<int32>(Context.DataLayout[DataIdx]);
					for (int32 FloatIdx = Offset; FloatIdx < Offset + NumFloats; ++FloatIdx)
					{
						PixelBuffer[FloatIdx] += SourceBuffer[FloatIdx] * FilterWeight;
					}
				}
			}
		}
	};

	auto OverwriteFn = [this, &ImageCoords, &Tile, &TileBuffer]
		(const TArray<int32>& EvaluatorIds, const float* SourceBuffer) -> void
	{
		const int64 BufferTilePixelLinearIdx = Tile.GetIndexFromSourceCoords(ImageCoords); 
		float* PixelBuffer = TileBuffer.GetPixel(BufferTilePixelLinearIdx);
		
		for (const int32 Idx : EvaluatorIds)
		{
			const FMeshMapEvaluator::FEvaluationContext& Context = BakeContexts[Idx];
			const int32 NumData = Context.DataLayout.Num();
			const int32 ResultOffset = BakeOffsets[Idx];
			for (int32 DataIdx = 0; DataIdx < NumData; ++DataIdx)
			{
				const int32 ResultIdx = ResultOffset + DataIdx;
				const int32 Offset = BakeSampleOffsets[ResultIdx];
				
				const int32 NumFloats = static_cast<int32>(Context.DataLayout[DataIdx]);
				for (int32 FloatIdx = Offset; FloatIdx < Offset + NumFloats; ++FloatIdx)
				{
					PixelBuffer[FloatIdx] = SourceBuffer[FloatIdx];
				}
			}
		}
	};

	if (SampleFilterF)
	{
		const float SampleMaskWeight = FMath::Clamp(SampleFilterF(ImageCoords, SampleFilterUVPosition, Sample.BaseSample.TriangleIndex), 0.0f, 1.0f);
		AddFn(EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Add), Buffer, SampleMaskWeight);
		AddFn(EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Add), BakeDefaults.GetData(), 1.0f - SampleMaskWeight);
		OverwriteFn(EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Overwrite), (SampleMaskWeight == 0) ? BakeDefaults.GetData() : Buffer);
	}
	else
	{
		AddFn(EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Add), Buffer, 1.0f);
		OverwriteFn(EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode::Overwrite), Buffer);
	}
}

int32 FMeshMapBaker::AddEvaluator(const TSharedPtr<FMeshMapEvaluator, ESPMode::ThreadSafe>& Eval)
{
	return Bakers.Add(Eval);
}

FMeshMapEvaluator* FMeshMapBaker::GetEvaluator(const int32 EvalIdx) const
{
	return Bakers[EvalIdx].Get();
}

void FMeshMapBaker::Reset()
{
	Bakers.Empty();
	BakeResults.Empty();
}

int32 FMeshMapBaker::NumEvaluators() const
{
	return Bakers.Num();
}

const TArrayView<TUniquePtr<TImageBuilder<FVector4f>>> FMeshMapBaker::GetBakeResults(const int32 EvalIdx)
{
	const int32 ResultIdx = BakeOffsets[EvalIdx];
	const int32 NumResults = BakeOffsets[EvalIdx + 1] - ResultIdx;
	return TArrayView<TUniquePtr<TImageBuilder<FVector4f>>>(&BakeResults[ResultIdx], NumResults);
}

void FMeshMapBaker::SetDimensions(const FImageDimensions DimensionsIn)
{
	Dimensions = DimensionsIn;
}

void FMeshMapBaker::SetGutterEnabled(const bool bEnabled)
{
	bGutterEnabled = bEnabled;
}

void FMeshMapBaker::SetGutterSize(const int32 GutterSizeIn)
{
	// GutterSize must be >= 1 since it is tied to MaxDistance for the
	// OccupancyMap spatial search.
	GutterSize = GutterSizeIn >= 1 ? GutterSizeIn : 1;
}

void FMeshMapBaker::SetSamplesPerPixel(const int32 SamplesPerPixelIn)
{
	SamplesPerPixel = SamplesPerPixelIn;
}

void FMeshMapBaker::SetFilter(const EBakeFilterType FilterTypeIn)
{
	FilterType = FilterTypeIn;
}

void FMeshMapBaker::SetTileSize(const int TileSizeIn)
{
	TileSize = TileSizeIn;
}

void FMeshMapBaker::InitFilter()
{
	FilterKernelSize = TilePadding;
	switch(FilterType)
	{
	case EBakeFilterType::None:
		FilterKernelSize = 0;
		TextureFilterEval = &EvaluateFilter<EBakeFilterType::None>;
		IsInFilterRegionEval = &EvaluateIsInFilterRegion<EBakeFilterType::None>;
		break;
	case EBakeFilterType::Box:
		TextureFilterEval = &EvaluateFilter<EBakeFilterType::Box>;
		IsInFilterRegionEval = &EvaluateIsInFilterRegion<EBakeFilterType::Box>;
		break;
	case EBakeFilterType::BSpline:
		TextureFilterEval = &EvaluateFilter<EBakeFilterType::BSpline>;
		IsInFilterRegionEval = &EvaluateIsInFilterRegion<EBakeFilterType::BSpline>;
		break;
	case EBakeFilterType::MitchellNetravali:
		TextureFilterEval = &EvaluateFilter<EBakeFilterType::MitchellNetravali>;
		IsInFilterRegionEval = &EvaluateIsInFilterRegion<EBakeFilterType::MitchellNetravali>;
		break;
	}
}

template<FMeshMapBaker::EBakeFilterType BakeFilterType>
float FMeshMapBaker::EvaluateFilter(const FVector2d& Dist)
{
	float Result = 0.0f;
	if constexpr(BakeFilterType == EBakeFilterType::None)
	{
		Result = 1.0f;
	}
	else if constexpr(BakeFilterType == EBakeFilterType::Box)
	{
		Result = BoxFilter.GetWeight(Dist);
	}
	else if constexpr(BakeFilterType == EBakeFilterType::BSpline)
	{
		Result = BSplineFilter.GetWeight(Dist);
	}
	else if constexpr(BakeFilterType == EBakeFilterType::MitchellNetravali)
	{
		Result = MitchellNetravaliFilter.GetWeight(Dist);
	}
	return Result;
}

template<FMeshMapBaker::EBakeFilterType BakeFilterType>
bool FMeshMapBaker::EvaluateIsInFilterRegion(const FVector2d& Dist)
{
	bool Result = false;
	if constexpr(BakeFilterType == EBakeFilterType::None)
	{
		Result = true;
	}
	else if constexpr(BakeFilterType == EBakeFilterType::Box)
	{
		Result = BoxFilter.IsInFilterRegion(Dist);
	}
	else if constexpr(BakeFilterType == EBakeFilterType::BSpline)
	{
		Result = BSplineFilter.IsInFilterRegion(Dist);
	}
	else if constexpr(BakeFilterType == EBakeFilterType::MitchellNetravali)
	{
		Result = MitchellNetravaliFilter.IsInFilterRegion(Dist);
	}
	return Result;
}

void FMeshMapBaker::ComputeUVCharts(const FDynamicMesh3& Mesh, TArray<int32>& MeshUVCharts)
{
	MeshUVCharts.SetNumUninitialized(Mesh.MaxTriangleID());
	for (int32& ChartId : MeshUVCharts)
	{
		ChartId = IndexConstants::InvalidID;
	}
	if (const FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes() ? Mesh.Attributes()->PrimaryUV() : nullptr)
	{
		FMeshConnectedComponents UVComponents(&Mesh);
		UVComponents.FindConnectedTriangles([UVOverlay](int32 Triangle0, int32 Triangle1) {
			return UVOverlay ? UVOverlay->AreTrianglesConnected(Triangle0, Triangle1) : false;
		});
		const int32 NumComponents = UVComponents.Num();
		for (int32 ComponentId = 0; ComponentId < NumComponents; ++ComponentId)
		{
			const FMeshConnectedComponents::FComponent& UVComp = UVComponents.GetComponent(ComponentId);
			for (const int32 TriId : UVComp.Indices)
			{
				MeshUVCharts[TriId] = ComponentId;
			}
		}
	}
}



