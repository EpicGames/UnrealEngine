// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshNormalMapBaker.h"

using namespace UE::Geometry;

void FMeshNormalMapBaker::Bake()
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	check(BakeCache);
	const FDynamicMesh3* DetailMesh = BakeCache->GetDetailMesh();
	const FDynamicMeshNormalOverlay* DetailNormalOverlay = BakeCache->GetDetailNormals();
	check(DetailNormalOverlay);

	// make sure we have per-triangle tangents computed so that GetInterpolatedTriangleTangent() below works
	int32 MaxTriID = BakeCache->GetBakeTargetMesh()->MaxTriangleID();
	check( BaseMeshTangents->GetTangents().Num() >= 3*BakeCache->GetBakeTargetMesh()->MaxTriangleID());

	auto NormalSampleFunction = [&](const FMeshImageBakingCache::FCorrespondenceSample& SampleData)
	{
		int32 DetailTriID = SampleData.DetailTriID;
		if (DetailMesh->IsTriangle(DetailTriID))
		{
			// get tangents on base mesh
			FVector3d BaseTangentX, BaseTangentY;
			BaseMeshTangents->GetInterpolatedTriangleTangent(
				SampleData.BaseSample.TriangleIndex, 
				SampleData.BaseSample.BaryCoords, 
				BaseTangentX, BaseTangentY);

			// sample normal on detail mesh
			FVector3d DetailNormal;
			DetailNormalOverlay->GetTriBaryInterpolate<double>(DetailTriID, &SampleData.DetailBaryCoords.X, &DetailNormal.X);
			Normalize(DetailNormal);

			// compute normal in tangent space
			double dx = DetailNormal.Dot(BaseTangentX);
			double dy = DetailNormal.Dot(BaseTangentY);
			double dz = DetailNormal.Dot(SampleData.BaseNormal);

			return FVector3f((float)dx, (float)dy, (float)dz);
		}
		return FVector3f::UnitZ();
	};


	NormalsBuilder = MakeUnique<TImageBuilder<FVector3f>>();
	NormalsBuilder->SetDimensions(BakeCache->GetDimensions());
	FVector3f DefaultMapNormal = (DefaultNormal + FVector3f::One()) * 0.5f;
	NormalsBuilder->Clear(DefaultMapNormal);


	BakeCache->EvaluateSamples([&](const FVector2i& Coords, const FMeshImageBakingCache::FCorrespondenceSample& Sample)
	{
		FVector3f RelativeDetailNormal = NormalSampleFunction(Sample);
		FVector3f MapNormal = (RelativeDetailNormal + FVector3f::One()) * 0.5f;
		NormalsBuilder->SetPixel(Coords, MapNormal);
	});

	const FImageOccupancyMap& Occupancy = *BakeCache->GetOccupancyMap();

	for (int64 k = 0; k < Occupancy.GutterTexels.Num(); k++)
	{
		TPair<int64, int64> GutterTexel = Occupancy.GutterTexels[k];
		NormalsBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
	}


}