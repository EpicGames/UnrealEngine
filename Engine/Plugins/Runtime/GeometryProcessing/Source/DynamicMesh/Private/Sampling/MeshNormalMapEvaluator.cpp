// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshNormalMapEvaluator.h"
#include "Sampling/MeshMapBaker.h"

using namespace UE::Geometry;


void FMeshNormalMapEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	// Cache data from the baker
	DetailSampler = Baker.GetDetailSampler();
	auto GetDetailNormalMaps = [this](const void* Mesh)
	{
		const FNormalTexture* NormalMap = DetailSampler->GetNormalTextureMap(Mesh);
		if (NormalMap)
		{
			// Require valid normal map and UV layer to enable normal map transfer.
			// Tangents also required if the map is in tangent space.
			const bool bDetailNormalTangentSpace = NormalMap->Get<2>() == IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent; 
			const bool bEnableNormalMapTransfer = DetailSampler->HasUVs(Mesh, NormalMap->Get<1>()) &&
				(!bDetailNormalTangentSpace || DetailSampler->HasTangents(Mesh));
			if (bEnableNormalMapTransfer)
			{
				DetailNormalMaps.Add(Mesh, *NormalMap);
				bHasDetailNormalTextures = true;
			}
		}
	};
	DetailSampler->ProcessMeshes(GetDetailNormalMaps);
	
	BaseMeshTangents = Baker.GetTargetMeshTangents();

	Context.Evaluate = bHasDetailNormalTextures ? &EvaluateSample<true> : &EvaluateSample<false>;
	Context.EvaluateDefault = &EvaluateDefault;
	Context.EvaluateColor = &EvaluateColor;
	Context.EvaluateChannel = &EvaluateChannel;
	Context.EvalData = this;
	Context.AccumulateMode = EAccumulateMode::Add;
	Context.DataLayout = DataLayout();
}

const TArray<FMeshMapEvaluator::EComponents>& FMeshNormalMapEvaluator::DataLayout() const
{
	static const TArray<EComponents> Layout{ EComponents::Float3 };
	return Layout;
}

template <bool bUseDetailNormalMap>
void FMeshNormalMapEvaluator::EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	const FMeshNormalMapEvaluator* Eval = static_cast<FMeshNormalMapEvaluator*>(EvalData);
	const FVector3f SampleResult = Eval->SampleFunction<bUseDetailNormalMap>(Sample);
	WriteToBuffer(Out, SampleResult);
}

void FMeshNormalMapEvaluator::EvaluateDefault(float*& Out, void* EvalData)
{
	const FMeshNormalMapEvaluator* Eval = static_cast<FMeshNormalMapEvaluator*>(EvalData);
	WriteToBuffer(Out, Eval->DefaultNormal);
}

void FMeshNormalMapEvaluator::EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
{
	// Map normal space [-1,1] to color space [0,1]
	const FVector3f Normal(In[0], In[1], In[2]);
	const FVector3f Color = (Normal + FVector3f::One()) * 0.5f;
	Out = FVector4f(Color.X, Color.Y, Color.Z, 1.0f);
	In += 3;
}

void FMeshNormalMapEvaluator::EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData)
{
	// Per-channel should not be an option for NormalMap evaluator
	ensure(false);
	Out = 0.0f;
	In += 1;
}

template <bool bUseDetailNormalMap>
FVector3f FMeshNormalMapEvaluator::SampleFunction(const FCorrespondenceSample& SampleData) const
{
	FVector3f Result = FVector3f::UnitZ();
	const void* DetailMesh = SampleData.DetailMesh;
	const int32 DetailTriID = SampleData.DetailTriID;

	// get tangents on base mesh
	FVector3d BaseTangentX, BaseTangentY;
	BaseMeshTangents->GetInterpolatedTriangleTangent(
		SampleData.BaseSample.TriangleIndex,
		SampleData.BaseSample.BaryCoords,
		BaseTangentX, BaseTangentY);

	// sample normal on detail mesh
	FVector3f DetailNormal;
	DetailSampler->TriBaryInterpolateNormal(
		DetailMesh,
		DetailTriID,
		SampleData.DetailBaryCoords,
		DetailNormal);
	Normalize(DetailNormal);
	
	if constexpr (bUseDetailNormalMap)
	{
		const TImageBuilder<FVector4f>* DetailNormalMap = nullptr;
		int DetailNormalUVLayer = 0;
		IMeshBakerDetailSampler::EBakeDetailNormalSpace DetailNormalSpace = IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent;
		const FNormalTexture* DetailNormalTexture = DetailNormalMaps.Find(DetailMesh);
		if (DetailNormalTexture)
		{
			Tie(DetailNormalMap, DetailNormalUVLayer, DetailNormalSpace) = *DetailNormalTexture;
		}

		if (DetailNormalMap)
		{
			FVector3d DetailTangentX, DetailTangentY;
			DetailSampler->TriBaryInterpolateTangents(
				DetailMesh,
				SampleData.DetailTriID,
				SampleData.DetailBaryCoords,
				DetailTangentX, DetailTangentY);
		
			FVector2f DetailUV;
			DetailSampler->TriBaryInterpolateUV(
				DetailMesh,
				DetailTriID,
				SampleData.DetailBaryCoords,
				DetailNormalUVLayer,
				DetailUV);
			const FVector4f DetailNormalColor4 = DetailNormalMap->BilinearSampleUV<float>(FVector2d(DetailUV), FVector4f(0, 0, 0, 1));

			// Map color space [0,1] to normal space [-1,1]
			const FVector3f DetailNormalColor(DetailNormalColor4.X, DetailNormalColor4.Y, DetailNormalColor4.Z);
			FVector3f DetailNormalObjectSpace = (DetailNormalColor * 2.0f) - FVector3f::One();
			// Ideally this branch could be made compile time. Unfortunately since each mesh could have its
			// own source normal map each with their own normal space, this branch must be resolved at runtime.
			if (DetailNormalSpace == IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent) //-V547
			{
				// Convert detail normal tangent space to object space
				const FVector3f DetailNormalTangentSpace = DetailNormalObjectSpace;
				DetailNormalObjectSpace = DetailNormalTangentSpace.X * FVector3f(DetailTangentX) + DetailNormalTangentSpace.Y * FVector3f(DetailTangentY) + DetailNormalTangentSpace.Z * DetailNormal;
			}
			Normalize(DetailNormalObjectSpace);
			DetailNormal = DetailNormalObjectSpace;
		}
	}

	// compute normal in tangent space
	const double dx = DetailNormal.Dot(FVector3f(BaseTangentX));
	const double dy = DetailNormal.Dot(FVector3f(BaseTangentY));
	const double dz = DetailNormal.Dot(FVector3f(SampleData.BaseNormal));

	return FVector3f((float)dx, (float)dy, (float)dz);
}



