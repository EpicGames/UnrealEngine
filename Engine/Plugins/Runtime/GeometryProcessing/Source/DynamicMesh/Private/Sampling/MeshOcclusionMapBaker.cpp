// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshOcclusionMapBaker.h"
#include "Sampling/SphericalFibonacci.h"
#include "Sampling/Gaussians.h"

#include "Math/RandomStream.h"

using namespace UE::Geometry;

void FMeshOcclusionMapBaker::Bake()
{
	if (OcclusionType == EOcclusionMapType::None)
	{
		return;
	}

	const FMeshImageBakingCache* BakeCache = GetCache();
	check(BakeCache);
	const FDynamicMesh3* DetailMesh = BakeCache->GetDetailMesh();
	const FDynamicMeshAABBTree3* DetailSpatial = BakeCache->GetDetailSpatial();
	const FDynamicMeshNormalOverlay* DetailNormalOverlay = BakeCache->GetDetailNormals();
	check(DetailNormalOverlay);

	double BiasDotThreshold = FMathd::Cos(FMathd::Clamp(90.0 - BiasAngleDeg, 0.0, 90.0) * FMathd::DegToRad);

	// Precompute occlusion ray directions
	TArray<FVector3d> RayDirections;
	THemisphericalFibonacci<double>::EDistribution Dist = THemisphericalFibonacci<double>::EDistribution::Cosine;
	switch (Distribution)
	{
	case EDistribution::Uniform:
		Dist = THemisphericalFibonacci<double>::EDistribution::Uniform;
		break;
	case EDistribution::Cosine:
		Dist = THemisphericalFibonacci<double>::EDistribution::Cosine;
		break;
	}
	THemisphericalFibonacci<double> Points(NumOcclusionRays, Dist);
	for (int32 k = 0; k < Points.Num(); ++k)
	{
		FVector3d P = Points[k];
		if (P.Z > 0)
		{
			RayDirections.Add( Normalized(P) );
		}
	}

	// Map occlusion ray hemisphere to conical area (SpreadAngle/2)
	double ConicalAngle = FMathd::Clamp(SpreadAngle * 0.5, 0.0001, 90.0);
	for (int32 k = 0; k < RayDirections.Num(); ++k)
	{
		FVector3d& RayDir = RayDirections[k];
		double RayAngle = AngleD(RayDir,FVector3d::UnitZ());
		FVector3d RayCross = RayDir.Cross(FVector3d::UnitZ());
		double RotationAngle = RayAngle - FMathd::Lerp(0.0, ConicalAngle, RayAngle / 90.0);
		FQuaterniond Rotation(RayCross, RotationAngle, true);
		RayDir = Rotation * RayDir;
	}

	// Randomized rotation of occlusion rays about the normal
	FRandomStream RotationGen(31337);
	FCriticalSection RotationLock;
	auto GetRandomRotation = [&RotationGen, &RotationLock]() {
		RotationLock.Lock();
		double Angle = RotationGen.GetFraction() * FMathd::TwoPi;
		RotationLock.Unlock();
		return Angle;
	};

	auto OcclusionSampleFunction = [&](const FMeshImageBakingCache::FCorrespondenceSample& SampleData, double& OutOcclusion, FVector3d& OutNormal)
	{
		// Fallback normal if invalid Tri or fully occluded
		FVector3d DefaultNormal = FVector3d::UnitZ();
		switch (NormalSpace)
		{
		case ESpace::Tangent:
			break;
		case ESpace::Object:
			DefaultNormal = SampleData.BaseNormal;
			break;
		}

		int32 DetailTriID = SampleData.DetailTriID;
		if (DetailMesh->IsTriangle(DetailTriID))
		{
			FIndex3i DetailTri = DetailMesh->GetTriangle(DetailTriID);
			//FVector3d DetailTriNormal = DetailMesh.GetTriNormal(DetailTriID);
			FVector3d DetailTriNormal;
			DetailNormalOverlay->GetTriBaryInterpolate<double>(DetailTriID, &SampleData.DetailBaryCoords.X, &DetailTriNormal.X);
			if (!DetailTriNormal.Normalize())
			{
				// degenerate triangle normal
				OutOcclusion = 0.0;
				OutNormal = DefaultNormal;
				return;
			}

			FVector3d BaseTangentX, BaseTangentY;
			if (WantBentNormal() && NormalSpace == ESpace::Tangent)
			{
				check(BaseMeshTangents);
				BaseMeshTangents->GetInterpolatedTriangleTangent(
					SampleData.BaseSample.TriangleIndex,
					SampleData.BaseSample.BaryCoords,
					BaseTangentX, BaseTangentY);
			}

			FVector3d DetailBaryCoords = SampleData.DetailBaryCoords;
			FVector3d DetailPos = DetailMesh->GetTriBaryPoint(DetailTriID, DetailBaryCoords.X, DetailBaryCoords.Y, DetailBaryCoords.Z);
			DetailPos += 10.0f * FMathf::ZeroTolerance * DetailTriNormal;
			FFrame3d SurfaceFrame(DetailPos, DetailTriNormal);

			double RotationAngle = GetRandomRotation();
			SurfaceFrame.Rotate(FQuaterniond(SurfaceFrame.Z(), RotationAngle, false));

			IMeshSpatial::FQueryOptions QueryOptions;
			QueryOptions.MaxDistance = MaxDistance;

			double AccumOcclusion = 0;
			FVector3d AccumNormal(FVector3d::Zero());
			double TotalPointWeight = 0;
			double TotalNormalWeight = 0;
			for (FVector3d SphereDir : RayDirections)
			{
				FRay3d OcclusionRay(DetailPos, SurfaceFrame.FromFrameVector(SphereDir));
				ensure(OcclusionRay.Direction.Dot(DetailTriNormal) > 0);

				bool bHit = DetailSpatial->TestAnyHitTriangle(OcclusionRay, QueryOptions);

				if (WantAmbientOcclusion())
				{
					// Have weight of point fall off as it becomes more coplanar with face. 
					// This reduces faceting artifacts that we would otherwise see because geometry does not vary smoothly
					double PointWeight = 1.0;
					double BiasDot = OcclusionRay.Direction.Dot(DetailTriNormal);
					if (BiasDot < BiasDotThreshold)
					{
						PointWeight = FMathd::Lerp(0.0, 1.0, FMathd::Clamp(BiasDot / BiasDotThreshold, 0.0, 1.0));
						PointWeight *= PointWeight;
					}
					TotalPointWeight += PointWeight;

					if (bHit)
					{
						AccumOcclusion += PointWeight;
					}
				}

				if (WantBentNormal())
				{
					FVector3d BentNormal = OcclusionRay.Direction;
					switch (NormalSpace)
					{
					case ESpace::Tangent:
					{
						// compute normal in tangent space
						double dx = BentNormal.Dot(BaseTangentX);
						double dy = BentNormal.Dot(BaseTangentY);
						double dz = BentNormal.Dot(SampleData.BaseNormal);
						BentNormal = FVector3d(dx, dy, dz);;
						break;
					}
					case ESpace::Object:
						break;
					}
					TotalNormalWeight += 1.0;

					if (!bHit)
					{
						AccumNormal += BentNormal;
					}
				}
			}

			if (WantAmbientOcclusion())
			{
				AccumOcclusion = (TotalPointWeight > 0.0001) ? (AccumOcclusion / TotalPointWeight) : 0.0;
			}
			if (WantBentNormal())
			{
				AccumNormal = (TotalNormalWeight > 0.0 && AccumNormal.Length() > 0.0) ? Normalized(AccumNormal / TotalNormalWeight) : DefaultNormal;
			}
			OutOcclusion = AccumOcclusion;
			OutNormal = AccumNormal;
			return;
		}
		OutOcclusion = 0.0;
		OutNormal = DefaultNormal;
	};

	if (WantAmbientOcclusion())
	{
		OcclusionBuilder = MakeUnique<TImageBuilder<FVector3f>>();
		OcclusionBuilder->SetDimensions(BakeCache->GetDimensions());
		OcclusionBuilder->Clear(FVector3f::One());
	}
	if (WantBentNormal())
	{
		NormalBuilder = MakeUnique<TImageBuilder<FVector3f>>();
		NormalBuilder->SetDimensions(BakeCache->GetDimensions());
		FVector3f DefaultNormal = FVector3f::UnitZ();
		switch (NormalSpace)
		{
		case ESpace::Tangent:
			break;
		case ESpace::Object:
			DefaultNormal = FVector3f::Zero();
			break;
		}
		FVector3f DefaultMapNormal = (DefaultNormal + FVector3f::One()) * 0.5f;
		NormalBuilder->Clear(DefaultMapNormal);
	}

	BakeCache->EvaluateSamples([&](const FVector2i& Coords, const FMeshImageBakingCache::FCorrespondenceSample& Sample)
	{
		double Occlusion = 0.0;
		FVector3d BentNormal;
		OcclusionSampleFunction(Sample, Occlusion, BentNormal);
		if (WantAmbientOcclusion())
		{
			FVector3f OcclusionColor = FMathd::Clamp(1.0f - (float)Occlusion, 0.0f, 1.0f) * FVector3f::One();
			OcclusionBuilder->SetPixel(Coords, OcclusionColor);
		}
		if (WantBentNormal())
		{
			FVector3f NormalColor = ((FVector3f)BentNormal + FVector3f::One()) * 0.5f;
			NormalBuilder->SetPixel(Coords, NormalColor);
		}
	});

	const FImageOccupancyMap& Occupancy = *BakeCache->GetOccupancyMap();
	for (int64 k = 0; k < Occupancy.GutterTexels.Num(); k++)
	{
		TPair<int64, int64> GutterTexel = Occupancy.GutterTexels[k];
		if (WantAmbientOcclusion())
		{
			OcclusionBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
		}
		if (WantBentNormal())
		{
			NormalBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
		}
	}

	if (WantAmbientOcclusion() && BlurRadius > 0.01)
	{
		TDiscreteKernel2f BlurKernel2d;
		TGaussian2f::MakeKernelFromRadius((float)BlurRadius, BlurKernel2d);
		TArray<float> AOBlurBuffer;
		Occupancy.ParallelProcessingPass<float>(
			[&](int64 Index) { return 0.0f; },
			[&](int64 LinearIdx, float Weight, float& CurValue) { CurValue += Weight * OcclusionBuilder->GetPixel(LinearIdx).X; },
			[&](int64 LinearIdx, float WeightSum, float& CurValue) { CurValue /= WeightSum; },
			[&](int64 LinearIdx, float& CurValue) { OcclusionBuilder->SetPixel(LinearIdx, FVector3f(CurValue, CurValue, CurValue)); },
			[&](const FVector2i& TexelOffset) { return BlurKernel2d.EvaluateFromOffset(TexelOffset); },
			BlurKernel2d.IntRadius,
			AOBlurBuffer);
	}



}
