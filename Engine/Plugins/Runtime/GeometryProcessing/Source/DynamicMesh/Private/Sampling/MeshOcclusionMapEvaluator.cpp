// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshOcclusionMapEvaluator.h"
#include "Sampling/MeshMapBaker.h"
#include "Sampling/SphericalFibonacci.h"
#include "Math/RandomStream.h"

using namespace UE::Geometry;

/**
 * Occlusion type tests
 * Using defines for if constexpr compile time branches
 */
#define WANT_AMBIENT_OCCLUSION(OccType) \
	(OccType & EMeshOcclusionMapType::AmbientOcclusion) == EMeshOcclusionMapType::AmbientOcclusion
#define WANT_BENT_NORMAL(OccType) \
	(OccType & EMeshOcclusionMapType::BentNormal) == EMeshOcclusionMapType::BentNormal
#define WANT_ALL(OccType) \
	(OccType & EMeshOcclusionMapType::All) == EMeshOcclusionMapType::All

const FVector3d FMeshOcclusionMapEvaluator::DefaultTangentNormal = FVector3d::UnitZ();
const FVector3d FMeshOcclusionMapEvaluator::DefaultObjectNormal = FVector3d::Zero();

void FMeshOcclusionMapEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	Context.DataLayout = DataLayout();

	if (WANT_ALL(OcclusionType))
	{
		if (NormalSpace == ESpace::Tangent)
		{
			Context.Evaluate = &EvaluateSample<EMeshOcclusionMapType::All, ESpace::Tangent>;
			Context.EvaluateDefault = &EvaluateDefault<EMeshOcclusionMapType::All, ESpace::Tangent>;
			Context.EvaluateColor = &EvaluateColor<EMeshOcclusionMapType::All, ESpace::Tangent>;
			Context.EvaluateChannel = &EvaluateChannel<EMeshOcclusionMapType::All, ESpace::Tangent>;
		}
		else // NormalSpace == ESpace::Object
		{
			Context.Evaluate = &EvaluateSample<EMeshOcclusionMapType::All, ESpace::Object>;
			Context.EvaluateDefault = &EvaluateDefault<EMeshOcclusionMapType::All, ESpace::Object>;
			Context.EvaluateColor = &EvaluateColor<EMeshOcclusionMapType::All, ESpace::Object>;
			Context.EvaluateChannel = &EvaluateChannel<EMeshOcclusionMapType::All, ESpace::Object>;
		}
		Context.EvalData = this;
		Context.AccumulateMode = EAccumulateMode::Add;
	}
	else if (WANT_AMBIENT_OCCLUSION(OcclusionType))
	{
		Context.Evaluate = &EvaluateSample<EMeshOcclusionMapType::AmbientOcclusion, ESpace::Tangent>;
		Context.EvaluateDefault = &EvaluateDefault<EMeshOcclusionMapType::AmbientOcclusion, ESpace::Tangent>;
		Context.EvaluateColor = &EvaluateColor<EMeshOcclusionMapType::AmbientOcclusion, ESpace::Tangent>;
		Context.EvaluateChannel = &EvaluateChannel<EMeshOcclusionMapType::AmbientOcclusion, ESpace::Tangent>;
		Context.EvalData = this;
		Context.AccumulateMode = EAccumulateMode::Add;
	}
	else if (WANT_BENT_NORMAL(OcclusionType))
	{
		if (NormalSpace == ESpace::Tangent)
		{
			Context.Evaluate = &EvaluateSample<EMeshOcclusionMapType::BentNormal, ESpace::Tangent>;
			Context.EvaluateDefault = &EvaluateDefault<EMeshOcclusionMapType::BentNormal, ESpace::Tangent>;
			Context.EvaluateColor = &EvaluateColor<EMeshOcclusionMapType::BentNormal, ESpace::Tangent>;
			Context.EvaluateChannel = &EvaluateChannel<EMeshOcclusionMapType::BentNormal, ESpace::Tangent>;
		}
		else // NormalSpace == ESpace::Object
		{
			Context.Evaluate = &EvaluateSample<EMeshOcclusionMapType::BentNormal, ESpace::Object>;
			Context.EvaluateDefault = &EvaluateDefault<EMeshOcclusionMapType::BentNormal, ESpace::Object>;
			Context.EvaluateColor = &EvaluateColor<EMeshOcclusionMapType::BentNormal, ESpace::Object>;
			Context.EvaluateChannel = &EvaluateChannel<EMeshOcclusionMapType::BentNormal, ESpace::Object>;
		}
		Context.EvalData = this;
		Context.AccumulateMode = EAccumulateMode::Add;
	}
	else
	{
		// TODO: Support error case in the baker to skip over invalid eval configs?
		checkSlow(false);
	}

	// Cache data from the baker
	DetailSampler = Baker.GetDetailSampler();

	if (WANT_BENT_NORMAL(OcclusionType) && NormalSpace == ESpace::Tangent)
	{
		BaseMeshTangents = Baker.GetTargetMeshTangents();
	}

	BiasDotThreshold = FMathd::Cos(FMathd::Clamp(90.0 - BiasAngleDeg, 0.0, 90.0) * FMathd::DegToRad);

	// Precompute occlusion ray directions
	RayDirections.Empty(NumOcclusionRays);
	THemisphericalFibonacci<double>::EDistribution Dist = THemisphericalFibonacci<double>::EDistribution::Cosine;
	switch (Distribution)
	{
	case EDistribution::Uniform:
		Dist = THemisphericalFibonacci<double>::EDistribution::Uniform;
		break;
	case EDistribution::Cosine:
		Dist = THemisphericalFibonacci<double>::EDistribution::Cosine;
		break;
	}
	THemisphericalFibonacci<double> Points(NumOcclusionRays, Dist);
	for (int32 k = 0; k < Points.Num(); ++k)
	{
		FVector3d P = Points[k];
		if (P.Z > 0)
		{
			RayDirections.Add(Normalized(P));
		}
	}

	// Map occlusion ray hemisphere to conical area (SpreadAngle/2)
	const double ConicalAngle = FMathd::Clamp(SpreadAngle * 0.5, 0.0001, 90.0);
	for (int32 k = 0; k < RayDirections.Num(); ++k)
	{
		FVector3d& RayDir = RayDirections[k];
		const double RayAngle = AngleD(RayDir, FVector3d::UnitZ());
		FVector3d RayCross = RayDir.Cross(FVector3d::UnitZ());
		const double RotationAngle = RayAngle - FMathd::Lerp(0.0, ConicalAngle, RayAngle / 90.0);
		FQuaterniond Rotation(RayCross, RotationAngle, true);
		RayDir = Rotation * RayDir;
	}
}

const TArray<FMeshMapEvaluator::EComponents>& FMeshOcclusionMapEvaluator::DataLayout() const
{
	if (WANT_ALL(OcclusionType))
	{
		static const TArray<EComponents> Layout{ EComponents::Float1, EComponents::Float3 };
		return Layout;
	}

	if (WANT_AMBIENT_OCCLUSION(OcclusionType))
	{
		static const TArray<EComponents> Layout{ EComponents::Float1 };
		return Layout;
	}

	if (WANT_BENT_NORMAL(OcclusionType))
	{
		static const TArray<EComponents> Layout{ EComponents::Float3 };
		return Layout;
	}

	// TODO: Support error case in the baker to skip over invalid eval configs?
	checkSlow(false);

	static const TArray<EComponents> Layout{ EComponents::Float1 };
	return Layout;
}

template <EMeshOcclusionMapType ComputeType, FMeshOcclusionMapEvaluator::ESpace ComputeSpace>
void FMeshOcclusionMapEvaluator::EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	FMeshOcclusionMapEvaluator* Eval = static_cast<FMeshOcclusionMapEvaluator*>(EvalData);
	const FVector3d& DefaultNormal = ComputeSpace == ESpace::Tangent ? DefaultTangentNormal : Sample.BaseNormal;

	double Occlusion = 0.0;
	FVector3f BentNormal = FVector3f::UnitZ();
	Tie(Occlusion, BentNormal) = Eval->SampleFunction<ComputeType, ComputeSpace>(Sample, DefaultNormal);

	if constexpr (WANT_AMBIENT_OCCLUSION(ComputeType))
	{
		WriteToBuffer(Out, FMathf::Clamp(1.0f - (float)Occlusion, 0.0f, 1.0f));
	}

	if constexpr (WANT_BENT_NORMAL(ComputeType))
	{
		WriteToBuffer(Out, BentNormal);
	}
}

template <EMeshOcclusionMapType ComputeType, FMeshOcclusionMapEvaluator::ESpace ComputeSpace>
void FMeshOcclusionMapEvaluator::EvaluateDefault(float*& Out, void* EvalData)
{
	if constexpr (WANT_AMBIENT_OCCLUSION(ComputeType))
	{
		WriteToBuffer(Out, 1.0f);
	}

	if constexpr (WANT_BENT_NORMAL(ComputeType))
	{
		const FVector3d& DefaultNormal = ComputeSpace == ESpace::Tangent ? DefaultTangentNormal : DefaultObjectNormal;
		WriteToBuffer(Out, (FVector3f)DefaultNormal);
	}
}

template <EMeshOcclusionMapType ComputeType, FMeshOcclusionMapEvaluator::ESpace ComputeSpace>
void FMeshOcclusionMapEvaluator::EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
{
	auto EvalOcclusion = [&In, &Out]()
	{
		Out = FVector4f(In[0], In[0], In[0], 1.0f);
		In += 1;
	};

	auto EvalNormal = [&In, &Out]()
	{
		const FVector3f Normal(In[0], In[1], In[2]);
		const FVector3f Color = (Normal + FVector3f::One()) * 0.5f;
		Out = FVector4f(Color.X, Color.Y, Color.Z, 1.0f);
		In += 3;
	};

	if constexpr (WANT_AMBIENT_OCCLUSION(ComputeType) && WANT_BENT_NORMAL(ComputeType))
	{
		DataIdx == 0 ? EvalOcclusion() : EvalNormal();
	}
	else if constexpr (WANT_AMBIENT_OCCLUSION(ComputeType))
	{
		EvalOcclusion();
	}
	else if constexpr (WANT_BENT_NORMAL(ComputeType))
	{
		EvalNormal();
	}
}

template <EMeshOcclusionMapType ComputeType, FMeshOcclusionMapEvaluator::ESpace ComputeSpace>
void FMeshOcclusionMapEvaluator::EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData)
{
	auto EvalOcclusion = [&In, &Out]()
	{
		Out = In[0];
		In += 1;
	};

	auto EvalNormal = [&In, &Out]()
	{
		ensure(false);	// Should not be able to do per-channel baking for normals
		Out = 0.0f;
		In += 1;
	};

	if constexpr (WANT_AMBIENT_OCCLUSION(ComputeType) && WANT_BENT_NORMAL(ComputeType))
	{
		DataIdx == 0 ? EvalOcclusion() : EvalNormal();
	}
	else if constexpr (WANT_AMBIENT_OCCLUSION(ComputeType))
	{
		EvalOcclusion();
	}
	else if constexpr (WANT_BENT_NORMAL(ComputeType))
	{
		EvalNormal();
	}
}


template <EMeshOcclusionMapType ComputeType, FMeshOcclusionMapEvaluator::ESpace ComputeSpace>
FMeshOcclusionMapEvaluator::FOcclusionTuple FMeshOcclusionMapEvaluator::SampleFunction(const FCorrespondenceSample& SampleData, const FVector3d& DefaultNormal)
{
	const void* DetailMesh = SampleData.DetailMesh;
	const int32 DetailTriID = SampleData.DetailTriID;

	FVector3f DetailTriNormal;
	DetailSampler->TriBaryInterpolateNormal(DetailMesh, DetailTriID, SampleData.DetailBaryCoords, DetailTriNormal);
	if (!DetailTriNormal.Normalize())
	{
		// degenerate triangle normal
		return FOcclusionTuple(0.0f, (FVector3f)DefaultNormal);
	}

	FVector3d BaseTangentX, BaseTangentY;
	if constexpr (WANT_BENT_NORMAL(ComputeType) && ComputeSpace == ESpace::Tangent)
	{
		check(BaseMeshTangents);
		BaseMeshTangents->GetInterpolatedTriangleTangent(
			SampleData.BaseSample.TriangleIndex,
			SampleData.BaseSample.BaryCoords,
			BaseTangentX, BaseTangentY);
	}

	const FVector3d DetailBaryCoords = SampleData.DetailBaryCoords;
	FVector3d DetailPos = DetailSampler->TriBaryInterpolatePoint(DetailMesh, DetailTriID, DetailBaryCoords);
	DetailPos += 10.0f * FMathf::ZeroTolerance * FVector3d(DetailTriNormal);
	FFrame3d SurfaceFrame(DetailPos, FVector3d(DetailTriNormal));

	// TODO: Consider pregenerating random rotations.
	FRandomStream RotationGen;
	RotationGen.GenerateNewSeed();
	const double RotationAngle = RotationGen.GetFraction() * FMathd::TwoPi;
	SurfaceFrame.Rotate(FQuaterniond(SurfaceFrame.Z(), RotationAngle, false));

	IMeshSpatial::FQueryOptions QueryOptions;
	QueryOptions.MaxDistance = MaxDistance;

	double AccumOcclusion = 0;
	FVector3d AccumNormal(FVector3d::Zero());
	double TotalPointWeight = 0;
	for (FVector3d SphereDir : RayDirections)
	{
		FRay3d OcclusionRay(DetailPos, SurfaceFrame.FromFrameVector(SphereDir));
		ensure(OcclusionRay.Direction.Dot(FVector3d(DetailTriNormal)) > 0);

		const bool bHit = DetailSampler->TestAnyHitTriangle(OcclusionRay, QueryOptions);

		if constexpr (WANT_AMBIENT_OCCLUSION(ComputeType))
		{
			// Have weight of point fall off as it becomes more coplanar with face. 
			// This reduces faceting artifacts that we would otherwise see because geometry does not vary smoothly
			double PointWeight = 1.0;
			const double BiasDot = OcclusionRay.Direction.Dot(FVector3d(DetailTriNormal));
			if (BiasDot < BiasDotThreshold)
			{
				PointWeight = FMathd::Lerp(0.0, 1.0, FMathd::Clamp(BiasDot / BiasDotThreshold, 0.0, 1.0));
				PointWeight *= PointWeight;
			}
			TotalPointWeight += PointWeight;

			if (bHit)
			{
				AccumOcclusion += PointWeight;
			}
		}

		if constexpr (WANT_BENT_NORMAL(ComputeType))
		{
			if (!bHit)
			{
				FVector3d BentNormal = OcclusionRay.Direction;
				if (ComputeSpace == ESpace::Tangent)
				{
					// compute normal in tangent space
					double dx = BentNormal.Dot(BaseTangentX);
					double dy = BentNormal.Dot(BaseTangentY);
					double dz = BentNormal.Dot(SampleData.BaseNormal);
					BentNormal = FVector3d(dx, dy, dz);;
				}
				AccumNormal += BentNormal;
			}
		}
	}

	if constexpr (WANT_AMBIENT_OCCLUSION(ComputeType))
	{
		AccumOcclusion = (TotalPointWeight > 0.0001) ? (AccumOcclusion / TotalPointWeight) : 0.0;
	}
	if constexpr (WANT_BENT_NORMAL(ComputeType))
	{
		AccumNormal = (AccumNormal.Length() > 0.0) ? Normalized(AccumNormal) : DefaultNormal;
	}
	return FOcclusionTuple( (float)AccumOcclusion, (FVector3f)AccumNormal);
}

#undef WANT_AMBIENT_OCCLUSION
#undef WANT_BENT_NORMAL
#undef WANT_ALL
