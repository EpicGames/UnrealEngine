// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshPropertyMapBaker.h"
#include "Util/ColorConstants.h"

using namespace UE::Geometry;

static FVector3f NormalToColor(const FVector3d Normal)
{
	return (FVector3f)((Normal + FVector3d::One()) * 0.5);
}

static FVector3f UVToColor(const FVector2d UV)
{
	double X = FMathd::Clamp(UV.X, 0.0, 1.0);
	double Y = FMathd::Clamp(UV.Y, 0.0, 1.0);
	return (FVector3f)FVector3d(X, Y, 0);
}

static FVector3f PositionToColor(const FVector3d Position, const FAxisAlignedBox3d SafeBounds)
{
	double X = (Position.X - SafeBounds.Min.X) / SafeBounds.Width();
	double Y = (Position.Y - SafeBounds.Min.Y) / SafeBounds.Height();
	double Z = (Position.Z - SafeBounds.Min.Z) / SafeBounds.Depth();
	return (FVector3f)FVector3d(X, Y, Z);
}


void FMeshPropertyMapBaker::Bake()
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	check(BakeCache);
	const FDynamicMesh3* DetailMesh = BakeCache->GetDetailMesh();
	const FDynamicMeshNormalOverlay* DetailNormalOverlay = BakeCache->GetDetailNormals();
	check(DetailNormalOverlay);
	const FDynamicMeshUVOverlay* DetailUVOverlay = DetailMesh->Attributes()->PrimaryUV();

	FAxisAlignedBox3d Bounds = DetailMesh->GetBounds();
	for (int32 j = 0; j < 3; ++j)
	{
		if (Bounds.Diagonal()[j] < FMathf::ZeroTolerance)
		{
			Bounds.Min[j] = Bounds.Center()[j] - FMathf::ZeroTolerance;
			Bounds.Max[j] = Bounds.Center()[j] + FMathf::ZeroTolerance;

		}
	}

	FVector3f DefaultValue(0, 0, 0);
	switch (this->Property)
	{
		case EMeshPropertyBakeType::Position:
			DefaultValue = PositionToColor(Bounds.Center(), Bounds);
			break;
		default:
		case EMeshPropertyBakeType::FacetNormal:
		case EMeshPropertyBakeType::Normal:
			DefaultValue = NormalToColor(FVector3d::UnitZ());
			break;
		case EMeshPropertyBakeType::UVPosition:
			DefaultValue = UVToColor(FVector2d::Zero());
			break;
	}

	auto PropertySampleFunction = [&](const FMeshImageBakingCache::FCorrespondenceSample& SampleData)
	{
		FVector3f Color = DefaultValue;
		int32 DetailTriID = SampleData.DetailTriID;
		if (DetailMesh->IsTriangle(DetailTriID))
		{
			switch (this->Property)
			{
				case EMeshPropertyBakeType::Position:
					{
						FVector3d Position = DetailMesh->GetTriBaryPoint(DetailTriID, SampleData.DetailBaryCoords[0], SampleData.DetailBaryCoords[1], SampleData.DetailBaryCoords[2]);
						Color = PositionToColor(Position, Bounds);
					}
					break;
				default:
				case EMeshPropertyBakeType::FacetNormal:
					{
						FVector3d FacetNormal = DetailMesh->GetTriNormal(DetailTriID);
						Color = NormalToColor(FacetNormal);
					}
					break;
				case EMeshPropertyBakeType::Normal:
					{
						if (DetailNormalOverlay->IsSetTriangle(DetailTriID))
						{
							FVector3d DetailNormal;
							DetailNormalOverlay->GetTriBaryInterpolate<double>(DetailTriID, &SampleData.DetailBaryCoords.X, &DetailNormal.X);
							Normalize(DetailNormal);
							Color = NormalToColor(DetailNormal);
						}
					}
					break;
				case EMeshPropertyBakeType::UVPosition:
					{
						if (DetailUVOverlay && DetailUVOverlay->IsSetTriangle(DetailTriID))
						{
							FVector2d DetailUV;
							DetailUVOverlay->GetTriBaryInterpolate<double>(DetailTriID, &SampleData.DetailBaryCoords.X, &DetailUV.X);
							Color = UVToColor(DetailUV);
						}
					}
					break;
				case EMeshPropertyBakeType::MaterialID:
					{
						if (DetailMesh->Attributes() && DetailMesh->Attributes()->HasMaterialID())
						{
							const FDynamicMeshMaterialAttribute* DetailMaterialIDAttrib = DetailMesh->Attributes()->GetMaterialID();
							const int32 MatID = DetailMaterialIDAttrib->GetValue(DetailTriID);
							Color = LinearColors::SelectColor<FVector3f>(MatID);
						}
						else
						{
							Color = FVector3f(LinearColors::LightPink3f());
						}
					}
					break;
			}
		}
		return Color;
	};


	ResultBuilder = MakeUnique<TImageBuilder<FVector3f>>();
	ResultBuilder->SetDimensions(BakeCache->GetDimensions());

	BakeCache->EvaluateSamples([&](const FVector2i& Coords, const FMeshImageBakingCache::FCorrespondenceSample& Sample)
	{
		FVector3f Color = PropertySampleFunction(Sample);
		ResultBuilder->SetPixel(Coords, Color);
	});

	const FImageOccupancyMap& Occupancy = *BakeCache->GetOccupancyMap();

	for (int64 k = 0; k < Occupancy.GutterTexels.Num(); k++)
	{
		TPair<int64, int64> GutterTexel = Occupancy.GutterTexels[k];
		ResultBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
	}


}