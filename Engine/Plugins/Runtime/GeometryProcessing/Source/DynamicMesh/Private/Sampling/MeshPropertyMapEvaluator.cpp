// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshPropertyMapEvaluator.h"
#include "Sampling/MeshMapBaker.h"
#include "Util/ColorConstants.h"

using namespace UE::Geometry;

void FMeshPropertyMapEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	// Cache data from the baker
	DetailSampler = Baker.GetDetailSampler();
	auto GetDetailNormalMaps = [this](const void* Mesh)
	{
		const FNormalTexture* NormalMap = DetailSampler->GetNormalTextureMap(Mesh);
		if (NormalMap)
		{
			// Require valid normal map and UV layer to enable normal map transfer.
			// Tangents also required if the map is in tangent space.
			const bool bDetailNormalTangentSpace = NormalMap->Get<2>() == IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent; 
			const bool bEnableNormalMapTransfer = DetailSampler->HasUVs(Mesh, NormalMap->Get<1>()) &&
				(!bDetailNormalTangentSpace || DetailSampler->HasTangents(Mesh));
			if (bEnableNormalMapTransfer)
			{
				DetailNormalMaps.Add(Mesh, *NormalMap);
				bHasDetailNormalTextures = true;
			}
		}
	};
	DetailSampler->ProcessMeshes(GetDetailNormalMaps);

	Context.Evaluate = bHasDetailNormalTextures ? &EvaluateSample<true> : &EvaluateSample<false>;
	Context.EvaluateDefault = &EvaluateDefault;
	Context.EvaluateColor = &EvaluateColor;
	Context.EvaluateChannel = &EvaluateChannel;
	Context.EvalData = this;
	Context.AccumulateMode = EAccumulateMode::Add;
	Context.DataLayout = DataLayout();

	if (Property == EMeshPropertyMapType::MaterialID || Property == EMeshPropertyMapType::PolyGroupID)
	{
		Context.AccumulateMode = EAccumulateMode::Overwrite;
	}

	Bounds = DetailSampler->GetBounds();
	for (int32 j = 0; j < 3; ++j)
	{
		if (Bounds.Diagonal()[j] < FMathf::ZeroTolerance)
		{
			Bounds.Min[j] = Bounds.Center()[j] - FMathf::ZeroTolerance;
			Bounds.Max[j] = Bounds.Center()[j] + FMathf::ZeroTolerance;
		}
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DefaultValue = GetDefaultValue(this->Property);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	DefaultValue4f = GetDefaultValue(this->Property);
}

const TArray<FMeshMapEvaluator::EComponents>& FMeshPropertyMapEvaluator::DataLayout() const
{
	static const TArray<EComponents> Layout{ EComponents::Float4 };
	return Layout;
}

FVector4f FMeshPropertyMapEvaluator::GetDefaultValue(const EMeshPropertyMapType InProperty) const
{
	FVector4f Value;
	switch (InProperty)
	{
	case EMeshPropertyMapType::Position:
		Value = PositionToColor(Bounds.Center(), Bounds);
		break;
	case EMeshPropertyMapType::FacetNormal:
		Value = NormalToColor(FVector3f::UnitZ());
		break;
	case EMeshPropertyMapType::Normal:
		Value = NormalToColor(FVector3f::UnitZ());
		break;
	case EMeshPropertyMapType::UVPosition:
		Value = UVToColor(FVector2f::Zero());
		break;
	case EMeshPropertyMapType::MaterialID:
	case EMeshPropertyMapType::PolyGroupID:
		Value = FVector4f(LinearColors::LightPink3f(), 1.0f);
		break;
	case EMeshPropertyMapType::VertexColor:
		Value = FVector4f::One();
		break;
	}
	return Value;
}


template <bool bUseDetailNormalMap>
void FMeshPropertyMapEvaluator::EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	const FMeshPropertyMapEvaluator* Eval = static_cast<FMeshPropertyMapEvaluator*>(EvalData);
	const FVector4f SampleResult = Eval->SampleFunction<bUseDetailNormalMap>(Sample);
	WriteToBuffer(Out, SampleResult);
}

void FMeshPropertyMapEvaluator::EvaluateDefault(float*& Out, void* EvalData)
{
	WriteToBuffer(Out, FVector4f::Zero());
}

void FMeshPropertyMapEvaluator::EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
{
	// TODO: Move property color space transformation from EvaluateSample/Default to here.
	Out = FVector4f(In[0], In[1], In[2], In[3]);
	In += 4;
}

void FMeshPropertyMapEvaluator::EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData)
{
	ensure(false);		// Should not be able to select per-channel evaluation for multi-dimensional properties
	Out = In[0];
	In += 1;
}

template <bool bUseDetailNormalMap>
FVector4f FMeshPropertyMapEvaluator::SampleFunction(const FCorrespondenceSample& SampleData) const
{
	const void* DetailMesh = SampleData.DetailMesh;
	const FVector3d& DetailBaryCoords = SampleData.DetailBaryCoords;
	FVector4f Color = DefaultValue4f;
	const int32 DetailTriID = SampleData.DetailTriID;

	switch (this->Property)
	{
	case EMeshPropertyMapType::Position:
	{
		const FVector3d Position = DetailSampler->TriBaryInterpolatePoint(DetailMesh, DetailTriID, DetailBaryCoords);
		Color = PositionToColor(Position, Bounds);
	}
	break;
	case EMeshPropertyMapType::FacetNormal:
	{
		const FVector3d FacetNormal = DetailSampler->GetTriNormal(DetailMesh, DetailTriID);
		Color = NormalToColor(FVector3f(FacetNormal));
	}
	break;
	case EMeshPropertyMapType::Normal:
	{
		FVector3f DetailNormal;
		if (DetailSampler->TriBaryInterpolateNormal(DetailMesh, DetailTriID, DetailBaryCoords, DetailNormal))
		{
			Normalize(DetailNormal);

			if constexpr (bUseDetailNormalMap)
			{
				const TImageBuilder<FVector4f>* DetailNormalMap = nullptr;
				int DetailNormalUVLayer = 0;
				IMeshBakerDetailSampler::EBakeDetailNormalSpace DetailNormalSpace = IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent;
				const FNormalTexture* DetailNormalTexture = DetailNormalMaps.Find(DetailMesh);
				if (DetailNormalTexture)
				{
					Tie(DetailNormalMap, DetailNormalUVLayer, DetailNormalSpace) = *DetailNormalTexture;
				}

				if (DetailNormalMap)
				{
					FVector3d DetailTangentX, DetailTangentY;
					DetailSampler->TriBaryInterpolateTangents(
						DetailMesh,
						SampleData.DetailTriID,
						SampleData.DetailBaryCoords,
						DetailTangentX, DetailTangentY);
	
					FVector2f DetailUV;
					DetailSampler->TriBaryInterpolateUV(DetailMesh, DetailTriID, DetailBaryCoords, DetailNormalUVLayer, DetailUV);
					const FVector4f DetailNormalColor4 = DetailNormalMap->BilinearSampleUV<float>(FVector2d(DetailUV), FVector4f(0, 0, 0, 1));

					// Map color space [0,1] to normal space [-1,1]
					const FVector3f DetailNormalColor(DetailNormalColor4.X, DetailNormalColor4.Y, DetailNormalColor4.Z);
					FVector3f DetailNormalObjectSpace = (DetailNormalColor * 2.0f) - FVector3f::One();
					// Ideally this branch could be made compile time. Unfortunately since each mesh could have its
					// own source normal map each with their own normal space, this branch must be resolved at runtime.
					if (DetailNormalSpace == IMeshBakerDetailSampler::EBakeDetailNormalSpace::Tangent) //-V547
					{
						// Convert detail normal tangent space to object space
						const FVector3f DetailNormalTangentSpace = DetailNormalObjectSpace;
						DetailNormalObjectSpace = DetailNormalTangentSpace.X * FVector3f(DetailTangentX) + DetailNormalTangentSpace.Y * FVector3f(DetailTangentY) + DetailNormalTangentSpace.Z * DetailNormal;
					}
					Normalize(DetailNormalObjectSpace);
					DetailNormal = DetailNormalObjectSpace;
				}
			}

			Color = NormalToColor(DetailNormal);
		}
	}
	break;
	case EMeshPropertyMapType::UVPosition:
	{
		FVector2f DetailUV;
		if (DetailSampler->TriBaryInterpolateUV(DetailMesh, DetailTriID, DetailBaryCoords, 0, DetailUV))
		{
			Color = UVToColor(DetailUV);
		}
	}
	break;
	case EMeshPropertyMapType::MaterialID:
	{
		const int32 MatID = DetailSampler->GetMaterialID(DetailMesh, DetailTriID);
		Color = LinearColors::SelectColor<FVector3f>(MatID);
	}
	break;
	case EMeshPropertyMapType::VertexColor:
	{
		FVector4f DetailColor;
		if (DetailSampler->TriBaryInterpolateColor(DetailMesh, DetailTriID, DetailBaryCoords, DetailColor))
		{
			Color = DetailColor;
		}
	}
	break;
	case EMeshPropertyMapType::PolyGroupID:
	{
		const int32 GroupID = DetailSampler->GetPolyGroupID(DetailMesh, DetailTriID);
		Color = LinearColors::SelectColor<FVector4f>(GroupID);
	}
	break;
	}
	return Color;
}


