// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshResampleImageBaker.h"

using namespace UE::Geometry;

void FMeshResampleImageBaker::Bake()
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	check(BakeCache);
	const FDynamicMesh3* DetailMesh = BakeCache->GetDetailMesh();

	check(DetailUVOverlay);

	FVector4f DefaultValue(0, 0, 0, 1.0);

	auto PropertySampleFunction = [&](const FMeshImageBakingCache::FCorrespondenceSample& SampleData)
	{
		FVector4f Color = DefaultValue;
		int32 DetailTriID = SampleData.DetailTriID;
		if (DetailMesh->IsTriangle(SampleData.DetailTriID) && DetailUVOverlay)
		{
			FVector2d DetailUV;
			DetailUVOverlay->GetTriBaryInterpolate<double>(DetailTriID, &SampleData.DetailBaryCoords.X, &DetailUV.X);

			Color = SampleFunction(DetailUV);
		}
		return Color;
	};

	ResultBuilder = MakeUnique<TImageBuilder<FVector4f>>();
	ResultBuilder->SetDimensions(BakeCache->GetDimensions());
	ResultBuilder->Clear(DefaultColor);

	BakeCache->EvaluateSamples([&](const FVector2i& Coords, const FMeshImageBakingCache::FCorrespondenceSample& Sample)
	{
		FVector4f Color = PropertySampleFunction(Sample);
		ResultBuilder->SetPixel(Coords, Color);
	});

	const FImageOccupancyMap& Occupancy = *BakeCache->GetOccupancyMap();

	for (int64 k = 0; k < Occupancy.GutterTexels.Num(); k++)
	{
		TPair<int64, int64> GutterTexel = Occupancy.GutterTexels[k];
		ResultBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
	}

}




void FMeshMultiResampleImageBaker::InitResult()
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	ResultBuilder = MakeUnique<TImageBuilder<FVector4f>>();
	ResultBuilder->SetDimensions(BakeCache->GetDimensions());
	ResultBuilder->Clear(DefaultColor);
}


void FMeshMultiResampleImageBaker::BakeMaterial(int32 MaterialID)
{
	const FMeshImageBakingCache* BakeCache = GetCache();
	if (!ensure(BakeCache))
	{
		return;
	}
	const FDynamicMesh3* DetailMesh = BakeCache->GetDetailMesh();
	if (!ensure(DetailMesh))
	{
		return;
	}
	const FDynamicMeshMaterialAttribute* DetailMaterialIDAttrib = DetailMesh->Attributes()->GetMaterialID();
	if (!ensure(DetailMaterialIDAttrib))
	{
		return;
	}
	if (!ensure(DetailUVOverlay))
	{
		return;
	}

	BakeCache->EvaluateSamples([this, DetailMesh, DetailMaterialIDAttrib, MaterialID](const FVector2i& Coords, const FMeshImageBakingCache::FCorrespondenceSample& Sample)
	{
		int32 DetailTriID = Sample.DetailTriID;

		if (DetailMesh->IsTriangle(DetailTriID))
		{
			if (DetailMaterialIDAttrib->GetValue(DetailTriID) == MaterialID)
			{
				FVector2d DetailUV;
				DetailUVOverlay->GetTriBaryInterpolate<double>(DetailTriID, &Sample.DetailBaryCoords.X, &DetailUV.X);
				FVector4f Color = SampleFunction(DetailUV);
				ResultBuilder->SetPixel(Coords, Color);
			}
			// otherwise leave the pixel alone
		}
	});

	const FImageOccupancyMap& Occupancy = *BakeCache->GetOccupancyMap();

	for (int64 k = 0; k < Occupancy.GutterTexels.Num(); k++)
	{
		TPair<int64, int64> GutterTexel = Occupancy.GutterTexels[k];
		ResultBuilder->CopyPixel(GutterTexel.Value, GutterTexel.Key);
	}
}


void FMeshMultiResampleImageBaker::Bake()
{
	InitResult();

	// Write into the sample buffer, separate pass for each material ID
	for (TPair< int32, TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>>& MaterialTexture : MultiTextures)
	{
		int32 MaterialID = MaterialTexture.Key;
		TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe> TextureImage = MaterialTexture.Value;

		this->SampleFunction = [&TextureImage](FVector2d UVCoord) 
		{
			return TextureImage->BilinearSampleUV<float>(UVCoord, FVector4f(0, 0, 0, 1));
		};

		BakeMaterial(MaterialID);
	}

}
