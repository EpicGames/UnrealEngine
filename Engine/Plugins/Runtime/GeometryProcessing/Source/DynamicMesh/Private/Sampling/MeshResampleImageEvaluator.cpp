// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshResampleImageEvaluator.h"
#include "Sampling/MeshMapBaker.h"

using namespace UE::Geometry;

//
// FMeshResampleImage Evaluator
//

void FMeshResampleImageEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	Context.Evaluate = &EvaluateSample;
	Context.EvaluateDefault = &EvaluateDefault;
	Context.EvaluateColor = &EvaluateColor;
	Context.EvalData = this;
	Context.AccumulateMode = EAccumulateMode::Add;
	Context.DataLayout = DataLayout();

	// Cache data from the baker
	DetailSampler = Baker.GetDetailSampler();
}

const TArray<FMeshMapEvaluator::EComponents>& FMeshResampleImageEvaluator::DataLayout() const
{
	static const TArray<FMeshMapEvaluator::EComponents> Layout{ EComponents::Float4 };
	return Layout;
}

void FMeshResampleImageEvaluator::EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	const FMeshResampleImageEvaluator* Eval = static_cast<FMeshResampleImageEvaluator*>(EvalData);
	const FVector4f SampleResult = Eval->ImageSampleFunction(Sample);
	WriteToBuffer(Out, SampleResult);
}

void FMeshResampleImageEvaluator::EvaluateDefault(float*& Out, void* EvalData)
{
	WriteToBuffer(Out, FVector4f(0.0f, 0.0f, 0.0f, 1.0f));
}

void FMeshResampleImageEvaluator::EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
{
	Out = FVector4f(In[0], In[1], In[2], In[3]);
	In += 4;
}

void FMeshResampleImageEvaluator::EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData)
{
	Out = In[0];
	In += 1;
}

FVector4f FMeshResampleImageEvaluator::ImageSampleFunction(const FCorrespondenceSample& SampleData) const
{
	const void* DetailMesh = SampleData.DetailMesh;
	const int32 DetailTriID = SampleData.DetailTriID;
	const TImageBuilder<FVector4f>* TextureImage = nullptr;
	int DetailUVLayer = 0;
	const IMeshBakerDetailSampler::FBakeDetailTexture* ColorMap = DetailSampler->GetTextureMap(DetailMesh);
	if (ColorMap)
	{
		Tie(TextureImage, DetailUVLayer) = *ColorMap;
	}

	FVector4f Color(0, 0, 0, 1);
	if (TextureImage)
	{
		FVector2f DetailUV;
		DetailSampler->TriBaryInterpolateUV(DetailMesh, DetailTriID, SampleData.DetailBaryCoords, DetailUVLayer, DetailUV);
		Color = TextureImage->BilinearSampleUV<float>(FVector2d(DetailUV), FVector4f(0, 0, 0, 1));
	}
	return Color;
}

//
// FMeshMultiResampleImage Evaluator
//

void FMeshMultiResampleImageEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	Context.Evaluate = &EvaluateSampleMulti;
	Context.EvaluateDefault = &EvaluateDefault;
	Context.EvaluateColor = &EvaluateColor;
	Context.EvaluateChannel = &EvaluateChannel;
	Context.EvalData = this;
	Context.AccumulateMode = EAccumulateMode::Add;
	Context.DataLayout = DataLayout();

	// Cache data from baker
	DetailSampler = Baker.GetDetailSampler();
	NumMultiTextures = MultiTextures.Num();
}

void FMeshMultiResampleImageEvaluator::EvaluateSampleMulti(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	const FMeshMultiResampleImageEvaluator* Eval = static_cast<FMeshMultiResampleImageEvaluator*>(EvalData);
	const FVector4f SampleResult = Eval->ImageSampleFunction(Sample);
	WriteToBuffer(Out, SampleResult);
}

FVector4f FMeshMultiResampleImageEvaluator::ImageSampleFunction(const FCorrespondenceSample& Sample) const
{
	FVector4f Color(0.0f, 0.0f, 0.0f, 1.0f);
	const void* DetailMesh = Sample.DetailMesh;
	const int32 DetailTriID = Sample.DetailTriID;
	const int32 MaterialID = DetailSampler->GetMaterialID(DetailMesh, DetailTriID);

	// TODO: Can we assume that the MaterialIDs are always valid?
	if (MaterialID >= 0 && MaterialID < NumMultiTextures)
	{
		if (const TImageBuilder<FVector4f>* TextureImage = MultiTextures[MaterialID].Get())
		{
			FVector2f DetailUV;
			DetailSampler->TriBaryInterpolateUV(DetailMesh, DetailTriID, Sample.DetailBaryCoords, DetailUVLayer, DetailUV);
			Color = TextureImage->BilinearSampleUV<float>(FVector2d(DetailUV), FVector4f(0, 0, 0, 1));
		}
	}
	return Color;
}
