// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshSurfaceSampler.h"

namespace UE
{
namespace Geometry
{


void FMeshSurfaceUVSampler::Initialize(
	const FDynamicMesh3* MeshIn,
	const FDynamicMeshUVOverlay* UVOverlayIn,
	EMeshSurfaceSamplerQueryType QueryTypeIn)
{
	Mesh = MeshIn;
	UVOverlay = UVOverlayIn;
	QueryType = QueryTypeIn;

	if (QueryType == EMeshSurfaceSamplerQueryType::UVOnly)
	{
		InitializeUVMeshSpatial();
	}
}




bool FMeshSurfaceUVSampler::QuerySampleInfo(const FVector2d& UV, FMeshUVSampleInfo& SampleInfo)
{
	check(QueryType == EMeshSurfaceSamplerQueryType::UVOnly);
	check(bUVMeshSpatialInitialized);

	FRay3d HitRay(FVector3d(UV.X, UV.Y, 100.0), -FVector3d::UnitZ());
	const int32 UVTriangleID = UVMeshSpatial.FindNearestHitTriangle(HitRay);

	return QuerySampleInfo(UVTriangleID, UV, SampleInfo);
}




bool FMeshSurfaceUVSampler::QuerySampleInfo(int32 UVTriangleID, const FVector2d& UV, FMeshUVSampleInfo& SampleInfo)
{
	SampleInfo.TriangleIndex = UVTriangleID;
	if (Mesh->IsTriangle(SampleInfo.TriangleIndex) == false)
	{
		return false;
	}
	check(UVOverlay->IsSetTriangle(SampleInfo.TriangleIndex));

	SampleInfo.MeshVertices = Mesh->GetTriangle(SampleInfo.TriangleIndex);
	SampleInfo.Triangle3D = FTriangle3d(
		Mesh->GetVertex(SampleInfo.MeshVertices.A),
		Mesh->GetVertex(SampleInfo.MeshVertices.B),
		Mesh->GetVertex(SampleInfo.MeshVertices.C));

	SampleInfo.UVVertices = UVOverlay->GetTriangle(SampleInfo.TriangleIndex);
	SampleInfo.TriangleUV = FTriangle2d(
		(FVector2d)UVOverlay->GetElement(SampleInfo.UVVertices.A),
		(FVector2d)UVOverlay->GetElement(SampleInfo.UVVertices.B),
		(FVector2d)UVOverlay->GetElement(SampleInfo.UVVertices.C));

	SampleInfo.BaryCoords = SampleInfo.TriangleUV.GetBarycentricCoords(UV);
	SampleInfo.SurfacePoint = Mesh->GetTriBaryPoint(
		SampleInfo.TriangleIndex,
		SampleInfo.BaryCoords.X,
		SampleInfo.BaryCoords.Y,
		SampleInfo.BaryCoords.Z);

	return true;
}




void FMeshSurfaceUVSampler::InitializeUVMeshSpatial()
{
	if (bUVMeshSpatialInitialized)
	{
		return;
	}

	check(UVOverlay);

	UVMeshAdapter.Mesh = Mesh;
	UVMeshAdapter.UV = UVOverlay;
	UVMeshSpatial.SetMesh(&UVMeshAdapter, true);

	bUVMeshSpatialInitialized = true;
}


} // end namespace Geometry
} // end namespace UE

