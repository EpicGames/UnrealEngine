// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshUVShellMapEvaluator.h"
#include "Sampling/MeshMapBaker.h"

using namespace UE::Geometry;

void FMeshUVShellMapEvaluator::Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context)
{
	// Cache data from the baker
	Context.Evaluate = &EvaluateSample;
	Context.EvaluateDefault = &EvaluateDefault;
	Context.EvaluateColor = &EvaluateColor;
	Context.EvaluateChannel = &EvaluateChannel;
	Context.EvalData = this;
	Context.AccumulateMode = EAccumulateMode::Add;
	Context.DataLayout = DataLayout();
	
	DetailSampler = Baker.GetDetailSampler();
}

const TArray<FMeshMapEvaluator::EComponents>& FMeshUVShellMapEvaluator::DataLayout() const
{
	static const TArray<EComponents> Layout{ EComponents::Float4 };
	return Layout;
}

void FMeshUVShellMapEvaluator::EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
{
	const FMeshUVShellMapEvaluator* Eval = static_cast<FMeshUVShellMapEvaluator*>(EvalData);
	const FVector4f SampleResult = Eval->SampleFunction(Sample);
	WriteToBuffer(Out, SampleResult);
}

void FMeshUVShellMapEvaluator::EvaluateDefault(float*& Out, void* EvalData)
{
	const FMeshUVShellMapEvaluator* Eval = static_cast<FMeshUVShellMapEvaluator*>(EvalData);
	WriteToBuffer(Out, Eval->BackgroundColor);
}

void FMeshUVShellMapEvaluator::EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
{
	Out = FVector4f(In[0], In[1], In[2], In[3]);
	In += 4;
}

void FMeshUVShellMapEvaluator::EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData)
{
	ensure(false);		// Should not be able to select per-channel evaluation for multi-dimensional properties
	Out = In[0];
	In += 1;
}

FVector4f FMeshUVShellMapEvaluator::SampleFunction(const FCorrespondenceSample& SampleData) const
{
	const void* DetailMesh = SampleData.DetailMesh;
	const FVector3d& DetailBaryCoords = SampleData.DetailBaryCoords;
	const int32 DetailTriID = SampleData.DetailTriID;
	FVector4f Color = BackgroundColor;

	if (DetailSampler->IsTriangle(DetailMesh, DetailTriID))
	{
		// Compute barycentric/areal coords in UV space.
		FVector2f UV0, UV1, UV2;
		DetailSampler->GetTriUVs(DetailMesh, DetailTriID, UVLayer, UV0, UV1, UV2);

		const FTriangle2f UVTri(UV0, UV1, UV2);
		const FVector3f UVArealCoords = FVector3f(DetailBaryCoords) * UVTri.Area();

		// Barycentric --> Trilinear to identify min distance to tri edges.
		const float M20 = (UV2-UV0).Length();
		const float M01 = (UV0-UV1).Length();
		const float M12 = (UV1-UV2).Length();

		const FVector3f Trilinear(
			2.0f * UVArealCoords.X / M20,
			2.0f * UVArealCoords.Y / M01,
			2.0f * UVArealCoords.Z / M12);
		const float MinTrilinear = FMath::Min3(Trilinear.X, Trilinear.Y, Trilinear.Z);

		// Check edge distance in texel space (assumes square texels).
		const float MinTexelDist = MinTrilinear / static_cast<float>(TexelSize.X);
		Color = MinTexelDist < WireframeThickness ? WireframeColor : ShellColor;
	}

	return Color;
}


