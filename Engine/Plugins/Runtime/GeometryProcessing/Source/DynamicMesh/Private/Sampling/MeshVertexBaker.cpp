// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sampling/MeshVertexBaker.h"
#include "Sampling/MeshBakerCommon.h"
#include "DynamicMesh/MeshNormals.h"
#include "ProfilingDebugging/ScopedTimers.h"

using namespace UE::Geometry;

FMeshConstantMapEvaluator FMeshVertexBaker::ZeroEvaluator(0.0f);
FMeshConstantMapEvaluator FMeshVertexBaker::OneEvaluator(1.0f);

void FMeshVertexBaker::Bake()
{
	TotalBakeDuration = 0.0;
	FScopedDurationTimer Timer(TotalBakeDuration);
	
	if (!ensure(TargetMesh && TargetMesh->HasAttributes() && TargetMesh->Attributes()->HasPrimaryColors()) ||
		!ensure(DetailSampler))
	{
		return;
	}

	// Convert Bake mode into internal list of bakers.
	Bakers.Reset();
	if (BakeMode == EBakeMode::RGBA)
	{
		FMeshMapEvaluator* Evaluator = ColorEvaluator.Get();
		Bakers.Add(Evaluator ? Evaluator : &ZeroEvaluator);
		BakeInternal = &BakeImpl<EBakeMode::RGBA>;
	}
	else // Mode == EBakeMode::PerChannel
	{
		for (int Idx = 0; Idx < 4; ++Idx)
		{
			// For alpha channel, default to 1.0, otherwise 0.0.
			FMeshMapEvaluator* DefaultEvaluator = Idx == 3 ? &OneEvaluator : &ZeroEvaluator;
			FMeshMapEvaluator* Evaluator = ChannelEvaluators[Idx].Get();
			Bakers.Add(Evaluator ? Evaluator : DefaultEvaluator);
		}
		BakeInternal = &BakeImpl<EBakeMode::PerChannel>;
	}

	const int NumBakers = Bakers.Num();
	if (NumBakers == 0)
	{
		return;
	}

	// Initialize BakeContext(s) and BakeDefaults
	BakeDefaults = FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
	float* DefaultBufferPtr = &BakeDefaults[0];
	BakeContexts.Reset();
	BakeContexts.SetNum(NumBakers);
	BakeSampleBufferSize = 0;
	for (int Idx = 0; Idx < NumBakers; ++Idx)
	{
		Bakers[Idx]->Setup(*this, BakeContexts[Idx]);

		for (FMeshMapEvaluator::EComponents Components : BakeContexts[Idx].DataLayout)
		{
			BakeSampleBufferSize += (int) Components;
		}
		if (!ensure(BakeSampleBufferSize <= 4))
		{
			return;
		}

		BakeContexts[Idx].EvaluateDefault(DefaultBufferPtr, BakeContexts[Idx].EvalData);
	}

	// Initialize BakeResult to unique vertex color elements
	const FDynamicMeshColorOverlay* ColorOverlay = TargetMesh->Attributes()->PrimaryColors();
	const int NumColors = ColorOverlay->ElementCount();
	Dimensions = FImageDimensions(NumColors, 1);

	BakeResult = MakeUnique<TImageBuilder<FVector4f>>();
	BakeResult->SetDimensions(Dimensions);
	BakeResult->Clear(BakeDefaults);

	BakeInternal(this);
}

const TImageBuilder<FVector4f>* FMeshVertexBaker::GetBakeResult() const
{
	return BakeResult.Get();
}

template<FMeshVertexBaker::EBakeMode ComputeMode>
void FMeshVertexBaker::BakeImpl(void* Data)
{
	if (!ensure(Data))
	{
		return;
	}

	FMeshVertexBaker* Baker = static_cast<FMeshVertexBaker*>(Data);
	
	ECorrespondenceStrategy UseStrategy = Baker->CorrespondenceStrategy;
	if (UseStrategy == ECorrespondenceStrategy::Identity)
	{
		bool bIsIdentity = true;
		int NumDetailMeshes = 0;
		auto CheckIdentity = [Baker, &bIsIdentity, &NumDetailMeshes](const void* DetailMesh)
		{
			// When the mesh pointers differ, loosely compare the meshes as a sanity check.
			// TODO: Expose additional comparison metrics on the detail sampler when the mesh pointers differ.
			bIsIdentity = bIsIdentity && (Baker->TargetMesh == DetailMesh || Baker->TargetMesh->TriangleCount() == Baker->DetailSampler->GetTriangleCount(DetailMesh));
			++NumDetailMeshes;
		};
		Baker->DetailSampler->ProcessMeshes(CheckIdentity);
		if (!ensure(bIsIdentity && NumDetailMeshes == 1))
		{
			// Identity strategy requires there to be only one mesh that is the same
			// as the target mesh.
			UseStrategy = ECorrespondenceStrategy::NearestPoint;
		}
	}
	
	const FDynamicMesh3* Mesh = Baker->TargetMesh;
	const FDynamicMeshColorOverlay* ColorOverlay = Baker->TargetMesh->Attributes()->PrimaryColors();
	const FDynamicMeshNormalOverlay* NormalOverlay = Baker->TargetMesh->Attributes()->PrimaryNormals();

	// TODO: Refactor into TMeshSurfaceSampler class (future vertex bake enhancements will require non-UV based surface sampling)
	auto SampleSurface = [Baker, Mesh, ColorOverlay, NormalOverlay, UseStrategy](int32 ElementIdx,
	                                                                FMeshMapEvaluator::FCorrespondenceSample& ValueOut)
	{
		if (!ColorOverlay->IsElement(ElementIdx))
		{
			return;
		}
	
		const int32 VertexId = ColorOverlay->GetParentVertex(ElementIdx);

		// Compute ray direction
		FVector SurfaceNormal = FVector::Zero();
		TArray<int> ColorElementTris;
		ColorOverlay->GetElementTriangles(ElementIdx, ColorElementTris);
		for (const int TriId : ColorElementTris)
		{
			FVector3d TriNormal, TriCentroid; double TriArea;
			Mesh->GetTriInfo(TriId, TriNormal, TriArea, TriCentroid);
			FVector3d NormalWeights = FMeshNormals::GetVertexWeightsOnTriangle(Mesh, TriId, TriArea, true, true);
			const FIndex3i TriVerts = Mesh->GetTriangle(TriId);
			const int TriVertexId = IndexUtil::FindTriIndex(VertexId, TriVerts);
			
			if (NormalOverlay->IsSetTriangle(TriId))
			{
				FVector3f Normal;
				NormalOverlay->GetElementAtVertex(TriId, VertexId, Normal);
				SurfaceNormal += NormalWeights[TriVertexId] * FVector(Normal);
			}
			else
			{
				SurfaceNormal += NormalWeights[TriVertexId] * TriNormal;
			}
		}
		Normalize(SurfaceNormal);

		// Compute surface point and barycentric coords
		const FVector3d SurfacePoint = Mesh->GetVertex(VertexId);
		const int32 TriangleIndex = ColorElementTris[0];
		const FIndex3i TriVerts = Mesh->GetTriangle(TriangleIndex);
		const int TriVertexId = IndexUtil::FindTriIndex(VertexId, TriVerts);
		FVector3d BaryCoords = FVector3d::Zero();
		BaryCoords[TriVertexId] = 1.0;

		ValueOut.BaseSample.TriangleIndex = TriangleIndex;
		ValueOut.BaseSample.SurfacePoint = SurfacePoint;
		ValueOut.BaseSample.BaryCoords = BaryCoords;
		ValueOut.DetailMesh = nullptr;
		ValueOut.DetailTriID = FDynamicMesh3::InvalidID;
		ValueOut.BaseNormal = SurfaceNormal;

		if (UseStrategy == ECorrespondenceStrategy::Identity)
		{
			ValueOut.DetailMesh = Mesh;
			ValueOut.DetailTriID = TriangleIndex;
			ValueOut.DetailBaryCoords = BaryCoords;
		}
		else if (UseStrategy == ECorrespondenceStrategy::NearestPoint)
		{
			ValueOut.DetailMesh = GetDetailMeshTrianglePoint_Nearest(Baker->DetailSampler, SurfacePoint,
			                                   ValueOut.DetailTriID, ValueOut.DetailBaryCoords);
		}
		else // Fall back to raycast strategy
		{
			const double SampleThickness = Baker->GetProjectionDistance(); // could modulate w/ a map here...

			// Find detail mesh triangle point
			ValueOut.DetailMesh = GetDetailMeshTrianglePoint_Raycast(Baker->DetailSampler, SurfacePoint, SurfaceNormal,
			                                   ValueOut.DetailTriID, ValueOut.DetailBaryCoords, SampleThickness,
			                                   (UseStrategy == ECorrespondenceStrategy::RaycastStandardThenNearest));
		}
	};

	// Perform bake
	constexpr int32 TileWidth = 1024;
	constexpr int32 TileHeight = 1;
	const FImageTiling Tiles(Baker->Dimensions, TileWidth, TileHeight);
	const int32 NumTiles = Tiles.Num();
	const int NumBakers = Baker->Bakers.Num();
	ParallelFor(NumTiles, [Baker, &Tiles, TileWidth, NumBakers, &SampleSurface](const int32 TileIdx)
	{
		if (Baker->CancelF())
		{
			return;
		}
		
		const FImageTile Tile = Tiles.GetTile(TileIdx);
		const int Width = Tile.GetWidth();
		for (int32 Idx = 0; Idx < Width; ++Idx)
		{
			const int ElemIdx = TileIdx * TileWidth + Idx;
			FMeshMapEvaluator::FCorrespondenceSample Sample;
			SampleSurface(ElemIdx, Sample);

			if (!Sample.DetailMesh || !Baker->DetailSampler->IsTriangle(Sample.DetailMesh, Sample.DetailTriID))
			{
				continue;
			}

			FVector4f& Pixel = Baker->BakeResult->GetPixel(ElemIdx);
			float* BufferPtr = &Pixel[0];
			for (int32 BakerIdx = 0; BakerIdx < NumBakers; ++BakerIdx)
			{
				Baker->BakeContexts[BakerIdx].Evaluate(BufferPtr, Sample, Baker->BakeContexts[BakerIdx].EvalData);
			}

			// TODO: Use a separate buffer rather than R/W from the same pixel.
			BufferPtr = &Pixel[0];
			for (int32 BakerIdx = 0; BakerIdx < NumBakers; ++BakerIdx)
			{
				if constexpr (ComputeMode == EBakeMode::RGBA)
				{
					Baker->BakeContexts[BakerIdx].EvaluateColor(0, BufferPtr, Pixel, Baker->BakeContexts[BakerIdx].EvalData);
				}
				else if constexpr (ComputeMode == EBakeMode::PerChannel)
				{
					// TODO: Are we guaranteed to have the same NumBakers as number of Pixel channels?
					ensure(NumBakers == 4);
					Baker->BakeContexts[BakerIdx].EvaluateChannel(0, BufferPtr, Pixel[BakerIdx], Baker->BakeContexts[BakerIdx].EvalData);
				}
			}
		}
	}, !Baker->bParallel ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);
}
