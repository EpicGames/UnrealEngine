// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selections/GeometrySelection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Algo/Find.h"

using namespace UE::Geometry;

void FGeometrySelectionEditor::Initialize(FGeometrySelection* TargetSelectionIn, const FGeometrySelectionHitQueryConfig& QueryConfigIn, bool bEnableTopologyIDFilteringIn)
{
	check(TargetSelectionIn != nullptr);
	TargetSelection = TargetSelectionIn;
	UpdateQueryConfig(QueryConfigIn, bEnableTopologyIDFilteringIn);
}

void FGeometrySelectionEditor::Initialize(FGeometrySelection* TargetSelectionIn, bool bEnableTopologyIDFilteringIn)
{
	check(TargetSelectionIn != nullptr);
	TargetSelection = TargetSelectionIn;
	FGeometrySelectionHitQueryConfig TmpQueryConfig{TargetSelection->TopologyType, TargetSelection->ElementType, false};
	UpdateQueryConfig(TmpQueryConfig, bEnableTopologyIDFilteringIn);
}


void FGeometrySelectionEditor::UpdateQueryConfig(const FGeometrySelectionHitQueryConfig& NewConfig, bool bEnableTopologyIDFilteringIn)
{
	check(NewConfig.ElementType == TargetSelection->ElementType);
	check(NewConfig.TopologyType == TargetSelection->TopologyType);
	QueryConfig = NewConfig;
	bEnableTopologyIDFiltering = bEnableTopologyIDFilteringIn;
}

void FGeometrySelectionEditor::ClearSelection(FGeometrySelectionDelta& DeltaOut)
{
	for (uint64 ID : TargetSelection->Selection)
	{
		DeltaOut.Removed.Add(ID);
	}

	TargetSelection->Reset();
}


bool FGeometrySelectionEditor::IsSelected(uint64 ID) const
{
	if (bEnableTopologyIDFiltering)
	{
		bool bFound;
		RemapToExistingTopologyID(ID, bFound);
		return bFound;
	}
	return TargetSelection->Selection.Contains(ID);
}

bool FGeometrySelectionEditor::RemoveFromSelection(uint64 ID)
{
	if (bEnableTopologyIDFiltering)
	{
		bool bFound;
		ID = RemapToExistingTopologyID(ID, bFound);
		if (!bFound)
		{
			return false;
		}
	}
	int32 NumRemoved = TargetSelection->Selection.Remove(ID);
	checkSlow(NumRemoved == 0 || NumRemoved == 1);
	return (NumRemoved > 0);
}

uint64 FGeometrySelectionEditor::RemapToExistingTopologyID(uint64 ID, bool& bFound) const
{
	bFound = false;
	uint32 TopologyID = FGeoSelectionID(ID).TopologyID;
	const uint64* Found = Algo::FindByPredicate(TargetSelection->Selection, [&](uint64 Item)
	{
		return FGeoSelectionID(Item).TopologyID == TopologyID;
	});
	if (Found != nullptr)
	{
		bFound = true;
		return *Found;
	}
	return ID;
}

bool FGeometrySelectionEditor::Replace(const FGeometrySelection& NewSelection, FGeometrySelectionDelta& DeltaOut)
{
	// currently hard-enforcing this...maybe we can allow switches? 
	if (ensure(TargetSelection->ElementType == NewSelection.ElementType) == false ||
		ensure(TargetSelection->TopologyType == NewSelection.TopologyType) == false)
	{
		return false;
	}

	for (uint64 ID : TargetSelection->Selection)
	{
		DeltaOut.Removed.Add(ID);
	}
	TargetSelection->Selection.Reset();

	for (uint64 ID : NewSelection.Selection)
	{
		TargetSelection->Selection.Add(ID);
		DeltaOut.Added.Add(ID);
	}

	return true;
}