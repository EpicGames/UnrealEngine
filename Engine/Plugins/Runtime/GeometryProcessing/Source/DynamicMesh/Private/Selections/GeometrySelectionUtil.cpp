// Copyright Epic Games, Inc. All Rights Reserved.


#include "Selections/GeometrySelectionUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/ColliderMesh.h"
#include "DynamicMesh/MeshNormals.h"
#include "TriangleTypes.h"
#include "SegmentTypes.h"
#include "GroupTopology.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshEdgeSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "Algo/Find.h"
#include "Selections/MeshFaceSelection.h"


namespace GeometrySelectionUtilLocals
{

// Return an integer in the range [0,5] which can be used to look up a handling function based on the Selection type
int GetSelectionTypeAsIndex(const UE::Geometry::FGeometrySelection& Selection)
{
	const int Index = ((int)Selection.ElementType / 2) + ((int)Selection.TopologyType / 2) * 3;
	checkSlow(Index >= 0);
	checkSlow(Index <= 5);
	return Index;
}

// We don't currently have an overload of EnumerateSelectionTriangles that takes a FGroupTopology
// instead of FPolygroupSet. If we build out the below version to support the other selection
// types (vertex, edge), we might want to expose it.
/**
 * Given a selection with elements of type EGeometryElementType::Face, call TriangleFunc on
 * each triangle of the selection.
 * 
 * @param GroupTopology must not be null if Selection is EGeometryTopologyType::Polygroup
 */
bool EnumerateFaceElementSelectionTriangles(
	const UE::Geometry::FGeometrySelection& Selection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FGroupTopology* GroupTopology,
	TFunctionRef<void(int32)> TriangleFunc)
{
	using namespace UE::Geometry;

	if (!ensure(Selection.ElementType == EGeometryElementType::Face))
	{
		return false;
	}

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		if (!ensure(GroupTopology))
		{
			return false;
		}

		for (uint64 EncodedID : Selection.Selection)
		{
			FGeoSelectionID GroupTriID(EncodedID);
			int32 SeedTriangleID = (int32)GroupTriID.GeometryID;
			int32 GroupID = (int32)GroupTriID.TopologyID;
			if (Mesh.IsTriangle(SeedTriangleID))
			{
				for (int32 Tid : GroupTopology->GetGroupTriangles(GroupID))
				{
					TriangleFunc(Tid);
				}
			}
		}
	}
	else if (Selection.TopologyType == EGeometryTopologyType::Triangle)
	{
		for (uint64 TriangleID : Selection.Selection)
		{
			if (Mesh.IsTriangle((int32)TriangleID))
			{
				TriangleFunc((int32)TriangleID);
			}
		}
	}
	else
	{
		return ensure(false);
	}

	return true;
}

/**
 * Given a selection with elements of type EGeometryElementType::Edge, call EdgeFunc on
 * each mesh edge (with the Eid passed in) of the selection.
 *
 * @param GroupTopology must not be null if Selection is EGeometryTopologyType::Polygroup
 */
bool EnumerateEdgeElementSelectionEdges(
	const UE::Geometry::FGeometrySelection& Selection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FGroupTopology* GroupTopology,
	TFunctionRef<void(uint32)> EdgeFunc)
{
	using namespace UE::Geometry;

	if (!ensure(Selection.ElementType == EGeometryElementType::Edge))
	{
		return false;
	}

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		if (!ensure(GroupTopology))
		{
			return false;
		}

		for (uint64 EncodedID : Selection.Selection)
		{
			FMeshTriEdgeID TriEdgeID(FGeoSelectionID(EncodedID).GeometryID);
			int32 SeedEdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if (Mesh.IsEdge(SeedEdgeID))
			{
				int32 GroupEdgeID = GroupTopology->FindGroupEdgeID(SeedEdgeID);
				for (int32 Eid : GroupTopology->GetGroupEdgeEdges(GroupEdgeID))
				{
					EdgeFunc(Eid);
				}
			}
		}
	}
	else if (Selection.TopologyType == EGeometryTopologyType::Triangle)
	{
		for (uint64 EncodedID : Selection.Selection)
		{
			FMeshTriEdgeID TriEdgeID(FGeoSelectionID(EncodedID).GeometryID);
			int32 EdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if (Mesh.IsEdge(EdgeID))
			{
				EdgeFunc(EdgeID);
			}
		}
	}
	else
	{
		return ensure(false);
	}

	return true;
}

bool EnumerateVertexElementSelectionVertices(
	const UE::Geometry::FGeometrySelection& Selection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FGroupTopology* GroupTopology,
	TFunctionRef<void(uint32)> VertexFunc)
{
	using namespace UE::Geometry;

	if (!ensure(Selection.ElementType == EGeometryElementType::Vertex))
	{
		return false;
	}

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		if (!ensure(GroupTopology))
		{
			return false;
		}

		for (uint64 EncodedID : Selection.Selection)
		{
			int32 VertexID = (int32)FGeoSelectionID(EncodedID).GeometryID;
			if (Mesh.IsVertex(VertexID))
			{
				VertexFunc(VertexID);
			}
		}
	}
	else if (Selection.TopologyType == EGeometryTopologyType::Triangle)
	{
		for (uint64 VertexID : Selection.Selection)
		{
			if (Mesh.IsVertex((int32)VertexID))
			{
				VertexFunc((int32)VertexID);
			}
		}
	}
	else
	{
		return ensure(false);
	}

	return true;
}

}//end GeometrySelectionUtilLocals

bool UE::Geometry::AreSelectionsIdentical(
	const FGeometrySelection& SelectionA, const FGeometrySelection& SelectionB)
{
	if ((SelectionA.ElementType != SelectionB.ElementType) || (SelectionA.TopologyType != SelectionB.TopologyType))
	{
		return false;
	}
	int32 Num = SelectionA.Num();
	if (Num != SelectionB.Num())
	{
		return false;
	}

	if (SelectionA.TopologyType == EGeometryTopologyType::Polygroup)
	{
		// for polygroup topology we may have stored an arbitrary geometry ID and so we cannot rely on TSet Contains
		for (uint64 ItemA : SelectionA.Selection)
		{
			uint32 TopologyID = FGeoSelectionID(ItemA).TopologyID;
			const uint64* Found = Algo::FindByPredicate(SelectionB.Selection, [&](uint64 Item)
			{
				return FGeoSelectionID(Item).TopologyID == TopologyID;
			});
			if (Found == nullptr)
			{
				return false;
			}
		}
	}
	else
	{
		for (uint64 ItemA : SelectionA.Selection)
		{
			if (SelectionB.Selection.Contains(ItemA) == false)
			{
				return false;
			}
		}
	}

	return true;
}



bool UE::Geometry::FindInSelectionByTopologyID(
	const FGeometrySelection& GeometrySelection,
	uint32 TopologyID,
	uint64& FoundValue)
{
	const uint64* Found = Algo::FindByPredicate(GeometrySelection.Selection, [&](uint64 Item)
	{
		return FGeoSelectionID(Item).TopologyID == TopologyID;
	});
	if (Found != nullptr)
	{
		FoundValue = *Found;
		return true;
	}
	FoundValue = FGeoSelectionID().Encoded();
	return false;
}


void UE::Geometry::UpdateTriangleSelectionViaRaycast(
	const FColliderMesh* ColliderMesh,
	FGeometrySelectionEditor* Editor,
	const FRay3d& LocalRay,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut )
{
	ensure(Editor->GetTopologyType() == EGeometryTopologyType::Triangle);

	ResultOut.bSelectionMissed = true;

	IMeshSpatial::FQueryOptions SpatialQueryOptions;
	if (!Editor->GetQueryConfig().bHitBackFaces)
	{
		SpatialQueryOptions.TriangleFilterF = [&](int32 Tid) {
			return ColliderMesh->GetTriNormal(Tid).Dot(LocalRay.Direction) < 0;
		};
	}
	double RayHitT; int32 HitTriangleID; FVector3d HitBaryCoords;
	if (ColliderMesh->FindNearestHitTriangle(LocalRay, RayHitT, HitTriangleID, HitBaryCoords, SpatialQueryOptions))
	{
		HitTriangleID = ColliderMesh->GetSourceTriangleID(HitTriangleID);
		if (HitTriangleID == IndexConstants::InvalidID)
		{
			return;
		}

		if (Editor->GetElementType() == EGeometryElementType::Face)
		{
			ResultOut.bSelectionModified = UpdateSelectionWithNewElements(Editor, UpdateConfig.ChangeType, TArray<uint64>{(uint64)HitTriangleID}, &ResultOut.SelectionDelta);
			ResultOut.bSelectionMissed = false;
		}
		else if (Editor->GetElementType() == EGeometryElementType::Vertex)
		{
			FVector3d HitPos = LocalRay.PointAt(RayHitT);
			FIndex3i TriVerts = ColliderMesh->GetTriangle(HitTriangleID);
			int32 NearestIdx = 0;
			double NearestDistSqr = DistanceSquared(ColliderMesh->GetVertex(TriVerts[0]), HitPos);
			for (int32 k = 1; k < 3; ++k)
			{
				double DistSqr = DistanceSquared(ColliderMesh->GetVertex(TriVerts[k]), HitPos);
				if (DistSqr < NearestDistSqr)
				{
					NearestDistSqr = DistSqr;
					NearestIdx = k;
				}
			}

			ResultOut.bSelectionModified = UpdateSelectionWithNewElements(Editor, UpdateConfig.ChangeType, TArray<uint64>{(uint64)TriVerts[NearestIdx]}, &ResultOut.SelectionDelta);
			ResultOut.bSelectionMissed = false;
		}
		else if (Editor->GetElementType() == EGeometryElementType::Edge)
		{
			FVector3d HitPos = LocalRay.PointAt(RayHitT);
			FIndex3i TriVerts = ColliderMesh->GetTriangle(HitTriangleID);
			FVector3d Positions[3];
			ColliderMesh->GetTriVertices(HitTriangleID, Positions[0], Positions[1], Positions[2]);
			int32 NearestIdx = 0;
			double NearestDistSqr = FSegment3d(Positions[0], Positions[1]).DistanceSquared(HitPos);
			for (int32 k = 1; k < 3; ++k)
			{
				double DistSqr = FSegment3d(Positions[k], Positions[(k+1)%3]).DistanceSquared(HitPos);
				if (DistSqr < NearestDistSqr)
				{
					NearestDistSqr = DistSqr;
					NearestIdx = k;
				}
			}
			FMeshTriEdgeID TriEdgeID(HitTriangleID, NearestIdx);

			ResultOut.bSelectionModified = UpdateSelectionWithNewElements(Editor, UpdateConfig.ChangeType, TArray<uint64>{(uint64)TriEdgeID.Encoded()}, &ResultOut.SelectionDelta);
			ResultOut.bSelectionMissed = false;
		}
	}
}





void UE::Geometry::UpdateGroupSelectionViaRaycast(
	const FColliderMesh* ColliderMesh,
	const FGroupTopology* GroupTopology,
	FGeometrySelectionEditor* Editor,
	const FRay3d& LocalRay,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut)
{
	ensure(Editor->GetTopologyType() == EGeometryTopologyType::Polygroup);

	ResultOut.bSelectionMissed = true;

	IMeshSpatial::FQueryOptions SpatialQueryOptions;
	if (!Editor->GetQueryConfig().bHitBackFaces)
	{
		SpatialQueryOptions.TriangleFilterF = [&](int32 Tid) {
			return ColliderMesh->GetTriNormal(Tid).Dot(LocalRay.Direction) < 0;
		};
	}
	double RayHitT; int32 HitTriangleID; FVector3d HitBaryCoords;
	if (ColliderMesh->FindNearestHitTriangle(LocalRay, RayHitT, HitTriangleID, HitBaryCoords, SpatialQueryOptions))
	{
		HitTriangleID = ColliderMesh->GetSourceTriangleID(HitTriangleID);
		if (HitTriangleID == IndexConstants::InvalidID)
		{
			return;
		}
		int32 GroupID = GroupTopology->GetGroupID(HitTriangleID);

		if (Editor->GetElementType() == EGeometryElementType::Face)
		{
			FGeoSelectionID GroupTriID((uint32)HitTriangleID, (uint32)GroupID);
			ResultOut.bSelectionModified = UpdateSelectionWithNewElements(Editor, UpdateConfig.ChangeType, TArray<uint64>{GroupTriID.Encoded()}, &ResultOut.SelectionDelta);
			ResultOut.bSelectionMissed = false;
		}
		else if (Editor->GetElementType() == EGeometryElementType::Vertex)
		{
			FVector3d HitPos = LocalRay.PointAt(RayHitT);
			FIndex3i TriVerts = ColliderMesh->GetTriangle(HitTriangleID);
			int32 NearestIdx = -1;
			int32 NearestCornerID = IndexConstants::InvalidID;
			double NearestDistSqr = TNumericLimits<double>::Max();
			for (int32 k = 0; k < 3; ++k)
			{
				int32 FoundCornerID = GroupTopology->GetCornerIDFromVertexID(TriVerts[k]);
				if (FoundCornerID != IndexConstants::InvalidID)
				{
					double DistSqr = DistanceSquared(ColliderMesh->GetVertex(TriVerts[k]), HitPos);
					if (DistSqr < NearestDistSqr)
					{
						NearestDistSqr = DistSqr;
						NearestIdx = k;
						NearestCornerID = FoundCornerID;
					}
				}
			}
			if (NearestCornerID != IndexConstants::InvalidID)
			{
				// do we need a group here?
				int32 VertexID = TriVerts[NearestIdx];
				FGeoSelectionID SelectionID(VertexID, NearestCornerID);
				ResultOut.bSelectionModified = UpdateSelectionWithNewElements(Editor, UpdateConfig.ChangeType, TArray<uint64>{SelectionID.Encoded()}, &ResultOut.SelectionDelta);
				ResultOut.bSelectionMissed = false;
			}
		}
		else if (Editor->GetElementType() == EGeometryElementType::Edge)
		{
			FVector3d HitPos = LocalRay.PointAt(RayHitT);
			FIndex3i TriVerts = ColliderMesh->GetTriangle(HitTriangleID);
			FVector3d Positions[3];
			ColliderMesh->GetTriVertices(HitTriangleID, Positions[0], Positions[1], Positions[2]);
			int32 NearestIdx = -1;
			double NearestDistSqr = TNumericLimits<double>::Max();
			for (int32 k = 0; k < 3; ++k)
			{
				if ( GroupTopology->IsGroupEdge(FMeshTriEdgeID(HitTriangleID, k), true) )
				{
					double DistSqr = FSegment3d(Positions[k], Positions[(k + 1) % 3]).DistanceSquared(HitPos);
					if (DistSqr < NearestDistSqr)
					{
						NearestDistSqr = DistSqr;
						NearestIdx = k;
					}
				}
			}
			if ( NearestIdx >= 0 )
			{
				// do we need a group here?
				FMeshTriEdgeID TriEdgeID(HitTriangleID, NearestIdx);
				int32 GroupEdgeID = GroupTopology->FindGroupEdgeID(TriEdgeID);
				checkSlow(GroupEdgeID >= 0);  // should never fail...
				if (GroupEdgeID >= 0)
				{
					FGeoSelectionID SelectionID(TriEdgeID.Encoded(), GroupEdgeID);
					ResultOut.bSelectionModified = UpdateSelectionWithNewElements(Editor, UpdateConfig.ChangeType, TArray<uint64>{SelectionID.Encoded()}, & ResultOut.SelectionDelta);
					ResultOut.bSelectionMissed = false;
				}
			}
		}
	}
}







bool UE::Geometry::UpdateSelectionWithNewElements(
	FGeometrySelectionEditor* Editor,
	EGeometrySelectionChangeType ChangeType,
	const TArray<uint64>& NewIDs,
	FGeometrySelectionDelta* Delta)
{
	FGeometrySelectionDelta LocalDelta;
	FGeometrySelectionDelta& UseDelta = (Delta != nullptr) ? (*Delta) : LocalDelta;

	if (ChangeType == EGeometrySelectionChangeType::Replace)
	{
		// [TODO] this could be optimized...
		Editor->ClearSelection(UseDelta);
		return Editor->Select(NewIDs, UseDelta);
	}
	else if (ChangeType == EGeometrySelectionChangeType::Add)
	{
		return Editor->Select(NewIDs, UseDelta);
	}
	else if (ChangeType == EGeometrySelectionChangeType::Remove)
	{
		return Editor->Deselect(NewIDs, UseDelta);
	}
	else
	{
		ensure(false);
		return false;
	}
	
}

bool UE::Geometry::EnumerateTriangleSelectionVertices(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FTransform& ApplyTransform,
	TFunctionRef<void(uint64, const FVector3d&)> VertexFunc)
{
	return EnumerateTriangleSelectionVertices(MeshSelection, Mesh, &ApplyTransform, VertexFunc);
}

bool UE::Geometry::EnumerateTriangleSelectionVertices(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FTransform* ApplyTransform,
	TFunctionRef<void(uint64, const FVector3d&)> VertexFunc)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Triangle ) == false )
	{
		return false;
	}

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		for (const uint64 TriangleID : MeshSelection.Selection)
		{
			if (Mesh.IsTriangle((int32)TriangleID))
			{
				const FIndex3i Triangle = Mesh.GetTriangle((int32)TriangleID);
				FVector3d TriVertexA, TriVertexB, TriVertexC;
				TriVertexA = Mesh.GetVertex(Triangle.A);
				TriVertexB = Mesh.GetVertex(Triangle.B);
				TriVertexC = Mesh.GetVertex(Triangle.C);
				
				if (ApplyTransform)
				{
					TriVertexA = ApplyTransform->TransformPosition(TriVertexA);
					TriVertexB = ApplyTransform->TransformPosition(TriVertexB);
					TriVertexC = ApplyTransform->TransformPosition(TriVertexC);
				}
				VertexFunc((uint64)Triangle.A, TriVertexA);
				VertexFunc((uint64)Triangle.B, TriVertexB);
				VertexFunc((uint64)Triangle.C, TriVertexC);
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FMeshTriEdgeID TriEdgeID(FGeoSelectionID(EncodedID).GeometryID);
			const int32 EdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if (Mesh.IsEdge(EdgeID))
			{
				const FIndex2i EdgeV = Mesh.GetEdgeV(EdgeID);
				FVector3d EdgeVertexA, EdgeVertexB;
				EdgeVertexA = Mesh.GetVertex(EdgeV.A);
				EdgeVertexB = Mesh.GetVertex(EdgeV.B);

				if (ApplyTransform)
				{
					EdgeVertexA = ApplyTransform->TransformPosition(EdgeVertexA);
					EdgeVertexB = ApplyTransform->TransformPosition(EdgeVertexB);
				}
				VertexFunc((uint64)EdgeV.A, EdgeVertexA);
				VertexFunc((uint64)EdgeV.B, EdgeVertexB);
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (const uint64 VertexID : MeshSelection.Selection)
		{
			if (Mesh.IsVertex((int32)VertexID))
			{
				FVector3d Vertex = Mesh.GetVertex((int32)VertexID);
				if (ApplyTransform)
				{
					Vertex = ApplyTransform->TransformPosition(Vertex);
				}
				VertexFunc(VertexID, Vertex);
			}
		}
	}
	else
	{
		return false;
	}

	return true;
}



bool UE::Geometry::EnumeratePolygroupSelectionVertices(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FTransform& ApplyTransform,
	TFunctionRef<void(uint64, const FVector3d&)> VertexFunc)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Polygroup ) == false )
	{
		return false;
	}

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FGeoSelectionID GroupTriID(EncodedID);
			const int32 SeedTriangleID = (int32)GroupTriID.GeometryID, GroupID = (int32)GroupTriID.TopologyID;
			if (Mesh.IsTriangle(SeedTriangleID))
			{
				for (const int32 TriangleID : GroupTopology->GetGroupFaces(GroupID))
				{
					const FIndex3i Triangle = Mesh.GetTriangle((int32)TriangleID);
					VertexFunc((uint64)Triangle.A, ApplyTransform.TransformPosition(Mesh.GetVertex(Triangle.A)));
					VertexFunc((uint64)Triangle.B, ApplyTransform.TransformPosition(Mesh.GetVertex(Triangle.B)));
					VertexFunc((uint64)Triangle.C, ApplyTransform.TransformPosition(Mesh.GetVertex(Triangle.C)));
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			const int32 SeedEdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if (Mesh.IsEdge(SeedEdgeID))
			{
				const int32 GroupEdgeID = GroupTopology->FindGroupEdgeID(SeedEdgeID);
				for (const int32 VertexID : GroupTopology->GetGroupEdgeVertices(GroupEdgeID))
				{
					FVector3d V = Mesh.GetVertex(VertexID);
					VertexFunc((uint64)VertexID, ApplyTransform.TransformPosition(V));
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const int32 VertexID = (int32)FGeoSelectionID(EncodedID).GeometryID;
			if (Mesh.IsVertex(VertexID))
			{
				VertexFunc((uint64)VertexID, ApplyTransform.TransformPosition(Mesh.GetVertex(VertexID)));
			}
		}
	}
	else
	{
		return false;
	}

	return true;
}



bool UE::Geometry::EnumerateSelectionTriangles(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> TriangleFunc,
	const UE::Geometry::FPolygroupSet* UseGroupSet
)
{
	if (MeshSelection.TopologyType == EGeometryTopologyType::Triangle)
	{
		return EnumerateTriangleSelectionTriangles(MeshSelection, Mesh, TriangleFunc);
	}
	else if (MeshSelection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		return EnumeratePolygroupSelectionTriangles(MeshSelection, Mesh, 
			(UseGroupSet != nullptr) ? *UseGroupSet : FPolygroupSet(&Mesh), 
			TriangleFunc);
	}
	return false;
}


bool UE::Geometry::EnumerateTriangleSelectionTriangles(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> TriangleFunc)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Triangle ) == false )
	{
		return false;
	}

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		for (const uint64 TriangleID : MeshSelection.Selection)
		{
			if (Mesh.IsTriangle((int32)TriangleID))
			{
				TriangleFunc((int32)TriangleID);
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			const int32 EdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			Mesh.EnumerateEdgeTriangles(EdgeID, [&TriangleFunc](const int32 TriangleID)
			{
				TriangleFunc(TriangleID);
			});
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (const uint64 VertexID : MeshSelection.Selection)
		{
			Mesh.EnumerateVertexTriangles((int32)VertexID, [&TriangleFunc](const int32 TriangleID)
			{
				TriangleFunc(TriangleID);
			});
		}
	}
	else
	{
		return false;
	}

	return true;
}


bool UE::Geometry::EnumeratePolygroupSelectionTriangles(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FPolygroupSet& GroupSet,
	TFunctionRef<void(int32)> TriangleFunc
)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Polygroup ) == false )
	{
		return false;
	}

	TArray<int32> SeedGroups;
	TArray<int32> SeedTriangles;
	TSet<int32> UniqueSeedGroups;

	// TODO: the code below will not work correctly if the selection contains
	// multiple disconnected-components with the same GroupID. They will be
	// filtered out by the UniqueSeedGroups test. Seems like it will be necessary
	// to detect this case up-front and do something more expensive, like filtering
	// out duplicates inside the connected-components loop instead of up-front

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FGeoSelectionID SelectionID(EncodedID);
			const int32 SeedTriangleID = (int32)SelectionID.GeometryID;
			if (Mesh.IsTriangle(SeedTriangleID))
			{
				const int32 GroupID = GroupSet.GetGroup(SeedTriangleID);
				// TODO: [TopologyMismatch] We don't have the ensure below because the selection system currently can have a
				//  different view of a converted target than a tool might after making some edits, and the group ID's can
				//  differ. For instance, a modified volume dynamic mesh may differ from the dynamic mesh that would result
				//  from converting to a volume and back to dynamic mesh again. Someday we may fix this, but for now we accept
				//  that the selection system and tools may disagree, and tools may end up setting incorrect selections after
				//  doing some edits.
				// ensure(GroupID == (int32)SelectionID.TopologyID);		// sanity-check that we are using the right group

				if ( GroupID >= 0 && UniqueSeedGroups.Contains(GroupID) == false)
				{
					UniqueSeedGroups.Add(GroupID);
					SeedGroups.Add(GroupID);
					SeedTriangles.Add(SeedTriangleID);
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			const int32 SeedEdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			Mesh.EnumerateEdgeTriangles(SeedEdgeID, [&GroupSet, &UniqueSeedGroups, &SeedGroups, &SeedTriangles](const int32 TriangleID)
			{
				const int32 GroupID = GroupSet.GetGroup(TriangleID);
				if (GroupID >= 0 && UniqueSeedGroups.Contains(GroupID) == false)
				{
					UniqueSeedGroups.Add(GroupID);
					SeedGroups.Add(GroupID);
					SeedTriangles.Add(TriangleID);
				}
			});
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const int32 VertexID = (int32)FGeoSelectionID(EncodedID).GeometryID;
			Mesh.EnumerateVertexTriangles(VertexID, [&GroupSet, &UniqueSeedGroups, &SeedGroups, &SeedTriangles](const int32 TriangleID)
			{
				const int32 GroupID = GroupSet.GetGroup(TriangleID);
				if (GroupID >= 0 && UniqueSeedGroups.Contains(GroupID) == false)
				{
					UniqueSeedGroups.Add(GroupID);
					SeedGroups.Add(GroupID);
					SeedTriangles.Add(TriangleID);
				}
			});
		}
	}
	else
	{
		return false;
	}


	TSet<int> TempROI;		// if we could provide this as input we would not need a temporary roi...
	TArray<int32> QueueBuffer;
	const int32 NumGroups = SeedGroups.Num();
	for (int32 k = 0; k < NumGroups; ++k)
	{
		ensure(GroupSet.GetGroup(SeedTriangles[k]) == SeedGroups[k]);
		const int32 GroupID = SeedGroups[k];
		FMeshConnectedComponents::GrowToConnectedTriangles(&Mesh, 
			TArray<int>{SeedTriangles[k]}, TempROI, &QueueBuffer, 
			[&GroupSet, &GroupID](const int32 T1, const int32 T2)
			{
				return GroupSet.GetGroup(T2) == GroupID;
			});
		for (const int32 TID : TempROI)
		{
			TriangleFunc(TID);
		}
	}

	return true;
}

bool UE::Geometry::EnumerateSelectionEdges(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> EdgeFunc,
	const UE::Geometry::FPolygroupSet* UseGroupSet
)
{
	if (MeshSelection.TopologyType == EGeometryTopologyType::Triangle)
	{
		return EnumerateTriangleSelectionEdges(MeshSelection, Mesh, EdgeFunc);
	}
	else if (MeshSelection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		return EnumeratePolygroupSelectionEdges(MeshSelection, Mesh, 
			(UseGroupSet != nullptr) ? *UseGroupSet : FPolygroupSet(&Mesh), 
			EdgeFunc);
	}
	return false;
}


bool UE::Geometry::EnumerateTriangleSelectionEdges(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> EdgeFunc)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Triangle ) == false )
	{
		return false;
	}

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		for (const uint64 TriangleID : MeshSelection.Selection)
		{
			if (Mesh.IsTriangle((int32)TriangleID))
			{
				const FIndex3i TriEdges = Mesh.GetTriEdges((int32)TriangleID);
				EdgeFunc(TriEdges[0]);
				EdgeFunc(TriEdges[1]);
				EdgeFunc(TriEdges[2]);
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			const int32 EdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if (Mesh.IsEdge((int32)EdgeID))
			{
				EdgeFunc((int32)EdgeID);
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (const uint64 VertexID : MeshSelection.Selection)
		{
			Mesh.EnumerateVertexEdges((int32)VertexID, [&EdgeFunc](const int32 EdgeID)
			{
				EdgeFunc(EdgeID);
			});
		}
	}
	else
	{
		return false;
	}

	return true;
}


bool UE::Geometry::EnumeratePolygroupSelectionEdges(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FPolygroupSet& GroupSet,
	TFunctionRef<void(int32)> EdgeFunc
)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Polygroup ) == false )
	{
		return false;
	}

	TArray<int32> SeedTriGroups;
	TArray<int32> SeedTriangles;

	TArray<int32> SeedEdges;

	// TODO: the face code below will not work correctly if the selection contains
	// multiple disconnected-components with the same GroupID. They will be
	// filtered out by the UniqueSeedGroups test. Seems like it will be necessary
	// to detect this case up-front and do something more expensive, like filtering
	// out duplicates inside the connected-components loop instead of up-front

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		TSet<int32> UniqueSeedTriGroups;
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FGeoSelectionID SelectionID(EncodedID);
			const int32 SeedTriangleID = (int32)SelectionID.GeometryID;
			if (Mesh.IsTriangle(SeedTriangleID))
			{
				const int32 GroupID = GroupSet.GetGroup(SeedTriangleID);

				// See TODO: [TopologyMismatch] above
				//ensure(GroupID == (int32)SelectionID.TopologyID);		// sanity-check that we are using the right group
				
				if ( GroupID >= 0 && UniqueSeedTriGroups.Contains(GroupID) == false)
				{
					UniqueSeedTriGroups.Add(GroupID);
					SeedTriGroups.Add(GroupID);
					SeedTriangles.Add(SeedTriangleID);
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			// record selected edges, need to find other edges that are part of the same polygroup edge to include in selection
			const FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			const int32 SeedEdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if (Mesh.IsEdge(SeedEdgeID))
			{
				SeedEdges.Add(SeedEdgeID);
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const int32 VertexID = (int32)FGeoSelectionID(EncodedID).GeometryID;
			if (Mesh.IsVertex(VertexID))
			{
				Mesh.EnumerateVertexEdges(VertexID, [&Mesh, &SeedEdges](const int32 EdgeID)
				{
					if (Mesh.IsEdge(EdgeID))
					{
						SeedEdges.Add(EdgeID);
					}
				});
			}
		}
	}
	else
	{
		return false;
	}


	TSet<int> TempROI;		// if we could provide this as input we would not need a temporary roi...
	TArray<int32> QueueBuffer;

	// Edge Type: currently enumerates all edges that are part of selected poly edge(s).
	// Vertex Type: currently enumerates all edges that are a part of a polyedge which contains the selected vertex/vertices
	// neither includes non poly-group edges
	if (MeshSelection.ElementType == EGeometryElementType::Vertex || MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		const int32 NumEdges = SeedEdges.Num();
		for (int32 j = 0; j < NumEdges; j++)
		{
			TArray<int32> EdgeGroups; // the 1 or 2 groups which an edge belongs to
			// retrieves the 1 or 2 triangles and groups to which a selected edge belongs, and finds which group(s) the triangles belong to
			Mesh.EnumerateEdgeTriangles(SeedEdges[j], [&GroupSet, &EdgeGroups](const int32 TriangleID)
			{
				const int32 GroupID = GroupSet.GetGroup(TriangleID);
				if (GroupID >= 0)
				{
					EdgeGroups.Add(GroupID);
				}
			});

			EdgeGroups.Sort();

			// if an edge's 2 triangles are from the same group, it is not a Polygroup boundary edge and can be disregarded
			const bool bIsInnerEdge = EdgeGroups.Num() == 2 && EdgeGroups[0] == EdgeGroups[1];

			if (!bIsInnerEdge)
			{
				// finds connected edges by looking at all edges in a mesh and retrieving which 2 (or 1) groups they belong to
				// if an edge's 2 groups are the same as the selected edges' 2 groups, we know they are part of the same poly edge
				FMeshConnectedComponents::GrowToConnectedEdges(Mesh, TArray<int>{SeedEdges[j]}, TempROI, &QueueBuffer, 
						[&Mesh, &GroupSet, &EdgeGroups](const int32 E1, const int32 E2)
						{
							TArray<int32> OtherEdgeGroups;
							Mesh.EnumerateEdgeTriangles(E2, [&GroupSet, &OtherEdgeGroups](const int32 TriangleID)
							{
								const int GrpID = GroupSet.GetGroup(TriangleID);
								if (GrpID >= 0)
								{
									OtherEdgeGroups.Add(GrpID);
								}
							});
							OtherEdgeGroups.Sort();

							// if selected edge groups and current edge groups are the same, they belong to the same poly edge
							// note: currently if a border edge is selected (i.e an edge has only 1 triangle and 1 group), function will include
							// all other border edges of the polygroup
							return  OtherEdgeGroups.Num() == EdgeGroups.Num() && OtherEdgeGroups == EdgeGroups;
						});
				// apply edge function to all edges in poly edge
				for (const int32 EdgeID : TempROI)
				{
					EdgeFunc(EdgeID);
				}
			}
			
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		const int32 NumGroups = SeedTriGroups.Num();
		for (int32 k = 0; k < NumGroups; ++k)
		{
			ensure(GroupSet.GetGroup(SeedTriangles[k]) == SeedTriGroups[k]);
			const int32 GroupID = SeedTriGroups[k];
			FMeshConnectedComponents::GrowToConnectedTriangles(&Mesh, 
				TArray<int>{SeedTriangles[k]}, TempROI, &QueueBuffer, 
				[&GroupSet, &GroupID](const int32 T1, const int32 T2)
				{
					return GroupSet.GetGroup(T2) == GroupID;
				});
			
			for (const int32 TID : TempROI)
			{
				FIndex3i TriEdges = Mesh.GetTriEdges(TID);
				EdgeFunc(TriEdges[0]);
				EdgeFunc(TriEdges[1]);
				EdgeFunc(TriEdges[2]);
			}
		
		}
	}
	else
	{
		return false;
	}

	return true;
}

bool UE::Geometry::EnumeratePolygroupSelectionEdges(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FGroupTopology& GroupTopology,
	TFunctionRef<void(int32)> EdgeFunc
)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Polygroup ) == false )
	{
		return false;
	}


	TArray<int32> GroupEdgeIDs;
	
	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		TSet<int32> SeedGroups;
		
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FGeoSelectionID SelectionID(EncodedID);
			const int32 SeedTriangleID = (int32)SelectionID.GeometryID;
			if (Mesh.IsTriangle(SeedTriangleID))
			{
				const int32 GroupID = GroupTopology.GetGroupID(SeedTriangleID);

				// See TODO: [TopologyMismatch] above
				//ensure(GroupID == (int32)SelectionID.TopologyID);		// sanity-check that we are using the right group
				
				if ( GroupID >= 0)
				{
					SeedGroups.Add(GroupID);
				}
			}
		}

		for (const int32 GroupID : SeedGroups)
		{
			TArray<int32> GroupTriangles = GroupTopology.GetGroupTriangles(GroupID);
			for (const int32 TriangleID : GroupTriangles)
			{
				FIndex3i TriEdges = Mesh.GetTriEdges(TriangleID);
				EdgeFunc(TriEdges[0]);
				EdgeFunc(TriEdges[1]);
				EdgeFunc(TriEdges[2]);
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			const int32 SeedEdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if (Mesh.IsEdge(SeedEdgeID))
			{
				const int32 GroupEdgeID = GroupTopology.FindGroupEdgeID(SeedEdgeID);
				if (GroupEdgeID != IndexConstants::InvalidID)
				{
					GroupEdgeIDs.Add (GroupEdgeID);
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (const uint64 VertexID : MeshSelection.Selection)
		{
			if (Mesh.IsVertex((int32)VertexID))
			{
				Mesh.EnumerateVertexEdges((int32)VertexID, [&GroupTopology, &GroupEdgeIDs](const int32 EdgeID)
				{
					const int32 GroupEdgeID = GroupTopology.FindGroupEdgeID(EdgeID);
					if (GroupEdgeID != IndexConstants::InvalidID)
					{
						GroupEdgeIDs.Add(GroupEdgeID);
					}
				});
			}
		}
	}
	else
	{
		return false;
	}

	if (MeshSelection.ElementType == EGeometryElementType::Vertex || MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const int32 GroupEdgeID : GroupEdgeIDs)
		{
			const FGroupTopology::FGroupEdge& GroupEdge = GroupTopology.Edges[GroupEdgeID];
			for (const int32 EID : GroupEdge.Span.Edges)
			{
				EdgeFunc(EID);
			}
		}
	}
	return true;
}


bool UE::Geometry::EnumerateTriangleSelectionElements(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32, const FVector3d&)> VertexFunc,
	TFunctionRef<void(int32, const FSegment3d&)> EdgeFunc,
	TFunctionRef<void(int32, const FTriangle3d&)> TriangleFunc,
	const FTransform* ApplyTransform,
	const bool bMapFacesToEdgeLoops
)
{
	return EnumerateTriangleSelectionElements(
		MeshSelection,
		Mesh,
		VertexFunc,
		EdgeFunc,
		TriangleFunc,
		ApplyTransform,
		EEnumerateSelectionMapping::Default | (bMapFacesToEdgeLoops ? EEnumerateSelectionMapping::FacesToEdges : EEnumerateSelectionMapping::None)
		);
}

bool UE::Geometry::EnumerateTriangleSelectionElements(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32, const FVector3d&)> VertexFunc,
	TFunctionRef<void(int32, const FSegment3d&)> EdgeFunc,
	TFunctionRef<void(int32, const FTriangle3d&)> TriangleFunc,
	const FTransform* ApplyTransform,
	const EEnumerateSelectionMapping Flags
)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Triangle ) == false )
	{
		return false;
	}
	
	auto ApplyTriangleFunc = [&ApplyTransform, &TriangleFunc, &Mesh](const int32 TriangleID)
	{
		FVector3d VertA, VertB, VertC;
		Mesh.GetTriVertices((int32)TriangleID, VertA, VertB, VertC);
		if (ApplyTransform)
		{
			VertA = ApplyTransform->TransformPosition(VertA);
			VertB = ApplyTransform->TransformPosition(VertB);
			VertC = ApplyTransform->TransformPosition(VertC);
		}
		TriangleFunc(TriangleID, FTriangle3d(VertA, VertB, VertC));	
	};

	auto ApplyEdgeFunc = [&ApplyTransform, &EdgeFunc, &Mesh](const int32 EdgeID)
	{
		FVector3d VertA, VertB;
		Mesh.GetEdgeV(EdgeID, VertA, VertB);
		if (ApplyTransform)
		{
			VertA = ApplyTransform->TransformPosition(VertA);
			VertB = ApplyTransform->TransformPosition(VertB);
		}
		EdgeFunc(EdgeID, FSegment3d(VertA, VertB));
	};

	auto ApplyVertexFunc = [&VertexFunc](const uint64 VertexID, const FVector3d& VertA)
	{
		VertexFunc((int32)VertexID, VertA);
	};
	

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		// Call TriangleFunc if we are mapping Faces to Faces
		if ((Flags & EEnumerateSelectionMapping::FacesToFaces) != EEnumerateSelectionMapping::None)
		{
			EnumerateTriangleSelectionTriangles(MeshSelection, Mesh, ApplyTriangleFunc);
		}

		// Call EdgeFunc if we are mapping Faces to Edges
		if ((Flags & EEnumerateSelectionMapping::FacesToEdges) != EEnumerateSelectionMapping::None)
		{
			EnumerateTriangleSelectionEdges(MeshSelection, Mesh, ApplyEdgeFunc);
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		EnumerateTriangleSelectionEdges(MeshSelection, Mesh, ApplyEdgeFunc);
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		EnumerateTriangleSelectionVertices(MeshSelection, Mesh, ApplyTransform, ApplyVertexFunc);
	}
	else
	{
		return false;
	}

	return true;
}

bool UE::Geometry::EnumeratePolygroupSelectionElements(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	TFunctionRef<void(int32, const FVector3d&)> VertexFunc,
	TFunctionRef<void(int32, const FSegment3d&)> EdgeFunc,
	TFunctionRef<void(int32, const FTriangle3d&)> TriangleFunc,
	const FTransform* ApplyTransform,
	const bool bMapFacesToEdgeLoops
)
{
	return EnumeratePolygroupSelectionElements(
		MeshSelection,
		Mesh,
		GroupTopology,
		VertexFunc,
		EdgeFunc,
		TriangleFunc,
		ApplyTransform,
		EEnumerateSelectionMapping::Default | (bMapFacesToEdgeLoops ? EEnumerateSelectionMapping::FacesToEdges : EEnumerateSelectionMapping::None)
		);
}

bool UE::Geometry::EnumeratePolygroupSelectionElements(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	TFunctionRef<void(int32, const FVector3d&)> VertexFunc,
	TFunctionRef<void(int32, const FSegment3d&)> EdgeFunc,
	TFunctionRef<void(int32, const FTriangle3d&)> TriangleFunc,
	const FTransform* ApplyTransform,
	const EEnumerateSelectionMapping Flags
)
{
if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Polygroup ) == false )
	{
		return false;
	}

	auto ProcessGroupEdgeID = [GroupTopology, &Mesh, ApplyTransform, &EdgeFunc](const int32 GroupEdgeID)
	{
		for (const int32 EdgeID : GroupTopology->GetGroupEdgeEdges(GroupEdgeID))
		{
			FVector3d A, B;
			Mesh.GetEdgeV(EdgeID, A, B);
			if (ApplyTransform)
			{
				A = ApplyTransform->TransformPosition(A);
				B = ApplyTransform->TransformPosition(B);
			}
			EdgeFunc(EdgeID, FSegment3d(A, B));
		}
	};

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FGeoSelectionID SelectionID(EncodedID);
			const int32 SeedTriangleID = (int32)SelectionID.GeometryID, GroupID = (int32)SelectionID.TopologyID;
			if (Mesh.IsTriangle(SeedTriangleID))
			{
				// Call TriangleFunc if we are mapping Faces to Faces
				// Note: While EnumeratePolygroupSelectionTriangles would also return all faces, implementing it this way minimizes redundancy in the
				// MeshSelection.Selection loop and more consistent with the rest of the Polygroup selection elements enumeration
				if ((Flags & EEnumerateSelectionMapping::FacesToFaces) != EEnumerateSelectionMapping::None)
				{
					for (const int32 TriangleID : GroupTopology->GetGroupFaces(GroupID))
					{
						FVector3d A, B, C;
						Mesh.GetTriVertices(TriangleID, A, B, C);
						if (ApplyTransform)
						{
							A = ApplyTransform->TransformPosition(A);
							B = ApplyTransform->TransformPosition(B);
							C = ApplyTransform->TransformPosition(C);
						}
						TriangleFunc(TriangleID, FTriangle3d(A, B, C));
					}
				}

				// Process the polygroup edges if we are mapping Faces to Edges
				if ((Flags & EEnumerateSelectionMapping::FacesToEdges) != EEnumerateSelectionMapping::None)
				{
					TArray<int32> GroupEdgeIDs;
					if ( const FGroupTopology::FGroup* Group = GroupTopology->FindGroupByID(GroupID) )
					{
						for (const FGroupTopology::FGroupBoundary& Boundary : Group->Boundaries)
						{
							for (const int32 GroupEdgeID : Boundary.GroupEdges)
							{
								GroupEdgeIDs.AddUnique(GroupEdgeID);
							}
						}
					}
					for (const int32 GroupEdgeID : GroupEdgeIDs)
					{
						ProcessGroupEdgeID(GroupEdgeID);
					}
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			const int32 SeedEdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if (Mesh.IsEdge(SeedEdgeID))
			{
				const int32 GroupEdgeID = GroupTopology->FindGroupEdgeID(SeedEdgeID);
				if (GroupEdgeID != INDEX_NONE)
				{
					ProcessGroupEdgeID(GroupEdgeID);
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (const uint64 EncodedID : MeshSelection.Selection)
		{
			const int32 VertexID = (int32)FGeoSelectionID(EncodedID).GeometryID;
			if (Mesh.IsVertex(VertexID))
			{
				const FVector3d A = Mesh.GetVertex(VertexID);
				
				VertexFunc(VertexID, (ApplyTransform) ? ApplyTransform->TransformPosition(A) : A);
			}
		}
	}
	else
	{
		return false;
	}

	return true;
}




bool UE::Geometry::ConvertPolygroupSelectionToTopologySelection(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	FGroupTopologySelection& TopologySelectionOut)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Polygroup ) == false )
	{
		return false;
	}

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		for (uint64 EncodedID : MeshSelection.Selection)
		{
			int32 GroupID = FGeoSelectionID(EncodedID).TopologyID;
			if (GroupTopology->FindGroupByID(GroupID) != nullptr)
			{
				TopologySelectionOut.SelectedGroupIDs.Add(GroupID);
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (uint64 EncodedID : MeshSelection.Selection)
		{
			FMeshTriEdgeID MeshEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			if ( Mesh.IsTriangle((int32)MeshEdgeID.TriangleID) )
			{
				int32 GroupEdgeID = GroupTopology->FindGroupEdgeID(MeshEdgeID);
				if (GroupEdgeID >= 0)
				{
					TopologySelectionOut.SelectedEdgeIDs.Add(GroupEdgeID);
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (uint64 EncodedID : MeshSelection.Selection)
		{
			int32 VertexID = (int32)FGeoSelectionID(EncodedID).GeometryID;
			if ( Mesh.IsVertex(VertexID) )
			{
				int32 CornerID = GroupTopology->GetCornerIDFromVertexID(VertexID);
				if (CornerID >= 0)
				{
					TopologySelectionOut.SelectedCornerIDs.Add(CornerID);
				}
			}
		}
	}
	else
	{
		return false;
	}

	return true;
}


bool UE::Geometry::InitializeSelectionFromTriangles(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	TArrayView<const int> Triangles,
	FGeometrySelection& SelectionOut)
{
	// TODO Refactor this to use GetSelectionTypeAsIndex

	if (SelectionOut.TopologyType == EGeometryTopologyType::Triangle)
	{
		if (SelectionOut.ElementType == EGeometryElementType::Vertex)
		{
			for (const int32 TID : Triangles)
			{
				if (Mesh.IsTriangle(TID))
				{
					const FIndex3i TriVertices = Mesh.GetTriangle(TID);
					SelectionOut.Selection.Add(FGeoSelectionID::MeshVertex(TriVertices.A).Encoded() );
					SelectionOut.Selection.Add(FGeoSelectionID::MeshVertex(TriVertices.B).Encoded());
					SelectionOut.Selection.Add(FGeoSelectionID::MeshVertex(TriVertices.C).Encoded());
				}
			}
		}
		else if (SelectionOut.ElementType == EGeometryElementType::Edge)
		{
			for (const int32 TID : Triangles)
			{
				if (Mesh.IsTriangle(TID))
				{
					Mesh.EnumerateTriEdgeIDsFromTriID(TID,
						[&SelectionOut](const FMeshTriEdgeID TriEdgeID)
						{
							SelectionOut.Selection.Add(FGeoSelectionID::MeshEdge(TriEdgeID).Encoded());
						});
				}
			}
		}
		else if (SelectionOut.ElementType == EGeometryElementType::Face)
		{
			for (const int32 TID : Triangles)
			{
				if (Mesh.IsTriangle(TID))
				{
					SelectionOut.Selection.Add(FGeoSelectionID::MeshTriangle(TID).Encoded());
				}
			}
		}
		else
		{
			return false;
		}
		return true;
	}
	else if (SelectionOut.TopologyType == EGeometryTopologyType::Polygroup)
	{
		if (!ensure(GroupTopology != nullptr))
		{
			return false;
		}

		if (SelectionOut.ElementType == EGeometryElementType::Vertex)
		{
			FMeshVertexSelection VertSelection(&Mesh);
			VertSelection.SelectTriangleVertices(Triangles);
			for (const int32 VID : VertSelection)
			{
				const int32 CornerID = GroupTopology->GetCornerIDFromVertexID(VID);
				if (CornerID != IndexConstants::InvalidID)
				{
					const FGroupTopology::FCorner& Corner = GroupTopology->Corners[CornerID];
					const FGeoSelectionID ID = FGeoSelectionID(Corner.VertexID, CornerID);
					SelectionOut.Selection.Add(ID.Encoded());
				}
			}
		}
		else if (SelectionOut.ElementType == EGeometryElementType::Edge)
		{
			FMeshEdgeSelection EdgeSelection(&Mesh);
			EdgeSelection.SelectTriangleEdges(Triangles);
			for (const int32 EID : EdgeSelection)
			{
				const int32 GroupEdgeID = GroupTopology->FindGroupEdgeID(EID);
				if (GroupEdgeID != IndexConstants::InvalidID)
				{
					const FGroupTopology::FGroupEdge& GroupEdge = GroupTopology->Edges[GroupEdgeID];
					const FMeshTriEdgeID MeshEdgeID = Mesh.GetTriEdgeIDFromEdgeID(GroupEdge.Span.Edges[0]);
					const FGeoSelectionID ID = FGeoSelectionID(MeshEdgeID.Encoded(), GroupEdgeID);
					SelectionOut.Selection.Add(ID.Encoded());
				}
			}
		}
		else if (SelectionOut.ElementType == EGeometryElementType::Face)
		{
			for (const int32 TID : Triangles)
			{
				if (Mesh.IsTriangle(TID))
				{
					const int32 GroupID = GroupTopology->GetGroupID(TID);
					const FGroupTopology::FGroup* GroupFace = GroupTopology->FindGroupByID(GroupID);
					if ( GroupFace )
					{
						const FGeoSelectionID ID = FGeoSelectionID(GroupFace->Triangles[0], GroupFace->GroupID);
						SelectionOut.Selection.Add(ID.Encoded());
					}
				}
			}
		}
		else
		{
			return false;
		}
		return true;
	}
	return false;
}

bool UE::Geometry::ConvertSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& FromSelectionIn,
	FGeometrySelection& ToSelectionOut)
{
	return ConvertSelection(Mesh, GroupTopology, FromSelectionIn, ToSelectionOut, EEnumerateSelectionConversionParams::ContainSelection);
}

bool UE::Geometry::ConvertSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& FromSelectionIn,
	FGeometrySelection& ToSelectionOut,
	const EEnumerateSelectionConversionParams ConversionParams)
{
	using namespace GeometrySelectionUtilLocals;

	const auto FromTypeToSame = [](
		const FDynamicMesh3& Mesh,
		const FGroupTopology* GroupTopology,
		const FGeometrySelection& FromSelectionIn,
		FGeometrySelection& ToSelectionOut,
		const EEnumerateSelectionConversionParams ConversionParams) -> bool
	{
		checkSlow(FromSelectionIn.IsSameType(ToSelectionOut));

		ToSelectionOut.Selection = FromSelectionIn.Selection;

		return true;
	};

	const auto ToTriFace = [](
		const FDynamicMesh3& Mesh,
		const FGroupTopology* GroupTopology,
		const FGeometrySelection& FromSelectionIn,
		FGeometrySelection& ToSelectionOut,
		const EEnumerateSelectionConversionParams ConversionParams) -> bool
	{
		checkSlow(ToSelectionOut.TopologyType == EGeometryTopologyType::Triangle);
		checkSlow(ToSelectionOut.ElementType == EGeometryElementType::Face);

		const FPolygroupSet GroupSet = FPolygroupSet(&Mesh);

		// TODO: these if statements could be consolidated to minimize minor repeat code; however its currently set up this way for readability and
		//		 scalability for if/when more EEnumerateSelectionConversionParams are added

		if (ConversionParams == EEnumerateSelectionConversionParams::ExpandSelection)
		{
			EnumerateSelectionTriangles(
				FromSelectionIn,
				Mesh,
				[&ToSelectionOut](const int32 TID) { ToSelectionOut.Selection.Add(FGeoSelectionID::MeshTriangle(TID).Encoded()); },
				&GroupSet);
		}

		else if (ConversionParams == EEnumerateSelectionConversionParams::ContainSelection)
		{
			// where FromSelection type is either PolyVerts, PolyEdges, PolyFaces, TriFaces -> ensuring stable selection is not applicable and can do same as in ExpandSelection
			if (FromSelectionIn.TopologyType == EGeometryTopologyType::Polygroup
				|| ( FromSelectionIn.ElementType == EGeometryElementType::Face && FromSelectionIn.TopologyType == EGeometryTopologyType::Triangle))
			{
				EnumerateSelectionTriangles(
				FromSelectionIn,
				Mesh,
				[&ToSelectionOut](const int32 TID) { ToSelectionOut.Selection.Add(FGeoSelectionID::MeshTriangle(TID).Encoded()); },
				&GroupSet);
			}
			// where FromSelection type is either TriEdge or TriVert
			else
			{
				TArray<int32> AllTriIDsConnectedToSelection;
				
				// retrieve all the triangles connected to the selected verts/edges but they will not all be included in final selection
				EnumerateSelectionTriangles(
					FromSelectionIn,
					Mesh,
					[&AllTriIDsConnectedToSelection](const int32 TID) { return AllTriIDsConnectedToSelection.Add(TID); },
					&GroupSet);


				if (FromSelectionIn.ElementType == EGeometryElementType::Edge)
				{
					// retrieve all the edges currently in the selection
					TSet<int32> SelectedEdges;
					for (const uint64 EncodedID : FromSelectionIn.Selection)
					{
						const FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
						const int32 EdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
						SelectedEdges.Add(EdgeID);
					}

					// for each triangle that's connected to the selection, determine if it is formed by 3 selected edges or not
					for (const int TriangleID : AllTriIDsConnectedToSelection) //-V1078
					{
						const FIndex3i TriEdges = Mesh.GetTriEdges(TriangleID);
						if (SelectedEdges.Contains(TriEdges.A) && SelectedEdges.Contains(TriEdges.B) && SelectedEdges.Contains(TriEdges.C))
						{
							ToSelectionOut.Selection.Add(FGeoSelectionID::MeshTriangle(TriangleID).Encoded());
						}
					}
				}

				if (FromSelectionIn.ElementType == EGeometryElementType::Vertex)
				{
					// retrieve all the vertices currently in the selection
					TSet<int32> SelectedVerts;
					for (const uint64 VertexID : FromSelectionIn.Selection)
					{
						SelectedVerts.Add((int32)VertexID);
					}
					
					// for each triangle that's connected to the selection, determine if it is formed by 3 selected vertices or not
					for (const int TriangleID : AllTriIDsConnectedToSelection) //-V1078
					{
						const FIndex3i TriVerts = Mesh.GetTriangle(TriangleID);
						if (SelectedVerts.Contains(TriVerts.A) && SelectedVerts.Contains(TriVerts.B) && SelectedVerts.Contains(TriVerts.C))
						{
							ToSelectionOut.Selection.Add(FGeoSelectionID::MeshTriangle(TriangleID).Encoded());
						}
					}
				}
			}
		}
		return true;
	};

	const auto ToTriEdge = [](
		const FDynamicMesh3& Mesh,
		const FGroupTopology* GroupTopology,
		const FGeometrySelection& FromSelectionIn,
		FGeometrySelection& ToSelectionOut,
		const EEnumerateSelectionConversionParams ConversionParams) -> bool
	{
		checkSlow(ToSelectionOut.TopologyType == EGeometryTopologyType::Triangle);
		checkSlow(ToSelectionOut.ElementType == EGeometryElementType::Edge);

		auto SelectEdgesFunc = [&Mesh, &GroupTopology, &ToSelectionOut] (const int32 EdgeID)
		{
			Mesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID,
				[&ToSelectionOut](const FMeshTriEdgeID TriEdgeID)
				{
					ToSelectionOut.Selection.Add(FGeoSelectionID::MeshEdge(TriEdgeID).Encoded());
				});
		};

		// TODO: these if statements could be consolidated to minimize repeat code; however its currently set up this way for readability and
		//		 scalability for if/when more EEnumerateSelectionConversionParams are added
		
		if (ConversionParams == EEnumerateSelectionConversionParams::ExpandSelection)
		{
			if (FromSelectionIn.TopologyType == EGeometryTopologyType::Triangle)
			{
				EnumerateTriangleSelectionEdges(FromSelectionIn, Mesh, SelectEdgesFunc);
			}
			else if (FromSelectionIn.TopologyType == EGeometryTopologyType::Polygroup)
			{
				EnumeratePolygroupSelectionEdges(FromSelectionIn, Mesh, *GroupTopology, SelectEdgesFunc);
			}
		}

		else if (ConversionParams == EEnumerateSelectionConversionParams::ContainSelection)
		{
			if (FromSelectionIn.TopologyType == EGeometryTopologyType::Triangle)
			{
				// in the case of TriVert->TriEdge, ensure that only edges where both verts are in the initial selection are included
				if (FromSelectionIn.ElementType == EGeometryElementType::Vertex)
				{
					TSet<uint64> SelectedVerts;
					TArray<int32> AllEdgesConnectedToVerts;
					for (const uint64 VertexID : FromSelectionIn.Selection)
					{
						SelectedVerts.Add(VertexID);
						Mesh.EnumerateVertexEdges((int32)VertexID, [&SelectEdgesFunc, &AllEdgesConnectedToVerts](const int32 EdgeID)
						{
							AllEdgesConnectedToVerts.Add(EdgeID);
						});
					}

					// ensure stable selection by only selecting the edges where BOTH its verts were in init selection
					// however a selection which includes a single vert or any verts without any of their adjacent verts selected will be lost in conversion
					for (const int32 EdgeID : AllEdgesConnectedToVerts) //-V1078
					{
						const FIndex2i EdgeVerts = Mesh.GetEdgeV(EdgeID);
						if (SelectedVerts.Contains(EdgeVerts.A) && SelectedVerts.Contains(EdgeVerts.B))
						{
							SelectEdgesFunc(EdgeID);
						}
					}
				}
				else // case of converting from TriEdge or TriFace -> ensuring stable selection is not applicable and can do same as in ExpandSelection
				{
					EnumerateTriangleSelectionEdges(FromSelectionIn, Mesh, SelectEdgesFunc);
				}
			}
			// case of converting from PolyVert, PolyEdge, PolyFace -> ensuring stable selection is not applicable and can do same as in ExpandSelection
			else if (FromSelectionIn.TopologyType == EGeometryTopologyType::Polygroup)
			{
				EnumeratePolygroupSelectionEdges(FromSelectionIn, Mesh, *GroupTopology, SelectEdgesFunc);
			}
		}
		return true;
	};

	const auto ToTriVtx = [](
		const FDynamicMesh3& Mesh,
		const FGroupTopology* GroupTopology,
		const FGeometrySelection& FromSelectionIn,
		FGeometrySelection& ToSelectionOut,
		const EEnumerateSelectionConversionParams ConversionParams) -> bool
	{
		checkSlow(ToSelectionOut.TopologyType == EGeometryTopologyType::Triangle);
		checkSlow(ToSelectionOut.ElementType == EGeometryElementType::Vertex);

		if (FromSelectionIn.TopologyType == EGeometryTopologyType::Triangle)
		{
			return EnumerateTriangleSelectionVertices(FromSelectionIn, Mesh, nullptr,
			[&ToSelectionOut](const uint64 Vid, const FVector3d& Unused)
			{
				ToSelectionOut.Selection.Add( FGeoSelectionID::MeshVertex((int32)Vid).Encoded() );
			});
		}
		else if (FromSelectionIn.TopologyType == EGeometryTopologyType::Polygroup)
		{
			// TODO Add a function which gets the Vids only, to remove the matrix-vector multiplication we just ignore
			const FTransform Transform = FTransform::Identity;
			return EnumeratePolygroupSelectionVertices(FromSelectionIn, Mesh, GroupTopology, Transform,
				[&ToSelectionOut](const uint64 VID, const FVector3d& Unused)
				{
					ToSelectionOut.Selection.Add( FGeoSelectionID::MeshVertex((int32)VID).Encoded() );
				});
		}
		return true;
	};

	const auto ToPolyFace = [](
		const FDynamicMesh3& Mesh,
		const FGroupTopology* GroupTopology,
		const FGeometrySelection& FromSelectionIn,
		FGeometrySelection& ToSelectionOut,
		const EEnumerateSelectionConversionParams ConversionParams) -> bool
	{
		checkSlow(ToSelectionOut.TopologyType == EGeometryTopologyType::Polygroup);
		checkSlow(ToSelectionOut.ElementType == EGeometryElementType::Face);

		const FPolygroupSet GroupSet = FPolygroupSet(&Mesh);
		EnumerateSelectionTriangles(FromSelectionIn, Mesh,
		[&ToSelectionOut, &GroupTopology](const int32 TID)
		{
			const int GroupID = GroupTopology->GetGroupID(TID);
			const TArray<int>& GroupTris = GroupTopology->GetGroupTriangles(GroupID);
			for (const int GroupTri : GroupTris)
			{
				ToSelectionOut.Selection.Add(FGeoSelectionID(GroupTri, GroupID).Encoded());
			}
		}, &GroupSet);
		return true;
	};

	const auto ToPolyEdge = [](
		const FDynamicMesh3& Mesh,
		const FGroupTopology* GroupTopology,
		const FGeometrySelection& FromSelectionIn,
		FGeometrySelection& ToSelectionOut,
		const EEnumerateSelectionConversionParams ConversionParams)
	{
		checkSlow(ToSelectionOut.TopologyType == EGeometryTopologyType::Polygroup);
		checkSlow(ToSelectionOut.ElementType == EGeometryElementType::Edge);

		bool bConverted = false;

		auto SelectEdgesFunc = [&Mesh, &GroupTopology, &ToSelectionOut, &bConverted] (const int32 EdgeID)
		{
			// similar but simplified version of code in GroupTopology->IsGroupEdge()
			const FIndex2i EdgeT = Mesh.GetEdgeT(EdgeID);
			if (EdgeT.B == IndexConstants::InvalidID)
			{
				bConverted = true;
			}
			const bool bIsGroupEdge = GroupTopology->GetGroupID(EdgeT.A) !=  GroupTopology->GetGroupID(EdgeT.B);
			bConverted = bConverted || bIsGroupEdge;

			if (bIsGroupEdge)
			{
				const int32 GroupEdgeID = GroupTopology->FindGroupEdgeID(EdgeID);
				for (const int32 EID : GroupTopology->Edges[GroupEdgeID].Span.Edges)
				{
					Mesh.EnumerateTriEdgeIDsFromEdgeID(EID,
				[&ToSelectionOut, &GroupEdgeID](const FMeshTriEdgeID TriEdgeID)
					{
						ToSelectionOut.Selection.Add(FGeoSelectionID(TriEdgeID.Encoded(), GroupEdgeID).Encoded());
					});
				}
			}
		};

		if (FromSelectionIn.TopologyType == EGeometryTopologyType::Triangle)
		{
			EnumerateTriangleSelectionEdges(FromSelectionIn, Mesh, SelectEdgesFunc);
		}
		else if (FromSelectionIn.TopologyType == EGeometryTopologyType::Polygroup)
		{
			EnumeratePolygroupSelectionEdges(FromSelectionIn, Mesh, *GroupTopology, SelectEdgesFunc);
		}
		
		return bConverted;
	};

	const auto ToPolyVtx = [](
		const FDynamicMesh3& Mesh,
		const FGroupTopology* GroupTopology,
		const FGeometrySelection& FromSelectionIn,
		FGeometrySelection& ToSelectionOut,
		const EEnumerateSelectionConversionParams ConversionParams) -> bool
	{
		checkSlow(ToSelectionOut.TopologyType == EGeometryTopologyType::Polygroup);
		checkSlow(ToSelectionOut.ElementType == EGeometryElementType::Vertex);

		auto ApplyPolyVertFunc = [&Mesh, GroupTopology, &ToSelectionOut](const uint64 VID, bool &bConverted)
		{
			if (Mesh.IsVertex((int32)VID))
			{
				const int32 CornerID = GroupTopology->GetCornerIDFromVertexID((int32)VID);
				if (CornerID != IndexConstants::InvalidID)
				{
					bConverted = true;
					const FGeoSelectionID ID = FGeoSelectionID((int32)VID, CornerID);
					ToSelectionOut.Selection.Add(ID.Encoded());
				}
			}
		};
		
		bool bConverted = false;
		if (FromSelectionIn.TopologyType == EGeometryTopologyType::Triangle)
		{
			EnumerateTriangleSelectionVertices(FromSelectionIn, Mesh, nullptr, 
			[&bConverted, &ApplyPolyVertFunc](const uint64 VID, const FVector& Unused)
			{
				ApplyPolyVertFunc(VID, bConverted);
			});
		}
		else if (FromSelectionIn.TopologyType == EGeometryTopologyType::Polygroup)
		{
			const FTransform Transform = FTransform::Identity;
			EnumeratePolygroupSelectionVertices(FromSelectionIn, Mesh, GroupTopology, Transform, 
				[&bConverted, &ApplyPolyVertFunc](const uint64 VID, const FVector& Unused)
				{
					ApplyPolyVertFunc(VID, bConverted);
				});
		}
		return bConverted;
	};

	typedef bool (*ConvertSelectionFunc)(
			const FDynamicMesh3& Mesh,
			const FGroupTopology* GroupTopology,
			const FGeometrySelection& FromSelectionIn,
			FGeometrySelection& ToSelectionOut,
			const EEnumerateSelectionConversionParams ConversionParams);

	constexpr ConvertSelectionFunc ConvertFuncs[6][6] = {
		{FromTypeToSame, ToTriEdge,      ToTriFace,      ToPolyVtx,     ToPolyEdge,     ToPolyFace },
		{ToTriVtx,		  FromTypeToSame, ToTriFace,      ToPolyVtx,     ToPolyEdge,     ToPolyFace },
		{ToTriVtx,		  ToTriEdge,      FromTypeToSame, ToPolyVtx,     ToPolyEdge,     ToPolyFace },
		{ToTriVtx,		  ToTriEdge,      ToTriFace,      FromTypeToSame,ToPolyEdge,     ToPolyFace },
		{ToTriVtx,		  ToTriEdge,      ToTriFace,      ToPolyVtx,     FromTypeToSame, ToPolyFace },
		{ToTriVtx,		  ToTriEdge,      ToTriFace,      ToPolyVtx,     ToPolyEdge,     FromTypeToSame }
	};

	const int FromIndex = GetSelectionTypeAsIndex(FromSelectionIn);
	const int ToIndex =   GetSelectionTypeAsIndex(ToSelectionOut);

	return ConvertFuncs[FromIndex][ToIndex](Mesh, GroupTopology, FromSelectionIn, ToSelectionOut, ConversionParams);
}

bool UE::Geometry::ConvertTriangleSelectionToOverlaySelection(
	const FDynamicMesh3& Mesh,
	const FGeometrySelection& MeshSelection,
	TSet<int>& TrianglesOut,
	TSet<int>& VerticesOut,
	FGeometrySelection* IncidentSelection)
{
	if (!ensure(MeshSelection.TopologyType == EGeometryTopologyType::Triangle))
	{
		return false;
	}

	TrianglesOut.Reset();
	VerticesOut.Reset();

	if (MeshSelection.IsEmpty())
	{
		return true;
	}

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		// In this case we get all the information by only visiting the triangles
		EnumerateTriangleSelectionTriangles(MeshSelection, Mesh,
			[&TrianglesOut, &VerticesOut, &Mesh](int32 ValidTid)
		{
			TrianglesOut.Add(ValidTid);
			const FIndex3i Verts = Mesh.GetTriangle(ValidTid);
			VerticesOut.Add(Verts.A);
			VerticesOut.Add(Verts.B);
			VerticesOut.Add(Verts.C);
		});
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge && IncidentSelection)
	{
		IncidentSelection->InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);

		EnumerateTriangleSelectionTriangles(MeshSelection, Mesh,
			[&TrianglesOut](int32 ValidTid)
		{
			TrianglesOut.Add(ValidTid);
		});

		EnumerateTriangleSelectionVertices(MeshSelection, Mesh, nullptr,
			[&VerticesOut, IncidentSelection](uint64 Vid, const FVector3d& Unused)
		{
			IncidentSelection->Selection.Add(FGeoSelectionID::MeshVertex((int)Vid).Encoded() );
			VerticesOut.Add((int)Vid);
		});
	}
	else
	{
		EnumerateTriangleSelectionTriangles(MeshSelection, Mesh,
			[&TrianglesOut](int32 ValidTid)
		{
			TrianglesOut.Add(ValidTid);
		});

		EnumerateTriangleSelectionVertices(MeshSelection, Mesh, nullptr,
			[&VerticesOut](uint64 Vid, const FVector3d& Unused)
		{
			VerticesOut.Add((int)Vid);
		});
	}

	return true;
}

bool UE::Geometry::ConvertPolygroupSelectionToOverlaySelection(
	const FDynamicMesh3& Mesh,
	const FPolygroupSet& GroupSet,
	const FGeometrySelection& MeshSelection,
	TSet<int>& TrianglesOut,
	TSet<int>& VerticesOut)
{
	return EnumeratePolygroupSelectionTriangles(MeshSelection, Mesh, GroupSet,
		[&TrianglesOut, &VerticesOut, &Mesh](int32 ValidTid)
	{
		TrianglesOut.Add(ValidTid);
		const FIndex3i Verts = Mesh.GetTriangle(ValidTid);
		VerticesOut.Add(Verts.A);
		VerticesOut.Add(Verts.B);
		VerticesOut.Add(Verts.C);
	});
}

bool UE::Geometry::ConvertPolygroupSelectionToIncidentOverlaySelection(
	const FDynamicMesh3& Mesh,
	const FGroupTopology& GroupTopology,
	const FGeometrySelection& MeshSelection,
	TSet<int>& TrianglesOut,
	TSet<int>& VerticesOut,
	FGeometrySelection* IncidentSelection)
{
	if (!ensure(MeshSelection.TopologyType == EGeometryTopologyType::Polygroup))
	{
		return false;
	}

	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		if (MeshSelection.TopologyType == EGeometryTopologyType::Polygroup)
		{
			// TODO This uses the polygroup set stored directly in the mesh, this is a source of potential inconsistency
			// with the given GroupTopology
			const FPolygroupSet GroupSet = FPolygroupSet(&Mesh);

			return EnumeratePolygroupSelectionTriangles(MeshSelection, Mesh, GroupSet,
				[&TrianglesOut, &VerticesOut, &Mesh](int32 ValidTid)
			{
				TrianglesOut.Add(ValidTid);
				const FIndex3i Verts = Mesh.GetTriangle(ValidTid);
				VerticesOut.Add(Verts.A);
				VerticesOut.Add(Verts.B);
				VerticesOut.Add(Verts.C);
			});
		}
		else
		{
			return ConvertTriangleSelectionToOverlaySelection(Mesh, MeshSelection, TrianglesOut, VerticesOut);
		}
	}
	else
	{
		FGeometrySelection TempIncidentSelection;
		if (IncidentSelection == nullptr)
		{
			IncidentSelection = &TempIncidentSelection;
		}

		IncidentSelection->InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);

		// GroupTopology argument is ignored if MeshSelection has Triangle topology
		bool bSuccess = ConvertSelection(Mesh, &GroupTopology, MeshSelection, *IncidentSelection, EEnumerateSelectionConversionParams::ContainSelection);
		ensure(bSuccess == true);
		ensure(!IncidentSelection->IsEmpty());

		return ConvertTriangleSelectionToOverlaySelection(Mesh, *IncidentSelection, TrianglesOut, VerticesOut);
	}
}

bool UE::Geometry::MakeSelectAllSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	TFunctionRef<bool(FGeoSelectionID)> SelectionIDPredicate,
	FGeometrySelection& AllSelection)
{
	if (AllSelection.TopologyType == EGeometryTopologyType::Triangle)
	{
		if (AllSelection.ElementType == EGeometryElementType::Vertex)
		{
			for (int32 vid : Mesh.VertexIndicesItr())
			{
				FGeoSelectionID ID = FGeoSelectionID::MeshVertex(vid);
				if ( SelectionIDPredicate(ID) )
				{
					AllSelection.Selection.Add(ID.Encoded());
				}
			}
		}
		else if (AllSelection.ElementType == EGeometryElementType::Edge)
		{
			for (int32 eid : Mesh.EdgeIndicesItr())
			{
				// Test if both half-edges pass the edge selection predicate
				bool bShouldSelect = true;
				Mesh.EnumerateTriEdgeIDsFromEdgeID(eid, [&SelectionIDPredicate, &bShouldSelect](FMeshTriEdgeID TriEdgeID)
				{
					bShouldSelect = bShouldSelect && SelectionIDPredicate(FGeoSelectionID::MeshEdge(TriEdgeID));
				});
				if (bShouldSelect)
				{
					// Select both half-edges
					Mesh.EnumerateTriEdgeIDsFromEdgeID(eid, [&AllSelection](FMeshTriEdgeID TriEdgeID)
					{
						AllSelection.Selection.Add(FGeoSelectionID::MeshEdge(TriEdgeID).Encoded());
					});
				}
			}
		}
		else if (AllSelection.ElementType == EGeometryElementType::Face)
		{
			for (int32 tid : Mesh.TriangleIndicesItr())
			{
				FGeoSelectionID ID = FGeoSelectionID::MeshTriangle(tid);
				if ( SelectionIDPredicate(ID) )
				{
					AllSelection.Selection.Add(ID.Encoded());
				}
			}
		}
		else
		{
			return false;
		}
		return true;
	}
	else if ( AllSelection.TopologyType == EGeometryTopologyType::Polygroup )
	{
		if (!ensure(GroupTopology != nullptr))
		{
			return false;
		}

		if (AllSelection.ElementType == EGeometryElementType::Vertex)
		{
			int32 NumCorners = GroupTopology->Corners.Num();
			for ( int32 CornerID = 0; CornerID < NumCorners; ++CornerID)
			{
				const FGroupTopology::FCorner& Corner = GroupTopology->Corners[CornerID];
				FGeoSelectionID ID = FGeoSelectionID(Corner.VertexID, CornerID);
				if ( SelectionIDPredicate(ID) )
				{
					AllSelection.Selection.Add(ID.Encoded());
				}
			}
		}
		else if (AllSelection.ElementType == EGeometryElementType::Edge)
		{
			int32 NumEdges = GroupTopology->Edges.Num();
			for ( int32 EdgeID = 0; EdgeID < NumEdges; ++EdgeID)
			{
				const FGroupTopology::FGroupEdge& GroupEdge = GroupTopology->Edges[EdgeID];
				FMeshTriEdgeID MeshEdgeID = Mesh.GetTriEdgeIDFromEdgeID(GroupEdge.Span.Edges[0]);
				FGeoSelectionID ID = FGeoSelectionID(MeshEdgeID.Encoded(), EdgeID);
				if ( SelectionIDPredicate(ID) )
				{
					AllSelection.Selection.Add(ID.Encoded());
				}
			}
		}
		else if (AllSelection.ElementType == EGeometryElementType::Face)
		{
			int32 NumFaces = GroupTopology->Groups.Num();
			for ( int32 FaceID = 0; FaceID < NumFaces; ++FaceID)
			{
				const FGroupTopology::FGroup& GroupFace = GroupTopology->Groups[FaceID];
				FGeoSelectionID ID = FGeoSelectionID(GroupFace.Triangles[0], GroupFace.GroupID);
				if ( SelectionIDPredicate(ID) )
				{
					AllSelection.Selection.Add(ID.Encoded());
				}
			}
		}
		else
		{
			return false;
		}
		return true;
	}
	return false;
}



bool UE::Geometry::MakeSelectAllConnectedSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& ReferenceSelection,
	TFunctionRef<bool(FGeoSelectionID)> SelectionIDPredicate,
	TFunctionRef<bool(FGeoSelectionID, FGeoSelectionID)> IsConnectedPredicate,
	FGeometrySelection& AllConnectedSelection)
{
	if ( ! ensure(ReferenceSelection.IsSameType(AllConnectedSelection)) ) return false;

	if (AllConnectedSelection.TopologyType == EGeometryTopologyType::Triangle)
	{
		TArray<int32> CurIndices;
		CurIndices.Reserve(ReferenceSelection.Num());

		if (AllConnectedSelection.ElementType == EGeometryElementType::Vertex)
		{
			for (uint64 ElementID : ReferenceSelection.Selection)
			{
				CurIndices.Add( FGeoSelectionID(ElementID).GeometryID );
			}
			TSet<int32> ConnectedVertices;
			FMeshConnectedComponents::GrowToConnectedVertices(Mesh, CurIndices, ConnectedVertices, nullptr,
				[&](int32 FromVertID, int32 ToVertID) {
					return SelectionIDPredicate(FGeoSelectionID::MeshVertex(ToVertID)) && 
							IsConnectedPredicate( FGeoSelectionID::MeshVertex(FromVertID), FGeoSelectionID::MeshVertex(ToVertID) );
				});
			for (int32 vid : ConnectedVertices)
			{
				AllConnectedSelection.Selection.Add( FGeoSelectionID::MeshVertex(vid).Encoded() );
			}
		}
		else if (AllConnectedSelection.ElementType == EGeometryElementType::Edge)
		{
			for (uint64 ElementID : ReferenceSelection.Selection)
			{
				FMeshTriEdgeID TriEdgeID( FGeoSelectionID(ElementID).GeometryID );
				CurIndices.Add( Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) );
			}
			TSet<int32> ConnectedEdges;
			FMeshConnectedComponents::GrowToConnectedEdges(Mesh, CurIndices, ConnectedEdges, nullptr,
				[&](int32 FromEdgeID, int32 ToEdgeID) {
					FMeshTriEdgeID ToTriEdgeID = Mesh.GetTriEdgeIDFromEdgeID(ToEdgeID), FromTriEdgeID = Mesh.GetTriEdgeIDFromEdgeID(FromEdgeID);
					// Grow if both half-edges pass the predicate
					bool bToEdgeID_SelectionPredicate = true;
					Mesh.EnumerateTriEdgeIDsFromEdgeID(ToEdgeID, [&SelectionIDPredicate, &bToEdgeID_SelectionPredicate](FMeshTriEdgeID TestTriEdgeID)
					{
						bToEdgeID_SelectionPredicate = bToEdgeID_SelectionPredicate && SelectionIDPredicate(FGeoSelectionID::MeshEdge(TestTriEdgeID));
					});
					return bToEdgeID_SelectionPredicate &&
							IsConnectedPredicate( FGeoSelectionID::MeshEdge(FromTriEdgeID), FGeoSelectionID::MeshEdge(ToTriEdgeID) );
				});
			for (int32 EdgeID : ConnectedEdges)
			{
				Mesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID, [&AllConnectedSelection](FMeshTriEdgeID TriEdgeID)
				{
					AllConnectedSelection.Selection.Add(FGeoSelectionID::MeshEdge(TriEdgeID).Encoded());
				});
			}
		}
		else if (AllConnectedSelection.ElementType == EGeometryElementType::Face)
		{
			for (uint64 ElementID : ReferenceSelection.Selection)
			{
				CurIndices.Add(FGeoSelectionID(ElementID).GeometryID);
			}
			TSet<int32> ConnectedTriangles;
			FMeshConnectedComponents::GrowToConnectedTriangles(&Mesh, CurIndices, ConnectedTriangles, nullptr,
				[&](int32 FromTriID, int32 ToTriID) {
					return SelectionIDPredicate(FGeoSelectionID::MeshTriangle(ToTriID)) && 
							IsConnectedPredicate( FGeoSelectionID::MeshTriangle(FromTriID), FGeoSelectionID::MeshTriangle(ToTriID) );
				});
			for (int32 tid : ConnectedTriangles)
			{
				AllConnectedSelection.Selection.Add( FGeoSelectionID::MeshTriangle(tid).Encoded() );
			}
		}
		else
		{
			return false;
		}
		return true;
	}
	else if ( AllConnectedSelection.TopologyType == EGeometryTopologyType::Polygroup )
	{
		if (!ensure(GroupTopology != nullptr))
		{
			return false;
		}
		FGeometrySelectionEditor Editor;
		Editor.Initialize(&AllConnectedSelection, true);
		AllConnectedSelection = ReferenceSelection;
		TArray<uint64> Queue;
		for (uint64 ID : ReferenceSelection.Selection)
		{
			Queue.Add( ID );
		}

		if (AllConnectedSelection.ElementType == EGeometryElementType::Vertex)
		{
			TArray<int32> NbrCornerIDs;
			while (Queue.Num() > 0)
			{
				FGeoSelectionID CurCornerSelectionID = FGeoSelectionID(Queue.Pop(EAllowShrinking::No));
				const FGroupTopology::FCorner& Corner = GroupTopology->Corners[CurCornerSelectionID.TopologyID];
				NbrCornerIDs.Reset();
				GroupTopology->FindCornerNbrCorners(CurCornerSelectionID.TopologyID, NbrCornerIDs);
				for ( int32 NbrCornerID : NbrCornerIDs )
				{
					FGeoSelectionID NbrCornerSelectionID( GroupTopology->Corners[NbrCornerID].VertexID, NbrCornerID );
					if ( Editor.IsSelected(NbrCornerSelectionID.Encoded()) == false 
						&& SelectionIDPredicate(NbrCornerSelectionID) 
						&& IsConnectedPredicate(CurCornerSelectionID, NbrCornerSelectionID) )
					{
						Queue.Add(NbrCornerSelectionID.Encoded());
						Editor.Select(NbrCornerSelectionID.Encoded());
					}
				}
			}
		}
		else if (AllConnectedSelection.ElementType == EGeometryElementType::Edge)
		{
			TArray<int32> NbrEdgeIDs;
			while (Queue.Num() > 0)
			{
				FGeoSelectionID CurEdgeSelectionID = FGeoSelectionID(Queue.Pop(EAllowShrinking::No));
				const FGroupTopology::FGroupEdge& Edge = GroupTopology->Edges[CurEdgeSelectionID.TopologyID];
				NbrEdgeIDs.Reset();
				GroupTopology->FindEdgeNbrEdges(CurEdgeSelectionID.TopologyID, NbrEdgeIDs);
				for ( int32 NbrEdgeID : NbrEdgeIDs )
				{
					FMeshTriEdgeID MeshEdgeID = Mesh.GetTriEdgeIDFromEdgeID(GroupTopology->Edges[NbrEdgeID].Span.Edges[0]);
					FGeoSelectionID NbrEdgeSelectionID( MeshEdgeID.Encoded(), NbrEdgeID);
					if ( Editor.IsSelected(NbrEdgeSelectionID.Encoded()) == false 
						&& SelectionIDPredicate(NbrEdgeSelectionID) 
						&& IsConnectedPredicate(CurEdgeSelectionID, NbrEdgeSelectionID) )
					{
						Queue.Add(NbrEdgeSelectionID.Encoded());
						Editor.Select(NbrEdgeSelectionID.Encoded());
					}
				}
			}
		}
		else if (AllConnectedSelection.ElementType == EGeometryElementType::Face)
		{
			TArray<int32> NbrGroupIDs;
			while (Queue.Num() > 0)
			{
				FGeoSelectionID CurGroupSelectionID = FGeoSelectionID(Queue.Pop(EAllowShrinking::No));
				NbrGroupIDs.Reset();
				for ( int32 NbrGroupID : GroupTopology->GetGroupNbrGroups(CurGroupSelectionID.TopologyID) )
				{
					const FGroupTopology::FGroup* NbrGroup = GroupTopology->FindGroupByID(NbrGroupID);
					FGeoSelectionID NbrGroupSelectionID( NbrGroup->Triangles[0], NbrGroupID);
					if ( Editor.IsSelected(NbrGroupSelectionID.Encoded()) == false 
						&& SelectionIDPredicate(NbrGroupSelectionID) 
						&& IsConnectedPredicate(CurGroupSelectionID, NbrGroupSelectionID) )
					{
						Queue.Add(NbrGroupSelectionID.Encoded());
						Editor.Select(NbrGroupSelectionID.Encoded());
					}
				}
			}

		}
		else
		{
			return false;
		}
		return true;
	}
	return false;

}



bool UE::Geometry::GetSelectionBoundaryVertices(
	const FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& ReferenceSelection,
	TSet<int32>& BorderVidsOut, TSet<int32>& CurVerticesOut)
{
	using namespace GeometrySelectionUtilLocals;

	BorderVidsOut.Reset();
	CurVerticesOut.Reset();
	
	switch (ReferenceSelection.ElementType)
	{
	case EGeometryElementType::Vertex:
		EnumerateVertexElementSelectionVertices(ReferenceSelection, Mesh, GroupTopology, [&CurVerticesOut](uint32 Vid)
		{
			CurVerticesOut.Add(Vid);
		});

		// Border vertices are ones that have some adjacent vertices not in selection
		for (int32 VertexID : CurVerticesOut)
		{
			// a boundary vertex is always on the selection boundary (for this and other selection types)
			bool bIsBoundary = Mesh.IsBoundaryVertex(VertexID);	
			if (!bIsBoundary)
			{
				Mesh.EnumerateVertexVertices(VertexID, [&](int32 NbrVertexID)
				{
					if (!CurVerticesOut.Contains(NbrVertexID))
					{
						bIsBoundary = true;
					}
				});
			}

			if (bIsBoundary)
			{
				BorderVidsOut.Add(VertexID);
			}
		}
		break;
	case EGeometryElementType::Edge:
	{
		// Border vertices are ones that have some adjacent edges that are not in selection, so
		// determine edges in selection.
		TSet<int32> EdgeIDsInSelection;
		EnumerateEdgeElementSelectionEdges(ReferenceSelection, Mesh, GroupTopology, [&EdgeIDsInSelection, &CurVerticesOut, &Mesh](uint32 Eid)
		{
			EdgeIDsInSelection.Add(Eid);
			FIndex2i EdgeV = Mesh.GetEdgeV(Eid);
			CurVerticesOut.Add(EdgeV.A);
			CurVerticesOut.Add(EdgeV.B);
		});

		for (int32 VertexID : CurVerticesOut)
		{
			bool bIsBoundary = Mesh.IsBoundaryVertex(VertexID);
			if (!bIsBoundary)
			{
				Mesh.EnumerateVertexEdges(VertexID, [&EdgeIDsInSelection, &bIsBoundary](int32 EdgeID)
				{
					if (!EdgeIDsInSelection.Contains(EdgeID))
					{
						bIsBoundary = true;
					}
				});
			}

			if (bIsBoundary)
			{
				BorderVidsOut.Add(VertexID);
			}
		}
	}
		break;
	case EGeometryElementType::Face:
	{
		// Border vertices are ones that have some adjacent triangles that are not in selection.
		TSet<int32> TriangleIDsInSelection;
		EnumerateFaceElementSelectionTriangles(ReferenceSelection, Mesh, GroupTopology, [&TriangleIDsInSelection, &CurVerticesOut, &Mesh](uint32 Tid)
		{
			TriangleIDsInSelection.Add(Tid);
			FIndex3i Triangle = Mesh.GetTriangle(Tid);
			CurVerticesOut.Add(Triangle.A);
			CurVerticesOut.Add(Triangle.B);
			CurVerticesOut.Add(Triangle.C);
		});

		for (int32 VertexID : CurVerticesOut)
		{
			bool bIsBoundary = Mesh.IsBoundaryVertex(VertexID);
			if (!bIsBoundary)
			{
				Mesh.EnumerateVertexTriangles(VertexID, [&TriangleIDsInSelection, &bIsBoundary](int32 TriangleID)
				{
					if (!TriangleIDsInSelection.Contains(TriangleID))
					{
						bIsBoundary = true;
					}
				});
			}

			if (bIsBoundary)
			{
				BorderVidsOut.Add(VertexID);
			}
		}
	}
		break;
	default:
		return ensure(false);
	}

	return true;
}



bool UE::Geometry::GetSelectionBoundaryCorners(
	const FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& ReferenceSelection,
	TSet<int32>& BorderCornerIDsOut, TSet<int32>& CurCornerIDsOut)
{
	BorderCornerIDsOut.Reset();
	CurCornerIDsOut.Reset();

	if (!ensure(GroupTopology))
	{
		return false;
	}

	if (!ensure(ReferenceSelection.TopologyType == EGeometryTopologyType::Polygroup))
	{
		// We don't currently support triangle selections here in part because it's not clear what to do. The
		// proper thing is likely to convert to an equivalent polygroup selection and find border corners, but
		// we haven't yet defined some of those conversions. Alternatively we could find the border vertices and 
		// keep whichever ones happen to be corners, but that gives the unintuitive result of not giving any corners
		// for selections that don't happen to line up with group boundaries.
		// There's also the fact that we don't yet have a use case for supporting this here.
		return false;
	}

	TArray<int32> NbrArray;

	switch (ReferenceSelection.ElementType)
	{
	case EGeometryElementType::Vertex:
	{
		// Assemble included corners
		for (uint64 ID : ReferenceSelection.Selection)
		{
			CurCornerIDsOut.Add(FGeoSelectionID(ID).TopologyID);		// TODO: can we rely on TopologyID being stable here, or do we need to look up from VertexID?
		}

		// Border corners are ones that have a corner neighbor not in the selection
		for (int32 CornerID : CurCornerIDsOut)
		{
			// Boundary vertex corners are always considered to be on selection boundary (for this and other selection types)
			bool bIsBoundary = Mesh.IsBoundaryVertex(GroupTopology->GetCornerVertexID(CornerID));
			if (!bIsBoundary)
			{
				NbrArray.Reset();
				GroupTopology->FindCornerNbrCorners(CornerID, NbrArray);
				for (int32 NbrCornerID : NbrArray)
				{
					if (!CurCornerIDsOut.Contains(NbrCornerID))
					{
						bIsBoundary = true;
						break;
					}
				}
			}

			if (bIsBoundary)
			{
				BorderCornerIDsOut.Add(CornerID);
			}
		}
	}
		break;
	case EGeometryElementType::Edge:
	{
		// Assemble the current group edge selection and the included corners.
		TSet<int32> GroupEdgeIDsInSelection;
		for (uint64 ID : ReferenceSelection.Selection)
		{
			int32 GroupEdgeID = FGeoSelectionID(ID).TopologyID;
			GroupEdgeIDsInSelection.Add(GroupEdgeID);

			const FGroupTopology::FGroupEdge& Edge = GroupTopology->Edges[GroupEdgeID];
			if (Edge.EndpointCorners.A != IndexConstants::InvalidID)
			{
				CurCornerIDsOut.Add(Edge.EndpointCorners.A);
			}
			if (Edge.EndpointCorners.B != IndexConstants::InvalidID)
			{
				CurCornerIDsOut.Add(Edge.EndpointCorners.B);
			}
		}

		// Border corners are ones that have some attached group edges that are not in the current selection.
		for (int32 CornerID : CurCornerIDsOut)
		{
			bool bIsBoundary = Mesh.IsBoundaryVertex(GroupTopology->GetCornerVertexID(CornerID));
			if (!bIsBoundary)
			{
				NbrArray.Reset();
				GroupTopology->FindCornerNbrEdges(CornerID, NbrArray);
				for (int32 GroupEdgeID : NbrArray)
				{
					if (!GroupEdgeIDsInSelection.Contains(GroupEdgeID))
					{
						bIsBoundary = true;
						break;
					}
				}
			}

			if (bIsBoundary)
			{
				BorderCornerIDsOut.Add(CornerID);
			}
		}
	}
		break;
	case EGeometryElementType::Face:
	{
		// Assemble current group selection and the included corners
		TSet<int32> GroupsInSelection;
		for (uint64 ID : ReferenceSelection.Selection)
		{
			int32 GroupID = FGeoSelectionID(ID).TopologyID;
			GroupsInSelection.Add(GroupID);
			GroupTopology->ForGroupEdges(GroupID, [&](const FGroupTopology::FGroupEdge& Edge, int)
			{
				if (Edge.EndpointCorners.A != IndexConstants::InvalidID)
				{
					CurCornerIDsOut.Add(Edge.EndpointCorners.A);
				}
				if (Edge.EndpointCorners.B != IndexConstants::InvalidID)
				{
					CurCornerIDsOut.Add(Edge.EndpointCorners.B);
				}
			});
		}

		// Boundary corners are ones that have an attached group not in selection
		for (int32 CornerID : CurCornerIDsOut)
		{
			bool bIsBoundary = Mesh.IsBoundaryVertex(GroupTopology->GetCornerVertexID(CornerID));
			if (!bIsBoundary)
			{
				NbrArray.Reset();
				GroupTopology->FindCornerNbrGroups(CornerID, NbrArray);
				for (int32 Group : NbrArray)
				{
					if (!GroupsInSelection.Contains(Group))
					{
						bIsBoundary = true;
						break;
					}
				}
			}

			if (bIsBoundary)
			{
				BorderCornerIDsOut.Add(CornerID);
			}
		}
	}
		break;
	default:
		return ensure(false);
	}

	return true;
}



bool UE::Geometry::MakeBoundaryConnectedSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& ReferenceSelection,
	TFunctionRef<bool(FGeoSelectionID)> SelectionIDPredicate,
	FGeometrySelection& BoundaryConnectedSelection)
{
	using namespace GeometrySelectionUtilLocals;

	if (BoundaryConnectedSelection.TopologyType == EGeometryTopologyType::Triangle)
	{
		TSet<int32> BorderVertices;
		TSet<int32> CurVertices;
		if (!GetSelectionBoundaryVertices(Mesh, GroupTopology, ReferenceSelection, BorderVertices, CurVertices))
		{
			return false;
		}

		// Now select elements connected to the border vertices.
		if (BoundaryConnectedSelection.ElementType == EGeometryElementType::Vertex)
		{
			TSet<int32> AdjacentVertices = BorderVertices;
			for (int32 VertexID : BorderVertices)
			{
				Mesh.EnumerateVertexVertices(VertexID, [&](int32 NbrVertexID)
				{
					// filter out interior vertices, maybe should be a parameter
					if ( CurVertices.Contains(NbrVertexID) == false )
					{
						AdjacentVertices.Add(NbrVertexID);
					}
				});
			}
			for (int32 VertexID : AdjacentVertices)
			{
				if (SelectionIDPredicate(FGeoSelectionID::MeshVertex(VertexID)))
				{
					BoundaryConnectedSelection.Selection.Add(FGeoSelectionID::MeshVertex(VertexID).Encoded());
				}
			}
		}
		else if (BoundaryConnectedSelection.ElementType == EGeometryElementType::Edge)
		{
			TSet<int32> AdjacentEdges;
			for (int32 VertexID : BorderVertices)
			{
				for ( int32 EdgeID : Mesh.VtxEdgesItr(VertexID) )
				{
					AdjacentEdges.Add(EdgeID);
				}
			}
			for (int32 EdgeID : AdjacentEdges)
			{
				// Test if both half-edges pass the edge selection predicate
				bool bShouldSelect = true;
				Mesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID, [&SelectionIDPredicate, &bShouldSelect](FMeshTriEdgeID TriEdgeID)
				{
					bShouldSelect = bShouldSelect && SelectionIDPredicate(FGeoSelectionID::MeshEdge(TriEdgeID));
					});
				if (bShouldSelect)
				{
					// Select both half-edges
					Mesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID, [&BoundaryConnectedSelection](FMeshTriEdgeID TriEdgeID)
					{
						BoundaryConnectedSelection.Selection.Add(FGeoSelectionID::MeshEdge(TriEdgeID).Encoded());
					});
				}
			}
		}
		else if (BoundaryConnectedSelection.ElementType == EGeometryElementType::Face)
		{
			TSet<int32> AdjacentTriangles;
			for (int32 VertexID : BorderVertices)
			{
				Mesh.EnumerateVertexTriangles(VertexID, [&](int32 NbrTriangleID)
				{
					AdjacentTriangles.Add(NbrTriangleID);
				});
			}
			for (int32 TriangleID : AdjacentTriangles)
			{
				if (SelectionIDPredicate(FGeoSelectionID::MeshTriangle(TriangleID)))
				{
					BoundaryConnectedSelection.Selection.Add(FGeoSelectionID::MeshTriangle(TriangleID).Encoded());
				}
			}
		}
		else
		{
			return false;
		}
		return true;
	}
	else if ( BoundaryConnectedSelection.TopologyType == EGeometryTopologyType::Polygroup )
	{
		if (!ensure(GroupTopology != nullptr))
		{
			return false;
		}

		TSet<int32> CurCornerIDs;
		TSet<int32> BorderCorners;
		
		if (!GetSelectionBoundaryCorners(Mesh, GroupTopology, ReferenceSelection, BorderCorners, CurCornerIDs))
		{
			return false;
		}

		TArray<int32> NbrArray;

		// now that we have boundary corners, iterate over them and select connected elements
		if (BoundaryConnectedSelection.ElementType == EGeometryElementType::Vertex)
		{
			TSet<int32> AdjacentCorners = BorderCorners;
			for (int32 CornerID : BorderCorners)
			{
				NbrArray.Reset();
				GroupTopology->FindCornerNbrCorners(CornerID, NbrArray);
				for (int32 NbrCornerID : NbrArray)
				{
					// filter out interior corners, maybe should be a parameter
					if ( CurCornerIDs.Contains(NbrCornerID) == false )
					{
						AdjacentCorners.Add(NbrCornerID);
					}
				}
			}
			for (int32 CornerID : AdjacentCorners)
			{
				FGeoSelectionID SelectionID( GroupTopology->GetCornerVertexID(CornerID), CornerID);
				if (SelectionIDPredicate(SelectionID) )
				{
					BoundaryConnectedSelection.Selection.Add( SelectionID.Encoded() );
				}
			}
		}
		else if (BoundaryConnectedSelection.ElementType == EGeometryElementType::Edge)
		{
			TSet<int32> AdjacentEdges;
			for (int32 CornerID : BorderCorners)
			{
				NbrArray.Reset();
				GroupTopology->FindCornerNbrEdges(CornerID, NbrArray);
				for (int32 NbrEdgeID : NbrArray)
				{
					AdjacentEdges.Add(NbrEdgeID);
				}
			}
			for (int32 EdgeID : AdjacentEdges)
			{
				FMeshTriEdgeID MeshEdgeID = Mesh.GetTriEdgeIDFromEdgeID(GroupTopology->GetGroupEdgeEdges(EdgeID)[0]);
				FGeoSelectionID SelectionID( MeshEdgeID.Encoded(), EdgeID);
				if (SelectionIDPredicate(SelectionID) )
				{
					BoundaryConnectedSelection.Selection.Add( SelectionID.Encoded() );
				}
			}
		}
		else   // already verified we are only vertex/edge/face above
		{
			TSet<int32> AdjacentGroups;
			for (int32 CornerID : BorderCorners)
			{
				NbrArray.Reset();
				GroupTopology->FindCornerNbrGroups(CornerID, NbrArray);
				for (int32 NbrGroupID : NbrArray)
				{
					AdjacentGroups.Add(NbrGroupID);
				}
			}
			for (int32 GroupID : AdjacentGroups)
			{
				FGeoSelectionID SelectionID( GroupTopology->GetGroupTriangles(GroupID)[0], GroupID);
				if (SelectionIDPredicate(SelectionID) )
				{
					BoundaryConnectedSelection.Selection.Add( SelectionID.Encoded() );
				}
			}
		}

		return true;
	}
	return false;
}





bool UE::Geometry::CombineSelectionInPlace(
	FGeometrySelection& SelectionA,
	const FGeometrySelection& SelectionB,
	EGeometrySelectionCombineModes CombineMode)
{
	if (SelectionA.IsSameType(SelectionB) == false)
	{
		return false;
	}

	if (SelectionA.TopologyType == EGeometryTopologyType::Triangle)
	{
		if (CombineMode == EGeometrySelectionCombineModes::Add)
		{
			for (uint64 ItemB : SelectionB.Selection)
			{
				SelectionA.Selection.Add(ItemB);
			}
		}
		else if (CombineMode == EGeometrySelectionCombineModes::Subtract)
		{
			if (SelectionB.IsEmpty() == false)
			{
				for (uint64 ItemB : SelectionB.Selection)
				{
					SelectionA.Selection.Remove(ItemB);
				}
				SelectionA.Selection.Compact();
			}
		}
		else if (CombineMode == EGeometrySelectionCombineModes::Intersection)
		{
			TArray<uint64, TInlineAllocator<32>> ToRemove;
			for (uint64 ItemA : SelectionA.Selection)
			{
				if (!SelectionB.Selection.Contains(ItemA))
				{
					ToRemove.Add(ItemA);
				}
			}
			if (ToRemove.Num() > 0)
			{
				for (uint64 ItemA : ToRemove)
				{
					SelectionA.Selection.Remove(ItemA);
				}
				SelectionA.Selection.Compact();
			}
		}

		return true;
	}
	else if (SelectionA.TopologyType == EGeometryTopologyType::Polygroup)
	{
		// for Polygroup selections, we cannot rely on TSet operations because we have set an arbitrary Triangle ID 
		// as the 'geometry' key.
		if (CombineMode == EGeometrySelectionCombineModes::Add)
		{
			for (uint64 ItemB : SelectionB.Selection)
			{
				uint64 FoundItemA;
				if ( UE::Geometry::FindInSelectionByTopologyID(SelectionA, FGeoSelectionID(ItemB).TopologyID, FoundItemA) == false)
				{
					SelectionA.Selection.Add(ItemB);
				}
			}
		}
		else if (CombineMode == EGeometrySelectionCombineModes::Subtract)
		{
			if (SelectionB.IsEmpty() == false)
			{
				for (uint64 ItemB : SelectionB.Selection)
				{
					uint64 FoundItemA;
					if (UE::Geometry::FindInSelectionByTopologyID(SelectionA, FGeoSelectionID(ItemB).TopologyID, FoundItemA))
					{
						SelectionA.Selection.Remove(FoundItemA);
					}
				}
				SelectionA.Selection.Compact();
			}
		}
		else if (CombineMode == EGeometrySelectionCombineModes::Intersection)
		{
			TArray<uint64, TInlineAllocator<32>> ToRemove;
			for (uint64 ItemA : SelectionA.Selection)
			{
				uint64 FoundItemB;
				if (UE::Geometry::FindInSelectionByTopologyID(SelectionA, FGeoSelectionID(ItemA).TopologyID, FoundItemB) == false)
				{
					ToRemove.Add(ItemA);
				}
			}
			if (ToRemove.Num() > 0)
			{
				for (uint64 ItemA : ToRemove)
				{
					SelectionA.Selection.Remove(ItemA);
				}
				SelectionA.Selection.Compact();
			}
		}

		return true;
	}

	return false;
}



bool UE::Geometry::GetTriangleSelectionFrame(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	FFrame3d& SelectionFrameOut)
{
	if ( ensure( MeshSelection.TopologyType == EGeometryTopologyType::Triangle ) == false )
	{
		return false;
	}

	FVector3d AccumulatedOrigin = FVector3d::Zero();
	FVector3d AccumulatedNormal = FVector3d::Zero();
	FVector3d AxisHint = FVector3d::Zero();
	double AccumWeight = 0;
	
	if (MeshSelection.ElementType == EGeometryElementType::Face)
	{
		for (uint64 EncodedID : MeshSelection.Selection)
		{
			int32 TriangleID = (int32)FGeoSelectionID(EncodedID).GeometryID;
			if (Mesh.IsTriangle(TriangleID))
			{
				FVector3d Normal, Centroid; double Area; 
				Mesh.GetTriInfo(TriangleID, Normal, Area, Centroid);
				if (Normal.SquaredLength() > 0.9)
				{
					Area = FMath::Max(Area, 0.000001);
					AccumulatedOrigin += Area * Centroid;
					AccumulatedNormal += Area * Normal;
					AccumWeight += Area;
				}
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Edge)
	{
		for (uint64 EncodedID : MeshSelection.Selection)
		{
			FMeshTriEdgeID TriEdgeID( FGeoSelectionID(EncodedID).GeometryID );
			int32 EdgeID = Mesh.IsTriangle(TriEdgeID.TriangleID) ? Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
			if ( Mesh.IsEdge(EdgeID) )
			{
				FVector3d A, B;
				Mesh.GetEdgeV(EdgeID, A, B);
				AccumulatedOrigin += (A + B) * 0.5;
				AccumulatedNormal += Mesh.GetEdgeNormal(EdgeID);
				AxisHint += Normalized(B - A);
				AccumWeight += 1.0;
			}
		}
	}
	else if (MeshSelection.ElementType == EGeometryElementType::Vertex)
	{
		for (uint64 EncodedID : MeshSelection.Selection)
		{
			int32 VertexID = (int32)FGeoSelectionID(EncodedID).GeometryID;
			if (Mesh.IsVertex(VertexID))
			{
				AccumulatedOrigin += Mesh.GetVertex(VertexID);
				AccumulatedNormal += FMeshNormals::ComputeVertexNormal(Mesh, VertexID);		// this could return area!
				AccumWeight += 1.0;
			}
		}
	}
	else
	{
		return false;
	}

	// todo use AxisHint!

	SelectionFrameOut = FFrame3d();
	if (AccumWeight > 0)
	{
		AccumulatedOrigin /= (double)AccumWeight;
		Normalize(AccumulatedNormal);

		// We set our frame Z to be accumulated normal, and the other two axes are unconstrained, so
		// we want to set them to something that will make our frame generally more useful. If the normal
		// is aligned with world Z, then the entire frame might as well be aligned with world.
		if (1 - AccumulatedNormal.Dot(FVector3d::UnitZ()) < KINDA_SMALL_NUMBER)
		{
			SelectionFrameOut = FFrame3d(AccumulatedOrigin, FQuaterniond::Identity());
		}
		else
		{
			// Otherwise, let's place one of the other axes into the XY plane so that the frame is more
			// useful for translation. We somewhat arbitrarily choose Y for this. 
			FVector3d FrameY = Normalized(AccumulatedNormal.Cross(FVector3d::UnitZ())); // orthogonal to world Z and frame Z 
			FVector3d FrameX = FrameY.Cross(AccumulatedNormal); // safe to not normalize because already orthogonal
			SelectionFrameOut = FFrame3d(AccumulatedOrigin, FrameX, FrameY, AccumulatedNormal);
		}
	}

	return true;
}

void UE::Geometry::TaperPerTriangleValues(const UE::Geometry::FDynamicMesh3& Mesh,
	const TSet<int32>& TriangleROI,
	TFunction<int32(int32)> ValueForTriangleFunc,
	TArray<int>& OutTriangleValues)
{
	using namespace UE::Geometry;

	OutTriangleValues.SetNum(Mesh.MaxTriangleID());
	for (const int32 TriangleIndex : Mesh.TriangleIndicesItr())
	{
		if (Mesh.IsTriangle(TriangleIndex))
		{
			OutTriangleValues[TriangleIndex] = ValueForTriangleFunc(TriangleIndex);
		}
	}

	// If a triangle is on the boundary of a mesh or on the boundary of the selection region, set its level to zero
	for (int32 TriangleID = 0; TriangleID < Mesh.MaxTriangleID(); ++TriangleID)
	{
		if (Mesh.IsTriangle(TriangleID))
		{
			bool bTriangleShouldHaveLevel = true;

			const FIndex3i Tri = Mesh.GetTriangle(TriangleID);
			for (int32 TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
			{
				const int32 VertexID = Tri[TriVertIndex];
				if (Mesh.IsBoundaryVertex(VertexID))
				{
					bTriangleShouldHaveLevel = false;
					break;
				}

				for (const int32 IncidentEdgeID : Mesh.VtxEdgesItr(VertexID))
				{
					const FIndex2i EdgeTriangles = Mesh.GetEdgeT(IncidentEdgeID);
					if (!TriangleROI.Contains(EdgeTriangles[0]) || !TriangleROI.Contains(EdgeTriangles[1]))
					{
						bTriangleShouldHaveLevel = false;
						break;
					}
				}
			}

			if (!bTriangleShouldHaveLevel)
			{
				OutTriangleValues[TriangleID] = 0;
			}
		}
	}

	// TODO: This could be slow in the worst case. Maybe swap out for a BFS
	bool bChanged = true;
	while (bChanged)
	{
		bChanged = false;

		for (int32 TriID : Mesh.TriangleIndicesItr())
		{
			int32 LowestNeighborLevel = ValueForTriangleFunc(TriID);
			const FIndex3i Tri = Mesh.GetTriangle(TriID);
			for (int32 TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
			{
				for (const int32 NeighborTri : Mesh.VtxTrianglesItr(Tri[TriVertIndex]))
				{
					if (Mesh.IsTriangle(NeighborTri) && TriangleROI.Contains(NeighborTri))
					{
						LowestNeighborLevel = FMath::Min(LowestNeighborLevel, OutTriangleValues[NeighborTri]);
					}
				}
			}

			if (OutTriangleValues[TriID] > LowestNeighborLevel + 1 && LowestNeighborLevel + 1 <= ValueForTriangleFunc(TriID))
			{
				OutTriangleValues[TriID] = LowestNeighborLevel + 1;
				bChanged = true;
			}
		}
	}
}
