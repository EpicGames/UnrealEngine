// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selections/QuadGridPatch.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Algo/Reverse.h"

using namespace UE::Geometry;


namespace UELocal
{

	static bool FindTriEdgeIndices(
		const FIndex3i& Triangle,
		int32 EdgeVert0, int32 EdgeVert1,
		int32& EdgeIndex0, int32& EdgeIndex1, int32& OtherIndex2)
	{
		for (int32 j = 0; j < 3; ++j)
		{
			if (Triangle[j] == EdgeVert0)
			{
				if (Triangle[(j + 1) % 3] == EdgeVert1)
				{
					EdgeIndex0 = j; EdgeIndex1 = (j + 1) % 3; OtherIndex2 = (j + 2) % 3; 
					return true;
				}
				else if (Triangle[(j + 2) % 3] == EdgeVert1)
				{
					EdgeIndex0 = j; EdgeIndex1 = (j + 2) % 3; OtherIndex2 = (j + 1) % 3; 
					return true;
				}
			}
		}
		return false;
	}

}


void FQuadGridPatch::InitializeFromQuadStrip(const FDynamicMesh3& Mesh, const TArray<FIndex2i>& SequentialQuadsIn, const TArray<int32>& FirstVertexSpan )
{
	int32 NumV = FirstVertexSpan.Num();
	int32 NumQ = SequentialQuadsIn.Num();
	check(NumQ == NumV-1);

	NumVertexColsU = NumV;
	NumVertexRowsV = 2;

	VertexSpans.SetNum(2);
	VertexSpans[0] = FirstVertexSpan;
	VertexSpans[1].Reserve(NumV);
	QuadTriangles.SetNum(1);
	QuadTriangles[0] = SequentialQuadsIn;

	for ( int32 k = 0; k < NumQ; ++k )
	{
		int Vertex0 = FirstVertexSpan[k];
		int Vertex1 = FirstVertexSpan[k+1];

		FIndex2i& QuadTris = QuadTriangles[0][k];
		FIndex3i TriA = Mesh.GetTriangle(QuadTris.A);
		FIndex3i TriB = Mesh.GetTriangle(QuadTris.B);
		// we want TriA to be the one containing the edge (Vertex0,Vertex1), so if it's TriB, swap them, and the quad
		if ( TriB.Contains(Vertex0) && TriB.Contains(Vertex1) )
		{
			Swap(TriA, TriB);
			Swap(QuadTris.A, QuadTris.B);
		}
		// at this point we are not sure if OtherVertex matches with Vertex0 or Vertex1
		int OtherVertex = IndexUtil::FindTriOtherVtx(Vertex0, Vertex1, TriA);
		bool bIsCase1 = (TriB.Contains(Vertex0) == false);
		int Other0 = (bIsCase1) ? OtherVertex : IndexUtil::FindTriOtherVtx(Vertex0, OtherVertex, TriB);
		int Other1 = (bIsCase1) ? IndexUtil::FindTriOtherVtx(Vertex1, OtherVertex, TriB) : OtherVertex;

		if (k == 0)
		{
			VertexSpans[1].Add(Other0);
		}
		checkSlow(VertexSpans[1].Last() == Other0);
		VertexSpans[1].Add(Other1);
	}
}



bool FQuadGridPatch::InitializeFromQuadPatch(const FDynamicMesh3& Mesh, const TArray<TArray<FIndex2i>>& QuadRowsIn, const TArray<TArray<int32>>& VertexSpansIn)
{
	int32 NumV = VertexSpansIn[0].Num();
	int32 NumQ = QuadRowsIn[0].Num();
	if (NumQ != NumV - 1 || QuadRowsIn.Num() != VertexSpansIn.Num()-1 )
	{
		ensure(false);
		return false;
	}

	NumVertexColsU = NumV;
	NumVertexRowsV = VertexSpansIn.Num();
	for (int32 j = 1; j < NumVertexRowsV; ++j)
	{
		if (VertexSpansIn[j].Num() != VertexSpansIn[0].Num())
		{
			ensure(false);
			return false;
		}
	}
	int32 NumQuadRows = QuadRowsIn.Num();
	for (int32 j = 1; j < NumQuadRows; ++j)
	{
		if (QuadRowsIn[j].Num() != QuadRowsIn[0].Num())
		{
			ensure(false);
			return false;
		}
	}

	VertexSpans = VertexSpansIn;
	QuadTriangles = QuadRowsIn;

	bool bAllOK = true;
	for (int32 j = 0; j < NumQuadRows && bAllOK; ++j)
	{
		for (int32 k = 0; k < NumQ && bAllOK; ++k)
		{
			// these should be the four vertices of the quad
			int VertexA = VertexSpans[j][k];
			int VertexB = VertexSpans[j][k+1];
			int VertexC = VertexSpans[j+1][k+1];
			int VertexD = VertexSpans[j+1][k];

			// figure out unique vertices from the quad info, and verify that there are 4 and they are A/B/C/D
			FIndex2i& QuadTris = QuadTriangles[j][k];
			if (Mesh.IsTriangle(QuadTris.A) == false || Mesh.IsTriangle(QuadTris.B) == false)
			{
				bAllOK = false;
				break;
			}
			FIndex3i TriA = Mesh.GetTriangle(QuadTris.A);
			FIndex3i TriB = Mesh.GetTriangle(QuadTris.B);
			TArray<int32, TInlineAllocator<6>> TriVerts({ TriA.A, TriA.B, TriA.C });
			TriVerts.AddUnique(TriB.A); TriVerts.AddUnique(TriB.B); TriVerts.AddUnique(TriB.C);
			if (TriVerts.Num() != 4) 
			{
				bAllOK = false;
				break;
			}
			if (TriVerts.Contains(VertexA) == false || TriVerts.Contains(VertexB) == false || TriVerts.Contains(VertexC) == false || TriVerts.Contains(VertexD) == false)
			{
				bAllOK = false;
				break;
			}

			// we want TriA to be the one containing the edge (Vertex0,Vertex1), so if it's TriB, swap them, and the quad
			if (TriB.Contains(VertexA) && TriB.Contains(VertexB))
			{
				Swap(TriA, TriB);
				Swap(QuadTris.A, QuadTris.B);
			}
		}
	}

	if (!bAllOK)
	{
		ensure(false);
		VertexSpans.Reset();
		QuadTriangles.Reset();
		NumVertexColsU = NumVertexRowsV = 0;
		return false;
	}

	return true;
}



void FQuadGridPatch::ReverseRows()
{
	Algo::Reverse(QuadTriangles);
	Algo::Reverse(VertexSpans);
}



bool FQuadGridPatch::AppendQuadPatchRows(FQuadGridPatch&& NextPatch, bool bChecked)
{
	if ( NextPatch.NumVertexColsU != NumVertexColsU ) return false;

	if ( bChecked )
	{
		const TArray<int32>& LastSpan = VertexSpans.Last();
		const TArray<int32>& FirstSpan = NextPatch.VertexSpans[0];
		for ( int32 k = 0; k < NumVertexColsU; ++k )
		{
			if (LastSpan[k] != FirstSpan[k])
			{
				check(false);
				return false;
			}
		}
	}

	int AddedV = NextPatch.NumVertexRowsV-1;
	for ( int32 k = 1; k < NextPatch.NumVertexRowsV; ++k )
	{
		VertexSpans.Add( MoveTemp(NextPatch.VertexSpans[k]) );
	}
	for ( int32 k = 0; k < NextPatch.NumVertexRowsV-1; ++k )
	{
		QuadTriangles.Add( MoveTemp(NextPatch.QuadTriangles[k]) );
	}
	NumVertexRowsV = VertexSpans.Num();
	return true;
}


void FQuadGridPatch::GetAllTriangles(TArray<int32>& AllTrianglesOut) const
{
	AllTrianglesOut.Reserve(AllTrianglesOut.Num() + 2*GetNumQuads());
	for ( const TArray<FIndex2i>& QuadStrip : QuadTriangles )
	{
		for (FIndex2i Quad : QuadStrip)
		{
			AllTrianglesOut.Add(Quad.A);
			AllTrianglesOut.Add(Quad.B);
		}
	}
}



FIndex3i FQuadGridPatch::GetQuadTriMappedToSpanIndices(const FDynamicMesh3& Mesh, int QuadRow, int QuadIndex, int TriIndex) const
{
	check(TriIndex == 0 || TriIndex == 1);
	check(QuadRow >= 0 && QuadRow < (NumVertexRowsV-1) );
	check(QuadIndex >= 0 && QuadIndex < (NumVertexColsU-1) );
	FIndex3i Tri = Mesh.GetTriangle( QuadTriangles[QuadRow][QuadIndex][TriIndex] );
	//        C - D
	//        |   |
	//        A - B
	FIndex4i QuadVertices(
		VertexSpans[QuadRow][QuadIndex], VertexSpans[QuadRow][QuadIndex+1],
		VertexSpans[QuadRow+1][QuadIndex], VertexSpans[QuadRow+1][QuadIndex+1] );
	FIndex4i QuadIndices(
		EncodeRowColumnIndex(QuadRow, QuadIndex), EncodeRowColumnIndex(QuadRow,QuadIndex+1),
		EncodeRowColumnIndex(QuadRow+1, QuadIndex), EncodeRowColumnIndex(QuadRow+1,QuadIndex+1));
	FIndex3i TriOrder, EncodedIndexTri = FIndex3i::Invalid();
	if ( UELocal::FindTriEdgeIndices(Tri, QuadVertices.A, QuadVertices.D, TriOrder.A, TriOrder.B, TriOrder.C) )
	{
		// edge A/D exists case
		EncodedIndexTri[TriOrder.A] = QuadIndices.A;
		EncodedIndexTri[TriOrder.B] = QuadIndices.D;
		EncodedIndexTri[TriOrder.C] = (Tri[TriOrder.C] == QuadVertices.C) ? QuadIndices.C : QuadIndices.B;
	}
	else if ( UELocal::FindTriEdgeIndices(Tri, QuadVertices.B, QuadVertices.C, TriOrder.A, TriOrder.B, TriOrder.C) )
	{
		// edge B/C exists case
		EncodedIndexTri[TriOrder.A] = QuadIndices.B;
		EncodedIndexTri[TriOrder.B] = QuadIndices.C;
		EncodedIndexTri[TriOrder.C] = (Tri[TriOrder.C] == QuadVertices.D) ? QuadIndices.D : QuadIndices.A;
	}
	else
	{
		check(false);		// ruh roh! it should have been one of those two other cases...
	}
	return EncodedIndexTri;
}



bool FQuadGridPatch::GetVertexColumn(int32 ColumnIndex, TArray<int32>& VerticesOut) const
{
	if (ColumnIndex < 0 || ColumnIndex >= VertexSpans[0].Num())
	{
		ensure(false);
		return false;
	}

	VerticesOut.Reset();
	for (int32 k = 0; k < VertexSpans.Num(); ++k)
	{
		VerticesOut.Add(VertexSpans[k][ColumnIndex]);
	}
	return true;
}

int32 FQuadGridPatch::FindColumnIndex(int32 VertexID) const
{
	for (const TArray<int32>& Span : VertexSpans)
	{
		int32 Index = Span.IndexOfByKey(VertexID);
		if (Index != INDEX_NONE)
		{
			return Index;
		}
	}
	return IndexConstants::InvalidID;
}


void FQuadGridPatch::GetSubPatchByQuadRange(int StartRow, int EndRow, int StartCol, int EndCol, FQuadGridPatch& PatchOut ) const
{
	int NumQuadRows = (EndRow-StartRow)+1;
	int NumQuadCols = (EndCol-StartCol)+1;

	PatchOut.NumVertexRowsV = NumQuadRows+1;
	PatchOut.NumVertexColsU = NumQuadCols+1;
	PatchOut.VertexSpans.SetNum(PatchOut.NumVertexRowsV);
	PatchOut.QuadTriangles.SetNum(PatchOut.NumVertexRowsV-1);
	for ( int32 ri = 0; ri < NumQuadRows; ++ri )
	{
		int32 row = StartRow + ri;
		for ( int32 ci = 0; ci < NumQuadCols; ++ci )
		{
			int col = StartCol + ci;
			PatchOut.VertexSpans[ri].Add( VertexSpans[row][col] );
			PatchOut.QuadTriangles[ri].Add( QuadTriangles[row][col] );
		}
		PatchOut.VertexSpans[ri].Add( VertexSpans[row][EndCol+1] );
	}
	// last row
	for ( int32 ci = 0; ci <= NumQuadCols; ++ci )
	{
		int col = StartCol + ci;
		PatchOut.VertexSpans[NumQuadRows].Add( VertexSpans[EndRow+1][col] );
	}
}


void FQuadGridPatch::SplitColumnsByPredicate( 
	TFunctionRef<bool(int32 Column0, int32 Column1)> PredicateFunc,
	TArray<FQuadGridPatch>& SplitPatchesOut) const
{
	int32 NumQuadCols = NumVertexColsU-1;
	int32 CurIndex = 0;
	int32 SpanStartIndex = 0;
	while ( CurIndex < NumQuadCols-1 )
	{
		bool bShouldSplit = PredicateFunc(CurIndex, CurIndex+1);
		if ( bShouldSplit )
		{
			FQuadGridPatch SubPatch;
			GetSubPatchByQuadRange(0, NumVertexRowsV-2, SpanStartIndex, CurIndex, SubPatch );
			SplitPatchesOut.Add(MoveTemp(SubPatch));
			CurIndex++;
			SpanStartIndex = CurIndex;
		}
		else
		{
			CurIndex++;
		}
	}
	// will we always have one final patch?
	FQuadGridPatch SubPatch;
	GetSubPatchByQuadRange(0, NumVertexRowsV-2, SpanStartIndex, CurIndex, SubPatch );
	SplitPatchesOut.Add(MoveTemp(SubPatch));
}



double FQuadGridPatch::GetQuadOpeningAngleDeg(const FDynamicMesh3& Mesh, int32 Column0, int32 Column1, int32 Row) const
{
	check(Column1 == (Column0+1));
	int32 Vertex0 = VertexSpans[Row][Column0];
	int32 Vertex1 = VertexSpans[Row+1][Column0];
	int32 EdgeID = Mesh.FindEdge(Vertex0, Vertex1);
	check(EdgeID != IndexConstants::InvalidID);
	FIndex2i EdgeTris = Mesh.GetEdgeT(EdgeID);
	check(EdgeTris.B != IndexConstants::InvalidID);
	FVector3d Normal0 = Mesh.GetTriNormal(EdgeTris.A);
	FVector3d Normal1 = Mesh.GetTriNormal(EdgeTris.B);
	return AngleD(Normal0, Normal1);
}