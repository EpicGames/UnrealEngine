// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShapeApproximation/MeshSimpleShapeApproximation.h"
#include "Async/ParallelFor.h"

#include "MinVolumeSphere3.h"
#include "MinVolumeBox3.h"
#include "FitCapsule3.h"
#include "CompGeom/ConvexDecomposition3.h"
//#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Implicit/SweepingMeshSDF.h"
#include "ShapeApproximation/ShapeDetection3.h"
#include "MeshQueries.h"
#include "Operations/MeshClusterSimplifier.h"
#include "Operations/MeshConvexHull.h"
#include "Operations/MeshProjectionHull.h"
#include "Util/ProgressCancel.h"

#define LOCTEXT_NAMESPACE "MeshSimpleShapeApproximation"

using namespace UE::Geometry;

namespace FMeshSimpleShapeApproximationLocals
{
	// Take the box, and while preserving its shape in space, swap its axes around so that they
	// are pointed roughly toward the world x/y/z. This makes it behave more intuitively under
	// transformations (for instance, scaling Z grows roughly in the Z direction rather than
	// to the side if the original box was actually on its "side").
	FOrientedBox3d ReparameterizeBoxCloserToWorldFrame(const FOrientedBox3d& Box)
	{
		return Box.ReparameterizedCloserToWorldFrame();
	}
}

void FMeshSimpleShapeApproximation::DetectAndCacheSimpleShapeType(const FDynamicMesh3* SourceMesh, FSourceMeshCache& CacheOut)
{
	if (UE::Geometry::IsBoxMesh(*SourceMesh, CacheOut.DetectedBox))
	{
		CacheOut.DetectedType = EDetectedSimpleShapeType::Box;
	}
	else if (UE::Geometry::IsSphereMesh(*SourceMesh, CacheOut.DetectedSphere))
	{
		CacheOut.DetectedType = EDetectedSimpleShapeType::Sphere;
	}
	else if (UE::Geometry::IsCapsuleMesh(*SourceMesh, CacheOut.DetectedCapsule))
	{
		CacheOut.DetectedType = EDetectedSimpleShapeType::Capsule;
	}
}




void FMeshSimpleShapeApproximation::InitializeSourceMeshes(const TArray<const FDynamicMesh3*>& InputMeshSet)
{
	SourceMeshes = InputMeshSet;
	SourceMeshCaches.Reset();
	SourceMeshCaches.SetNum(SourceMeshes.Num());

	ParallelFor(SourceMeshes.Num(), [&](int32 k) {
		DetectAndCacheSimpleShapeType( SourceMeshes[k], SourceMeshCaches[k]);
	});

}







bool FMeshSimpleShapeApproximation::GetDetectedSimpleShape(
	const FSourceMeshCache& Cache,
	FSimpleShapeSet3d& ShapeSetOut,
	FCriticalSection& ShapeSetLock)
{
	using namespace FMeshSimpleShapeApproximationLocals;

	if (Cache.DetectedType == EDetectedSimpleShapeType::Sphere && bDetectSpheres)
	{
		ShapeSetLock.Lock();
		ShapeSetOut.Spheres.Add(Cache.DetectedSphere);
		ShapeSetLock.Unlock();
		return true;
	}
	else if (Cache.DetectedType == EDetectedSimpleShapeType::Box && bDetectBoxes)
	{
		FOrientedBox3d BoxToUse = ReparameterizeBoxCloserToWorldFrame(Cache.DetectedBox);

		ShapeSetLock.Lock();
		ShapeSetOut.Boxes.Add(BoxToUse);
		ShapeSetLock.Unlock();
		return true;
	}
	else if (Cache.DetectedType == EDetectedSimpleShapeType::Capsule && bDetectCapsules)
	{
		ShapeSetLock.Lock();
		ShapeSetOut.Capsules.Add(Cache.DetectedCapsule);
		ShapeSetLock.Unlock();
		return true;
	}

	return false;
}




namespace UE {
namespace Geometry {

struct FSimpleShapeFitsResult
{
	bool bHaveSphere = false;
	UE::Geometry::FSphere3d Sphere;

	bool bHaveBox = false;
	FOrientedBox3d Box;

	bool bHaveCapsule = false;
	FCapsule3d Capsule;

	bool bHaveConvex = false;
	FDynamicMesh3 Convex;
};


static void ComputeSimpleShapeFits(const FDynamicMesh3& Mesh,
									bool bSphere, bool bBox, bool bCapsule, double MinDimension, bool bUseExactComputationForBox,
								   FSimpleShapeFitsResult& FitResult,
								   FProgressCancel* Progress = nullptr)
{
	TArray<int32> ToLinear, FromLinear;
	if (bSphere || bBox || bCapsule)
	{
		FromLinear.SetNum(Mesh.VertexCount());
		int32 LinearIndex = 0;
		for (int32 vid : Mesh.VertexIndicesItr())
		{
			FromLinear[LinearIndex++] = vid;
		}
	}

	FitResult.bHaveBox = false;
	if (bBox)
	{
		FMinVolumeBox3d MinBoxCalc;
		bool bMinBoxOK = MinBoxCalc.Solve(FromLinear.Num(),
			[&](int32 Index) { return Mesh.GetVertex(FromLinear[Index]); }, bUseExactComputationForBox, Progress);
		if (bMinBoxOK && MinBoxCalc.IsSolutionAvailable())
		{
			FitResult.bHaveBox = true;
			MinBoxCalc.GetResult(FitResult.Box);
			double MinHalfDimension = MinDimension * .5;
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				FitResult.Box.Extents[SubIdx] = FMath::Max(MinHalfDimension, FitResult.Box.Extents[SubIdx]);
			}
		}
	}

	FitResult.bHaveSphere = false;
	if (bSphere)
	{
		FMinVolumeSphere3d MinSphereCalc;
		bool bMinSphereOK = MinSphereCalc.Solve(FromLinear.Num(),
			[&](int32 Index) { return Mesh.GetVertex(FromLinear[Index]); });
		if (bMinSphereOK && MinSphereCalc.IsSolutionAvailable())
		{
			FitResult.bHaveSphere = true;
			MinSphereCalc.GetResult(FitResult.Sphere);
			FitResult.Sphere.Radius = FMath::Max(MinDimension * .5, FitResult.Sphere.Radius);
		}
	}

	FitResult.bHaveCapsule = false;
	if (bCapsule)
	{
		FitResult.bHaveCapsule = TFitCapsule3<double>::Solve(FromLinear.Num(),
			[&](int32 Index) { return Mesh.GetVertex(FromLinear[Index]); }, FitResult.Capsule);
		if (FitResult.bHaveCapsule)
		{
			FitResult.Capsule.Radius = FMath::Max(MinDimension * .5, FitResult.Capsule.Radius);
			// Note: No need to clamp Length based on MinDimension; a capsule's min dimension can't be along its length since a capsule w/ length of zero is still a sphere ...
		}
	}
}


}
}




void FMeshSimpleShapeApproximation::Generate_AlignedBoxes(FSimpleShapeSet3d& ShapeSetOut)
{
	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 idx)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[idx], ShapeSetOut, GeometryLock))
		{
			return;
		}

		FAxisAlignedBox3d Bounds = SourceMeshes[idx]->GetBounds();

		if (!Bounds.IsEmpty())
		{
			FBoxShape3d NewBox;
			NewBox.Box = FOrientedBox3d(Bounds);
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				NewBox.Box.Extents[SubIdx] = FMath::Max(MinDimension * .5, NewBox.Box.Extents[SubIdx]);
			}

			GeometryLock.Lock();
			ShapeSetOut.Boxes.Add(NewBox);
			GeometryLock.Unlock();
		}
	});
}



void FMeshSimpleShapeApproximation::Generate_OrientedBoxes(FSimpleShapeSet3d& ShapeSetOut, FProgressCancel* Progress)
{
	using namespace FMeshSimpleShapeApproximationLocals;

	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 idx)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[idx], ShapeSetOut, GeometryLock))
		{
			return;
		}

		const FDynamicMesh3& SourceMesh = *SourceMeshes[idx];
		FSimpleShapeFitsResult FitResult;
		ComputeSimpleShapeFits(SourceMesh, false, true, false, MinDimension, bUseExactComputationForBox, FitResult, Progress);

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		if (FitResult.bHaveBox)
		{
			FOrientedBox3d BoxToUse = ReparameterizeBoxCloserToWorldFrame(FitResult.Box);

			GeometryLock.Lock();
			ShapeSetOut.Boxes.Add(FBoxShape3d(BoxToUse));
			GeometryLock.Unlock();
		}
	});
}

void FMeshSimpleShapeApproximation::Generate_MinimalSpheres(FSimpleShapeSet3d& ShapeSetOut)
{
	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 idx)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[idx], ShapeSetOut, GeometryLock))
		{
			return;
		}

		const FDynamicMesh3& SourceMesh = *SourceMeshes[idx];
		FSimpleShapeFitsResult FitResult;
		ComputeSimpleShapeFits(SourceMesh, true, false, false, MinDimension, bUseExactComputationForBox, FitResult);

		if (FitResult.bHaveSphere)
		{
			GeometryLock.Lock();
			ShapeSetOut.Spheres.Add(FSphereShape3d(FitResult.Sphere));
			GeometryLock.Unlock();
		}
	});
}

void FMeshSimpleShapeApproximation::Generate_Capsules(FSimpleShapeSet3d& ShapeSetOut)
{
	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 idx)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[idx], ShapeSetOut, GeometryLock))
		{
			return;
		}

		const FDynamicMesh3& SourceMesh = *SourceMeshes[idx];
		FSimpleShapeFitsResult FitResult;
		ComputeSimpleShapeFits(SourceMesh, false, false, true, MinDimension, bUseExactComputationForBox, FitResult);

		if (FitResult.bHaveCapsule)
		{
			GeometryLock.Lock();
			ShapeSetOut.Capsules.Add(UE::Geometry::FCapsuleShape3d(FitResult.Capsule));
			GeometryLock.Unlock();
		}
	});
}



void FMeshSimpleShapeApproximation::Generate_ConvexHulls(FSimpleShapeSet3d& ShapeSetOut, FProgressCancel* Progress)
{
	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 idx)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[idx], ShapeSetOut, GeometryLock))
		{
			return;
		}

		const FDynamicMesh3& SourceMesh = *SourceMeshes[idx];
		FMeshConvexHull Hull(&SourceMesh);

		Hull.bPostSimplify = bSimplifyHulls;
		Hull.SimplifyOptions.MaxTargetFaceCount = HullTargetFaceCount;
		Hull.SimplifyOptions.GeometricToleranceAfterTarget = HullGeometricToleranceAfterTarget;
		Hull.MinDimension = MinDimension;

		if (Hull.Compute(Progress))
		{
			GeometryLock.Lock();
			ShapeSetOut.Convexes.Emplace(MoveTemp(Hull.ConvexHull));
			GeometryLock.Unlock();
		}
	});
}



void FMeshSimpleShapeApproximation::Generate_ConvexHullDecompositions(FSimpleShapeSet3d& ShapeSetOut, FProgressCancel* Progress)
{
	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 idx)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[idx], ShapeSetOut, GeometryLock))
		{
			return;
		}

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		const FDynamicMesh3& SourceMesh = *SourceMeshes[idx];
		FConvexDecomposition3::FPreprocessMeshOptions PreprocessOptions;
		PreprocessOptions.bMergeEdges = true;

		// Hull thickening allows us to compute decompositions planar inputs
		PreprocessOptions.ThickenInputAfterHullFailure = ConvexDecompositionMinPartThickness;
		// If protecting negative space, we also clamp thickening to at most half of the tolerance value so that we keep within the tolerance distance of the input shape
		if (bConvexDecompositionProtectNegativeSpace)
		{
			PreprocessOptions.ThickenInputAfterHullFailure = FMath::Min(PreprocessOptions.ThickenInputAfterHullFailure, NegativeSpaceTolerance * .5);
		}
		PreprocessOptions.ThickenInputAfterHullFailure = FMath::Max(PreprocessOptions.ThickenInputAfterHullFailure, FMathd::ZeroTolerance);

		PreprocessOptions.CustomPreprocess = [this](FDynamicMesh3& ProcessMesh, const FAxisAlignedBox3d& Bounds) -> void
		{
			// for solid inputs, flip orientation if the initial volume is negative
			if (ProcessMesh.IsClosed())
			{
				double InitialVolume = TMeshQueries<FDynamicMesh3>::GetVolumeArea(ProcessMesh).X;
				if (InitialVolume < 0)
				{
					ProcessMesh.ReverseOrientation();
				}
			}
			
			if (bDecompositionPreSimplifyWithEdgeLength)
			{
				// Run pre-simplification to the target edge length
				MeshClusterSimplify::FSimplifyOptions SimplifyOptions;
				SimplifyOptions.TargetEdgeLength = DecompositionPreSimplifyEdgeLength;
				// Only consider shape for the decomposition -- do not constrain seams, do not transfer attributes/groups
				SimplifyOptions.bTransferAttributes = false;
				SimplifyOptions.bTransferGroups = false;
				SimplifyOptions.PreserveEdges.SetSeamConstraints(MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free);
				FDynamicMesh3 ResultMesh;
				MeshClusterSimplify::Simplify(ProcessMesh, ResultMesh, SimplifyOptions);
				ProcessMesh = ResultMesh;
			}
		};
		FConvexDecomposition3 Decomposition(SourceMesh, PreprocessOptions);
		const bool bIsSolid = Decomposition.IsInputSolid();
		Decomposition.bTreatAsSolid = bIsSolid;

		if (bConvexDecompositionProtectNegativeSpace)
		{
			// Use settings tuned for navigation-driven decomposition
			
			FNegativeSpaceSampleSettings Settings;
			Settings.MarchingCubesGridScale = 1.0;
			Settings.MaxVoxelsPerDim = 1024;
			Settings.MinSpacing = 0.0;
			Settings.TargetNumSamples = 0;
			Settings.VoxelExpandBoundsFactor = UE_DOUBLE_KINDA_SMALL_NUMBER;
			Settings.bOnlyConnectedToHull = bIgnoreInternalNegativeSpace;
			Settings.MinRadius = NegativeSpaceMinRadius;
			Settings.ReduceRadiusMargin = NegativeSpaceTolerance;
			Settings.MinRadius = FMath::Max(1, (NegativeSpaceMinRadius + NegativeSpaceTolerance) * .5);
			Settings.SampleMethod = FNegativeSpaceSampleSettings::ESampleMethod::NavigableVoxelSearch;
			Settings.bRequireSearchSampleCoverage = true;
			Settings.TargetNumSamples = 0; // let the sample coverage determine the number of spheres to place
			Settings.bAllowSamplesInsideMesh = !bIsSolid;

			Decomposition.MaxConvexEdgePlanes = 4;
			Decomposition.bSplitDisconnectedComponents = false;
			Decomposition.ConvexEdgeAngleMoreSamplesThreshold = 180;
			Decomposition.ThickenAfterHullFailure = PreprocessOptions.ThickenInputAfterHullFailure;
			
			Decomposition.InitializeNegativeSpace(Settings);

			constexpr int32 MaxAllowedSplits = 1000000; // more parts than any expected / reasonable decomposition
			int32 TargetNumSplits = bUseConvexDecompositionMaxPieces ? ConvexDecompositionMaxPieces - 1 : MaxAllowedSplits + 1;
			for (int32 Split = 0; Split < TargetNumSplits; Split++)
			{
				int32 NumSplit = Decomposition.SplitWorst(false, -1, true, Settings.ReduceRadiusMargin * .5);

				if (NumSplit == 0)
				{
					break;
				}

				if (!ensureMsgf(Split < MaxAllowedSplits, TEXT("Convex decomposition split the input %d times; likely stuck in a loop"), Split))
				{
					break;
				}
			}

			Decomposition.FixHullOverlapsInNegativeSpace();
			int32 TargetNumPieces = bUseConvexDecompositionMaxPieces ? ConvexDecompositionMaxPieces : -1;
			Decomposition.MergeBest(TargetNumPieces, 0, ConvexDecompositionMinPartThickness, true);
		}
		else
		{
			int32 NumAdditionalSplits = FMath::FloorToInt32(float(ConvexDecompositionMaxPieces) * ConvexDecompositionSearchFactor);
			Decomposition.Compute(ConvexDecompositionMaxPieces, NumAdditionalSplits, ConvexDecompositionErrorTolerance, ConvexDecompositionMinPartThickness);
		}

		for (int32 HullIdx = 0; HullIdx < Decomposition.NumHulls(); HullIdx++)
		{
			FDynamicMesh3 HullMesh = Decomposition.GetHullMesh(HullIdx);
			FMeshConvexHull::FSimplifyOptions SimplifyOptions;
			SimplifyOptions.MaxTargetFaceCount = HullTargetFaceCount;
			SimplifyOptions.GeometricToleranceAfterTarget = HullGeometricToleranceAfterTarget;
			if (bSimplifyHulls && FMeshConvexHull::SimplifyHull(HullMesh, SimplifyOptions, Progress) == false)
			{
				return;
			}

			GeometryLock.Lock();
			ShapeSetOut.Convexes.Emplace(MoveTemp(HullMesh));
			GeometryLock.Unlock();
		}
	});
}






void FMeshSimpleShapeApproximation::Generate_ProjectedHulls(FSimpleShapeSet3d& ShapeSetOut, EProjectedHullAxisMode AxisMode)
{
	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 idx)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[idx], ShapeSetOut, GeometryLock))
		{
			return;
		}

		const FDynamicMesh3& Mesh = *SourceMeshes[idx];

		FFrame3d ProjectionPlane(FVector3d::Zero(), FVector3d::UnitY());
		if (AxisMode == EProjectedHullAxisMode::SmallestBoxDimension)
		{
			FAxisAlignedBox3d Bounds = Mesh.GetBounds();
			int32 AxisIndex = MinAbsElementIndex(Bounds.Diagonal());
			check(MinAbsElement(Bounds.Diagonal()) == Bounds.Diagonal()[AxisIndex]);
			ProjectionPlane = FFrame3d(FVector3d::Zero(), MakeUnitVector3<double>(AxisIndex));
		}
		else if (AxisMode == EProjectedHullAxisMode::SmallestVolume)
		{
			FMeshProjectionHull HullX(&Mesh);
			HullX.ProjectionFrame = FFrame3d(FVector3d::Zero(), FVector3d::UnitX());
			HullX.MinThickness = FMathd::Max(MinDimension, 0);
			bool bHaveX = HullX.Compute();
			FMeshProjectionHull HullY(&Mesh);
			HullY.ProjectionFrame = FFrame3d(FVector3d::Zero(), FVector3d::UnitY());
			HullY.MinThickness = FMathd::Max(MinDimension, 0);
			bool bHaveY = HullY.Compute();
			FMeshProjectionHull HullZ(&Mesh);
			HullZ.ProjectionFrame = FFrame3d(FVector3d::Zero(), FVector3d::UnitZ());
			HullZ.MinThickness = FMathd::Max(MinDimension, 0);
			bool bHaveZ = HullZ.Compute();
			int32 MinIdx = FMathd::Min3Index(
				(bHaveX) ? TMeshQueries<FDynamicMesh3>::GetVolumeArea(HullX.ConvexHull3D).X : TNumericLimits<double>::Max(),
				(bHaveY) ? TMeshQueries<FDynamicMesh3>::GetVolumeArea(HullY.ConvexHull3D).X : TNumericLimits<double>::Max(),
				(bHaveZ) ? TMeshQueries<FDynamicMesh3>::GetVolumeArea(HullZ.ConvexHull3D).X : TNumericLimits<double>::Max());
			ProjectionPlane = (MinIdx == 0) ? HullX.ProjectionFrame : ((MinIdx == 1) ? HullY.ProjectionFrame : HullZ.ProjectionFrame);
		}
		else
		{
			ProjectionPlane = FFrame3d(FVector3d::Zero(), MakeUnitVector3<double>((int32)AxisMode));
		}

		FMeshProjectionHull Hull(&Mesh);
		Hull.ProjectionFrame = ProjectionPlane;
		Hull.MinThickness = FMathd::Max(MinDimension, 0);
		Hull.bSimplifyPolygon = bSimplifyHulls;
		Hull.MinEdgeLength = HullSimplifyTolerance;
		Hull.DeviationTolerance = HullSimplifyTolerance;

		if (Hull.Compute())
		{
			FConvexShape3d NewConvex;
			NewConvex.Mesh = MoveTemp(Hull.ConvexHull3D);
			GeometryLock.Lock();
			ShapeSetOut.Convexes.Add(NewConvex);
			GeometryLock.Unlock();
		}
	});
}


void FMeshSimpleShapeApproximation::Generate_LevelSets(FSimpleShapeSet3d& ShapeSetOut, FProgressCancel* Progress)
{
	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 MeshIndex)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[MeshIndex], ShapeSetOut, GeometryLock))
		{
			return;
		}

		const FAxisAlignedBox3d Bounds = SourceMeshes[MeshIndex]->GetBounds();
		const double CellSize = Bounds.MaxDim() / LevelSetGridResolution;

		TMeshAABBTree3<FDynamicMesh3> Spatial(SourceMeshes[MeshIndex]);

		TSweepingMeshSDF<FDynamicMesh3> SDF;
		SDF.Mesh = SourceMeshes[MeshIndex];
		SDF.Spatial = &Spatial;
		SDF.ComputeMode = TSweepingMeshSDF<FDynamicMesh3>::EComputeModes::NarrowBand_SpatialFloodFill;
		SDF.CellSize = (float)CellSize;
		SDF.NarrowBandMaxDistance = 2.0 * CellSize;
		SDF.ExactBandWidth = FMath::CeilToInt32(SDF.NarrowBandMaxDistance / CellSize);
		SDF.ExpandBounds = 2.0 * CellSize * FVector3d::One();

		if (SDF.Compute(Bounds))
		{
			FLevelSetShape3d NewLevelSet;
			NewLevelSet.GridTransform = FTransform((FVector3d)SDF.GridOrigin);
			NewLevelSet.Grid = MoveTemp(SDF.Grid);
			NewLevelSet.CellSize = SDF.CellSize;

			GeometryLock.Lock();
			ShapeSetOut.LevelSets.Add(MoveTemp(NewLevelSet));
			GeometryLock.Unlock();
		}
		else if (Progress)
		{
			GeometryLock.Lock();
			Progress->AddWarning(LOCTEXT("Generate_LevelSets_Failed", "Generating a new Level Set failed"), FProgressCancel::EMessageLevel::UserWarning);
			GeometryLock.Unlock();
		}
	});
}

void FMeshSimpleShapeApproximation::Generate_MinVolume(FSimpleShapeSet3d& ShapeSetOut)
{
	FCriticalSection GeometryLock;
	ParallelFor(SourceMeshes.Num(), [&](int32 idx)
	{
		if (GetDetectedSimpleShape(SourceMeshCaches[idx], ShapeSetOut, GeometryLock))
		{
			return;
		}

		const FDynamicMesh3& SourceMesh = *SourceMeshes[idx];

		FOrientedBox3d AlignedBox = FOrientedBox3d(SourceMesh.GetBounds());
		double MinHalfDimension = MinDimension * .5;
		for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
		{
			AlignedBox.Extents[SubIdx] = FMath::Max(MinHalfDimension, AlignedBox.Extents[SubIdx]);
		}

		FSimpleShapeFitsResult FitResult;
		ComputeSimpleShapeFits(SourceMesh, true, true, true, MinDimension, bUseExactComputationForBox, FitResult);

		double Volumes[4];
		Volumes[0] = AlignedBox.Volume();
		Volumes[1] = (FitResult.bHaveBox) ? FitResult.Box.Volume() : TNumericLimits<double>::Max();
		Volumes[2] = (FitResult.bHaveSphere) ? FitResult.Sphere.Volume() : TNumericLimits<double>::Max();
		Volumes[3] = (FitResult.bHaveCapsule) ? FitResult.Capsule.Volume() : TNumericLimits<double>::Max();

		int32 MinVolIndex = 0;
		for (int32 k = 1; k < 4; ++k)
		{
			if (Volumes[k] < Volumes[MinVolIndex])
			{
				MinVolIndex = k;
			}
		}

		if (Volumes[MinVolIndex] < TNumericLimits<double>::Max())
		{
			GeometryLock.Lock();
			switch (MinVolIndex)
			{
			case 0:
				ShapeSetOut.Boxes.Add(FBoxShape3d(AlignedBox));
				break;
			case 1:
				ShapeSetOut.Boxes.Add(FBoxShape3d(FitResult.Box));
				break;
			case 2:
				ShapeSetOut.Spheres.Add(FSphereShape3d(FitResult.Sphere));
				break;
			case 3:
				ShapeSetOut.Capsules.Add(UE::Geometry::FCapsuleShape3d(FitResult.Capsule));
				break;
			}
			GeometryLock.Unlock();
		}
	});
}

#undef LOCTEXT_NAMESPACE
