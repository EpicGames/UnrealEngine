// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShapeApproximation/ShapeDetection3.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshQueries.h"
#include "FitCapsule3.h"
#include "Math/UnrealMathUtility.h"

using namespace UE::Geometry;

bool UE::Geometry::IsSphereMesh(const FDynamicMesh3& Mesh, FSphere3d& SphereOut, double RelativeDeviationTol, double MaxAngleRangeDegrees)
{
	// assume that we aren't going to count it as a sphere unless it has 4 slices/sections, which means at least 10 vertices
	if (Mesh.VertexCount() < 10)
	{
		return false;
	}

	// compute bounding box and centroid
	FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
	FVector3d Centroid = FVector3d::Zero();
	for (FVector3d Position : Mesh.VerticesItr())
	{
		Centroid += Position;
		Bounds.Contain(Position);
	}
	Centroid *= 1.0 / (double)Mesh.VertexCount();
	SphereOut.Center = Centroid;

	// Early out if bbox is not sufficiently cubical.
	// Need to more permissive here for low-poly meshes because depending on orientation the
	// AABB can deviate quite a bit. This threshold has been tested down to 4 verts per sphere slice/span
	// (3 makes a diamond which is arguably no longer a sphere)
	double MaxBoxSkewRatio = (Mesh.VertexCount() < 50) ? 1.25 : 1.1;
	if (Bounds.MinDim() <= 0 || (Bounds.MaxDim() / Bounds.MinDim()) > MaxBoxSkewRatio)
	{
		return false;
	}

	// incrementally improve sphere fit for a few iterations
	double UseFitTolerance = FMathf::ZeroTolerance;
	for (int32 Iteration = 0; Iteration < 5; ++Iteration)
	{
		FVector3d PrevCenter = SphereOut.Center;

		double AvgLen = 0;
		FVector3d AvgLenDeriv = FVector3d::Zero();		// gradient of AvgLen wrt sphere center
		for (FVector3d Position : Mesh.VerticesItr())
		{
			FVector3d Delta = Position - SphereOut.Center;
			double DeltaLen = Delta.Length();
			if (DeltaLen > 0)
			{
				AvgLen += DeltaLen;
				AvgLenDeriv -= (1.0 / DeltaLen) * Delta;
			}
		}
		AvgLen *= 1.0 / (double)Mesh.VertexCount();
		AvgLenDeriv *= 1.0 / (double)Mesh.VertexCount();

		SphereOut.Center = Centroid + AvgLen * AvgLenDeriv;
		SphereOut.Radius = AvgLen;

		if (DistanceSquared(SphereOut.Center, PrevCenter) < UseFitTolerance)
		{
			break;
		}
	}

	// We need to make sure this is actually a sphere. Cannot rely on vertices because
	// they may all actually lie on a sphere. However if this a sphere, each edge should be
	// a chord of a circle with the sphere radius. So compare deviation from sphere radius
	// at the center of each mesh edge, with the analytic chord height, or "saggita",
	// computed with formula sagitta = r - sqrt(r*r - l*l), where l = chordlen/2
	double UseRadius = SphereOut.Radius;
	double DeviationTol = 2.0 * UseRadius * RelativeDeviationTol;
	double CosAngleTolerance = FMathd::Cos(FMathd::DegToRad * MaxAngleRangeDegrees);
	for (int32 EdgeID : Mesh.EdgeIndicesItr())
	{
		FVector3d A, B;
		Mesh.GetEdgeV(EdgeID, A, B);

		// if a single edge spans too wide an angular range, the shape is too coarsely tesselated to be considered a sphere
		FVector3d ToA = (A - SphereOut.Center).GetSafeNormal();
		FVector3d ToB = (B - SphereOut.Center).GetSafeNormal();
		if (ToA.Dot(ToB) <= CosAngleTolerance)
		{
			return false;
		}

		double HalfChordLen = Distance(A, B) * 0.5;
		double MaxChordHeight = UseRadius - FMathd::Sqrt(UseRadius*UseRadius - HalfChordLen*HalfChordLen);   // "sagitta" height

		double MidpointSignedDist = SphereOut.SignedDistance( (A+B)*0.5 );
		if (FMathd::Abs(MidpointSignedDist) > (MaxChordHeight + DeviationTol) )
		{
			return false;
		}
	}

	return true;
}



bool UE::Geometry::IsBoxMesh(const FDynamicMesh3& Mesh, FOrientedBox3d& BoxOut, double AngleToleranceDeg, double PlaneDistanceTolerance)
{
	// minimal box has at least 6 vertices
	if (Mesh.VertexCount() < 6)
	{
		return false;
	}

	double PerpDotTolerance = FMathd::Cos((90.0 - AngleToleranceDeg) * FMathd::DegToRad);
	double ParallelDotTolerance = FMathd::Cos(AngleToleranceDeg * FMathd::DegToRad);

	// test a subset of internal mesh edges to see if we can find faces with opening angles
	// that are not ~0 or ~90 degrees, if so this is not a box. This will immediately reject 
	// most non-box meshes without having to do expensive normal clustering first
	int32 NumEdges = Mesh.EdgeCount();
	int32 Step = FMath::Max(Mesh.EdgeCount() / 10, 2);
	for (int32 k = 0; k < NumEdges; k += Step)
	{
		if (Mesh.IsEdge(k))
		{
			FIndex2i EdgeT = Mesh.GetEdgeT(k);
			if (EdgeT.B != FDynamicMesh3::InvalidID)
			{
				double Dot = Mesh.GetTriNormal(EdgeT.A).Dot(Mesh.GetTriNormal(EdgeT.B));
				Dot = FMathd::Abs(Dot);
				if (Dot > PerpDotTolerance && Dot < ParallelDotTolerance)
				{
					return false;
				}
			}
		}
	}

	// Greedily cluster up to six planes
	constexpr int32 ExpectPlanes = 6;
	FVector3d Normals[ExpectPlanes];
	double Distances[ExpectPlanes];
	int32 FoundPlanes = 0;

	// Helper to bin a new position+normal into a plane cluster, or return -1 if no valid cluster is found
	auto NormalMatch = [&Normals, &Distances, &FoundPlanes, ExpectPlanes, ParallelDotTolerance, PlaneDistanceTolerance](FVector3d Pos, FVector3d Normal) -> int32
	{
		int32 Idx = 0;
		for (; Idx < FoundPlanes; ++Idx)
		{
			if (Normals[Idx].Dot(Normal) > ParallelDotTolerance)
			{
				double Distance = Pos.Dot(Normal);
				if (FMath::IsNearlyEqual(Distance, Distances[Idx], PlaneDistanceTolerance)) //-V614
				{
					return Idx;
				}
				else
				{
					// A box cannot have multiple parallel planes with the same normal
					// (note opposite faces will be parallel but with opposite normals)
					return INDEX_NONE;
				}
			}
		}
		if (FoundPlanes < ExpectPlanes)
		{
			Normals[Idx] = Normal;
			Distances[Idx] = Pos.Dot(Normal);
			FoundPlanes++;
			return Idx;
		}
		return INDEX_NONE;
	};

	for (int32 TriID : Mesh.TriangleIndicesItr())
	{
		FIndex3i Tri = Mesh.GetTriangle(TriID);
		FVector3d Vert = Mesh.GetVertex(Tri.A);
		FVector3d Normal = Mesh.GetTriNormal(TriID);
		int32 PlaneIdx = NormalMatch(Vert, Normal);
		if (PlaneIdx == INDEX_NONE)
		{
			return false;
		}
	}
	if (FoundPlanes != ExpectPlanes)
	{
		return false;
	}

	// Check that each of those 6 normals is either perpendicular or parallel-with-reversed-sign from all the others
	// Simultaneously find the 3 unique directions/axes
	FIndex3i UniqueAxes(-1, -1, -1);
	int32 UniqueCount = 0;
	bool bDone[6] = { false,false,false,false,false,false };
	for (int32 k = 0; k < 6; ++k)
	{
		if (bDone[k]) continue;

		FVector3d Normal0 = Normals[k];
		int32 ParallelPair = -1;

		for (int32 j = k + 1; j < 6; ++j)
		{
			double Dot = Normal0.Dot(Normals[j]);
			if (FMathd::Abs(Dot) > PerpDotTolerance)
			{
				if (Dot > -ParallelDotTolerance)		// if dot is not zero, it needs to be -1
				{
					return false;
				}
				ParallelPair = j;
				bDone[j] = true;
				break;
			}
		}

		// accumulate unique axis if we haven't seen this one and we haven't found 3 already
		if (UniqueCount < 3)
		{
			int32 UniqueIdx = (ParallelPair == -1) ? k : FMath::Min(k, ParallelPair);
			UniqueAxes[UniqueCount++] = UniqueIdx;
		}
	}
			
	// if we found the 3 unique axes, it's a box and we know its orientation, so just fit minimal 
	// container aligned to box axes, and shift center point to center of that oriented-AABB
	if (UniqueCount == 3)
	{
		// would be nice to cycle these so that X = most-aligned-with-X, etc, or longest?
		FVector3d X = Normals[UniqueAxes[0]];
		FVector3d Y = Normals[UniqueAxes[1]];
		FVector3d Z = Normals[UniqueAxes[2]];
		// make sure the frame vectors are oriented as expected for a rotation matrix
		if ((X.Cross(Y)).Dot(Z) < 0)
		{
			Z = -Z;
		}
		// compute AABB in the frame of the box
		FQuaterniond Rotation(FMatrix3d(X, Y, Z, false));
		FMatrix3d UnorientRotation = Rotation.Inverse().ToRotationMatrix();
		FAxisAlignedBox3d FitBox = FAxisAlignedBox3d::Empty();
		for (FVector3d VtxPos : Mesh.VerticesItr())
		{
			VtxPos = UnorientRotation * VtxPos;
			FitBox.Contain(VtxPos);
		}
		// recenter output OBB to the center of oriented AABB
		FVector3d Extents = FitBox.Extents();
		FVector3d Center = Rotation * FitBox.Center();
		BoxOut = FOrientedBox3d( FFrame3d(Center, Rotation), Extents);
		return true;
	}

	return false;
}



bool UE::Geometry::IsCapsuleMesh(const FDynamicMesh3& Mesh, FCapsule3d& CapsuleOut, double RelativeDeviationTol, double MaxAngleRangeDegrees)
{
	// minimal 4-slice capsule has at least 10 vertices
	if (Mesh.VertexCount() < 10)		
	{
		return false;
	}
	// any more early-out tests we can do here? Not immediately obvious...

	// fit a capsule using TFitCapsule3. If mesh is not compact this (currently) requires linearize vertices, unfortunately
	bool bFitCapsule = false;
	if (Mesh.IsCompactV())
	{
		bFitCapsule = TFitCapsule3<double>::Solve(Mesh.VertexCount(),
			[&](int32 Index) { return Mesh.GetVertex(Index); }, CapsuleOut);
	}
	else
	{
		TArray<FVector3d> LinearVertices;
		LinearVertices.Reserve(Mesh.VertexCount());
		for (FVector3d Position : Mesh.VerticesItr())
		{
			LinearVertices.Add(Position);
		}
		bFitCapsule = TFitCapsule3<double>::Solve(LinearVertices.Num(),
			[&](int32 Index) { return LinearVertices[Index]; }, CapsuleOut);
	}
	if (!bFitCapsule)
	{
		return false;
	}

	// We use logic similar to IsSphereMesh() to test edge midpoints vs the capsule's spherical endcaps or cylindrical middle.
	// When the edge midpoint projects to the cylindrical middle, we project the problem to a segment-aligned plane,
	// so it becomes a test vs the circular cross-section.
	double UseRadius = CapsuleOut.Radius;
	double DeviationTol = 2.0 * UseRadius * RelativeDeviationTol;
	double CosAngleTolerance = FMathd::Cos(FMathd::DegToRad * MaxAngleRangeDegrees);

	for (int32 EdgeID : Mesh.EdgeIndicesItr())
	{
		FVector3d A, B;
		Mesh.GetEdgeV(EdgeID, A, B);

		FVector3d MidPoint = (A + B) * .5;
		double ProjParam = CapsuleOut.Segment.Project(MidPoint);

		// Find the relevant point on the segment to use for distance calculations
		FVector3d RefSegmentPt;
		if (ProjParam + UE_DOUBLE_KINDA_SMALL_NUMBER >= CapsuleOut.Segment.Extent)
		{
			RefSegmentPt = CapsuleOut.Segment.EndPoint();
		}
		else if (ProjParam - UE_DOUBLE_KINDA_SMALL_NUMBER <= -CapsuleOut.Segment.Extent)
		{
			RefSegmentPt = CapsuleOut.Segment.StartPoint();
		}
		else
		{
			// Cylinder case: do a projection so we only measure distances in the space of the circular cross section
			A = A - (A - CapsuleOut.Segment.Center).Dot(CapsuleOut.Segment.Direction) * CapsuleOut.Segment.Direction;
			B = B - (B - CapsuleOut.Segment.Center).Dot(CapsuleOut.Segment.Direction) * CapsuleOut.Segment.Direction;
			MidPoint = (A + B) * .5;
			RefSegmentPt = CapsuleOut.Segment.Center;
		}
		
		// if a single edge spans too wide an angular range, the shape is too coarsely tesselated to be considered a capsule
		FVector3d ToA = (A - RefSegmentPt).GetSafeNormal();
		FVector3d ToB = (B - RefSegmentPt).GetSafeNormal();
		if (ToA.Dot(ToB) <= CosAngleTolerance)
		{
			return false;
		}

		double HalfChordLen = Distance(A, B) * 0.5;
		double MaxChordHeight = UseRadius - FMathd::Sqrt(UseRadius * UseRadius - HalfChordLen * HalfChordLen);   // "sagitta" height

		double MidpointSignedDist = FVector3d::Distance(MidPoint, RefSegmentPt) - UseRadius;
		if (FMathd::Abs(MidpointSignedDist) > (MaxChordHeight + DeviationTol))
		{
			return false;
		}
	}

	// track where the vertices project to along the segment, to see if the capsule vertices cover the expected range along the capsule's major axis
	FInterval1d ProjRange;
	for (int32 VertID : Mesh.VertexIndicesItr())
	{
		FVector3d Vertex = Mesh.GetVertex(VertID);
		double ProjParam = CapsuleOut.Segment.Project(Vertex);
		ProjRange.Contain(ProjParam);
	}

	// Expect to come within DeviationTol of covering the full range of the capsule
	if (ProjRange.Max < CapsuleOut.Segment.Extent + CapsuleOut.Radius - DeviationTol || ProjRange.Min > -CapsuleOut.Segment.Extent - CapsuleOut.Radius + DeviationTol)
	{
		return false;
	}

	return true;
}
