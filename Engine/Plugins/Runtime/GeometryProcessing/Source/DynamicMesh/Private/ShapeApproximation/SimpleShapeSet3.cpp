// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShapeApproximation/SimpleShapeSet3.h"

#include "MeshQueries.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Intersection/ContainmentQueries3.h"

#include "CompGeom/ConvexDecomposition3.h"
#include "DynamicMeshEditor.h"
#include "Generators/MeshShapeGenerator.h"
#include "Generators/GridBoxMeshGenerator.h"
#include "Generators/BoxSphereGenerator.h"
#include "Generators/CapsuleGenerator.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Spatial/FastWinding.h"

using namespace UE::Geometry;


// FSimpleShapeElementKey identifies an element of a FSimpleShapeSet3d, via the Type
// of that element, and the Index into the per-element-type arrays.
struct FSimpleShapeElementKey
{
	// type of element
	ESimpleShapeType Type;
	// index of this element in per-type element lists inside a FSimpleShapeSet3d
	int32 Index;

	// volume of the element, used for sorting/etc
	double Volume;

	FSimpleShapeElementKey() = default;

	FSimpleShapeElementKey(ESimpleShapeType TypeIn, int32 IndexIn, double VolumeIn)
		: Type(TypeIn), Index(IndexIn), Volume(VolumeIn)
	{}
};



static void FilterContained(const FSimpleShapeSet3d& Geometry, const FSphereShape3d& Sphere, const TArray<FSimpleShapeElementKey>& Elements, int32 k, TArray<bool>& RemovedInOut)
{
	int32 N = Elements.Num();
	for (int32 j = k + 1; j < N; ++j)
	{
		if (RemovedInOut[j] == false)
		{
			bool bContained = false;
			int32 ElemIdx = Elements[j].Index;
			if (Elements[j].Type == ESimpleShapeType::Sphere)
			{
				bContained = UE::Geometry::IsInside<double>(Sphere.Sphere, Geometry.Spheres[ElemIdx].Sphere);
			}
			else if (Elements[j].Type == ESimpleShapeType::Box)
			{
				bContained = UE::Geometry::IsInside<double>(Sphere.Sphere, Geometry.Boxes[ElemIdx].Box);
			}
			else if (Elements[j].Type == ESimpleShapeType::Capsule)
			{
				bContained = UE::Geometry::IsInside<double>(Sphere.Sphere, Geometry.Capsules[ElemIdx].Capsule);
			}
			else if (Elements[j].Type == ESimpleShapeType::Convex)
			{
				bContained = UE::Geometry::IsInside(Sphere.Sphere, Geometry.Convexes[ElemIdx].Mesh.VerticesItr());
			}
			else if (Elements[j].Type == ESimpleShapeType::LevelSet)
			{
				bContained = UE::Geometry::IsInside(Sphere.Sphere, Geometry.LevelSets[ElemIdx].GetGridBox());
			}
			else
			{
				ensure(false);		// not implemented yet?
			}

			if (bContained)
			{
				RemovedInOut[j] = true;
			}
		}
	}
}




static void FilterContained(const FSimpleShapeSet3d& Geometry, const UE::Geometry::FCapsuleShape3d& Capsule, const TArray<FSimpleShapeElementKey>& Elements, int32 k, TArray<bool>& RemovedInOut)
{
	int32 N = Elements.Num();
	for (int32 j = k + 1; j < N; ++j)
	{
		if (RemovedInOut[j] == false)
		{
			bool bContained = false;
			int32 ElemIdx = Elements[j].Index;
			if (Elements[j].Type == ESimpleShapeType::Sphere)
			{
				bContained = UE::Geometry::IsInside<double>(Capsule.Capsule, Geometry.Spheres[ElemIdx].Sphere);
			}
			else if (Elements[j].Type == ESimpleShapeType::Box)
			{
				bContained = UE::Geometry::IsInside<double>(Capsule.Capsule, Geometry.Boxes[ElemIdx].Box);
			}
			else if (Elements[j].Type == ESimpleShapeType::Capsule)
			{
				bContained = UE::Geometry::IsInside<double>(Capsule.Capsule, Geometry.Capsules[ElemIdx].Capsule);
			}
			else if (Elements[j].Type == ESimpleShapeType::Convex)
			{
				bContained = UE::Geometry::IsInside(Capsule.Capsule, Geometry.Convexes[ElemIdx].Mesh.VerticesItr());
			}
			else if (Elements[j].Type == ESimpleShapeType::LevelSet)
			{
				bContained = UE::Geometry::IsInside(Capsule.Capsule, Geometry.LevelSets[ElemIdx].GetGridBox());
			}
			else
			{
				ensure(false);
			}

			if (bContained)
			{
				RemovedInOut[j] = true;
			}
		}
	}
}




static void FilterContained(const FSimpleShapeSet3d& Geometry, const FBoxShape3d& Box, const TArray<FSimpleShapeElementKey>& Elements, int32 k, TArray<bool>& RemovedInOut)
{
	int32 N = Elements.Num();
	for (int32 j = k + 1; j < N; ++j)
	{
		if (RemovedInOut[j] == false)
		{
			bool bContained = false;
			int32 ElemIdx = Elements[j].Index;
			if (Elements[j].Type == ESimpleShapeType::Sphere)
			{
				bContained = UE::Geometry::IsInside<double>(Box.Box, Geometry.Spheres[ElemIdx].Sphere);
			}
			else if (Elements[j].Type == ESimpleShapeType::Box)
			{
				bContained = UE::Geometry::IsInside<double>(Box.Box, Geometry.Boxes[ElemIdx].Box);
			}
			else if (Elements[j].Type == ESimpleShapeType::Capsule)
			{
				bContained = UE::Geometry::IsInside<double>(Box.Box, Geometry.Capsules[ElemIdx].Capsule);
			}
			else if (Elements[j].Type == ESimpleShapeType::Convex)
			{
				bContained = UE::Geometry::IsInside(Box.Box, Geometry.Convexes[ElemIdx].Mesh.VerticesItr());
			}
			else if (Elements[j].Type == ESimpleShapeType::LevelSet)
			{
				bContained = UE::Geometry::IsInside(Box.Box, Geometry.LevelSets[ElemIdx].GetGridBox());
			}
			else
			{
				ensure(false);
			}

			if (bContained)
			{
				RemovedInOut[j] = true;
			}
		}
	}
}





static void FilterContained(const FSimpleShapeSet3d& Geometry, const FConvexShape3d& Convex, const TArray<FSimpleShapeElementKey>& Elements, int32 k, TArray<bool>& RemovedInOut)
{
	TArray<FHalfspace3d> Planes;
	for (int32 tid : Convex.Mesh.TriangleIndicesItr())
	{
		FVector3d Normal, Centroid; double Area;
		Convex.Mesh.GetTriInfo(tid, Normal, Area, Centroid);
		Planes.Add(FHalfspace3d(Normal, Centroid));
	}

	int32 N = Elements.Num();
	for (int32 j = k + 1; j < N; ++j)
	{
		if (RemovedInOut[j] == false)
		{
			bool bContained = false;
			int32 ElemIdx = Elements[j].Index;
			if (Elements[j].Type == ESimpleShapeType::Sphere)
			{
				bContained = UE::Geometry::IsInsideHull<double>(Planes, Geometry.Spheres[ElemIdx].Sphere);
			}
			else if (Elements[j].Type == ESimpleShapeType::Box)
			{
				bContained = UE::Geometry::IsInsideHull<double>(Planes, Geometry.Boxes[ElemIdx].Box);
			}
			else if (Elements[j].Type == ESimpleShapeType::Capsule)
			{
				bContained = UE::Geometry::IsInsideHull<double>(Planes, Geometry.Capsules[ElemIdx].Capsule);
			}
			else if (Elements[j].Type == ESimpleShapeType::Convex)
			{
				bContained = UE::Geometry::IsInsideHull<double>(Planes, Geometry.Convexes[ElemIdx].Mesh.VerticesItr());
			}
			else if (Elements[j].Type == ESimpleShapeType::LevelSet)
			{
				bContained = UE::Geometry::IsInsideHull<double>(Planes, Geometry.LevelSets[ElemIdx].GetGridBox());
			}
			else
			{
				ensure(false);
			}

			if (bContained)
			{
				RemovedInOut[j] = true;
			}
		}
	}
}


static void FilterContained(const FSimpleShapeSet3d& Geometry, FLevelSetShape3d& LevelSetShape, const TArray<FSimpleShapeElementKey>& Elements, int32 k, TArray<bool>& RemovedInOut)
{
	TTriLinearGridInterpolant<FDenseGrid3f> GridInterp(&LevelSetShape.Grid, FVector3d::ZeroVector, LevelSetShape.CellSize, LevelSetShape.Grid.GetDimensions());

	int32 N = Elements.Num();
	for (int32 j = k + 1; j < N; ++j)
	{
		if (RemovedInOut[j] == false)
		{
			bool bContained = false;
			int32 ElemIdx = Elements[j].Index;
			if (Elements[j].Type == ESimpleShapeType::Sphere)
			{
				bContained = UE::Geometry::IsInside(GridInterp, LevelSetShape.GridTransform, Geometry.Spheres[ElemIdx].Sphere);
			}
			else if (Elements[j].Type == ESimpleShapeType::Box)
			{
				bContained = UE::Geometry::IsInside(GridInterp, LevelSetShape.GridTransform, Geometry.Boxes[ElemIdx].Box);
			}
			else if (Elements[j].Type == ESimpleShapeType::Capsule)
			{
				bContained = UE::Geometry::IsInside(GridInterp, LevelSetShape.GridTransform, Geometry.Capsules[ElemIdx].Capsule);
			}
			else if (Elements[j].Type == ESimpleShapeType::Convex)
			{
				bContained = UE::Geometry::IsInside<double>(GridInterp, LevelSetShape.GridTransform, Geometry.Convexes[ElemIdx].Mesh.VerticesItr());
			}
			else if (Elements[j].Type == ESimpleShapeType::LevelSet)
			{
				bContained = UE::Geometry::IsInside(GridInterp, LevelSetShape.GridTransform, Geometry.LevelSets[ElemIdx].GetGridBox());
			}
			else
			{
				ensure(false);
			}

			if (bContained)
			{
				RemovedInOut[j] = true;
			}
		}
	}
}


static void GetElementsList(FSimpleShapeSet3d& GeometrySet, TArray<FSimpleShapeElementKey>& Elements)
{
	for (int32 k = 0; k < GeometrySet.Spheres.Num(); ++k)
	{
		Elements.Add(FSimpleShapeElementKey(ESimpleShapeType::Sphere, k, GeometrySet.Spheres[k].Sphere.Volume()));
	}
	for (int32 k = 0; k < GeometrySet.Boxes.Num(); ++k)
	{
		Elements.Add(FSimpleShapeElementKey(ESimpleShapeType::Box, k, GeometrySet.Boxes[k].Box.Volume()));
	}
	for (int32 k = 0; k < GeometrySet.Capsules.Num(); ++k)
	{
		Elements.Add(FSimpleShapeElementKey(ESimpleShapeType::Capsule, k, GeometrySet.Capsules[k].Capsule.Volume()));
	}
	for (int32 k = 0; k < GeometrySet.Convexes.Num(); ++k)
	{
		FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(GeometrySet.Convexes[k].Mesh);
		Elements.Add(FSimpleShapeElementKey(ESimpleShapeType::Convex, k, VolArea.X));
	}
	for (int32 k = 0; k < GeometrySet.LevelSets.Num(); ++k)
	{
		Elements.Add(FSimpleShapeElementKey(ESimpleShapeType::LevelSet, k, GeometrySet.LevelSets[k].GetGridBox().Volume()));
	}

}

static void GetElementsSortedByDecreasing(FSimpleShapeSet3d& GeometrySet, TArray<FSimpleShapeElementKey>& Elements)
{
	GetElementsList(GeometrySet, Elements);
	// sort by decreasing volume
	Elements.Sort([](const FSimpleShapeElementKey& A, const FSimpleShapeElementKey& B) { return A.Volume > B.Volume; });
}


void FSimpleShapeSet3d::RemoveContainedGeometry()
{
	TArray<FSimpleShapeElementKey> Elements;
	GetElementsSortedByDecreasing(*this, Elements);

	int32 N = Elements.Num();
	TArray<bool> Removed;
	Removed.Init(false, N);

	// remove contained elements
	for (int32 k = 0; k < N; ++k)
	{
		if (Removed[k]) continue;

		ESimpleShapeType ElemType = Elements[k].Type;
		int32 ElemIdx = (int32)Elements[k].Index;

		if (ElemType == ESimpleShapeType::Sphere)
		{
			FilterContained(*this, Spheres[ElemIdx], Elements, k, Removed);
		}
		else if (ElemType == ESimpleShapeType::Capsule)
		{
			FilterContained(*this, Capsules[ElemIdx], Elements, k, Removed);
		}
		else if (ElemType == ESimpleShapeType::Box)
		{
			FilterContained(*this, Boxes[ElemIdx], Elements, k, Removed);
		}
		else if (ElemType == ESimpleShapeType::Convex)
		{
			FilterContained(*this, Convexes[ElemIdx], Elements, k, Removed);
		}
		else if (ElemType == ESimpleShapeType::LevelSet)
		{
			// Pass in zero here instead of k because we don't currently have a good volume value for level set objects
			FilterContained(*this, LevelSets[ElemIdx], Elements, 0, Removed);
		}
		else
		{
			ensure(false);
		}
	}


	// build a new shape set
	FSimpleShapeSet3d NewSet;
	for (int32 k = 0; k < N; ++k)
	{
		if (Removed[k] == false)
		{
			ESimpleShapeType ElemType = Elements[k].Type;
			int32 ElemIdx = Elements[k].Index;

			switch (ElemType)
			{
			case ESimpleShapeType::Sphere:
				NewSet.Spheres.Add(Spheres[ElemIdx]);
				break;
			case ESimpleShapeType::Box:
				NewSet.Boxes.Add(Boxes[ElemIdx]);
				break;
			case ESimpleShapeType::Capsule:
				NewSet.Capsules.Add(Capsules[ElemIdx]);
				break;
			case ESimpleShapeType::Convex:
				NewSet.Convexes.Add(Convexes[ElemIdx]);		// todo movetemp here...
				break;
			case ESimpleShapeType::LevelSet:
				NewSet.LevelSets.Add(LevelSets[ElemIdx]);		// todo movetemp here...
				break;
			}
		}
	}

	// replace our lists with new set
	Spheres = MoveTemp(NewSet.Spheres);
	Boxes = MoveTemp(NewSet.Boxes);
	Capsules = MoveTemp(NewSet.Capsules);
	Convexes = MoveTemp(NewSet.Convexes);
	LevelSets = MoveTemp(NewSet.LevelSets);
}

template<typename TransformType>
static void TransformSphereShape(FSphereShape3d& SphereShape, const TransformType& Transform)
{
	double RadiusScale = Transform.GetScale3D().GetAbsMin(); // Scale radius by the minimum axis scale, to match how the editor applies non-uniform scales to spheres
	SphereShape.Sphere.Center = Transform.TransformPosition(SphereShape.Sphere.Center);
	SphereShape.Sphere.Radius *= RadiusScale;
}
template<typename TransformType>
static void TransformBoxShape(FBoxShape3d& BoxShape, const TransformType& Transform)
{
	BoxShape.Box.Frame.Transform(Transform);

	// There isn't a perfect approach to transforming the extents in the presence of non-uniform scaling.
	// One approach we've done in the past has been to transform the corner vector, but depending on that
	// vector's orientation relative to scaling axis, the extents could grow/shrink in unexpected ways,
	// or not change at all if direction happened to be orthogonal.
	// The approach we now use is to apply the scaling in the reference frame of the box itself, because
	// this what the editor does for box collision shapes, and it is helpful to have this work the same
	// way here.
	// For uniform scaling, it doesn't matter which approach we use.
	BoxShape.Box.Extents *= Transform.GetScale3D();
}
template<typename TransformType>
static void TransformCapsuleShape(UE::Geometry::FCapsuleShape3d& CapsuleShape, const TransformType& Transform)
{
	FVector3d P0 = Transform.TransformPosition(CapsuleShape.Capsule.Segment.StartPoint());
	FVector3d P1 = Transform.TransformPosition(CapsuleShape.Capsule.Segment.EndPoint());

	CapsuleShape.Capsule.Segment.Center = 0.5 * (P0 + P1);
	CapsuleShape.Capsule.Segment.Direction = (P1 - P0);
	CapsuleShape.Capsule.Segment.Extent = Normalize(CapsuleShape.Capsule.Segment.Direction) * 0.5;

	double CurRadius = CapsuleShape.Capsule.Radius;
	FFrame3d CapsuleFrame(CapsuleShape.Capsule.Segment.Center, CapsuleShape.Capsule.Segment.Direction);
	FVector3d SideVec = CapsuleFrame.PointAt(FVector3d(CurRadius, CurRadius, 0)) - CapsuleFrame.Origin;
	FVector3d NewSideVec = Transform.TransformVector(SideVec);
	double RadiusScale = NewSideVec.Length() / SideVec.Length();
	CapsuleShape.Capsule.Radius *= RadiusScale;
}

template<typename TransformType>
static void TransformLevelSetShape(UE::Geometry::FLevelSetShape3d& LevelSetShape, const TransformType& Transform)
{
	const FTransform3d Sub(-0.5 * LevelSetShape.CellSize * FVector3d::One());
	const FTransform3d Add(0.5 * LevelSetShape.CellSize * FVector3d::One());
	LevelSetShape.GridTransform = LevelSetShape.GridTransform * Sub * (FTransform3d)Transform * Add;
}


template<typename TransformArrayType>
static void TransformSphereShapeByArray(FSphereShape3d& SphereShape, const TransformArrayType& TransformSequence)
{
	for (const auto& XForm : TransformSequence)
	{
		SphereShape.Sphere.Center = XForm.TransformPosition(SphereShape.Sphere.Center);
		double RadiusScale = XForm.GetScale3D().GetAbsMin(); // Scale radius by the minimum axis scale, to match how the editor applies non-uniform scales to spheres
		SphereShape.Sphere.Radius *= RadiusScale;
	}
}
template<typename TransformArrayType>
static void TransformBoxShapeByArray(FBoxShape3d& BoxShape, const TransformArrayType& TransformSequence)
{
	for (const auto& XForm : TransformSequence)
	{
		TransformBoxShape(BoxShape, XForm);
	}
}
template<typename TransformArrayType>
static void TransformCapsuleShapeByArray(UE::Geometry::FCapsuleShape3d& CapsuleShape, const TransformArrayType& TransformSequence)
{
	FVector3d P0 = CapsuleShape.Capsule.Segment.StartPoint();
	FVector3d P1 = CapsuleShape.Capsule.Segment.EndPoint();

	double CurRadius = CapsuleShape.Capsule.Radius;
	FFrame3d CapsuleFrame(CapsuleShape.Capsule.Segment.Center, CapsuleShape.Capsule.Segment.Direction);
	FVector3d InitialSideVec = CapsuleFrame.PointAt(FVector3d(CurRadius, CurRadius, 0)) - CapsuleFrame.Origin;
	FVector3d NewSideVec = InitialSideVec;

	for (const auto& XForm : TransformSequence)
	{
		P0 = XForm.TransformPosition(P0);
		P1 = XForm.TransformPosition(P1);
		NewSideVec = XForm.TransformVector(NewSideVec);
	}

	CapsuleShape.Capsule.Segment.Center = 0.5 * (P0 + P1);
	CapsuleShape.Capsule.Segment.Direction = (P1 - P0);
	CapsuleShape.Capsule.Segment.Extent = Normalize(CapsuleShape.Capsule.Segment.Direction) * 0.5;
	double RadiusScale = NewSideVec.Length() / InitialSideVec.Length();
	CapsuleShape.Capsule.Radius *= RadiusScale;
}
template<typename TransformArrayType>
static void TransformLevelSetShapeByArray(UE::Geometry::FLevelSetShape3d& LevelSetShape, const TransformArrayType& TransformSequence)
{
	const FTransform3d Sub(-0.5 * LevelSetShape.CellSize * FVector3d::One());
	const FTransform3d Add(0.5 * LevelSetShape.CellSize * FVector3d::One());

	LevelSetShape.GridTransform = LevelSetShape.GridTransform * Sub;

	for (const auto& XForm : TransformSequence)
	{
		LevelSetShape.GridTransform = LevelSetShape.GridTransform * XForm;
	}

	LevelSetShape.GridTransform = LevelSetShape.GridTransform * Add;
}


void FSimpleShapeSet3d::Append(const FSimpleShapeSet3d& OtherShapeSet)
{
	for (FSphereShape3d SphereShape : OtherShapeSet.Spheres)
	{
		Spheres.Add(SphereShape);
	}

	for (FBoxShape3d BoxShape : OtherShapeSet.Boxes)
	{
		Boxes.Add(BoxShape);
	}

	for (UE::Geometry::FCapsuleShape3d CapsuleShape : OtherShapeSet.Capsules)
	{
		Capsules.Add(CapsuleShape);
	}

	Convexes.Reserve(Convexes.Num() + OtherShapeSet.Convexes.Num());
	for (const FConvexShape3d& ConvexShape : OtherShapeSet.Convexes)
	{
		Convexes.Add(ConvexShape);
	}

	LevelSets.Reserve(LevelSets.Num() + OtherShapeSet.LevelSets.Num());
	for (const FLevelSetShape3d& LevelSetShape : OtherShapeSet.LevelSets)
	{
		LevelSets.Add(LevelSetShape);
	}
}


void FSimpleShapeSet3d::Append(const FSimpleShapeSet3d& OtherShapeSet, const FTransform3d& Transform)
{
	for (FSphereShape3d SphereShape : OtherShapeSet.Spheres)
	{
		TransformSphereShape(SphereShape, Transform);
		Spheres.Add(SphereShape);
	}

	for (FBoxShape3d BoxShape : OtherShapeSet.Boxes)
	{
		TransformBoxShape(BoxShape, Transform);
		Boxes.Add(BoxShape);
	}

	for (UE::Geometry::FCapsuleShape3d CapsuleShape : OtherShapeSet.Capsules)
	{
		TransformCapsuleShape(CapsuleShape, Transform);
		Capsules.Add(CapsuleShape);
	}

	Convexes.Reserve(Convexes.Num() + OtherShapeSet.Convexes.Num());
	for (const FConvexShape3d& ConvexShape : OtherShapeSet.Convexes)
	{
		Convexes.Add(ConvexShape);
		MeshTransforms::ApplyTransform(Convexes.Last().Mesh, Transform, true);
	}

	LevelSets.Reserve(LevelSets.Num() + OtherShapeSet.LevelSets.Num());
	for (const FLevelSetShape3d& LevelSetShape : OtherShapeSet.LevelSets)
	{
		LevelSets.Add(LevelSetShape);
		TransformLevelSetShape(LevelSets.Last(), Transform);
	}
}



void FSimpleShapeSet3d::Append(const FSimpleShapeSet3d& OtherShapeSet, const TArray<FTransform3d>& TransformSequence)
{
	for (FSphereShape3d SphereShape : OtherShapeSet.Spheres)
	{
		TransformSphereShapeByArray(SphereShape, TransformSequence);
		Spheres.Add(SphereShape);
	}

	for (FBoxShape3d BoxShape : OtherShapeSet.Boxes)
	{
		TransformBoxShapeByArray(BoxShape, TransformSequence);
		Boxes.Add(BoxShape);
	}

	for (UE::Geometry::FCapsuleShape3d CapsuleShape : OtherShapeSet.Capsules)
	{
		TransformCapsuleShapeByArray(CapsuleShape, TransformSequence);
		Capsules.Add(CapsuleShape);
	}

	Convexes.Reserve(Convexes.Num() + OtherShapeSet.Convexes.Num());
	for (const FConvexShape3d& ConvexShape : OtherShapeSet.Convexes)
	{
		Convexes.Add(ConvexShape);
		for (const FTransform3d& XForm : TransformSequence)
		{
			MeshTransforms::ApplyTransform(Convexes.Last().Mesh, XForm, true);
		}
	}

	LevelSets.Reserve(LevelSets.Num() + OtherShapeSet.LevelSets.Num());
	for (const FLevelSetShape3d& LevelSetShape : OtherShapeSet.LevelSets)
	{
		LevelSets.Add(LevelSetShape);
		TransformLevelSetShapeByArray(LevelSets.Last(), TransformSequence);
	}
}


void FSimpleShapeSet3d::Append(const FSimpleShapeSet3d& OtherShapeSet, const FTransformSequence3d& TransformSequence)
{
	for (FSphereShape3d SphereShape : OtherShapeSet.Spheres)
	{
		TransformSphereShapeByArray(SphereShape, TransformSequence.GetTransforms());
		Spheres.Add(SphereShape);
	}

	for (FBoxShape3d BoxShape : OtherShapeSet.Boxes)
	{
		TransformBoxShapeByArray(BoxShape, TransformSequence.GetTransforms());
		Boxes.Add(BoxShape);
	}

	for (UE::Geometry::FCapsuleShape3d CapsuleShape : OtherShapeSet.Capsules)
	{
		TransformCapsuleShapeByArray(CapsuleShape, TransformSequence.GetTransforms());
		Capsules.Add(CapsuleShape);
	}

	Convexes.Reserve(Convexes.Num() + OtherShapeSet.Convexes.Num());
	for (const FConvexShape3d& ConvexShape : OtherShapeSet.Convexes)
	{
		Convexes.Add(ConvexShape);
		for (const FTransformSRT3d& XForm : TransformSequence.GetTransforms())
		{
			MeshTransforms::ApplyTransform(Convexes.Last().Mesh, XForm, true);
		}
	}

	LevelSets.Reserve(LevelSets.Num() + OtherShapeSet.LevelSets.Num());
	for (const FLevelSetShape3d& LevelSetShape : OtherShapeSet.LevelSets)
	{
		LevelSets.Add(LevelSetShape);
		TransformLevelSetShapeByArray(LevelSets.Last(), TransformSequence.GetTransforms());
	}
}




void FSimpleShapeSet3d::FilterByVolume(int32 MaximumCount)
{
	TArray<FSimpleShapeElementKey> Elements;
	GetElementsSortedByDecreasing(*this, Elements);
	if (Elements.Num() <= MaximumCount)
	{
		return;
	}

	FSimpleShapeSet3d NewSet;
	for (int32 k = 0; k < MaximumCount; ++k)
	{
		ESimpleShapeType ElemType = Elements[k].Type;
		int32 ElemIdx = Elements[k].Index;

		switch (ElemType)
		{
		case ESimpleShapeType::Sphere:
			NewSet.Spheres.Add(Spheres[ElemIdx]);
			break;
		case ESimpleShapeType::Box:
			NewSet.Boxes.Add(Boxes[ElemIdx]);
			break;
		case ESimpleShapeType::Capsule:
			NewSet.Capsules.Add(Capsules[ElemIdx]);
			break;
		case ESimpleShapeType::Convex:
			NewSet.Convexes.Add(Convexes[ElemIdx]);		// todo movetemp here...
			break;
		case ESimpleShapeType::LevelSet:
			NewSet.LevelSets.Add(LevelSets[ElemIdx]);		// todo movetemp here...
			break;
		}
	}

	Spheres = MoveTemp(NewSet.Spheres);
	Boxes = MoveTemp(NewSet.Boxes);
	Capsules = MoveTemp(NewSet.Capsules);
	Convexes = MoveTemp(NewSet.Convexes);
	LevelSets = MoveTemp(NewSet.LevelSets);
}



void FSimpleShapeSet3d::ApplyTransform(const FTransform3d& Transform)
{
	for (FSphereShape3d& SphereShape : Spheres)
	{
		TransformSphereShape(SphereShape, Transform);
	}

	for (FBoxShape3d& BoxShape : Boxes)
	{
		TransformBoxShape(BoxShape, Transform);
	}

	for (UE::Geometry::FCapsuleShape3d& CapsuleShape : Capsules)
	{
		TransformCapsuleShape(CapsuleShape, Transform);
	}

	for (FConvexShape3d& ConvexShape : Convexes)
	{
		MeshTransforms::ApplyTransform(ConvexShape.Mesh, Transform, true);
	}

	for (FLevelSetShape3d& LevelSetShape : LevelSets)
	{
		TransformLevelSetShape(LevelSetShape, Transform);
	}
}

bool FSimpleShapeSet3d::MergeShapes(int32 MergeAboveCount, const FMergeShapesSettings& MergeSettings)
{
	int32 NumShapes = TotalElementsNum();
	//Note: We currently do not support merging level sets, because the merge logic assumes we can treat each simple shape as a convex hull
	int32 NonLevelSet = NumShapes - LevelSets.Num();
	if (NumShapes <= MergeAboveCount || NonLevelSet < 2)
	{
		// No merges needed / possible
		return false;
	}

	struct FSourceShapeRef
	{
		int32 SourceIdx = -1;
		ESimpleShapeType ShapeType;
	};

	TArray<FVector> HullVertices;
	TArray<int32> HullVertexCounts;
	TArray<double> HullVolumes;
	TArray<FSourceShapeRef> HullToShapeSource;
	TUniquePtr<FDynamicMesh3> CollisionMesh;
	if (MergeSettings.bMergeShapesProtectNegativeSpace)
	{
		CollisionMesh = MakeUnique<FDynamicMesh3>();
	}

	auto AppendGeneratorToCollisionMesh = [&CollisionMesh](const FMeshShapeGenerator& Generator)
	{
		if (!CollisionMesh)
		{
			return;
		}
		checkSlow(CollisionMesh->IsCompact());
		int32 NewVertStart = CollisionMesh->MaxVertexID();
		for (FVector3d V : Generator.Vertices)
		{
			CollisionMesh->AppendVertex(V);
		}
		for (FIndex3i T : Generator.Triangles)
		{
			CollisionMesh->AppendTriangle(T.A + NewVertStart, T.B + NewVertStart, T.C + NewVertStart);
		}
	};
	auto TransformVertices = [](TArrayView<FVector3d> Vertices, const FTransform& Transform)
	{
		for (FVector3d& Vertex : Vertices)
		{
			Vertex = Transform.TransformPosition(Vertex);
		}
	};
	auto AppendHullVertices = [&HullToShapeSource, &HullVolumes, &HullVertices, &HullVertexCounts]
	(TArrayView<const FVector3d> Vertices, double Volume, const FSourceShapeRef ShapeSource, const FTransform* ShapeTransform = nullptr)
	{
		check(HullToShapeSource.Num() == HullVolumes.Num());
		HullToShapeSource.Add(ShapeSource);
		int32 HullIdxStart = HullVertices.Num();
		HullVertices.Append(Vertices);
		if (ShapeTransform)
		{
			for (int32 Idx = HullIdxStart; Idx < HullVertices.Num(); ++Idx)
			{
				HullVertices[Idx] = ShapeTransform->TransformPosition(HullVertices[Idx]);
			}
		}
		HullVertexCounts.Add(Vertices.Num());
		HullVolumes.Add(Volume);
	};
	auto GeneratorVolume = [](FMeshShapeGenerator* Generator) -> double
	{
		TIndexVectorMeshArrayAdapter<FIndex3i, double, FVector3d> GenMeshAdapter(&Generator->Vertices, &Generator->Triangles);
		FVector2d VolArea = TMeshQueries<TIndexVectorMeshArrayAdapter<FIndex3i, double, FVector3d>>::GetVolumeArea(GenMeshAdapter);
		return VolArea.X;
	};


	// Simple shape approximation options, for conversion to convex hull for merges
	// TODO: consider exposing as parameters
	constexpr int32 SphereStepsPerSide = 6;
	constexpr int32 CapsuleCircleSteps = 16;
	constexpr int32 CapsuleHemisphereSteps = 8;

	for (int32 BoxIdx = 0; BoxIdx < Boxes.Num(); ++BoxIdx)
	{
		FSourceShapeRef Source{ BoxIdx, ESimpleShapeType::Box };
		FOrientedBox3d OrientedBox = Boxes[BoxIdx].Box;
		if (CollisionMesh)
		{
			FGridBoxMeshGenerator BoxGenerator;
			BoxGenerator.EdgeVertices = FIndex3i(1, 1, 1);
			BoxGenerator.Box = OrientedBox;
			BoxGenerator.Generate();
			AppendHullVertices(BoxGenerator.Vertices, OrientedBox.Volume(), Source);
			AppendGeneratorToCollisionMesh(BoxGenerator);
		}
		else
		{
			TArray<FVector3d, TFixedAllocator<8>> BoxVertices;
			OrientedBox.EnumerateCorners([&](FVector3d Corner) { BoxVertices.Add(Corner); });
			AppendHullVertices(BoxVertices, OrientedBox.Volume(), Source);
		}
	}
	for (int32 SphereIdx = 0; SphereIdx < Spheres.Num(); ++SphereIdx)
	{
		FSourceShapeRef Source{ SphereIdx, ESimpleShapeType::Sphere };
		const FSphere3d& Sphere = Spheres[SphereIdx].Sphere;
		FBoxSphereGenerator SphereGenerator;
		SphereGenerator.Box.Frame.Origin = Sphere.Center;
		SphereGenerator.Radius = (float)FMath::Max(FMathf::ZeroTolerance, Sphere.Radius);
		int32 StepsPerSide = FMath::Max(1, SphereStepsPerSide);
		SphereGenerator.EdgeVertices = FIndex3i(StepsPerSide, StepsPerSide, StepsPerSide);
		SphereGenerator.Generate();
		double Volume = GeneratorVolume(&SphereGenerator);
		AppendHullVertices(SphereGenerator.Vertices, Volume, Source);
		AppendGeneratorToCollisionMesh(SphereGenerator);
	}
	for (int32 CapsuleIdx = 0; CapsuleIdx < Capsules.Num(); ++CapsuleIdx)
	{
		FSourceShapeRef Source{ CapsuleIdx, ESimpleShapeType::Capsule };
		const FCapsule3d& Capsule = Capsules[CapsuleIdx].Capsule;
		FCapsuleGenerator CapsuleGenerator;
		CapsuleGenerator.Radius = Capsule.Radius;
		CapsuleGenerator.SegmentLength = Capsule.Segment.Length();
		CapsuleGenerator.NumHemisphereArcSteps = FMath::Max(2, CapsuleHemisphereSteps);
		CapsuleGenerator.NumCircleSteps = FMath::Max(3, CapsuleCircleSteps);
		CapsuleGenerator.Generate();
		Geometry::FQuaterniond CapsuleRot(FVector3d::ZAxisVector, Capsule.Segment.Direction);
		FTransform CapsuleTransform(FRotator(CapsuleRot), Capsule.Center() - Capsule.Segment.Direction * Capsule.Segment.Length() * .5);
		TransformVertices(CapsuleGenerator.Vertices, CapsuleTransform);
		double Volume = GeneratorVolume(&CapsuleGenerator);
		AppendHullVertices(CapsuleGenerator.Vertices, Volume, Source);
		AppendGeneratorToCollisionMesh(CapsuleGenerator);
	}
	for (int32 ConvexIdx = 0; ConvexIdx < Convexes.Num(); ++ConvexIdx)
	{
		FSourceShapeRef Source{ ConvexIdx, ESimpleShapeType::Convex };
		const FDynamicMesh3& Mesh = Convexes[ConvexIdx].Mesh;
		double Volume = TMeshQueries<FDynamicMesh3>::GetVolumeArea(Mesh).X;
		for (FVector3d V : Mesh.VerticesItr())
		{
			HullVertices.Add(V);
		}
		HullVertexCounts.Add(Mesh.VertexCount());
		HullVolumes.Add(Volume);
		HullToShapeSource.Add(Source);
		if (CollisionMesh)
		{
			FDynamicMeshEditor Editor(CollisionMesh.Get());
			FMeshIndexMappings UnusedMappings;
			Editor.AppendMesh(&Mesh, UnusedMappings);
		}
	}

	const int32 InitialNumConvex = HullVertexCounts.Num();
	// Nothing we are able to merge
	if (InitialNumConvex <= 1)
	{
		return false;
	}

	TArray<int32> HullVertexStarts;
	HullVertexStarts.SetNumUninitialized(InitialNumConvex);
	HullVertexStarts[0] = 0; // Note InitialNumConvex is > 1 due to above test
	for (int32 HullIdx = 1, LastEnd = HullVertexCounts[0]; HullIdx < InitialNumConvex; LastEnd += HullVertexCounts[HullIdx++])
	{
		HullVertexStarts[HullIdx] = LastEnd;
	}

	TArray<TPair<int32, int32>> HullProximity;

	FConvexDecomposition3 Decomposition;
	Decomposition.InitializeFromHulls(HullVertexStarts.Num(),
		[&HullVolumes](int32 HullIdx) { return HullVolumes[HullIdx]; }, [&HullVertexCounts](int32 HullIdx) { return HullVertexCounts[HullIdx]; },
		[&HullVertexStarts, &HullVertices](int32 HullIdx, int32 VertIdx) { return HullVertices[HullVertexStarts[HullIdx] + VertIdx]; }, HullProximity);
	double MinProximityOverlapTolerance = 0;
	FSphereCovering NegativeSpace;
	// Build the negative space of the collision shapes, if requested
	if (MergeSettings.bMergeShapesProtectNegativeSpace)
	{
		FNegativeSpaceSampleSettings SampleSettings;
		SampleSettings.bOnlyConnectedToHull = MergeSettings.bIgnoreInternalNegativeSpace;
		SampleSettings.MinRadius = MergeSettings.NegativeSpaceMinRadius;
		SampleSettings.ReduceRadiusMargin = MergeSettings.NegativeSpaceTolerance;
		SampleSettings.MinRadius = FMath::Max(1, (MergeSettings.NegativeSpaceMinRadius + MergeSettings.NegativeSpaceTolerance) * .5);
		SampleSettings.SampleMethod = FNegativeSpaceSampleSettings::ESampleMethod::VoxelSearch;
		SampleSettings.bRequireSearchSampleCoverage = true;
		SampleSettings.TargetNumSamples = 1; // let the sample coverage determine the number of spheres to place

		MinProximityOverlapTolerance = FMath::Max(SampleSettings.ReduceRadiusMargin * .5, MinProximityOverlapTolerance);
		FDynamicMeshAABBTree3 CollisionAABBTree(CollisionMesh.Get(), true);
		TFastWindingTree<FDynamicMesh3> CollisionFastWinding(&CollisionAABBTree, true);
		NegativeSpace.AddNegativeSpace(CollisionFastWinding, SampleSettings, false);
	}
	FSphereCovering* UseNegativeSpace = NegativeSpace.Num() > 0 ? &NegativeSpace : nullptr;

	// Find possible shape merges based on the shape bounding box overlaps,
	// where bounds are expanded by: Max(a quarter their min dimension, a tenth their max dimension, the half reduce radius margin if negative space is computed)
	Decomposition.InitializeProximityFromDecompositionBoundingBoxOverlaps(.25, .1, MinProximityOverlapTolerance);

	int32 MaxShapeCount = FMath::Max(0, MergeAboveCount - LevelSets.Num());
	Decomposition.RestrictMergeSearchToLocalAfterTestNumConnections = 1000 + MaxShapeCount * MaxShapeCount; // Restrict searches in very large search cases, when not close to max shape count, to avoid excessive search time
	constexpr double ErrorTolerance = 0, MinThicknessTolerance = 0;
	Decomposition.MergeBest(MaxShapeCount, ErrorTolerance, MinThicknessTolerance, true, false, MaxShapeCount, UseNegativeSpace, nullptr /*optional FTransform for negative space*/);

	// Algorithm decided not to merge
	if (Decomposition.NumHulls() == InitialNumConvex)
	{
		return false;
	}


	TArray<FSphereShape3d> SourceSpheres = MoveTemp(Spheres);
	TArray<FBoxShape3d> SourceBoxes = MoveTemp(Boxes);
	TArray<UE::Geometry::FCapsuleShape3d> SourceCapsules = MoveTemp(Capsules);
	TArray<FConvexShape3d> SourceConvexes = MoveTemp(Convexes);

	for (int32 HullIdx = 0; HullIdx < Decomposition.Decomposition.Num(); ++HullIdx)
	{
		const FConvexDecomposition3::FConvexPart& Part = Decomposition.Decomposition[HullIdx];
		// If part was not merged, use the source ID to map it back to the original collision primitive
		if (Part.HullSourceID >= 0)
		{
			const FSourceShapeRef& SourceRef = HullToShapeSource[Part.HullSourceID];
			bool bHandledShape = true;
			switch (SourceRef.ShapeType)
			{
			case ESimpleShapeType::Box:
				Boxes.Add(SourceBoxes[SourceRef.SourceIdx]);
				break;
			case ESimpleShapeType::Sphere:
				Spheres.Add(SourceSpheres[SourceRef.SourceIdx]);
				break;
			case ESimpleShapeType::Capsule:
				Capsules.Add(SourceCapsules[SourceRef.SourceIdx]);
				break;
			case ESimpleShapeType::Convex:
				Convexes.Add(MoveTemp(SourceConvexes[SourceRef.SourceIdx]));
				break;
			default:
				// Note: All shapes that we add to the HullToShapeSource array should be handled above, so we should not reach here
				ensureMsgf(false, TEXT("Unhandled shape element type could not be restored from source shapes"));
				bHandledShape = false;
			}
			if (bHandledShape)
			{
				continue;
			}
		}
		// Add the merged part
		Convexes.Emplace(Decomposition.GetHullMesh(HullIdx));
	}

	return true;
}

