// Copyright Epic Games, Inc. All Rights Reserved.

#include "Solvers/ConstrainedMeshDeformer.h"
#include "Solvers/Internal/ConstrainedMeshDeformers.h"


TUniquePtr<UE::Solvers::IConstrainedMeshSolver> UE::MeshDeformation::ConstructConstrainedMeshDeformer(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& DynamicMesh)
{
	TUniquePtr<UE::Solvers::IConstrainedMeshSolver> Deformer(new FConstrainedMeshDeformer(DynamicMesh, WeightScheme));
	return Deformer;
}

TUniquePtr<UE::Solvers::IConstrainedMeshSolver> UE::MeshDeformation::ConstructUniformConstrainedMeshDeformer(const UE::MeshDeformation::FDynamicGraph3d& Graph)
{
	TUniquePtr<UE::Solvers::IConstrainedMeshSolver> Deformer(new FConstrainedMeshDeformer(Graph));
	return Deformer;
}

TUniquePtr<UE::Solvers::IConstrainedLaplacianMeshSolver> UE::MeshDeformation::ConstructSoftMeshDeformer(const FDynamicMesh3& DynamicMesh)
{
	TUniquePtr<UE::Solvers::IConstrainedLaplacianMeshSolver> Deformer(new FSoftMeshDeformer(DynamicMesh));
	return Deformer;
}
