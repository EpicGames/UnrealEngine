// Copyright Epic Games, Inc. All Rights Reserved.

#include "Solvers/ConstrainedMeshSmoother.h"
#include "Solvers/Internal/ConstrainedMeshSmoothers.h"


TUniquePtr<UE::Solvers::IConstrainedMeshSolver> UE::MeshDeformation::ConstructConstrainedMeshSmoother(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& DynamicMesh)
{
	TUniquePtr<UE::Solvers::IConstrainedMeshSolver> Deformer(new FBiHarmonicMeshSmoother(DynamicMesh, WeightScheme));
	return Deformer;
}

