// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstrainedMeshDeformationSolver.h"
#include "Solvers/LaplacianMatrixAssembly.h"
#include "LaplacianOperators.h"
#include "MatrixSolver.h"
#include "ConstrainedPoissonSolver.h"

using namespace UE::Geometry;

#ifdef TIME_LAPLACIAN_SMOOTHERS

#include "ProfilingDebugging/ScopedTimers.h"
DEFINE_LOG_CATEGORY_STATIC(LogMeshSmoother, Log, All);

#endif

double ComputeDistSqrd(const FSOAPositions& VecA, const FSOAPositions& VecB)
{
	const int32 NumA = VecA.Num();
	const int32 NumB = VecB.Num();
	checkSlow(NumA == NumB);

#if 0
	// the eigne way?
	double Tmp = (VecA.Array(0) - VecB.Array(0)).dot(VecA.Array(0) - VecB.Array(0))
		+ (VecA.Array(1) - VecB.Array(1)).dot(VecA.Array(1) - VecB.Array(1))
		+ (VecA.Array(2) - VecB.Array(2)).dot(VecA.Array(2) - VecB.Array(2));

#endif

	// doing it by hand so we can break when the error is large
	double DistSqrd = 0.;
	{
		const auto& AX = VecA.Array(0);
		const auto& AY = VecA.Array(1);
		const auto& AZ = VecA.Array(2);

		const auto& BX = VecB.Array(0);
		const auto& BY = VecB.Array(1);
		const auto& BZ = VecB.Array(2);


		for (int32 i = 0; i < NumA; ++i)
		{
			double TmpX = AX(i) - BX(i);
			double TmpY = AY(i) - BY(i);
			double TmpZ = AZ(i) - BZ(i);

			TmpX *= TmpX;
			TmpY *= TmpY;
			TmpZ *= TmpZ;
			double TmpT = TmpX + TmpY + TmpZ;
			DistSqrd += TmpT;
		}
	}
	return DistSqrd;
}


FConstrainedMeshDeformationSolver::~FConstrainedMeshDeformationSolver() 
{
}

FConstrainedMeshDeformationSolver::FConstrainedMeshDeformationSolver(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme, const EMatrixSolverType MatrixSolverType)
	: VertexCount(DynamicMesh.VertexCount())
{
	FSparseMatrixD LaplacianInternal;
	FSparseMatrixD LaplacianBoundary;
	ConstructLaplacian(Scheme, DynamicMesh, VtxLinearization, LaplacianInternal, LaplacianBoundary);

	Init(DynamicMesh, Scheme, MatrixSolverType, LaplacianInternal, LaplacianBoundary);
}

void FConstrainedMeshDeformationSolver::AddConstraint(const int32 VtxId, const double Weight, const FVector3d& Pos, const bool bPostFix)
{
	const auto& ToIndex = VtxLinearization.ToIndex();

	if (VtxId > ToIndex.Num())
	{
		return;
	}

	const int32 Index = ToIndex[VtxId];

	// Only add the constraint if the vertex is actually in the interior.  We aren't solving for edge vertices.
	if (Index != FDynamicMesh3::InvalidID && Index < InternalVertexCount)
	{
		bConstraintPositionsDirty = true;
		bConstraintWeightsDirty = true;

		FConstraintPosition NewConstraint;
		NewConstraint.ElementID = VtxId;
		NewConstraint.ConstraintIndex = Index;
		NewConstraint.Position = Pos;
		NewConstraint.Weight = Weight;
		NewConstraint.bPostFix = bPostFix;

		ConstraintPositionMap.Add(TTuple<int32, FConstraintPosition>(Index, NewConstraint));
		ConstraintWeightMap.Add(TTuple<int32, double>(Index, Weight));
	}
}


bool FConstrainedMeshDeformationSolver::UpdateConstraintPosition(const int32 VtxId, const FVector3d& Pos, const bool bPostFix)
{
	bool Result = false;
	const auto& ToIndex = VtxLinearization.ToIndex();

	if (VtxId > ToIndex.Num())
	{
		return Result;
	}

	const int32 Index = ToIndex[VtxId];


	if (Index != FDynamicMesh3::InvalidID && Index < InternalVertexCount)
	{
		FConstraintPosition* Found = ConstraintPositionMap.Find(Index);
		if (ensure(Found != nullptr))
		{
			Found->Position = Pos;
			Found->bPostFix = bPostFix;
			bConstraintPositionsDirty = true;
			Result = true;
		}
	}
	return Result;
}

bool FConstrainedMeshDeformationSolver::UpdateConstraintWeight(const int32 VtxId, const double Weight)
{
	bool Result = false;
	const auto& ToIndex = VtxLinearization.ToIndex();

	if (VtxId > ToIndex.Num())
	{
		return Result;
	}

	const int32 Index = ToIndex[VtxId];
	if (Index != FDynamicMesh3::InvalidID && Index < InternalVertexCount)
	{
		bConstraintWeightsDirty = true;

		// Add should over-write any existing value for this key
		ConstraintWeightMap.Add(TTuple<int32, double>(Index, Weight));

		Result = ConstraintPositionMap.Contains(VtxId);
	}
	return Result;
}


bool FConstrainedMeshDeformationSolver::IsConstrained(const int32 VtxId) const
{
	bool Result = false;

	const auto& ToIndex = VtxLinearization.ToIndex();

	if (VtxId > ToIndex.Num())
	{
		return Result;
	}

	const int32 Index = ToIndex[VtxId];

	if (Index != FDynamicMesh3::InvalidID && Index < InternalVertexCount)
	{
		Result = ConstraintWeightMap.Contains(Index);
	}

	return Result;
}

void FConstrainedMeshDeformationSolver::UpdateSolverConstraints()
{
	if (bConstraintWeightsDirty)
	{
		ConstrainedSolver->SetConstraintWeights(ConstraintWeightMap);
		bConstraintWeightsDirty = false;
	}

	if (bConstraintPositionsDirty)
	{
		ConstrainedSolver->SetContraintPositions(ConstraintPositionMap);
		bConstraintPositionsDirty = false;
	}
}

bool FConstrainedMeshDeformationSolver::CopyInternalPositions(const FSOAPositions& PositionalVector, TArray<FVector3d>& LinearArray) const
{
	// Number of positions

	const int32 Num = PositionalVector.Num();

	// early out if the x,y,z arrays in the PositionalVector have different lengths
	if (!PositionalVector.bHasSize(Num))
	{
		return false;
	}

	checkSlow(Num == InternalVertexCount);

	// 
	const auto& ToVtxId = VtxLinearization.ToId();
	const int32 MaxVtxId = ToVtxId.Num(); // NB: this is really max_used + 1 in the mesh.  See  FDynamicMesh3::MaxVertexID()

	if (LinearArray.Num() != MaxVtxId)
	{
		return false;
	}

	// Update the internal positions.
	for (int32 i = 0; i < InternalVertexCount; ++i)
	{
		const int32 VtxId = ToVtxId[i];

		LinearArray[VtxId] = FVector3d(PositionalVector.X(i), PositionalVector.Y(i), PositionalVector.Z(i));
	}

	return true;
}

bool FConstrainedMeshDeformationSolver::CopyBoundaryPositions(TArray<FVector3d>& LinearArray) const
{
	const auto& ToVtxId = VtxLinearization.ToId();
	const int32 MaxVtxId = ToVtxId.Num();

	if (LinearArray.Num() != MaxVtxId)
	{
		return false;
	}

	int32 BoundaryVertexCount = VertexCount - InternalVertexCount;

	for (int32 i = 0; i < BoundaryVertexCount; ++i)
	{
		const int32 VtxId = ToVtxId[i + InternalVertexCount];
		LinearArray[VtxId] = FVector3d(BoundaryPositions.X(i), BoundaryPositions.Y(i), BoundaryPositions.Z(i)); //BoundaryPositions[i];
	}

	return true;
}



void FConstrainedMeshDeformationSolver::UpdateWithPostFixConstraints(FSOAPositions& PositionVector) const
{
	for (const auto& ConstraintPosition : ConstraintPositionMap)
	{
		const int32 Index = ConstraintPosition.Key;
		const FConstraintPosition& Constraint = ConstraintPosition.Value;

		checkSlow(Index < InternalVertexCount);

		// we only care about post-fix constraints

		if (Constraint.bPostFix)
		{
			const FVector3d& Pos = Constraint.Position;
			PositionVector.SetXYZ(Index, Pos);
		}
	}
}




bool FConstrainedMeshDeformationSolver::SetMaxIterations(int32 MaxIterations)
{
	IIterativeMatrixSolverBase* Solver = ConstrainedSolver->GetMatrixSolverIterativeBase();
	if (Solver)
	{
		Solver->SetIterations(MaxIterations);
		return true;
	}
	return false;
}

bool FConstrainedMeshDeformationSolver::SetTolerance(double Tol)
{
	IIterativeMatrixSolverBase* Solver = ConstrainedSolver->GetMatrixSolverIterativeBase();
	if (Solver)
	{
		Solver->SetTolerance(Tol);
		return true;
	}
	return false;
}












FSoftMeshDeformationSolver::FSoftMeshDeformationSolver(const FDynamicMesh3& DynamicMesh)
{
	EMatrixSolverType MatrixSolverType = EMatrixSolverType::LU;

	typedef FSparseMatrixD::Scalar  ScalarT;
	typedef Eigen::Triplet<ScalarT> MatrixTripletT;

	// compute linearization so we can store constraints at linearized indices
	VtxLinearization.Reset(DynamicMesh);

	const TArray<int32>& ToMeshV = VtxLinearization.ToId();
	const TArray<int32>& ToIndex = VtxLinearization.ToIndex();
	const int32 NumVerts = VtxLinearization.NumVerts();

	FEigenSparseMatrixAssembler LaplacianAssembler(NumVerts, NumVerts);

	UE::MeshDeformation::ConstructFullCotangentLaplacian<double>(DynamicMesh, VtxLinearization, LaplacianAssembler,
		UE::MeshDeformation::ECotangentWeightMode::ClampedMagnitude,
		UE::MeshDeformation::ECotangentAreaMode::VoronoiArea);

	FSparseMatrixD CotangentLaplacian;
	LaplacianAssembler.ExtractResult(CotangentLaplacian);

	checkSlow(CotangentLaplacian.rows() == CotangentLaplacian.cols());

	TUniquePtr<FSparseMatrixD> LTLPtr(new FSparseMatrixD(CotangentLaplacian.rows(), CotangentLaplacian.cols()));
	FSparseMatrixD& LTLMatrix = *(LTLPtr);

	// Construct the Biharmonic system matrix
	// Note that if Laplacian was symmetric (eg if uniform/etc) then LTLMatrix = CotangentLaplacian * CotangentLaplacian
	LTLMatrix = CotangentLaplacian.transpose() * CotangentLaplacian;
	ConstrainedSolver.Reset(new FConstrainedSolver(LTLPtr, MatrixSolverType));
}

FSoftMeshDeformationSolver::~FSoftMeshDeformationSolver()
{
}



void FSoftMeshDeformationSolver::UpdateSolverConstraints()
{
	if (bConstraintWeightsDirty)
	{
		ConstrainedSolver->SetConstraintWeights(ConstraintMap);
		bConstraintWeightsDirty = false;
	}

	if (bConstraintPositionsDirty)
	{
		ConstrainedSolver->SetContraintPositions(ConstraintMap);
		bConstraintPositionsDirty = false;
	}
}



void FSoftMeshDeformationSolver::AddConstraint(const int32 VtxId, const double Weight, const FVector3d& Position, const bool bPostFix)
{
	if (ensure(VtxLinearization.IsValidId(VtxId)) == false) return;
	int32 Index = VtxLinearization.GetIndex(VtxId);

	FPositionConstraint NewConstraint;
	NewConstraint.ElementID = VtxId;
	NewConstraint.ConstraintIndex = Index;
	NewConstraint.Position = Position;
	NewConstraint.Weight = Weight;
	NewConstraint.bPostFix = bPostFix;

	ConstraintMap.Add(Index, NewConstraint);

	bConstraintPositionsDirty = true;
	bConstraintWeightsDirty = true;
}


bool FSoftMeshDeformationSolver::UpdateConstraintPosition(const int32 VtxId, const FVector3d& NewPosition, const bool bPostFix)
{
	if (ensure(VtxLinearization.IsValidId(VtxId)) == false) return false;
	int32 Index = VtxLinearization.GetIndex(VtxId);

	FPositionConstraint* Found = ConstraintMap.Find(Index);
	if (ensure(Found != nullptr))
	{
		Found->Position = NewPosition;
		Found->bPostFix = bPostFix;
		bConstraintPositionsDirty = true;
		return true;
	}
	return false;
}


bool FSoftMeshDeformationSolver::UpdateConstraintWeight(const int32 VtxId, const double NewWeight)
{
	if (ensure(VtxLinearization.IsValidId(VtxId)) == false) return false;
	int32 Index = VtxLinearization.GetIndex(VtxId);

	FPositionConstraint* Found = ConstraintMap.Find(Index);
	if (ensure(Found != nullptr))
	{
		Found->Weight = NewWeight;
		bConstraintWeightsDirty = true;
		return true;
	}
	return false;
}


bool FSoftMeshDeformationSolver::IsConstrained(const int32 VtxId) const
{
	if (VtxLinearization.IsValidId(VtxId) == false) return false;
	int32 Index = VtxLinearization.GetIndex(VtxId);
	return ConstraintMap.Contains(Index);
}


void FSoftMeshDeformationSolver::ClearConstraints()
{
	ConstraintMap.Empty();
	bConstraintPositionsDirty = true;
	bConstraintWeightsDirty = true;
}



void FSoftMeshDeformationSolver::UpdateLaplacianScale(double UniformScale)
{
	LaplacianScale = UniformScale;
}

bool FSoftMeshDeformationSolver::HasLaplacianScale() const
{
	return LaplacianScale != 1.0;
}

double FSoftMeshDeformationSolver::GetLaplacianScale(int32 Index) const
{
	return LaplacianScale;
}