// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "LaplacianOperators.h"
#include "Solvers/MatrixInterfaces.h"
#include "Solvers/MeshLaplacian.h"
#include "Solvers/MeshLinearization.h"
#include "Solvers/ConstrainedMeshSolver.h"
#include "ConstrainedPoissonSolver.h"

#include "MatrixBase.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{


/**
 * FConstrainedMeshDeformationSolver is an implmentation of IConstrainedMeshSolver that solves
 * Mesh Deformation problems by using quadratic energy functions based on the vertex-graph Laplacians. 
 *
 * All constraints are "soft", ie they are included in the system as weighted quadratic energies
 * rather than had constraints.
 *
 * Both the Laplacian weighting scheme/type and The Sparse Matrix Solver type are configurable.
 */
class FConstrainedMeshDeformationSolver : public UE::Solvers::IConstrainedMeshSolver
{
public:
	typedef UE::Solvers::FPositionConstraint  FConstraintPosition;

	UE_API FConstrainedMeshDeformationSolver(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme, const EMatrixSolverType MatrixSolverType);
	UE_API virtual ~FConstrainedMeshDeformationSolver();

	// Create the solver using a generic mesh type. Assume uniform weight scheme.
	template<typename MeshType>
	FConstrainedMeshDeformationSolver(const MeshType& Mesh, const EMatrixSolverType MatrixSolverType);

	// Add constraint associated with given vertex id.  Boundary vertices will be ignored
	UE_API void AddConstraint(const int32 VtxId, const double Weight, const FVector3d& Pos, const bool bPostFix) override;

	// Update the position of an existing constraint.  Bool return if a corresponding constraint weight exists. Boundary vertices will be ignored (and return false).  
	UE_API bool UpdateConstraintPosition(const int32 VtxId, const FVector3d& Position, const bool bPostFix) override;

	// The underlying solver will have to refactor the matrix if this is done. Bool return if a corresponding constraint position exists. Boundary vertices will be ignored (and return false).  
	UE_API bool UpdateConstraintWeight(const int32 VtxId, const double Weight) override;

	// Clear all constraints associated with this smoother
	void ClearConstraints() override { FConstrainedMeshDeformationSolver::ClearConstraintPositions(); FConstrainedMeshDeformationSolver::ClearConstraintWeights(); }

	void ClearConstraintWeights() override { ConstraintWeightMap.Empty();    bConstraintWeightsDirty = true; }

	void ClearConstraintPositions() override { ConstraintPositionMap.Empty();  bConstraintPositionsDirty = true; }


	// Test if for constraint associated with given vertex id. Will return false for any boundary vert.
	UE_API bool IsConstrained(const int32 VtxId) const override;

	virtual bool Deform(TArray<FVector3d>& PositionBuffer) override { return false; }

	// Sync constraints with internal solver.  If in the process any internal matrix factoring is dirty, it will be rebuilt.
	// Note: this is called from within the Deform() method.   Only call this method if you want to trigger the matrix refactor yourself.
	UE_API void UpdateSolverConstraints();



public:
	// these only apply to iterative solvers

	UE_API bool SetMaxIterations(int32 MaxIterations);
	UE_API bool SetTolerance(double Tol);


protected:

	// Call this from constructors after building matrices
	template<typename MeshType>
	void Init(const MeshType& Mesh, const ELaplacianWeightScheme Scheme, const EMatrixSolverType MatrixSolverType, const FSparseMatrixD& LaplacianInternal, const FSparseMatrixD& LaplacianBoundary)
	{
		// Copy the original boundary vertex locations
		const int32 BoundaryVertexCount = VtxLinearization.NumBoundaryVerts();

		// Number of vertices in the interior of the mesh
		InternalVertexCount = VertexCount - BoundaryVertexCount;

		// Copy the original boundary vertex locations
		{
			const auto ToVertId = VtxLinearization.ToId();
			BoundaryPositions.SetZero(BoundaryVertexCount);
			for (int32 i = 0; i < BoundaryVertexCount; ++i)
			{
				const int32 VtxId = ToVertId[i + InternalVertexCount];
				const FVector3d Pos = Mesh.GetVertex(VtxId);
				BoundaryPositions.SetXYZ(i, Pos);
			}
		}

		checkSlow(LaplacianInternal.rows() == LaplacianInternal.cols());

		TUniquePtr<FSparseMatrixD> LTLPtr(new FSparseMatrixD(LaplacianInternal.rows(), LaplacianInternal.cols()));
		FSparseMatrixD& LTLMatrix = *(LTLPtr);

		bool bIsLaplacianSymmetric = (Scheme == ELaplacianWeightScheme::Valence || Scheme == ELaplacianWeightScheme::Uniform);

		if (bIsLaplacianSymmetric)
		{
			// Laplacian is symmetric, i.e. equal to its transpose
			LTLMatrix = LaplacianInternal * LaplacianInternal;
			BoundaryOperator = -1. * LaplacianInternal * LaplacianBoundary;

			ConstrainedSolver.Reset(new FConstrainedSolver(LTLPtr, MatrixSolverType));
		}
		else
		{
			// the laplacian 
			LTLMatrix = LaplacianInternal.transpose() * LaplacianInternal;
			BoundaryOperator = -1. * LaplacianInternal.transpose() * LaplacianBoundary;

			ConstrainedSolver.Reset(new FConstrainedSolver(LTLPtr, MatrixSolverType));
		}
	}
	
	template<typename MeshType>
	void ExtractInteriorVertexPositions(const MeshType& Mesh, FSOAPositions& VertexPositions) const
	{
		VertexPositions.SetZero(InternalVertexCount);

		const auto& ToVtxId = VtxLinearization.ToId();

		for (int32 i = 0; i < InternalVertexCount; ++i)
		{
			const int32 VtxId = ToVtxId[i];
			const FVector3d& Pos = Mesh.GetVertex(VtxId);
			VertexPositions.SetXYZ(i, Pos);
		}
	}


	//void UpdateMeshWithConstraints();
	// Respect any bPostFix constraints by moving those vertices to position defined by said constraint.
	UE_API void UpdateWithPostFixConstraints(FSOAPositions& PositionalVector) const;


	// converts the positionalvector to a TArray<FVector3d> where the offset in the array is implicitly the
	// VtxId in the mesh, and not ness the matrix row id
	// NB: the resulting array is treated as sparse and may have un-initialized elements.
	UE_API bool CopyInternalPositions(const FSOAPositions& PositionalVector, TArray<FVector3d>& LinearArray) const;

	UE_API bool CopyBoundaryPositions(TArray<FVector3d>& LinearArray) const;


protected:

	// The Key (int32) here is the vertex index not vertex ID
	// making it the same as the matrix row
	TMap<int32, FConstraintPosition> ConstraintPositionMap;
	TMap<int32, double>              ConstraintWeightMap;

	bool bConstraintPositionsDirty = true;
	bool bConstraintWeightsDirty = true;

	// Cache the vertex count. boundary + internal
	int32                 VertexCount;

	// Cache the number of internal vertices
	int32                 InternalVertexCount;

	// Used to map between VtxId and vertex Index in linear vector..
	FVertexLinearization  VtxLinearization;

	// Boundary points, split into 3 arrays (x,y,z).
	FSOAPositions         BoundaryPositions;


	// Actual solver that manages the various linear algebra bits.
	TUniquePtr<FConstrainedSolver>         ConstrainedSolver;

	// Sparse Matrix that holds L^T * B where B has the boundary terms.
	FSparseMatrixD                         BoundaryOperator;
};



template<typename MeshType>
FConstrainedMeshDeformationSolver::FConstrainedMeshDeformationSolver(const MeshType& Mesh, const EMatrixSolverType MatrixSolverType)
	: VertexCount(Mesh.VertexCount())
{
	FSparseMatrixD LaplacianInternal;
	FSparseMatrixD LaplacianBoundary;
	ConstructUniformLaplacian(Mesh, VtxLinearization, LaplacianInternal, LaplacianBoundary);

	Init(Mesh, ELaplacianWeightScheme::Uniform, MatrixSolverType, LaplacianInternal, LaplacianBoundary);
}

/**
 * FSoftMeshDeformationSolver is an implmentation of IConstrainedLaplacianMeshSolver that solves
 * Mesh Deformation problems by using quadratic energy functions based on the vertex-graph Laplacians.
 *
 * The primary difference with FConstrainedMeshDeformationSolver is that boundary vertices do not
 * receive special treatment, they are included in the system and solved in the same way. As a result
 * it is generally necessary to add constraints for boundary vertices.
 *
 * All constraints are "soft", ie they are included in the system as weighted quadratic energies
 * rather than had constraints.
 *
 * Voronoi-Area Clamped Cotangent weights are used for the Laplacian, and an LU Sover
 */
class FSoftMeshDeformationSolver : public UE::Solvers::IConstrainedLaplacianMeshSolver
{
public:
	typedef UE::Solvers::FPositionConstraint  FPositionConstraint;

	UE_API FSoftMeshDeformationSolver(const FDynamicMesh3& DynamicMesh);
	UE_API virtual ~FSoftMeshDeformationSolver();

	//
	// IConstrainedMeshSolver API
	//

	// Add constraint associated with given vertex id.  Boundary vertices will be ignored
	UE_API void AddConstraint(const int32 VtxId, const double Weight, const FVector3d& Position, const bool bPostFix) override;

	// Update the position of an existing constraint.  Bool return if a corresponding constraint weight exists. Boundary vertices will be ignored (and return false).  
	UE_API bool UpdateConstraintPosition(const int32 VtxId, const FVector3d& Position, const bool bPostFix) override;

	// The underlying solver will have to refactor the matrix if this is done. Bool return if a corresponding constraint position exists. Boundary vertices will be ignored (and return false).  
	UE_API bool UpdateConstraintWeight(const int32 VtxId, const double Weight) override;

	// Clear all constraints associated with this smoother
	UE_API void ClearConstraints() override;

	// do not support these
	void ClearConstraintWeights() override { check(false); }
	void ClearConstraintPositions() override { check(false); }

	// Test if for constraint associated with given vertex id. Will return false for any boundary vert.
	UE_API bool IsConstrained(const int32 VtxId) const override;

	virtual bool Deform(TArray<FVector3d>& PositionBuffer) override { return false;	}


	//
	// IConstrainedLaplacianMeshSolver API
	//


	// Update global scale applied to Laplacian vectors before solve.
	UE_API virtual void UpdateLaplacianScale(double UniformScale);


public:

	// Sync constraints with internal solver.  If in the process any internal matrix factoring is dirty, it will be rebuilt.
	// Note: this is called from within the Deform() method.   Only call this method if you want to trigger the matrix refactor yourself.
	UE_API void UpdateSolverConstraints();


protected:

	// The Key (int32) here is the vertex index not vertex ID
	// making it the same as the matrix row
	TMap<int32, FPositionConstraint> ConstraintMap;

	bool bConstraintPositionsDirty = true;
	bool bConstraintWeightsDirty = true;

	// Used to map between VtxId and vertex Index in linear vector..
	FVertexLinearization  VtxLinearization;

	// Actual solver that manages the various linear algebra bits.
	TUniquePtr<FConstrainedSolver> ConstrainedSolver;

	double LaplacianScale = 1.0;
	UE_API bool HasLaplacianScale() const;
	UE_API double GetLaplacianScale(int32 LinearVtxIndex) const;
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
