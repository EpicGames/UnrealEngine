// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstrainedMeshDeformers.h"
#include "MatrixSolver.h"
#include "ConstrainedPoissonSolver.h"

using namespace UE::Geometry;


FConstrainedMeshDeformer::FConstrainedMeshDeformer(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme LaplacianType)
	: FConstrainedMeshDeformationSolver(DynamicMesh, LaplacianType, EMatrixSolverType::LU)
	, LaplacianVectors(FConstrainedMeshDeformationSolver::InternalVertexCount)
{


	// The current vertex positions 

	// Note: the OriginalInteriorPositions are being stored as member data 
	// for use if the solver is iterative.
	// FSOAPositions OriginalInteriorPositions; 
	ExtractInteriorVertexPositions(DynamicMesh, OriginalInteriorPositions);


	// The biharmonic part of the constrained solver
	//   Biharmonic := Laplacian^{T} * Laplacian

	const auto& Biharmonic = ConstrainedSolver->Biharmonic();

	// Compute the Laplacian Vectors
	//    := Biharmonic * VertexPostion
	// In the case of the cotangent laplacian this can be identified as the mean curvature * normal.
	checkSlow(LaplacianVectors.Num() == OriginalInteriorPositions.Num());

	for (int32 i = 0; i < 3; ++i)
	{
		LaplacianVectors.Array(i) = Biharmonic * OriginalInteriorPositions.Array(i);
	}
}


bool FConstrainedMeshDeformer::Deform(TArray<FVector3d>& PositionBuffer)
{

	// Update constraints.  This only trigger solver rebuild if the weights were updated.
	UpdateSolverConstraints();

	// If no internal vertices to solve for, fail out early (the solver expects at least one)
	if (InternalVertexCount == 0)
	{
		return false;
	}

	// Allocate space for the result as a struct of arrays
	FSOAPositions SolutionVector(InternalVertexCount);

	// Solve the linear system
	// NB: the original positions will only be used if the underlying solver type is iterative	
	bool bSuccess = ConstrainedSolver->SolveWithGuess(OriginalInteriorPositions, LaplacianVectors, SolutionVector);

	// Move any vertices to match bPostFix constraints

	UpdateWithPostFixConstraints(SolutionVector);

	// Allocate Position Buffer for random access writes
	int32 MaxVtxId = VtxLinearization.ToId().Num();
	PositionBuffer.Empty(MaxVtxId);
	PositionBuffer.AddUninitialized(MaxVtxId);

	// Export the computed internal positions:
	// Copy the results into the array of structs form.  
	// NB: this re-indexes so the results can be looked up using VtxId

	CopyInternalPositions(SolutionVector, PositionBuffer);

	// Copy the boundary
	// NB: this re-indexes so the results can be looked up using VtxId
	CopyBoundaryPositions(PositionBuffer);

	// the matrix solve state
	return bSuccess;

}











FSoftMeshDeformer::FSoftMeshDeformer(const FDynamicMesh3& DynamicMesh)
	: FSoftMeshDeformationSolver(DynamicMesh)
{
	int32 NumVertices = VtxLinearization.NumVerts();
	LaplacianVectors = FSOAPositions(NumVertices);

	// The current vertex positions 

	// Note: the OriginalInteriorPositions are being stored as member data 
	// for use if the solver is iterative.
	OriginalPositions.SetZero(NumVertices);
	const TArray<int32>& ToVtxId = VtxLinearization.ToId();
	for (int32 i = 0; i < NumVertices; ++i)
	{
		OriginalPositions.SetXYZ(i, DynamicMesh.GetVertex(ToVtxId[i]));
	}

	// The biharmonic part of the constrained solver
	//   Biharmonic := Laplacian^{T} * Laplacian

	const auto& Biharmonic = ConstrainedSolver->Biharmonic();

	// Compute the Laplacian Vectors
	//    := Biharmonic * VertexPostion
	// In the case of the cotangent laplacian this can be identified as the mean curvature * normal.
	for (int32 i = 0; i < 3; ++i)
	{
		LaplacianVectors.Array(i) = Biharmonic * OriginalPositions.Array(i);
	}
}


bool FSoftMeshDeformer::Deform(TArray<FVector3d>& PositionBuffer)
{
	// Update constraints.  This only trigger solver rebuild if the weights were updated.
	UpdateSolverConstraints();

	// Allocate space for the result as a struct of arrays
	FSOAPositions SolutionVector(VtxLinearization.NumVerts());

	// solve linear system
	bool bSuccess = false;
	if (HasLaplacianScale())
	{
		FSOAPositions ScaledLaplacians = LaplacianVectors;
		int32 NumVectors = ScaledLaplacians.Num();
		for (int32 k = 0; k < NumVectors; ++k)
		{
			ScaledLaplacians.Set(k, GetLaplacianScale(k) * ScaledLaplacians.Get(k));
		}
		bSuccess = ConstrainedSolver->SolveWithGuess(OriginalPositions, ScaledLaplacians, SolutionVector);
	}
	else
	{
		// NB: the original positions will only be used if the underlying solver type is iterative	
		bSuccess = ConstrainedSolver->SolveWithGuess(OriginalPositions, LaplacianVectors, SolutionVector);
	}

	// Move any vertices to match bPostFix constraints
	for (const auto& Constraint : ConstraintMap)
	{
		const int32 Index = Constraint.Key;
		if (Constraint.Value.bPostFix)
		{
			SolutionVector.SetXYZ(Index, Constraint.Value.Position);
		}
	}

	// Allocate Position Buffer for random access writes
	int32 MaxVtxId = VtxLinearization.ToId().Num();
	PositionBuffer.Empty(MaxVtxId);
	PositionBuffer.AddUninitialized(MaxVtxId);

	// Export the computed positions
	int32 NumVertices = VtxLinearization.NumVerts();
	const TArray<int32>& ToVtxId = VtxLinearization.ToId();
	for (int32 i = 0; i < NumVertices; ++i)
	{
		const int32 VtxId = ToVtxId[i];
		PositionBuffer[VtxId] = FVector3d(SolutionVector.X(i), SolutionVector.Y(i), SolutionVector.Z(i));
	}

	// the matrix solve state
	return bSuccess;

}
