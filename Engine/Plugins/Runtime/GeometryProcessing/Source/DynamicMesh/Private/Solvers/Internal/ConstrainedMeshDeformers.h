// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ConstrainedMeshDeformationSolver.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{


/**
 * FConstrainedMeshDeformer solves detail-preserving mesh deformation problems with arbitrary position constraints.
 * The initial Mesh Laplacians are defined as Biharmonic * VtxPositions
 * A direct solver is used, currently LU decomposition.
 *
 * Boundary vertices are fixed to their input positions.
 */
class FConstrainedMeshDeformer : public FConstrainedMeshDeformationSolver
{
public:
	UE_API FConstrainedMeshDeformer(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme LaplacianType);
	~FConstrainedMeshDeformer() override = default;

	// Generic mesh type, assume uniform weight scheme
	template<typename MeshT>
	FConstrainedMeshDeformer(const MeshT& Mesh);
	
	UE_API bool Deform(TArray<FVector3d>& PositionBuffer) override;

private:

	FSOAPositions LaplacianVectors;
	FSOAPositions OriginalInteriorPositions;
};

// Generic mesh type, assume uniform weight scheme
template<typename MeshT>
FConstrainedMeshDeformer::FConstrainedMeshDeformer(const MeshT& Mesh) :
	FConstrainedMeshDeformationSolver(Mesh, EMatrixSolverType::LU),
	LaplacianVectors(FConstrainedMeshDeformationSolver::InternalVertexCount)
{
	// The current vertex positions 
	// Note: the OriginalInteriorPositions are being stored as member data 
	// for use if the solver is iterative.
	// FSOAPositions OriginalInteriorPositions; 
	ExtractInteriorVertexPositions(Mesh, OriginalInteriorPositions);

	// The biharmonic part of the constrained solver
	//   Biharmonic := Laplacian^{T} * Laplacian

	const auto& Biharmonic = ConstrainedSolver->Biharmonic();

	// Compute the Laplacian Vectors
	//    := Biharmonic * VertexPostion
	// In the case of the cotangent laplacian this can be identified as the mean curvature * normal.
	checkSlow(LaplacianVectors.Num() == OriginalInteriorPositions.Num());

	for (int32 i = 0; i < 3; ++i)
	{
		LaplacianVectors.Array(i) = Biharmonic * OriginalInteriorPositions.Array(i);
	}
}


/**
 * FSoftMeshDeformer solves detail-preserving mesh deformation problems with arbitrary position constraints.
 * The initial Mesh Laplacians are defined as Biharmonic * VtxPositions
 * A direct solver is used, currently LU decomposition.
 * Clamped Cotangent weights with Voronoi area are used.
 * Boundary Vertices are *not* fixed, they are included in the system and so should have soft constraints set similar to any other vertex
 */
class FSoftMeshDeformer : public FSoftMeshDeformationSolver
{
public:
	UE_API FSoftMeshDeformer(const FDynamicMesh3& DynamicMesh);
	~FSoftMeshDeformer() override = default;

	UE_API bool Deform(TArray<FVector3d>& PositionBuffer) override;

private:

	FSOAPositions LaplacianVectors;
	FSOAPositions OriginalPositions;
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
