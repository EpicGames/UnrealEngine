// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstrainedMeshSmoothers.h"
#include "MatrixSolver.h"
#include "ConstrainedPoissonSolver.h"

using namespace UE::Geometry;

bool FBiHarmonicMeshSmoother::ComputeSmoothedMeshPositions(TArray<FVector3d>& UpdatedPositions)
{

	UpdateSolverConstraints();

	// Compute the source vector
	FSOAPositions SourceVector(InternalVertexCount);

	if (InternalVertexCount != VertexCount) // have boundary points
	{
		for (int32 dir = 0; dir < 3; ++dir)
		{
			SourceVector.Array(dir) = BoundaryOperator * BoundaryPositions.Array(dir);
		}
	}
	else
	{
		SourceVector.SetZero(InternalVertexCount);
	}

	// Solves the constrained system and updates the mesh 

	FSOAPositions SolutionVector(InternalVertexCount);

	bool bSuccess = ConstrainedSolver->Solve(SourceVector, SolutionVector);


	// Move any vertices to match bPostFix constraints

	UpdateWithPostFixConstraints(SolutionVector);

	// Allocate Position Buffer for random access writes
	int32 MaxVtxId = VtxLinearization.ToId().Num();
	UpdatedPositions.Empty(MaxVtxId);
	UpdatedPositions.AddUninitialized(MaxVtxId);

	// Export the computed internal positions:
	// Copy the results into the array of structs form.  
	// NB: this re-indexes so the results can be looked up using VtxId

	CopyInternalPositions(SolutionVector, UpdatedPositions);

	// Copy the boundary
	// NB: this re-indexes so the results can be looked up using VtxId
	CopyBoundaryPositions(UpdatedPositions);

	return bSuccess;
}






bool FCGBiHarmonicMeshSmoother::ComputeSmoothedMeshPositions(TArray<FVector3d>& UpdatedPositions)
{
	// NB: This conjugate gradient solver could be updated to use  solveWithGuess() method on the iterative solver

	UpdateSolverConstraints();

	// Compute the source vector
	FSOAPositions SourceVector(InternalVertexCount);

	if (InternalVertexCount != VertexCount) // have boundary points
	{
		for (int32 dir = 0; dir < 3; ++dir)
		{
			SourceVector.Array(dir) = BoundaryOperator * BoundaryPositions.Array(dir);
		}
	}
	else
	{
		SourceVector.SetZero(InternalVertexCount);
	}

	// Solves the constrained system and updates the mesh 

	// Solves the constrained system

	FSOAPositions SolutionVector(InternalVertexCount);

	bool bSuccess = ConstrainedSolver->Solve(SourceVector, SolutionVector);


	// Move any vertices to match bPostFix constraints

	UpdateWithPostFixConstraints(SolutionVector);

	// Allocate Position Buffer for random access writes
	int32 MaxVtxId = VtxLinearization.ToId().Num();
	UpdatedPositions.Empty(MaxVtxId);
	UpdatedPositions.AddUninitialized(MaxVtxId);

	// Export the computed internal postions:
	// Copy the results into the array of structs form.  
	// NB: this re-indexes so the results can be looked up using VtxId

	CopyInternalPositions(SolutionVector, UpdatedPositions);

	// Copy the boundary
	// NB: this re-indexes so the results can be looked up using VtxId
	CopyBoundaryPositions(UpdatedPositions);

	return bSuccess;
}

