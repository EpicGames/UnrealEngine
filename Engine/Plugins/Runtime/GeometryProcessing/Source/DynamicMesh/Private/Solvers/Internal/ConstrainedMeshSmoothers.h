// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "ConstrainedMeshDeformationSolver.h"


namespace UE
{
namespace Geometry
{


/**
 * FConstrainedMeshDeformer solves implicit mesh smoothing problems with arbitrary position constraints.
 * A direct solver is used, currently LU decomposition.
 *
 * Boundary vertices are fixed to their input positions.
 */
class FBiHarmonicMeshSmoother : public FConstrainedMeshDeformationSolver
{
public:
	typedef FConstrainedMeshDeformationSolver         MyBaseType;

	FBiHarmonicMeshSmoother(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme) :
		MyBaseType(DynamicMesh, Scheme, EMatrixSolverType::LU)
	{}

	bool Deform(TArray<FVector3d>& UpdatedPositions) override
	{
		return	ComputeSmoothedMeshPositions(UpdatedPositions);
	}

	// (Direct) Solve the constrained system and populate the UpdatedPositions with the result 
	bool ComputeSmoothedMeshPositions(TArray<FVector3d>& UpdatedPositions);

};





/**
 * FConstrainedMeshDeformer solves implicit mesh smoothing problems with arbitrary position constraints.
 * An iterative Conjugate Gradient solver is used to compute the solution.
 *
 * Boundary vertices are fixed to their input positions.
 */
class FCGBiHarmonicMeshSmoother : public FConstrainedMeshDeformationSolver
{
public:
	typedef FConstrainedMeshDeformationSolver      MyBaseType;

	FCGBiHarmonicMeshSmoother(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme) :
		MyBaseType(DynamicMesh, Scheme, EMatrixSolverType::BICGSTAB)
	{}

	bool Deform(TArray<FVector3d>& UpdatedPositions) override
	{
		return	ComputeSmoothedMeshPositions(UpdatedPositions);
	}

	// (Iterative) Solve the constrained system and populate the UpdatedPositions with the result 
	bool ComputeSmoothedMeshPositions(TArray<FVector3d>& UpdatedPositions);

};


} // end namespace UE::Geometry
} // end namespace UE