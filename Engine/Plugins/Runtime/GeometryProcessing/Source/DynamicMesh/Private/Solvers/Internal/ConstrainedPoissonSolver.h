// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "SparseMatrix.h"
#include "MatrixSolver.h"
#include "Math/NumericLimits.h"

// only needed if you enable logging by changing the typedef for FConstrainedSolverTimeLogger
//#include "ProfilingDebugging/ScopedTimers.h"

namespace UE
{
namespace Geometry
{


/**
* Least Squares constrained poisson solver.
*
* Where L is a Laplacian type matrix
*       {lambda_i, cvec_i} are k weights (lambda) and positional constraints.
*
* Solves in the lease squares sense the equation
*         L p_vec = d_vec
*         lambda_i * (p_vec)_i  = lambda_i (c_vec)_i  for some constraints.
*
* this amounts to
*         ( Transpose(L) * L   + (0  0      )  ) p_vec = source_vec + ( 0              )
*    	  (                      (0 lambda^2)  )                      ( lambda^2 c_vec )
*
* where source_vec = Transpose(L)*L p_vec 
*
* C.F. http://sites.fas.harvard.edu/~cs277/papers/deformation_survey.pdf  section V, subsection C
*
* @todo consider reworking to hold TUniquePtr<> to the various objects.
*/

class FConstrainedSolver
{
public:

	typedef typename FSparseMatrixD::Scalar     ScalarType;
	typedef typename FSOAPositions::VectorType  VectorType;

	struct FDummyTimeLogger
	{
		FDummyTimeLogger(FString InMsg = TEXT("Scoped action"), FOutputDevice* InDevice = GLog) {}
	};

	// Change this to enable time logging for testing.
	//typedef FScopedDurationTimeLogger FConstrainedSolverTimeLogger; 
	typedef FDummyTimeLogger FConstrainedSolverTimeLogger;


	FConstrainedSolver(TUniquePtr<FSparseMatrixD>& SymmatrixMatrixOperator, const EMatrixSolverType MatrixSolverType)
		: ConstraintPositions((int32)SymmatrixMatrixOperator->cols())
	{
		checkSlow(SymmatrixMatrixOperator->cols() <= MAX_int32);
		SymmetricMatrixPtr.Reset(SymmatrixMatrixOperator.Release());

		bMatrixSolverDirty = true;
		MatrixSolver = ConstructMatrixSolver(MatrixSolverType);
	}


	// Updates the diagonal weights matrix.
	void SetConstraintWeights(const TMap<int32, double>& WeightMap)
	{
		typedef FSparseMatrixD::Scalar    ScalarT;
		typedef Eigen::Triplet<ScalarT>   MatrixTripletT;

		ClearWeights();

		std::vector<MatrixTripletT> MatrixTripelList;
		MatrixTripelList.reserve(WeightMap.Num());

		for (const auto& WeightPair : WeightMap)
		{
			const int32 i = WeightPair.Key; // row id
			double Weight = WeightPair.Value;

			checkSlow(i < SymmetricMatrixPtr->cols());

			// the soft constrained system uses the square of the weight.
			Weight *= Weight;

			//const int32 i = ToIndex[VertId];
			MatrixTripelList.push_back(MatrixTripletT(i, i, Weight));

		}

		// Construct matrix with weights on the diagonal for the constrained verts ( and zero everywhere else)
		WeightsSqrdMatrix.setFromTriplets(MatrixTripelList.begin(), MatrixTripelList.end());
		WeightsSqrdMatrix.makeCompressed();

		// The solver matrix will have to be updated and re-factored
		UpdateSolverWithContraints();
	}


	// Updates the diagonal weights matrix.
	void SetConstraintWeights(const TMap<int32, UE::Solvers::FPositionConstraint>& ConstraintMap)
	{
		typedef FSparseMatrixD::Scalar    ScalarT;
		typedef Eigen::Triplet<ScalarT>   MatrixTripletT;

		ClearWeights();

		std::vector<MatrixTripletT> MatrixTripleList;
		MatrixTripleList.reserve(ConstraintMap.Num());

		for (const auto& ConstraintPair : ConstraintMap)
		{
			const int32 i = ConstraintPair.Key; // row id
			double Weight = ConstraintPair.Value.Weight;

			checkSlow(i < SymmetricMatrixPtr->cols());

			// the soft constrained system uses the square of the weight.
			Weight *= Weight;

			//const int32 i = ToIndex[VertId];
			MatrixTripleList.push_back(MatrixTripletT(i, i, Weight));

		}

		// Construct matrix with weights on the diagonal for the constrained verts ( and zero everywhere else)
		WeightsSqrdMatrix.setFromTriplets(MatrixTripleList.begin(), MatrixTripleList.end());
		WeightsSqrdMatrix.makeCompressed();

		// The solver matrix will have to be updated and re-factored
		UpdateSolverWithContraints();
	}


	// Updates the positional source term
	void SetContraintPositions(const TMap<int32, UE::Solvers::FPositionConstraint>& PositionMap)
	{
		ClearConstraintPositions();

		for (const auto& PositionPair : PositionMap)
		{
			const int32 i = PositionPair.Key; // row id
			const FVector3d& Pos = PositionPair.Value.Position;

			checkSlow(i < SymmetricMatrixPtr->cols());


			// constrained source vector
			ConstraintPositions.SetX(i, Pos.X);
			ConstraintPositions.SetY(i, Pos.Y);
			ConstraintPositions.SetZ(i, Pos.Z);

		}
	}

	ScalarType GetWeightSqrd(int32 VtxIndex) const
	{
		return WeightsSqrdMatrix.coeff(VtxIndex, VtxIndex);
	}

	/**
	* Access to the underlying solver
	*/
	IMatrixSolverBase* GetMatrixSolverBase() { return MatrixSolver.Get(); }
	
	const IMatrixSolverBase* GetMatrixSolverBase() const { return MatrixSolver.Get(); }

	/**
	* Access to the underlying solver in iterative form - will return null if the solver is not iterative.
	*/
	IIterativeMatrixSolverBase* GetMatrixSolverIterativeBase()
	{
		IIterativeMatrixSolverBase* IterativeBasePtr = NULL;
		if (MatrixSolver->bIsIterative())
		{
			IterativeBasePtr = (IIterativeMatrixSolverBase*)MatrixSolver.Get();
		}
		return IterativeBasePtr;
	}
	const IIterativeMatrixSolverBase* GetMatrixSolverIterativeBase() const 
	{
		const IIterativeMatrixSolverBase* IterativeBasePtr = NULL;
		if (MatrixSolver->bIsIterative())
		{
			IterativeBasePtr = (const IIterativeMatrixSolverBase*)MatrixSolver.Get();
		}
		return IterativeBasePtr;
	}

	/**
	* @param SourceVector   - Required that each component vector has size equal to the the number of columns in the laplacian
	* @param SolutionVector - the resulting solution to the least squares problem
	*/
	bool Solve(const FSOAPositions& SourceVector, FSOAPositions& SolutionVector) const
	{
		checkSlow(bMatrixSolverDirty == false);

		if (MatrixSolver->bSucceeded() == false)  // check that the factorization succeeded 
		{
			return false;
		}

		FConstrainedSolverTimeLogger Timer(TEXT("Post-setup solve time"));

		FSparseMatrixD& SymmetricMatrix = *SymmetricMatrixPtr;
		// Set up the source vector
		FSOAPositions RHSVector((int32)SymmetricMatrix.cols());
		for (int32 Dir = 0; Dir < 3; ++Dir) 
		{
			RHSVector.Array(Dir) = SourceVector.Array(Dir) + WeightsSqrdMatrix * ConstraintPositions.Array(Dir);
		}
		
		MatrixSolver->Solve(RHSVector, SolutionVector);

		bool bSuccess = MatrixSolver->bSucceeded();

		return bSuccess;
	}


	/**
	* Special case when the source vector is identically zero.
	* NB: this is used for region smoothing.
	*/
	bool Solve(FSOAPositions& SolutionVector) const
	{
		checkSlow(bMatrixSolverDirty == false);

		if (MatrixSolver->bSucceeded() == false)  // check that the factorization succeeded 
		{
			return false;
		}

		FConstrainedSolverTimeLogger Timer(TEXT("Post-setup solve time"));

		FSparseMatrixD& SymmetricMatrix = *SymmetricMatrixPtr;
		FSOAPositions RHSVector((int32)SymmetricMatrix.cols());
		for (int32 Dir = 0; Dir < 3; ++Dir)
		{
			RHSVector.Array(Dir) = WeightsSqrdMatrix * ConstraintPositions.Array(Dir);
		}
	
		MatrixSolver->Solve(RHSVector, SolutionVector);

		bool bSuccess = MatrixSolver->bSucceeded();
		return bSuccess;
	}

	/*
	* For use with iterative solvers.
	*
	* Reverts to Solve(SolutionVector); 
	* if the matrix solver type is direct.
	*/
	bool SolveWithGuess(const FSOAPositions& Guess, FSOAPositions& SolutionVector) const
	{
		checkSlow(bMatrixSolverDirty == false);

		if (MatrixSolver->bIsIterative())
		{
			const IIterativeMatrixSolverBase* IterativeSolver = (IIterativeMatrixSolverBase*)MatrixSolver.Get();

			FConstrainedSolverTimeLogger Timmer(TEXT("Post-setup solve time"));

			FSparseMatrixD& SymmetricMatrix = *SymmetricMatrixPtr;
			FSOAPositions RHSVector((int32)SymmetricMatrix.cols());
			for (int32 Dir = 0; Dir < 3; ++Dir)
			{
				RHSVector.Array(Dir) = WeightsSqrdMatrix * ConstraintPositions.Array(Dir);
			}
			
			IterativeSolver->SolveWithGuess(Guess, RHSVector, SolutionVector);

			bool bSuccess = IterativeSolver->bSucceeded();
			return bSuccess;
		}
		else
		{
			return Solve(SolutionVector);
		}
	}

	/**
	* For use with iterative solvers.
	*
	* Reverts to Solve(SourceVector, SolutionVector);
	* if the matrix solver type is direct
	*/
	bool SolveWithGuess(const FSOAPositions& Guess, const FSOAPositions& SourceVector, FSOAPositions& SolutionVector) const
	{
		checkSlow(bMatrixSolverDirty == false);

		if (MatrixSolver->bIsIterative())
		{
			const IIterativeMatrixSolverBase* IterativeSolver = (IIterativeMatrixSolverBase*)MatrixSolver.Get();

			FConstrainedSolverTimeLogger Timmer(TEXT("Post-setup solve time"));

			FSparseMatrixD& SymmetricMatrix = *SymmetricMatrixPtr;
			// Set up the source vector
			FSOAPositions RHSVector((int32)SymmetricMatrix.cols());
			for (int32 Dir = 0; Dir < 3; ++Dir)
			{
				RHSVector.Array(Dir) = SourceVector.Array(Dir) + WeightsSqrdMatrix * ConstraintPositions.Array(Dir);
			}
		
			IterativeSolver->SolveWithGuess(Guess, RHSVector, SolutionVector);

			bool bSuccess = IterativeSolver->bSucceeded();

			return bSuccess;
		}
		else
		{
			return Solve(SourceVector, SolutionVector);
		}
	}


	const FSparseMatrixD& Biharmonic() const { return *SymmetricMatrixPtr; }


protected:

	// Zero the diagonal matrix that holds constraints
	void ClearConstraints()
	{
		ClearConstraintPositions();
		ClearWeights();
	}

	void ClearConstraintPositions()
	{
		const FSparseMatrixD& SymmetricMatrix = *SymmetricMatrixPtr;
		const int32 NumColumns = (int32)SymmetricMatrix.cols();

		ConstraintPositions.SetZero(NumColumns);

	}
	void ClearWeights()
	{
		WeightsSqrdMatrix.setZero();

		WeightsSqrdMatrix.resize(SymmetricMatrixPtr->rows(), SymmetricMatrixPtr->cols());

		// The constraints are part of the matrix
		bMatrixSolverDirty = true;
	}

	// Note: If constraints haven't been set, or if constraints have been cleared, this must be called prior to solve.
	void UpdateSolverWithContraints()
	{
		FConstrainedSolverTimeLogger Timmer(TEXT("Matrix setup time"));

		// The constrained verts are part of the matrix

		FSparseMatrixD& SymmetricMatrix = *SymmetricMatrixPtr;
		

		LHSMatrix =  SymmetricMatrix + WeightsSqrdMatrix;
		
		LHSMatrix.makeCompressed();
		// This matrix by construction is symmetric
		const bool bIsSymmetric = true;


		MatrixSolver->Reset();

		MatrixSolver->SetUp(LHSMatrix, bIsSymmetric);

		bMatrixSolverDirty = false;
	}

private:


	// Positional Constraint Arrays	
	FSOAPositions                  ConstraintPositions;

	TUniquePtr<FSparseMatrixD>     SymmetricMatrixPtr; // Transpose(Laplacian) * Laplacain
	FSparseMatrixD                 WeightsSqrdMatrix; // Weights Squared Diagonal Matrix.

	FSparseMatrixD                 LHSMatrix;

	bool                           bMatrixSolverDirty;
	TUniquePtr<IMatrixSolverBase>  MatrixSolver;


};


} // end namespace UE::Geometry
} // end namespace UE