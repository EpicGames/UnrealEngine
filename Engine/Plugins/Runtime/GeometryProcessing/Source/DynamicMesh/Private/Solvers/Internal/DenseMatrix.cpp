// Copyright Epic Games, Inc. All Rights Reserved.

#include "DenseMatrix.h"
#include "Math/RandomStream.h"

namespace UE
{
namespace Geometry
{

	template <typename ScalarType, int RowsAtCompileTime, int ColsAtCompileTime>
	void RandomDenseMatrix(TDenseMatrix<ScalarType, RowsAtCompileTime, ColsAtCompileTime>& OutMatrix,
		const int32 InRowsAtRunTime,
		const int32 InColsAtRunTime,
		const FRandomStream& InRandomStream)
	{
		auto RandNumFunc = [&InRandomStream]() { return InRandomStream.FRandRange(-1.0f, 1.0f); };
		OutMatrix = TDenseMatrix<ScalarType, RowsAtCompileTime, ColsAtCompileTime>::NullaryExpr(InRowsAtRunTime, InColsAtRunTime, RandNumFunc);
	}

	template <typename ScalarType, int RowsAtCompileTime, int ColsAtCompileTime>
	bool SliceDenseMatrix(const TDenseMatrix<ScalarType, RowsAtCompileTime,ColsAtCompileTime>& InMatrix, 
	                      const TArray<int>& InRowsToSlice,
	                      const TArray<int>& InColsToSlice,
	                      TDenseMatrixX<ScalarType>& OutSlicedMatrix)
	{
        const int32 NumRows = static_cast<int32>(InMatrix.rows());
        const int32 NumCols = static_cast<int32>(InMatrix.cols());
        int32 NumRowsSliced = InRowsToSlice.Num();
        int32 NumColsSliced = InColsToSlice.Num();

        if (NumRowsSliced == 0 && NumColsSliced == 0)
        {
            // if both slice masks are empty then simply return the copy
            OutSlicedMatrix = InMatrix;
            return true;
        }
        
        if (NumRows == 0 || NumCols == 0)
        {
            return true; // empty matrix, nothing to do
        }

        if (NumRowsSliced == 0)
        {
            NumRowsSliced = NumRows;
        }

        if (NumColsSliced == 0)
        {
			NumColsSliced = NumCols;
        }
        
        OutSlicedMatrix.resize(NumRowsSliced, NumColsSliced);
        for (int SlicedRowIndex = 0; SlicedRowIndex < NumRowsSliced; ++SlicedRowIndex)
        {
            for (int SlicedColIndex = 0; SlicedColIndex < NumColsSliced; ++SlicedColIndex)
            {	
				const int RowIndex = InRowsToSlice[SlicedRowIndex];
				const int ColIndex = InColsToSlice[SlicedColIndex];
				if (RowIndex < 0 || RowIndex >= NumRows || ColIndex < 0 || ColIndex >= NumCols)
				{
					return false;
				}

                OutSlicedMatrix(SlicedRowIndex, SlicedColIndex) = InMatrix(RowIndex, ColIndex);
            }
        }

        return true;
	}

	template <typename ScalarType, int RowsAtCompileTime, int ColsAtCompileTime>
	bool SliceDenseMatrix(const TDenseMatrix<ScalarType, RowsAtCompileTime, ColsAtCompileTime>& InMatrix,
		const TArray<int>& InRowsToSlice,
		TDenseMatrixX<ScalarType>& OutSlicedMatrix)
	{
		TArray<int> EmptyColArray;
		SliceDenseMatrix(InMatrix, InRowsToSlice, EmptyColArray, OutSlicedMatrix);

		return true;
	}


	// Explicit template instatiation
	template void UE::Geometry::RandomDenseMatrix<double, -1, 1>(class Eigen::Matrix<double, -1, 1, 0, -1, 1>&, int, int, struct FRandomStream const&);
	template bool UE::Geometry::SliceDenseMatrix<double, -1, 1>(class Eigen::Matrix<double, -1, 1, 0, -1, 1> const&, class TArray<int, class TSizedDefaultAllocator<32> > const&, class TArray<int, class TSizedDefaultAllocator<32> > const&, class Eigen::Matrix<double, -1, -1, 0, -1, -1>&); 
	template bool UE::Geometry::SliceDenseMatrix<double, -1, 1>(class Eigen::Matrix<double, -1, 1, 0, -1, 1> const&, class TArray<int, class TSizedDefaultAllocator<32> > const&, class Eigen::Matrix<double, -1, -1, 0, -1, -1>&);
}
}