// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif

#if defined(__clang__) && __clang_major__ > 9
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wimplicit-int-float-conversion"
#endif

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

#if defined(__clang__) && __clang_major__ > 9
#pragma clang diagnostic pop
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

// Forward declarations
struct FRandomStream;

namespace UE
{
namespace Geometry
{

// Template wrapper around Eigen's Dense Matrix class 
template <typename ScalarType, int RowsAtCompileTime, int ColsAtCompileTime>
using TDenseMatrix = Eigen::Matrix<ScalarType, RowsAtCompileTime, ColsAtCompileTime>;

// Dense matrix of arbitrary dimensions 
template <typename ScalarType>
using TDenseMatrixX = TDenseMatrix<ScalarType, Eigen::Dynamic, Eigen::Dynamic>;
using FDenseMatrixD = TDenseMatrixX<double>;
using FDenseMatrixF = TDenseMatrixX<float>;
using FDenseMatrixI = TDenseMatrixX<int32>;

// Dense matrix representing a column vector
template <class ScalarType>
using TColumnVector = TDenseMatrix<ScalarType, Eigen::Dynamic, 1>;
using FColumnVectorD = TColumnVector<double>;
using FColumnVectorF = TColumnVector<float>;
using FColumnVectorI = TColumnVector<int32>;

/**
 * Initialize a Dense Matrix with random values from the stream.
 * 
 * @param OutMatrix        Matrix will be resized and populated with the random values from the stream.
 * @param InRowsAtRunTime  Number of rows in the output matrix.
 * @param InColsAtRunTime  Number of columns in the output matrix.
 * @param InRandomStream   Random stream used for fetching random values.
 */
template <typename ScalarType, int RowsAtCompileTime, int ColsAtCompileTime>
void RandomDenseMatrix(TDenseMatrix<ScalarType, RowsAtCompileTime, ColsAtCompileTime>& OutMatrix, 
					   const int32 InRowsAtRunTime,
					   const int32 InColsAtRunTime,
					   const FRandomStream& InRandomStream);

/** 
 * Slice the matrix such that OutSlicedMatrix = InMatrix[InRowsToSlice, InColsToSlice].
 * 
 * @param InMatrix The matrix we are slicing.
 * @param InRowsToSlice Rows slice mask. If empty all rows will be used. 
 * @param InColsToSlice Columns slice mask. If empty all columns will be used.
 * @param OutSlicedMatrix The sliced matrix.
 * 
 * @return true if slicing succeeds, false otherwise. Slicing can fail if one of the input parameters is invalid.
 */
template <typename ScalarType, int RowsAtCompileTime, int ColsAtCompileTime>
bool SliceDenseMatrix(const TDenseMatrix<ScalarType, RowsAtCompileTime,ColsAtCompileTime>& InMatrix, 
                      const TArray<int>& InRowsToSlice,
                      const TArray<int>& InColsToSlice,
                      TDenseMatrixX<ScalarType>& OutSlicedMatrix);

/**
 * Slice the matrix only along the rows such that OutSlicedMatrix = InMatrix[InRowsToSlice, :].
 *
 * @param InMatrix The matrix we are slicing.
 * @param InRowsToSlice Rows slice mask. If empty all rows will be used.
 * @param OutSlicedMatrix The sliced matrix.
 *
 * @return true if slicing succeeds, false otherwise. Slicing can fail if one of the input parameters is invalid.
 */
template <typename ScalarType, int RowsAtCompileTime, int ColsAtCompileTime>
bool SliceDenseMatrix(const TDenseMatrix<ScalarType, RowsAtCompileTime, ColsAtCompileTime>& InMatrix,
					  const TArray<int>& InRowsToSlice,
					  TDenseMatrixX<ScalarType>& OutSlicedMatrix);

} // end namespace UE::Geometry
} // end namespace UE