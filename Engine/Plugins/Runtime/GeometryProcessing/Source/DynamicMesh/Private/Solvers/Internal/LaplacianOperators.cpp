// Copyright Epic Games, Inc. All Rights Reserved.

#include "LaplacianOperators.h"

#include "Solvers/MatrixInterfaces.h"


#include <cmath> // double version of sqrt
#include <vector> // used by eigen to initialize sparse matrix

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Dense> // for Matrix4d in testing
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

using namespace UE::Geometry;


void ConstructUmbrellaLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary)
{
	// Sync the mapping between the mesh vertex ids and their offsets in a nominal linear array.
	VertexMap.Reset(DynamicMesh);
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	FEigenSparseMatrixAssembler Interior(NumInteriorVerts, NumInteriorVerts);
	FEigenSparseMatrixAssembler Boundary(NumInteriorVerts, NumBoundaryVerts);
	UE::MeshDeformation::ConstructUmbrellaLaplacian<double>(DynamicMesh, VertexMap, Interior, Boundary);
	Interior.ExtractResult(LaplacianInterior);
	Boundary.ExtractResult(LaplacianBoundary);
}


void ConstructValenceWeightedLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary)
{
	// Sync the mapping between the mesh vertex ids and their offsets in a nominal linear array.
	VertexMap.Reset(DynamicMesh);
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	FEigenSparseMatrixAssembler Interior(NumInteriorVerts, NumInteriorVerts);
	FEigenSparseMatrixAssembler Boundary(NumInteriorVerts, NumBoundaryVerts);
	UE::MeshDeformation::ConstructValenceWeightedLaplacian<double>(DynamicMesh, VertexMap, Interior, Boundary);
	Interior.ExtractResult(LaplacianInterior);
	Boundary.ExtractResult(LaplacianBoundary);
}



void ConstructMeanValueWeightLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary)
{
	// Sync the mapping between the mesh vertex ids and their offsets in a nominal linear array.
	VertexMap.Reset(DynamicMesh);
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	FEigenSparseMatrixAssembler Interior(NumInteriorVerts, NumInteriorVerts);
	FEigenSparseMatrixAssembler Boundary(NumInteriorVerts, NumBoundaryVerts);
	UE::MeshDeformation::ConstructMeanValueWeightLaplacian<double>(DynamicMesh, VertexMap, Interior, Boundary);
	Interior.ExtractResult(LaplacianInterior);
	Boundary.ExtractResult(LaplacianBoundary);
}




void ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& AreaMatrix, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary)
{
	// Sync the mapping between the mesh vertex ids and their offsets in a nominal linear array.
	VertexMap.Reset(DynamicMesh);
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	FEigenSparseMatrixAssembler Interior(NumInteriorVerts, NumInteriorVerts);
	FEigenSparseMatrixAssembler Boundary(NumInteriorVerts, NumBoundaryVerts);
	FEigenSparseMatrixAssembler Area(NumInteriorVerts, NumInteriorVerts);
	UE::MeshDeformation::ConstructCotangentLaplacian<double>(DynamicMesh, VertexMap, Area, Interior, Boundary);
	Interior.ExtractResult(LaplacianInterior);
	Boundary.ExtractResult(LaplacianBoundary);
	Area.ExtractResult(AreaMatrix);
}


void ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary, const bool bClampWeights)
{
	// Sync the mapping between the mesh vertex ids and their offsets in a nominal linear array.
	VertexMap.Reset(DynamicMesh);
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	FEigenSparseMatrixAssembler Interior(NumInteriorVerts, NumInteriorVerts);
	FEigenSparseMatrixAssembler Boundary(NumInteriorVerts, NumBoundaryVerts);
	UE::MeshDeformation::ConstructCotangentLaplacian<double>(DynamicMesh, VertexMap, Interior, Boundary, bClampWeights);
	Interior.ExtractResult(LaplacianInterior);
	Boundary.ExtractResult(LaplacianBoundary);
}

void ConstructIDTCotangentLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary, const bool bClampWeights)
{
	// Sync the mapping between the mesh vertex ids and their offsets in a nominal linear array.
	VertexMap.Reset(DynamicMesh);
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	FEigenSparseMatrixAssembler Interior(NumInteriorVerts, NumInteriorVerts);
	FEigenSparseMatrixAssembler Boundary(NumInteriorVerts, NumBoundaryVerts);
	UE::MeshDeformation::ConstructIDTCotangentLaplacian<double>(DynamicMesh, VertexMap, Interior, Boundary, bClampWeights);
	Interior.ExtractResult(LaplacianInterior);
	Boundary.ExtractResult(LaplacianBoundary);
}

double ConstructScaledCotangentLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary, const bool bClampAreas)
{
	typedef FSparseMatrixD::Scalar  ScalarT;
	typedef Eigen::Triplet<ScalarT> MatrixTripletT;

	// diagonal mass matrix.
	FSparseMatrixD AreaMatrix;
	FSparseMatrixD CotangentInterior;
	FSparseMatrixD CotangentBoundary;
	ConstructCotangentLaplacian(DynamicMesh, VertexMap, AreaMatrix, CotangentInterior, CotangentBoundary);

	// Find average entry in the area matrix
	const int32 Rank = (int32)AreaMatrix.cols();
	double AveArea = 0.;
	for (int32 i = 0; i < Rank; ++i)
	{
		double Area = AreaMatrix.coeff(i, i);
		checkSlow(Area > 0.);  // Area must be positive.
		AveArea += Area;
	}
	AveArea /= (double)Rank;
	
	std::vector<MatrixTripletT> ScaledInvAreaTriplets;
	ScaledInvAreaTriplets.reserve(Rank);
	for (int32 i = 0; i < Rank; ++i)
	{
		double Area = AreaMatrix.coeff(i, i);
		double ScaledInvArea = AveArea / Area;
		if (bClampAreas)
		{
			ScaledInvArea = FMathd::Clamp(ScaledInvArea, 0.5, 5.); // when  squared this gives largest scales 100 x smallest
		}

		ScaledInvAreaTriplets.push_back(MatrixTripletT(i, i, ScaledInvArea));
	}

	FSparseMatrixD ScaledInvAreaMatrix(Rank, Rank);
	ScaledInvAreaMatrix.setFromTriplets(ScaledInvAreaTriplets.begin(), ScaledInvAreaTriplets.end());
	ScaledInvAreaMatrix.makeCompressed();

	LaplacianBoundary = ScaledInvAreaMatrix * CotangentBoundary;
	LaplacianBoundary.makeCompressed();
	LaplacianInterior = ScaledInvAreaMatrix * CotangentInterior;
	LaplacianInterior.makeCompressed();	

	return AveArea;
}





void ConstructLaplacian(const ELaplacianWeightScheme Scheme, const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary)
{

	switch (Scheme)
	{
	default:
	case ELaplacianWeightScheme::Uniform:
		ConstructUniformLaplacian(DynamicMesh, VertexMap, LaplacianInterior, LaplacianBoundary);
		break;
	case ELaplacianWeightScheme::Umbrella:
		ConstructUmbrellaLaplacian(DynamicMesh, VertexMap, LaplacianInterior, LaplacianBoundary);
		break;
	case ELaplacianWeightScheme::Valence:
		ConstructValenceWeightedLaplacian(DynamicMesh, VertexMap, LaplacianInterior, LaplacianBoundary);
		break;
	case ELaplacianWeightScheme::Cotangent:
	{
		bool bClampWeights = false;
		ConstructScaledCotangentLaplacian(DynamicMesh, VertexMap, LaplacianInterior, LaplacianBoundary, bClampWeights);
		break;
	}
	case ELaplacianWeightScheme::ClampedCotangent:
	{
		bool bClampWeights = true;
		ConstructScaledCotangentLaplacian(DynamicMesh, VertexMap, LaplacianInterior, LaplacianBoundary, bClampWeights);
		break;
	}
	case ELaplacianWeightScheme::IDTCotanget:
	{
		bool bClampWeights = true;
		ConstructIDTCotangentLaplacian(DynamicMesh, VertexMap, LaplacianInterior, LaplacianBoundary, bClampWeights);
		break;
	}
	case ELaplacianWeightScheme::MeanValue:
		ConstructMeanValueWeightLaplacian(DynamicMesh, VertexMap, LaplacianInterior, LaplacianBoundary);
		break;
	}
}






//
//
//


void ExtractBoundaryVerts(const FVertexLinearization& VertexMap, TArray<int32>& BoundaryVerts)
{
	int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	int32 NumInternalVerts = VertexMap.NumVerts() - NumBoundaryVerts;
	BoundaryVerts.Empty(NumBoundaryVerts);

	const auto& ToId = VertexMap.ToId();
	for (int32 i = NumInternalVerts; i < VertexMap.NumVerts(); ++i)
	{
		int32 VtxId = ToId[i];
		BoundaryVerts.Add(VtxId);
	}
}

TUniquePtr<FSparseMatrixD> ConstructUmbrellaLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, TArray<int32>* BoundaryVerts)
{
	TUniquePtr<FSparseMatrixD> LaplacianMatrix(new FSparseMatrixD);
	FSparseMatrixD BoundaryMatrix;

	ConstructUmbrellaLaplacian(DynamicMesh, VertexMap, *LaplacianMatrix, BoundaryMatrix);

	if (BoundaryVerts)
	{
		ExtractBoundaryVerts(VertexMap, *BoundaryVerts);
	}

	return LaplacianMatrix;
}


TUniquePtr<FSparseMatrixD> ConstructValenceWeightedLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, TArray<int32>* BoundaryVerts)
{
	TUniquePtr<FSparseMatrixD> LaplacianMatrix(new FSparseMatrixD);
	FSparseMatrixD BoundaryMatrix;

	ConstructValenceWeightedLaplacian(DynamicMesh, VertexMap, *LaplacianMatrix, BoundaryMatrix);

	if (BoundaryVerts)
	{
		ExtractBoundaryVerts(VertexMap, *BoundaryVerts);
	}

	return LaplacianMatrix;

}


TUniquePtr<FSparseMatrixD> ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, FSparseMatrixD& AreaMatrix, TArray<int32>* BoundaryVerts)
{
	TUniquePtr<FSparseMatrixD> LaplacianMatrix(new FSparseMatrixD);
	FSparseMatrixD BoundaryMatrix;

	ConstructCotangentLaplacian(DynamicMesh, VertexMap, AreaMatrix, *LaplacianMatrix, BoundaryMatrix);

	if (BoundaryVerts)
	{
		ExtractBoundaryVerts(VertexMap, *BoundaryVerts);
	}

	return LaplacianMatrix;
}



TUniquePtr<FSparseMatrixD> ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, const bool bClampWeights, TArray<int32>* BoundaryVerts)
{
	TUniquePtr<FSparseMatrixD> LaplacianMatrix(new FSparseMatrixD);
	FSparseMatrixD BoundaryMatrix;

	ConstructCotangentLaplacian(DynamicMesh, VertexMap, *LaplacianMatrix, BoundaryMatrix, bClampWeights);

	if (BoundaryVerts)
	{
		ExtractBoundaryVerts(VertexMap, *BoundaryVerts);
	}

	return LaplacianMatrix;
}

TUniquePtr<FSparseMatrixD> ConstructIDTCotangentLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, const bool bClampWeights, TArray<int32>* BoundaryVerts)
{
	TUniquePtr<FSparseMatrixD> LaplacianMatrix(new FSparseMatrixD);
	FSparseMatrixD BoundaryMatrix;

	ConstructIDTCotangentLaplacian(DynamicMesh, VertexMap, *LaplacianMatrix, BoundaryMatrix, bClampWeights);

	if (BoundaryVerts)
	{
		ExtractBoundaryVerts(VertexMap, *BoundaryVerts);
	}

	return LaplacianMatrix;
}


TUniquePtr<FSparseMatrixD> ConstructMeanValueWeightLaplacian(const FDynamicMesh3& DynamicMesh, FVertexLinearization& VertexMap, TArray<int32>* BoundaryVerts)
{
	TUniquePtr<FSparseMatrixD> LaplacianMatrix(new FSparseMatrixD);
	FSparseMatrixD BoundaryMatrix;

	ConstructMeanValueWeightLaplacian(DynamicMesh, VertexMap, *LaplacianMatrix, BoundaryMatrix);

	if (BoundaryVerts)
	{
		ExtractBoundaryVerts(VertexMap, *BoundaryVerts);
	}

	return LaplacianMatrix;
}




