// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Solvers/MeshLaplacian.h"
#include "Solvers/MeshLinearization.h"
#include "Solvers/LaplacianMatrixAssembly.h"
#include "SparseMatrix.h"

// this is a private header
using namespace UE::Geometry;

/**
* Construct a sparse matrix representation of the requested "Laplacian" type operator.
*
* The mesh itself is assumed to have N interior vertices, and M boundary vertices.
*
*
* @param Scheme             Selects the actual type of "laplacian" constructed.
* @param DynamicMesh        The triangle mesh
* @param VertexMap          On return, Additional arrays used to map between vertexID and offset in a linear array (i.e. the row).
*                           The vertices are ordered so that last M ( = VertexMap.NumBoundaryVerts() )  correspond to those on the boundary.
* @param LaplacianInterior  On return, the laplacian operator that acts on the interior vertices: sparse  N x N matrix
* @param LaplacianBoundary  On return, the portion of the operator that acts on the boundary vertices: sparse  N x M matrix
*
*   LaplacianInterior * Vector_InteriorVerts + LaplacianBoundary * Vector_BoundaryVerts = Full Laplacian applied to interior vertices.
*/
void ConstructLaplacian(const ELaplacianWeightScheme Scheme,
	                   const FDynamicMesh3& DynamicMesh,
	                   FVertexLinearization& VertexMap,
	                   FSparseMatrixD& LaplacianInterior,
	                   FSparseMatrixD& LaplacianBoundary);






//
// Functions below are only used in current Unit Test code, and can be removed if/when 
// those tests are rewritten against ConstructLaplacian() above
//

void ConstructCotangentLaplacian(
	const FDynamicMesh3& DynamicMesh, 
	FVertexLinearization& VertexMap, 
	FSparseMatrixD& LaplacianInterior, 
	FSparseMatrixD& LaplacianBoundary, 
	const bool bClampWeights);

TUniquePtr<FSparseMatrixD> ConstructLaplacian(const ELaplacianWeightScheme Scheme,
	const FDynamicMesh3& DynamicMesh,
	FVertexLinearization& VertexMap,
	TArray<int32>* BoundaryVerts = NULL);

TUniquePtr<FSparseMatrixD> ConstructUmbrellaLaplacian(const FDynamicMesh3& DynamicMesh,
	FVertexLinearization& VertexMap,
	TArray<int32>* BoundaryVerts = NULL);

TUniquePtr<FSparseMatrixD>  ConstructValenceWeightedLaplacian(const FDynamicMesh3& DynamicMesh3,
	FVertexLinearization& VertexMap,
	TArray<int32>* BoundaryVerts = NULL);

TUniquePtr<FSparseMatrixD> ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh,
	FVertexLinearization& VertexMap,
	const bool bClampWeights = false,
	TArray<int32>* BoundaryVerts = NULL);

TUniquePtr<FSparseMatrixD> ConstructIDTCotangentLaplacian(const FDynamicMesh3& DynamicMesh,
	FVertexLinearization& VertexMap,
	const bool bClampWeights = false,
	TArray<int32>* BoundaryVerts = NULL);

TUniquePtr<FSparseMatrixD> ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh,
	FVertexLinearization& VertexMap,
	FSparseMatrixD& AreaMatrix,
	TArray<int32>* BoundaryVerts = NULL);

TUniquePtr<FSparseMatrixD> ConstructMeanValueWeightLaplacian(const FDynamicMesh3& DynamicMesh,
	FVertexLinearization& VertexMap,
	TArray<int32>* BoundaryVerts = NULL);

void ExtractBoundaryVerts(const FVertexLinearization& VertexMap, TArray<int32>& BoundaryVerts);


template<typename MeshType>
void ConstructUniformLaplacian(const MeshType& Mesh, FVertexLinearization& VertexMap, FSparseMatrixD& LaplacianInterior, FSparseMatrixD& LaplacianBoundary)
{
	// Sync the mapping between the mesh vertex ids and their offsets in a nominal linear array.
	VertexMap.Reset(Mesh);
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	FEigenSparseMatrixAssembler Interior(NumInteriorVerts, NumInteriorVerts);
	FEigenSparseMatrixAssembler Boundary(NumInteriorVerts, NumBoundaryVerts);
	UE::MeshDeformation::ConstructUniformLaplacian<double>(Mesh, VertexMap, Interior, Boundary);
	Interior.ExtractResult(LaplacianInterior);
	Boundary.ExtractResult(LaplacianBoundary);
}

template<typename MeshType>
TUniquePtr<FSparseMatrixD> ConstructUniformLaplacian(const MeshType& Mesh, FVertexLinearization& VertexMap, TArray<int32>* BoundaryVerts)
{
	TUniquePtr<FSparseMatrixD> LaplacianMatrix(new FSparseMatrixD);
	FSparseMatrixD BoundaryMatrix;

	ConstructUniformLaplacian(Mesh, VertexMap, *LaplacianMatrix, BoundaryMatrix);

	if (BoundaryVerts)
	{
		ExtractBoundaryVerts(VertexMap, *BoundaryVerts);
	}

	return LaplacianMatrix;
}

