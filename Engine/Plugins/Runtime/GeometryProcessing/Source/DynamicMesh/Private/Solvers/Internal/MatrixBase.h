// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "SparseMatrix.h"

namespace UE
{
namespace Geometry
{

enum class EMatrixSolverType
{
	LU /* SparseLU*/,
	QR /* SparseQR */,
	BICGSTAB /* Iterative Bi-conjugate gradient */,
	PCG /* Pre-conditioned conjugate gradient  - requires symmetric positive def.*/,
#ifndef EIGEN_MPL2_ONLY
	LDLT /**Not included due to MPL2 */,
#endif
	FastestPSD /* Get the best available solver for the positive semidefinite matrix.*/,
};

static FString MatrixSolverName(const EMatrixSolverType SolverType)
{
	FString String;
	switch (SolverType)
	{
	case EMatrixSolverType::LU:
		String = FString(TEXT(" Direct LU "));
		break;
	case EMatrixSolverType::QR:
		String = FString(TEXT(" Direct QR "));
		break;
	case EMatrixSolverType::BICGSTAB:
		String = FString(TEXT(" Iterative BiConjugate Gradient "));
		break;
	case EMatrixSolverType::PCG:
		String = FString(TEXT(" Iterative Preconditioned Conjugate Gradient "));
		break;
#ifndef EIGEN_MPL2_ONLY
	case EMatrixSolverType::LDLT:
		String = FString(TEXT(" Direct Cholesky "));
		break;
#endif
	case EMatrixSolverType::FastestPSD:
		String = FString(TEXT(" Fastest positive semidefinite "));
		break; 
	default:
		check(0);
	}

	return String;
}

class FMatrixSolverSettings
{
public:
	EMatrixSolverType  MatrixSolverType;

	// Used by iterative solvers
	int32  MaxIterations = 600;
	double Tolerance = 1e-4;
};



// Pure Virtual Base Class used to wrap Eigen Solvers.
class IMatrixSolverBase
{
public:
	typedef typename FSparseMatrixD::Scalar    ScalarType;
	typedef typename FSOAPositions::VectorType VectorType;
	typedef typename Eigen::Matrix<ScalarType, Eigen::Dynamic, 1>  RealVectorType;

	IMatrixSolverBase() {}
	virtual ~IMatrixSolverBase() = 0;
	virtual bool bIsIterative() const = 0;
	virtual void Solve(const RealVectorType& BVector, RealVectorType& SolVector) const = 0;
	virtual void Solve(const VectorType& BVector, VectorType& SolVector) const = 0;
	virtual void Solve(const FSOAPositions& BVectors, FSOAPositions& SolVectors) const = 0;

	virtual void SetUp(const FSparseMatrixD& Matrix, bool bIsSymmetric) = 0;
	//virtual void SetIterations(int32 MaxIterations) = 0;
	virtual void Reset() = 0;
	virtual bool bSucceeded() const = 0;
};

// Additional methods particular to the iterative solvers
class IIterativeMatrixSolverBase : public IMatrixSolverBase
{
public:
	IIterativeMatrixSolverBase() {}

	virtual ~IIterativeMatrixSolverBase() = 0;
	virtual void SetIterations(int32 MaxIterations) = 0;
	virtual void SetTolerance(double Tol) = 0;
	virtual void SolveWithGuess(const RealVectorType& GuessVector, const RealVectorType& BVector, RealVectorType& SolVector)  const = 0;
	virtual void SolveWithGuess(const VectorType& GuessVector, const VectorType& BVector, VectorType& SolVector)  const = 0;
	virtual void SolveWithGuess(const FSOAPositions& GuessVector, const FSOAPositions& BVector, FSOAPositions& SolVector)  const = 0;

};

}
}