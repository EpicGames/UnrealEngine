// Copyright Epic Games, Inc. All Rights Reserved.

#include "MatrixSolver.h"

namespace UE
{
namespace Geometry
{

IMatrixSolverBase::~IMatrixSolverBase() {};
IIterativeMatrixSolverBase::~IIterativeMatrixSolverBase() {};

TUniquePtr<IMatrixSolverBase> ConstructMatrixSolver(const EMatrixSolverType& MatrixSolverType)
{
	TUniquePtr<IMatrixSolverBase> ResultPtr;

	switch (MatrixSolverType)
	{
	default:

	case  EMatrixSolverType::LU:
		ResultPtr.Reset(new FLUMatrixSolver());
		break;
	case EMatrixSolverType::QR:
		ResultPtr.Reset(new FQRMatrixSolver());
		break;
	case EMatrixSolverType::PCG:
		ResultPtr.Reset(new FPCGMatrixSolver());
		break;	
	case EMatrixSolverType::BICGSTAB:
		ResultPtr.Reset(new FBiCGMatrixSolver());
		break;
#ifndef EIGEN_MPL2_ONLY
	case EMatrixSolverType::LDLT:
		ResultPtr.Reset(new FLDLTMatrixSolver());
		break;
#endif
	case EMatrixSolverType::FastestPSD:
		#ifndef EIGEN_MPL2_ONLY
			ResultPtr.Reset(new FLDLTMatrixSolver());
		#else
			ResultPtr.Reset(new FLUMatrixSolver());
		#endif
		break;
	}

	return ResultPtr;
}

}
}