// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Async/ParallelFor.h"

#include "MatrixBase.h"

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Sparse>
#include <Eigen/Core>
#include <Eigen/SparseLU>
#include <Eigen/SparseQR>
#include <Eigen/OrderingMethods>
#include <Eigen/IterativeLinearSolvers> // BiCGSTAB & IncompleteLUT
#ifndef EIGEN_MPL2_ONLY
#include <Eigen/SparseCholesky>
#endif
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

#include "ProfilingDebugging/ScopedTimers.h"

namespace UE
{
namespace Geometry
{
// Matrix Solver Factory
TUniquePtr<IMatrixSolverBase> ConstructMatrixSolver(const EMatrixSolverType& MatrixSolverType);

template <typename DirectSolverType>
class TMatrixSolver : public IMatrixSolverBase
{
public:
	typedef DirectSolverType  SolverType;
	TMatrixSolver() {}
	~TMatrixSolver() override {};

	bool bIsIterative() const override { return false; }

	void Solve(const RealVectorType& BVector, RealVectorType& SolVector) const override
	{
		if (ensure(bSetup))
		{
			SolVector = MatrixSolver.solve(BVector);
		}
	}

	void Solve(const VectorType& BVector, VectorType& SolVector) const override
	{
		if (ensure(bSetup))
		{
			SolVector = MatrixSolver.solve(BVector);
		}
	}

	void Solve(const FSOAPositions& BVectors, FSOAPositions& SolVectors) const override
	{
		const bool bForceSingleThreaded = false;
		if (ensure(bSetup))
		{
			ParallelFor(3, [&](int Dir)
			{ SolVectors.Array(Dir) = MatrixSolver.solve(BVectors.Array(Dir)); }, bForceSingleThreaded);
		}
	}

	void SetUp(const FSparseMatrixD& SparseMatrix, bool bIsSymmetric) override
	{
		SetSymmetry(bIsSymmetric);
		SetUp(SparseMatrix);
	}

	void SetSymmetry(bool bIsSymmetric);

	//void SetIterations(int32 MaxIterations) override {}

	void Reset() override
	{
		//MatrixSolver = FMatrixSolver();
		bSetup = false;
	}

	bool bSucceeded() const override
	{
		const auto Info = MatrixSolver.info();
		bool bSuccess = bSetup && (Info == Eigen::ComputationInfo::Success);
		return bSuccess;
	}


private:

	void SetUp(const FSparseMatrixD& SparseMatrix)
	{
		if (SparseMatrix.rows() == 0)
		{
			return;
		}
		
		// The analyzePattern could be done just once if
		// the sparsity pattern of the matrix is fixed.
		// But testing indicates that takes little time compared 
		// with factorizing:  e.g. dim 14508 matrix.  Analyze 0.145s, Factorize 0.935s 

		MatrixSolver.analyzePattern(SparseMatrix);
		MatrixSolver.factorize(SparseMatrix);

		// if the factorize succeeded, info() status should return 'success'
		bSetup = (MatrixSolver.info() == Eigen::ComputationInfo::Success);
	}
private:

	bool bSetup = false;
	DirectSolverType  MatrixSolver;
};


/**
* Define wrappers for Direct Matrix Solver Types
*/
// Timing Tests:
//    7k Tri  3.6k Verts.  10 sets of 3 backsolves  0.17s
//    29k tri  14.5k Verts                           1.3s
//   100k tri  51.5k Verts                           5.39s
//   127k tri  63.4k Verts                           3.0s
typedef TMatrixSolver<Eigen::SparseLU <FSparseMatrixD, Eigen::COLAMDOrdering<int>>>  FLUMatrixSolver;
typedef TMatrixSolver<Eigen::SparseQR<FSparseMatrixD, Eigen::COLAMDOrdering<int>>>   FQRMatrixSolver;

#ifndef EIGEN_MPL2_ONLY
// Not included due to MPL2.  But in general much faster than standard LU

// Timing info: 
//              29k tris 14.5k verts    10 sets of 3 backsolves - 0.42s   Analyze 0.03s Factorize 0.28s
//              45k tris 23k verts with 10 sets of 3 backsolves - 0.8s    Analyze 0.08s Factorize 0.54s
//                       49k verts with 10 sets of 3 backsolves - 3.35s   Analyze 0.17s Factorize 2.67s
//             101k tris 50k verts with 10 sets of 3 backsolves - 1.34s   Analyze 0.12s Factorize 0.84s 
//             126k tris 63k verts with 10 sets of 3 backsolves - 0.8s    Analyze 0.17s Factorize 0.22s
//             205k tris 102k verts with 10 sets of 3 backsolves - 3.5s    Analyze 0.41s Factorize 2.2s
typedef TMatrixSolver< Eigen::SimplicialLDLT<FSparseMatrixD> >   FLDLTMatrixSolver
template<>
void TMatrixSolver<typename FLUMatrixSolver::SolverType>::SetSymmetry(bool bIsSymmetric) { check(bIsSymmetric); };

#endif  //  EIGEN_MPL2_ONLY

template<>
inline void TMatrixSolver<typename FLUMatrixSolver::SolverType>::SetSymmetry(bool bIsSymmetric) { MatrixSolver.isSymmetric(bIsSymmetric); };

template<>
inline void TMatrixSolver<typename FQRMatrixSolver::SolverType>::SetSymmetry(bool bIsSymmetric) {};


template <typename IterativeMatrixSolverType>
class  TIterativeMatrixSolver : public  IIterativeMatrixSolverBase
{
public:
	typedef IterativeMatrixSolverType  SolverType;

	TIterativeMatrixSolver() { MatrixSolver.setMaxIterations(1000); MatrixSolver.setTolerance(1e-4); }
	~TIterativeMatrixSolver() override {};

	bool bIsIterative() const override { return true; }

	void Solve(const RealVectorType& BVector, RealVectorType& SolVector) const override
	{
		if (bSetup)
		{
			SolVector = MatrixSolver.solve(BVector);
		}
	}

	void Solve(const VectorType& BVector, VectorType& SolVector) const override
	{
		if (bSetup)
		{
			SolVector = MatrixSolver.solve(BVector);
		}
	}

	void Solve(const FSOAPositions& BVectors, FSOAPositions& SolVectors) const override
	{
		const bool bForceSingleThreaded = false;
		if (bSetup)
		{
			ParallelFor(3, [&](int Dir)
			{ SolVectors.Array(Dir) = MatrixSolver.solve(BVectors.Array(Dir)); }, bForceSingleThreaded);
		}
	}

	void SolveWithGuess(const RealVectorType& GuessVector, const RealVectorType& BVector, RealVectorType& SolVector)  const override
	{

		if (bSetup)
		{
			SolVector = MatrixSolver.solveWithGuess(BVector, GuessVector);
		}
	}

	void SolveWithGuess(const VectorType& GuessVector, const VectorType& BVector, VectorType& SolVector)  const override
	{

		if (bSetup)
		{
			SolVector = MatrixSolver.solveWithGuess(BVector, GuessVector);
		}
	}

	void SolveWithGuess(const FSOAPositions& GuessVectors, const FSOAPositions& BVectors, FSOAPositions& SolVectors) const override
	{
		const bool bForceSingleThreaded = false;
		if (bSetup)
		{
			ParallelFor(3, [&](int Dir)
			{ SolVectors.Array(Dir) = MatrixSolver.solveWithGuess(BVectors.Array(Dir), GuessVectors.Array(Dir)); }, bForceSingleThreaded);
		}
	}
	void SetUp(const FSparseMatrixD& SparseMatrix, bool bIsSymmetric) override
	{
		SetUp(SparseMatrix);
	}

	void SetIterations(int32 MaxIterations) override { MatrixSolver.setMaxIterations(MaxIterations); }
	void SetTolerance(double Tol) override { MatrixSolver.setTolerance(Tol); }

	void Reset() override
	{
		//MatrixSolver = FMatrixSolver();
		bSetup = false;
	}

	bool bSucceeded() const override
	{
		const auto Info = MatrixSolver.info();
		bool bSuccess = bSetup && (Info == Eigen::ComputationInfo::Success);
		return bSuccess;
	}

private:
	void SetUp(const FSparseMatrixD& SparseMatrix)
	{
		MatrixSolver.analyzePattern(SparseMatrix);
		MatrixSolver.factorize(SparseMatrix);

		bSetup = (MatrixSolver.info() == Eigen::ComputationInfo::Success);

		int32 n = Eigen::nbThreads();
	}

private:
	bool bSetup = false;

	IterativeMatrixSolverType MatrixSolver;
};

/**
* Define wrappers for Iterative Matrix Solver Types
*/
//typedef Eigen::IncompleteCholesky< double, Eigen::Lower | Eigen::Upper>  FICPreConditioner;
typedef TIterativeMatrixSolver<Eigen::ConjugateGradient<FSparseMatrixD, Eigen::Lower | Eigen::Upper>>               FPCGMatrixSolver;
typedef TIterativeMatrixSolver<Eigen::BiCGSTAB<FSparseMatrixD, Eigen::IncompleteLUT<FSparseMatrixD::Scalar, int> >> FBiCGMatrixSolver;


}
}