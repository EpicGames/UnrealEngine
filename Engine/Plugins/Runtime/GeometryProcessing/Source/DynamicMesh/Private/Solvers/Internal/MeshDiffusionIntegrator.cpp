// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDiffusionIntegrator.h"
#include "MatrixSolver.h"

using namespace UE::Geometry;

FMeshDiffusionIntegrator::FMeshDiffusionIntegrator(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme, FProgressCancel* Progress)
{
	Id = 0;
	bIsSymmetric = false;
	MinDiagonalValue = 0;

	VertexCount = DynamicMesh.VertexCount();

	ProgressCancel = Progress;
}

void FMeshDiffusionIntegrator::Initialize(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme)
{
	// Construct the laplacian, and extract the mapping for vertices (VtxLinearization)
	//DiffusionOperator = ConstructDiffusionOperator(Scheme, DynamicMesh, bIsSymmetric, VtxLinearization, &EdgeVerts);


	ConstructOperators(Scheme, DynamicMesh, bIsSymmetric, VtxLinearization, DiffusionOperator, BoundaryOperator);

	const int32 BoundaryVertexCount = VtxLinearization.NumBoundaryVerts();
	InternalVertexCount = VertexCount - BoundaryVertexCount;

	// Allocate the double buffers.
	Tmp[0].SetZero(InternalVertexCount);
	Tmp[1].SetZero(InternalVertexCount);

	const auto& ToVertId = VtxLinearization.ToId();

	// Extract current internal positions.
	for (int32 i = 0; i < InternalVertexCount; ++i)
	{
		const int32 VtxId = ToVertId[i];
		const FVector3d Pos = DynamicMesh.GetVertex(VtxId);
		Tmp[0].SetXYZ(i, Pos);
	}

	// backup the locations of the boundary verts.
	{
		BoundaryPositions.SetZero(BoundaryVertexCount);
		for (int32 i = 0; i < BoundaryVertexCount; ++i)
		{
			const int32 VtxId = ToVertId[i + InternalVertexCount];
			const FVector3d Pos = DynamicMesh.GetVertex(VtxId);
			BoundaryPositions.SetXYZ(i, Pos);
		}
	}

	const FSparseMatrixD& M = DiffusionOperator;

	// Find the min diagonal entry (all should be negative).
	int32 Rank = (int32)M.rows();
	MinDiagonalValue = FSparseMatrixD::Scalar(0);
	for (int32 i = 0; i < Rank; ++i)
	{
		auto Diag = M.coeff(i, i);
		MinDiagonalValue = FMath::Min(Diag, MinDiagonalValue);
	}
	// The matrix should have a row for each internal vertex
	checkSlow(Rank == InternalVertexCount);

#if 0
	// testing - how to print the matrix to debug output 

	std::stringstream  ss;
	ss << Eigen::MatrixXd(M) << std::endl;

	FString Foo = ss.str().c_str();
	FPlatformMisc::LowLevelOutputDebugString(*Foo);
#endif

}



void FMeshDiffusionIntegrator::Integrate_ForwardEuler(const int32 NumSteps, const double Speed)
{
	double Alpha = FMath::Clamp(Speed, 0., 1.);

	FSparseMatrixD::Scalar TimeStep = -Alpha / MinDiagonalValue;
	Id = 0;
	for (int32 s = 0; s < NumSteps; ++s)
	{
		if (ProgressCancel && ProgressCancel->Cancelled()) return;

		int32 SrcBuffer = Id;
		Id = 1 - Id;
		Tmp[Id].Array(0) = Tmp[SrcBuffer].Array(0) + TimeStep * (DiffusionOperator * Tmp[SrcBuffer].Array(0) + BoundaryOperator * BoundaryPositions.Array(0));
		Tmp[Id].Array(1) = Tmp[SrcBuffer].Array(1) + TimeStep * (DiffusionOperator * Tmp[SrcBuffer].Array(1) + BoundaryOperator * BoundaryPositions.Array(1));
		Tmp[Id].Array(2) = Tmp[SrcBuffer].Array(2) + TimeStep * (DiffusionOperator * Tmp[SrcBuffer].Array(2) + BoundaryOperator * BoundaryPositions.Array(2));

	}

}



void FMeshDiffusionIntegrator::Integrate_BackwardEuler(const EMatrixSolverType MatrixSolverType, const int32 NumSteps, const double TimeStepSize)
{

	//typedef typename TMatrixSolverTrait<EMatrixSolverType::LU>::MatrixSolverType   MatrixSolverType;

	// We solve 
	// p^{n+1} - dt * L[p^{n+1}] = p^{n} + dt * B[boundaryPts]
	// 
	// i.e.
	// [I - dt * L ] p^{n+1} = p^{n} + dt * B[boundaryPts]
	//
	// NB: in the case of the cotangent laplacian this would be better if we broke the L int
	// L = (A^{-1}) H  where A is the "area matrix" (think "mass matrix"), then this would
	// become
	// [A - dt * H] p^{n+1} = Ap^{n}  dt * A *B[boundaryPts]
	//  
	// A - dt * H would be symmetric
	//


	// Identity matrix
	FSparseMatrixD Ident(DiffusionOperator.rows(), DiffusionOperator.cols());
	Ident.setIdentity();

	FSparseMatrixD::Scalar TimeStep = FMath::Abs(TimeStepSize);// Alpha * FMath::Min(Intensity, 1.e6);

	FSparseMatrixD SparseMatrix = Ident - TimeStep * DiffusionOperator;


	SparseMatrix.makeCompressed();

	TUniquePtr<IMatrixSolverBase> MatrixSolver = ConstructMatrixSolver(MatrixSolverType);

	MatrixSolver->SetUp(SparseMatrix, bIsSymmetric);

	if (MatrixSolver->bSucceeded() == false)  // check that the factorization succeeded 
	{
		return;
	}

	// We are going to solve the system 
	FSOAPositions Source(InternalVertexCount);

	if (MatrixSolver->bIsIterative())
	{
		IIterativeMatrixSolverBase* IterativeSolver = (IIterativeMatrixSolverBase*)MatrixSolver.Get();

		bool bForceSingleThreaded = false;
		Id = 0;
		for (int32 s = 0; s < NumSteps; ++s)
		{
			if (ProgressCancel && ProgressCancel->Cancelled()) return;

			int32 SrcBuffer = Id;
			Id = 1 - Id;

			for (int32 i = 0; i < 3; ++i)
			{
				Source.Array(i) = Tmp[SrcBuffer].Array(i) + TimeStep * BoundaryOperator * BoundaryPositions.Array(i);
			}
			// Old solution is the guess.
			IterativeSolver->SolveWithGuess(Tmp[SrcBuffer], Source, Tmp[Id]);

		}
	}
	else
	{
		bool bForceSingleThreaded = false;
		Id = 0;
		for (int32 s = 0; s < NumSteps; ++s)
		{
			if (ProgressCancel && ProgressCancel->Cancelled()) return;

			int32 SrcBuffer = Id;
			Id = 1 - Id;

			for (int32 i = 0; i < 3; ++i)
			{
				Source.Array(i) = Tmp[SrcBuffer].Array(i) + TimeStep * BoundaryOperator * BoundaryPositions.Array(i);
			}

			MatrixSolver->Solve(Source, Tmp[Id]);
		}
	}


}

void FMeshDiffusionIntegrator::GetPositions(TArray<FVector3d>& PositionBuffer) const
{
	// Allocate Position Buffer for random access writes
	int32 MaxVtxId = VtxLinearization.ToId().Num();
	PositionBuffer.Empty(MaxVtxId);
	PositionBuffer.AddUninitialized(MaxVtxId);

	CopyInternalPositions(Tmp[Id], PositionBuffer);


	// Copy the boundary
	// NB: this re-indexes so the results can be looked up using VtxId
	CopyBoundaryPositions(PositionBuffer);

}

bool FMeshDiffusionIntegrator::CopyInternalPositions(const FSOAPositions& PositionalVector, TArray<FVector3d>& LinearArray) const
{
	// Number of positions

	const int32 Num = PositionalVector.Num();

	// early out if the x,y,z arrays in the PositionalVector have different lengths
	if (!PositionalVector.bHasSize(Num))
	{
		return false;
	}

	// 
	const auto& ToVtxId = VtxLinearization.ToId();
	const int32 MaxVtxId = ToVtxId.Num(); // NB: this is really max_used + 1 in the mesh.  See  FDynamicMesh3::MaxVertexID()
	const int32 BoundaryVertexCount = VtxLinearization.NumBoundaryVerts();

	if (MaxVtxId != LinearArray.Num())
	{
		return false;
	}

	// Copy the updated internal vertex locations over
	for (int32 i = 0; i < InternalVertexCount; ++i)
	{
		const int32 VtxId = ToVtxId[i];

		LinearArray[VtxId] = FVector3d(PositionalVector.X(i), PositionalVector.Y(i), PositionalVector.Z(i));
	}

	return true;
}

bool FMeshDiffusionIntegrator::CopyBoundaryPositions(TArray<FVector3d>& LinearArray) const
{
	const auto& ToVtxId = VtxLinearization.ToId();
	const int32 MaxVtxId = ToVtxId.Num();

	if (LinearArray.Num() != MaxVtxId)
	{
		return false;
	}

	int32 BoundaryVertexCount = VertexCount - InternalVertexCount;

	for (int32 i = 0; i < BoundaryVertexCount; ++i)
	{
		const int32 VtxId = ToVtxId[i + InternalVertexCount];
		LinearArray[VtxId] = FVector3d(BoundaryPositions.X(i), BoundaryPositions.Y(i), BoundaryPositions.Z(i));
	}

	return true;
}

