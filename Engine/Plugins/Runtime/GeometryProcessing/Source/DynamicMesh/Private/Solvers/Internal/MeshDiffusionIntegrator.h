// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Solvers/MeshLaplacian.h"
#include "Solvers/MeshLinearization.h"

#include "MatrixSolver.h"
#include "Util/ProgressCancel.h"

namespace UE
{
namespace Geometry
{


/**
 * FMeshDiffusionIntegrator solves diffusion problems over vertices of a 3D triangle mesh.
 * Note: if canceled using the optional FProgressCancel, the results may be in an unusable state. 
 * (Could likely be generalized to other graph diffusion problems)
 */
class FMeshDiffusionIntegrator
{
public:

	FMeshDiffusionIntegrator(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme, FProgressCancel* Progress = nullptr);
	virtual ~FMeshDiffusionIntegrator() = default;

	void Integrate_ForwardEuler(const int32 NumSteps, const double Speed);

	// Note: 
	void Integrate_BackwardEuler(const EMatrixSolverType MatrixSolverType, const int32 NumSteps, const double TimeStepSize);

	void GetPositions(TArray<FVector3d>& PositionArray) const;

protected:


	// The derived class has to implement this.
	// responsible for constructing the Diffusion Operator, the Boundary Operator etc
	virtual void ConstructOperators(const ELaplacianWeightScheme Scheme,
		const FDynamicMesh3& Mesh,
		bool& bIsOperatorSymmetric,
		FVertexLinearization& Linearization,
		FSparseMatrixD& DiffusionOp,
		FSparseMatrixD& BoundaryOp) = 0;

	void Initialize(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme);

protected:

	// Copy the 
	bool CopyInternalPositions(const FSOAPositions& PositionalVector, TArray<FVector3d>& LinearArray) const;

	bool CopyBoundaryPositions(TArray<FVector3d>& LinearArray) const;

	// Cache the vertex count.
	int32                 VertexCount;

	// Cache the number of internal vertices
	int32                 InternalVertexCount;

	// Used to map between VtxId and vertex Index in linear vector..
	FVertexLinearization  VtxLinearization;

	// I don't know if we want to keep this after the constructor
	bool                                   bIsSymmetric;
	FSparseMatrixD                         DiffusionOperator;
	FSparseMatrixD			               BoundaryOperator;
	TArray<int32>                          EdgeVerts;
	FSparseMatrixD::Scalar                 MinDiagonalValue;

	FSOAPositions                       BoundaryPositions;

	FSOAPositions                       Tmp[2];
	int32                               Id; // double buffer id

	FProgressCancel* ProgressCancel;
};


} // end namespace UE::Geometry
} // end namespace UE
