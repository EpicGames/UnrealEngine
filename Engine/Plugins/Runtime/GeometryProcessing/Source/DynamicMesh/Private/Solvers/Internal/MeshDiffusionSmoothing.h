// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshDiffusionIntegrator.h"
#include "LaplacianOperators.h"

namespace UE
{
namespace Geometry
{

/**
 * Solve mesh diffusion using Laplacian matrix of various types
 * Note: if canceled using the optional FProgressCancel, the results may be in an unusable state. 
 */
class FLaplacianDiffusionMeshSmoother : public FMeshDiffusionIntegrator
{
public:

	FLaplacianDiffusionMeshSmoother(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme, FProgressCancel* Progress = nullptr)
		: FMeshDiffusionIntegrator(DynamicMesh, Scheme, Progress)
	{
		Initialize(DynamicMesh, Scheme);
	}

protected:

	void ConstructOperators(const ELaplacianWeightScheme Scheme,
		const FDynamicMesh3& Mesh,
		bool& bIsOperatorSymmetric,
		FVertexLinearization& Linearization,
		FSparseMatrixD& DiffusionOp,
		FSparseMatrixD& BoundaryOp) override
	{
		bIsOperatorSymmetric = bIsSymmetricLaplacian(Scheme);
		ConstructLaplacian(Scheme, Mesh, VtxLinearization, DiffusionOp, BoundaryOp);
	}
};



/**
 * Solve mesh diffusion using Biharmonic (ie squared Laplacian) matrix of various types
 * Note: if canceled using the optional FProgressCancel, the results may be in an unusable state. 
 */
class  FBiHarmonicDiffusionMeshSmoother : public FMeshDiffusionIntegrator
{
public:

	FBiHarmonicDiffusionMeshSmoother(const FDynamicMesh3& DynamicMesh, const ELaplacianWeightScheme Scheme, FProgressCancel* Progress = nullptr)
		: FMeshDiffusionIntegrator(DynamicMesh, Scheme, Progress)
	{
		Initialize(DynamicMesh, Scheme);
	}

protected:

	void ConstructOperators(const ELaplacianWeightScheme Scheme,
		const FDynamicMesh3& Mesh,
		bool& bIsOperatorSymmetric,
		FVertexLinearization& Linearization,
		FSparseMatrixD& DiffusionOp,
		FSparseMatrixD& BoundaryOp) override
	{
		bIsOperatorSymmetric = true;

		FSparseMatrixD Laplacian;
		FSparseMatrixD BoundaryTerms;
		ConstructLaplacian(Scheme, Mesh, VtxLinearization, Laplacian, BoundaryTerms);

		bool bIsLaplacianSymmetric = bIsSymmetricLaplacian(Scheme);

		// It is actually unclear the best way to approximate the boundary conditions in this case.  
		// because we are repeatedly applying the operator ( for example thing about the way ( f(x+d)-f(x-d) )/ d will spread if you apply it twice
		// as opposed to (f(x+d)-2f(x) + f(x-d) ) / d*d

		// Anyhow here is a guess..

		if (bIsLaplacianSymmetric)
		{
			DiffusionOp = -1. * Laplacian * Laplacian;
			BoundaryOp = -1. * Laplacian * BoundaryTerms;
		}
		else
		{
			FSparseMatrixD LTran = Laplacian.transpose();
			DiffusionOp = -1. * LTran * Laplacian;
			BoundaryOp = -1. * LTran * BoundaryTerms;
		}

		DiffusionOp.makeCompressed();
		BoundaryOp.makeCompressed();
	}
};



} // end namespace UE::Geometry
} // end namespace UE