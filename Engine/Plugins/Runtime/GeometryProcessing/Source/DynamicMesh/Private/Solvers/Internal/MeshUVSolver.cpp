// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshUVSolver.h"

#include "Solvers/MatrixInterfaces.h"
#include "Solvers/LaplacianMatrixAssembly.h"
#include "PowerMethodSolver.h"
#include "MatrixSolver.h"
#include "MeshBoundaryLoops.h"
#include "MeshQueries.h"

using namespace UE::Geometry;

using ScalarT = FSparseMatrixD::Scalar;
using MatrixTripletT = Eigen::Triplet<ScalarT>;

//
// Extension of TSparseMatrixAssembler suitable for eigen sparse matrix,
// that stores each element twice. The DNCP linear system is 2Vx2V for a mesh with
// V vertices, ie a block matrix [ [X;0] , [0;Y] ], where X and Y are copies of the
// Cotan-Laplacian matrix. So in the AddEntryFunc below when we get a new entry we
// store it in both locations.
//
class FEigenDNCPSparseMatrixAssembler : public UE::Solvers::TSparseMatrixAssembler<double>
{
public:
	
	FEigenDNCPSparseMatrixAssembler(const int32 InNumVert)
	{
		NumVert = InNumVert;

		Matrix = MakeUnique<FSparseMatrixD>(2 * NumVert, 2 * NumVert);

		ReserveEntriesFunc = [this](int32 NumElements)
		{
			EntryTriplets.reserve(NumElements);
		};

		AddEntryFunc = [this](int32 RowIdx, int32 ColIdx, double Value)
		{
			// set upper-left block value
			EntryTriplets.push_back(MatrixTripletT(RowIdx, ColIdx, Value));
			// set lower-right block value
			EntryTriplets.push_back(MatrixTripletT(NumVert + RowIdx, NumVert + ColIdx, Value));
		};

		AddEntriesFunc = [this](const FTupleData* Data, int32 NumData)
		{
			for (int32 Index = 0; Index < NumData; Index++)
			{
				// set upper-left block value
				EntryTriplets.push_back(MatrixTripletT(Data[Index].I, Data[Index].J, Data[Index].Value));
				// set lower-right block value
				EntryTriplets.push_back(MatrixTripletT(NumVert + Data[Index].I, NumVert + Data[Index].J, Data[Index].Value));
			}
		};
	}

	void ExtractResult(FSparseMatrixD& Result)
	{
		Matrix->setFromTriplets(EntryTriplets.begin(), EntryTriplets.end());
		Matrix->makeCompressed();
		Result.swap(*Matrix);
	}

public:

	TUniquePtr<FSparseMatrixD> Matrix;
	std::vector<MatrixTripletT> EntryTriplets;

	int32 NumVert;
};


//
// FConstrainedMeshUVSolver
//

FConstrainedMeshUVSolver::FConstrainedMeshUVSolver(const FDynamicMesh3& DynamicMesh)
:
IConstrainedMeshUVSolver(), Mesh(DynamicMesh)
{
	// compute linearization so we can store constraints at linearized indices
	VtxLinearization.Reset(DynamicMesh, false);
}


void FConstrainedMeshUVSolver::AddConstraint(const int32 VtxId, const double Weight, const FVector2d& Pos, const bool bPostFix)
{
	if ( ensure(VtxLinearization.IsValidId(VtxId)) == false) return;
	int32 Index = VtxLinearization.GetIndex(VtxId);

	FUVConstraint NewConstraint;
	NewConstraint.ElementID = VtxId;
	NewConstraint.ConstraintIndex = Index;
	NewConstraint.Position = Pos;
	NewConstraint.Weight = Weight;
	NewConstraint.bPostFix = bPostFix;

	ConstraintMap.Add(Index, NewConstraint);

	bConstraintPositionsDirty = true;
	bConstraintWeightsDirty = true;
}


bool FConstrainedMeshUVSolver::UpdateConstraintPosition(const int32 VtxId, const FVector2d& NewPosition, const bool bPostFix)
{
	if (ensure(VtxLinearization.IsValidId(VtxId)) == false) return false;
	int32 Index = VtxLinearization.GetIndex(VtxId);

	FUVConstraint* Found = ConstraintMap.Find(Index);
	if (ensure(Found != nullptr))
	{
		Found->Position = NewPosition;
		Found->bPostFix = bPostFix;
		bConstraintPositionsDirty = true;
		return true;
	}
	return false;
}


bool FConstrainedMeshUVSolver::UpdateConstraintWeight(const int32 VtxId, const double NewWeight)
{
	if (ensure(VtxLinearization.IsValidId(VtxId)) == false) return false;
	int32 Index = VtxLinearization.GetIndex(VtxId);

	FUVConstraint* Found = ConstraintMap.Find(Index);
	if ( ensure(Found != nullptr) )
	{
		Found->Weight = NewWeight;
		bConstraintWeightsDirty = true;
		return true;
	}
	return false;
}


bool FConstrainedMeshUVSolver::IsConstrained(const int32 VtxId) const
{
	if (VtxLinearization.IsValidId(VtxId) == false) return false;
	int32 Index = VtxLinearization.GetIndex(VtxId);
	return ConstraintMap.Contains(Index);
}


void FConstrainedMeshUVSolver::ClearConstraints()
{ 
	ConstraintMap.Empty();
	bConstraintPositionsDirty = true;
	bConstraintWeightsDirty = true;
}


//
//  FConformalMeshUVSolver
//

void FConformalMeshUVSolver::ConstructVectorAreaMatrix(FSparseMatrixD& OutAreaMatrix) 
{
	const int32 NumVert = VtxLinearization.NumVerts();
	 
	FMeshBoundaryLoops Loops(&Mesh, true);	

	// Compute how many matrix entries to expect 
	int32 NumTripletEntries = 0; 
	for (const FEdgeLoop& Loop : Loops.Loops)
	{
		NumTripletEntries += 4*Loop.GetVertexCount(); // 4 entries per loop edge
	}	

	std::vector<MatrixTripletT> Triplets;
	Triplets.reserve(NumTripletEntries);

	for (FEdgeLoop& Loop : Loops.Loops)
	{
		const int32 NumLoopVert = Loop.GetVertexCount();
		for (int32 Idx = 0; Idx < NumLoopVert; ++Idx)
		{
			// Directed edge is [EdgeVertex1, EdgeVertex2] with UVs [(U1,V1), (U2,V2)]
			// Note we are reversing the direction of the edge to handle UE mesh orientation.
			// Otherwise our Area matrix will be the negative of the correct Area matrix.
			const int32 EdgeVertex1 = Loop.Vertices[(Idx + 1) % NumLoopVert];
			const int32 EdgeVertex2 = Loop.Vertices[Idx];
			
			const int32 U1 = VtxLinearization.GetIndex(EdgeVertex1);
			const int32 V1 = U1 + NumVert;
			
			const int32 U2 = VtxLinearization.GetIndex(EdgeVertex2);
			const int32 V2 = U2 + NumVert;

			Triplets.push_back(MatrixTripletT(U1, V2, 1));
			Triplets.push_back(MatrixTripletT(U2, V1, -1));
			
			// Make it symmetric
			Triplets.push_back(MatrixTripletT(V2, U1, 1));
			Triplets.push_back(MatrixTripletT(V1, U2, -1));
		}
	}

	OutAreaMatrix.resize(2 * NumVert, 2 * NumVert);
	OutAreaMatrix.setFromTriplets(Triplets.begin(), Triplets.end());
}


void FConformalMeshUVSolver::ConstructWeightedVectorAreaMatrix(FSparseMatrixD& OutAreaMatrix,
															   double& OutMaxTriArea,
											  				   const double InSmallTriangleArea) 
{
	const int32 NumVert = VtxLinearization.NumVerts();
	
	std::vector<MatrixTripletT> Triplets;

	const int32 NumTripletEntries = 4*3*Mesh.TriangleCount(); // 4 entries for each of the 3 edges of each triangle
	Triplets.reserve(NumTripletEntries);

	// Compute the areas of all triangles and find max.
	// At the same time cache the results to be used later
	TArray<double> TriAreasArray;
	TriAreasArray.SetNum(Mesh.MaxTriangleID());
	OutMaxTriArea = -1;
	for (const int32 TriId : Mesh.TriangleIndicesItr()) 
	{
		const double TriArea = Mesh.GetTriArea(TriId);
		OutMaxTriArea = FMathd::Max(OutMaxTriArea, TriArea);
		TriAreasArray[TriId] = TriArea;
	}

	for (const int32 TriId : Mesh.TriangleIndicesItr()) 
	{
		const FIndex3i TriVert = Mesh.GetTriangle(TriId);
		
		double TriArea = TriAreasArray[TriId];

		if (TriArea < InSmallTriangleArea) 
		{
			TriArea = InSmallTriangleArea;
		}

		// Normalize all area values with respect to the largest area among all triangles.
		// This helps with the numerics being affected by the mesh scale and doesn't affect 
		// the result of the solve.
		TriArea /= OutMaxTriArea;
		
		const double Value = 1.0/TriArea;

		// Iterator over each edge of the triangle
		for (int32 VertIndex = 0; VertIndex < 3; ++VertIndex) 
		{
			// Directed edge is [EdgeVertex1, EdgeVertex2] with UVs [(U1,V1), (U2,V2)]
			// Note we are reversing the direction of the edge to handle UE mesh orientation.
			// Otherwise our Area matrix will be the negative of the correct Area matrix.
			const int32 EdgeVertex1 = TriVert[(VertIndex + 1) % 3];
			const int32 EdgeVertex2 = TriVert[VertIndex];

			const int32 U1 = VtxLinearization.GetIndex(EdgeVertex1);
			const int32 V1 = U1 + NumVert;
			
			const int32 U2 = VtxLinearization.GetIndex(EdgeVertex2);
			const int32 V2 = U2 + NumVert;
	
			Triplets.push_back(MatrixTripletT(U1, V2,  Value));
			Triplets.push_back(MatrixTripletT(V1, U2, -Value));

			// Make it symmetric
			Triplets.push_back(MatrixTripletT(V2, U1,  Value));
			Triplets.push_back(MatrixTripletT(U2, V1, -Value));
		}
	}

	OutAreaMatrix.resize(2 * NumVert, 2 * NumVert);
	OutAreaMatrix.setFromTriplets(Triplets.begin(), Triplets.end());
}


void FConformalMeshUVSolver::ConstructConformalEnergyMatrix(FSparseMatrixD& OutConfromalEnergy,
															const bool bInAreaWeighted, 
															const double InSmallTriangleArea,
															FSparseMatrixD* OutCotangentMatrix,
															FSparseMatrixD* OutAreaMatrix)
{
	using namespace UE::MeshDeformation;

	const int32 NumVerts = VtxLinearization.NumVerts();
	
	// Construct 2Nx2N system that includes both X and Y values for UVs. We will use block form [X, 0; 0, Y]
	FEigenDNCPSparseMatrixAssembler DNCPLaplacianAssembler(NumVerts);

	ConstructFullCotangentLaplacian<double>(Mesh,
											VtxLinearization, 
											DNCPLaplacianAssembler, 
											bInAreaWeighted ? ECotangentWeightMode::TriangleArea : ECotangentWeightMode::ClampedMagnitude,
											ECotangentAreaMode::NoArea);
	
	FSparseMatrixD CotangentMatrixLocal;
	FSparseMatrixD* CotangentMatrix = OutCotangentMatrix == nullptr ? &CotangentMatrixLocal : OutCotangentMatrix;
	DNCPLaplacianAssembler.ExtractResult(*CotangentMatrix);
	
	FSparseMatrixD AreaMatrixLocal;
	FSparseMatrixD* AreaMatrix = OutAreaMatrix == nullptr ? &AreaMatrixLocal : OutAreaMatrix;

	
	// we want diagonal of the CotangentMatrix to be positive, so that sign of quadratic form is the same as
	// the area matrix (which is a positive area)
	if (bInAreaWeighted) 
	{
		double MaxTriArea;
		ConstructWeightedVectorAreaMatrix(*AreaMatrix, MaxTriArea, InSmallTriangleArea);
		
		OutConfromalEnergy = -MaxTriArea*(*CotangentMatrix) - *AreaMatrix;
	}
	else 
	{
		ConstructVectorAreaMatrix(*AreaMatrix); 
		
		OutConfromalEnergy = -1*(*CotangentMatrix) - *AreaMatrix;
	}
}


//
// FLeastSquaresConformalMeshUVSolver
//

void FLeastSquaresConformalMeshUVSolver::ConstructSystemMatrices(FSparseMatrixD& OutSystemMatrix)
{
	const int32 NumVerts = VtxLinearization.NumVerts();
	
	ConstructConformalEnergyMatrix(OutSystemMatrix);

	// Set fixed vertex rows to M(i,i) = 1;
	// TODO: we can move the constrained columns to the RHS to keep the matrix symmetric.
	// This would allow for more efficient solving, if we had a symmetric solver...however it
	// also means we need a way to pass back the vector that needs to be added to the RHS
	for (const TPair<int32, FUVConstraint>& ElemPair : ConstraintMap)
	{
		const int32 RowIdx = ElemPair.Value.ConstraintIndex;
		
		// clear existing rows
		OutSystemMatrix.row(RowIdx) *= 0;
		OutSystemMatrix.row(RowIdx + NumVerts) *= 0;

		// set rows to identity, these coefficients should already exist so no insertion happens here
		OutSystemMatrix.coeffRef(RowIdx, RowIdx) = 1.0;
		OutSystemMatrix.coeffRef(RowIdx + NumVerts, RowIdx + NumVerts) = 1.0;
	}	
	
	OutSystemMatrix.makeCompressed();
}

bool FLeastSquaresConformalMeshUVSolver::SolveParameterization(const FSparseMatrixD& InSystemMatrix, 
															   const EMatrixSolverType InSolverType,
															   FColumnVectorD& OutSolution)
{
	const int32 NumVerts = VtxLinearization.NumVerts();
	
	// create a suitable matrix solver
	TUniquePtr<IMatrixSolverBase> MatrixSolver = ConstructMatrixSolver(InSolverType);
	MatrixSolver->SetUp(InSystemMatrix, false);
	if (MatrixSolver->bSucceeded() == false)  // check that the factorization succeeded 
	{
		return false;
	}

	// set the constraint positions in the RHS
	FColumnVectorD RHSVector(2 * NumVerts);
	RHSVector.setZero();
	for (const TPair<int32, FUVConstraint>& ElemPair : ConstraintMap)
	{
		const int32 RowIdx = ElemPair.Value.ConstraintIndex;

		RHSVector(RowIdx) = ElemPair.Value.Position.X;
		RHSVector(RowIdx + NumVerts) = ElemPair.Value.Position.Y;
	}

	// solve the linear system
	OutSolution.resize(2 * NumVerts, 1);
	MatrixSolver->Solve(RHSVector, OutSolution);
	const bool bSuccess = MatrixSolver->bSucceeded();

	return bSuccess;
}


bool FLeastSquaresConformalMeshUVSolver::SolveUVs(const FDynamicMesh3* InMeshPtr /*ignored*/, TArray<FVector2d>& OutUVBuffer)
{
	// InMeshPtr and Mesh class member variable should be the same meshes.
	checkSlow(InMeshPtr->IsSameAs(Mesh, FDynamicMesh3::FSameAsOptions()));

	// build DNCP system
	FSparseMatrixD SystemMatrix;
	ConstructSystemMatrices(SystemMatrix);

	// Transfer to solver and solve
	FColumnVectorD Solution;
	bool bSolveOK = SolveParameterization(SystemMatrix, EMatrixSolverType::LU, Solution);

	if (ensure(bSolveOK) == false)
	{
		// If solve failed we will try QR solver which is more robust.
		// This should perhaps be optional as the QR solve is much more expensive.
		bSolveOK = SolveParameterization(SystemMatrix, EMatrixSolverType::QR, Solution);
	}
	
	OutUVBuffer.SetNum(Mesh.MaxVertexID());
	if (ensure(bSolveOK)) 
	{
		// Copy back to input buffer
		const int32 NumVerts = VtxLinearization.NumVerts();
		for (int32 Idx = 0; Idx < VtxLinearization.NumVerts(); ++Idx)
		{
			const int32 VertexID = VtxLinearization.GetId(Idx);
			OutUVBuffer[VertexID] = FVector2d(Solution(Idx), Solution(Idx + NumVerts)); //Solution[Idx];
		}
	}
	else 
	{
		// TODO: This should be deleted, the caller of the method should handle the case if SolveUVs returns false.
		// Keeping this for now for the backwards compatibility
		for (int32 Idx = 0; Idx < VtxLinearization.NumVerts(); ++Idx)
		{
			const int32 VertexID = VtxLinearization.GetId(Idx);
			OutUVBuffer[VertexID] = FVector2d::Zero();
		}
	}

	return bSolveOK;
}


//
// FSpectralConformalMeshUVSolver
//

void FSpectralConformalMeshUVSolver::ConstructSystemMatrices(FSparseMatrixD& OutConformalEnergy,
															 FSparseMatrixD& OutMatrixB,
															 FSparseMatrixD& OutMatrixE)
{
	using namespace UE::MeshDeformation;

	const int32 NumVerts = VtxLinearization.NumVerts();

	// Ensures that we are using the same area floor value as UE::MeshDeformation::CotanTriangleData::Initialize
	// method which computes cotangent entries for the Laplacian matrix.
	const double SmallTriangleArea = UE::MeshDeformation::CotanTriangleData::SmallTriangleArea;
	ConstructConformalEnergyMatrix(OutConformalEnergy, bPreserveIrregularity, SmallTriangleArea);
	OutConformalEnergy.makeCompressed();

	// construct matrix B
	const int32 NumBndrVerts = ConstraintMap.Num();
	std::vector<MatrixTripletT> TripletsB;	
	TripletsB.reserve(2*NumBndrVerts);

	for (const TPair<int32, FUVConstraint>& ElemPair : ConstraintMap)
	{
		const int32 RowIdx = ElemPair.Value.ConstraintIndex;
		checkSlow(RowIdx < NumVerts);

		TripletsB.push_back(MatrixTripletT(RowIdx, RowIdx, 1.0));
		TripletsB.push_back(MatrixTripletT(RowIdx + NumVerts, RowIdx + NumVerts, 1.0));
	}

	OutMatrixB = FSparseMatrixD(2*NumVerts, 2*NumVerts); 
	OutMatrixB.setFromTriplets(TripletsB.begin(), TripletsB.end());
	OutMatrixB.makeCompressed();
		
	// construct matrix E
	std::vector<MatrixTripletT> TripletsE;
	TripletsE.reserve(2*NumBndrVerts);

	const ScalarT SqrtBndrNum = 1.0/FMath::Sqrt(static_cast<ScalarT>(NumBndrVerts));		
	for (const TPair<int32, FUVConstraint>& ElemPair : ConstraintMap)
	{
		const int32 RowIdx = ElemPair.Value.ConstraintIndex;

		TripletsE.push_back(MatrixTripletT(RowIdx, 0, SqrtBndrNum));
		TripletsE.push_back(MatrixTripletT(RowIdx + NumVerts, 1, SqrtBndrNum)) ;
	}
	
	OutMatrixE = FSparseMatrixD(2*NumVerts, 2); 
	OutMatrixE.setFromTriplets(TripletsE.begin(), TripletsE.end());
	OutMatrixE.makeCompressed();
}


bool FSpectralConformalMeshUVSolver::SolveParameterization(const FSparseMatrixD& InSystemMatrix,
														   const FSparseMatrixD& InMatrixB,
														   const FSparseMatrixD& InMatrixE,
														   FColumnVectorD& OutSolution)
{	
	static constexpr FSparseMatrixD::Scalar Eps = 1e-8;
	FSparseMatrixD IdentityMat(InSystemMatrix.rows(), InSystemMatrix.cols());
	IdentityMat.setIdentity();
	
	// Make sure we are working with a PSD matrix
	FSparseMatrixD SystemMatrixPSD = InSystemMatrix + Eps * IdentityMat;
	
	// Setup a custom matrix operator to avoid the expensive computation of the large dense matrix E*E^T and avoid 
	// turning (B-E*E^T)*x into a dense matrix-vector product.
	FPowerMethod::MatrixOperator OpMatrixB;
	OpMatrixB.Product = [&InMatrixB, &InMatrixE](const FPowerMethod::RealVectorType& InVector, 
												 FPowerMethod::RealVectorType& OutVector)
	{
		FPowerMethod::RealVectorType EEVector = (InMatrixE.transpose() * InVector).eval();
		OutVector = InMatrixB * InVector - InMatrixE * EEVector;
	};

	// User-specified solver parameters
	FPowerMethod::Parameters Parms;
	Parms.Tolerance = 1e-10;
	
	// check if we need to set a custom starting vector
	if (InitialUVs.IsSet() && ensure(InitialUVs.GetValue().Num() == Mesh.MaxVertexID())) 
	{
		const int32 NumVerts = VtxLinearization.NumVerts();

		FPowerMethod::RealVectorType InitialSolution;
		InitialSolution.resize(2*NumVerts);
		
		for (int32 Idx = 0; Idx < NumVerts; ++Idx)
		{
			const int32 VertexID = VtxLinearization.GetId(Idx);
			InitialSolution(Idx) = InitialUVs.GetValue()[VertexID].X;
			InitialSolution(Idx + NumVerts) = InitialUVs.GetValue()[VertexID].Y;
		}
		Parms.InitialSolution = InitialSolution;
	}
	else if (RandomStream.IsSet()) 
	{
		const int32 NumVerts = VtxLinearization.NumVerts();
		FPowerMethod::RealVectorType InitialSolution;
        RandomDenseMatrix(InitialSolution, 2*NumVerts, 1, RandomStream.GetValue());
		Parms.InitialSolution = InitialSolution;
	}
	
	FPowerMethod::ScalarType EigenValue;
	FPowerMethod::RealVectorType EigenVector;
	TUniquePtr<FPowerMethod> PMSolverPtr = MakeUnique<FSparsePowerMethod>(SystemMatrixPSD, 
																		  FSparsePowerMethod::FMatrixHints(true), 
																		  Parms);
	PMSolverPtr->OpMatrixB = OpMatrixB;
	
	FPowerMethod::ESolveStatus SolveStatus = PMSolverPtr->Solve(EigenValue, OutSolution);
	
	if (ensure(SolveStatus == FPowerMethod::ESolveStatus::Success || SolveStatus == FPowerMethod::ESolveStatus::NotConverged))
	{
		// We allow the NotConverged case to go through since most likely the result should still be ok.
		// But let the user know that tha result is not optimal.
		if (SolveStatus == FPowerMethod::ESolveStatus::NotConverged)
		{
			checkSlow(false);
			UE_LOG(LogTemp, Error, TEXT("Spectral Conformal UV Solver failed to converge. Resulting parametarization \
										 might be invalid. You alternatively can try the default Conformal solve instead."));
		} 
		
		return true;
	}
	else
	{
		UE_LOGF(LogTemp, Error, "Spectral Conformal UV Solver failed.");
		
		return false;
	}
}


bool FSpectralConformalMeshUVSolver::SolveUVs(const FDynamicMesh3* InMeshPtr, TArray<FVector2d>& OutUVBuffer)
{
	// InMeshPtr and Mesh class member variable should be the same meshes.
	checkSlow(InMeshPtr->IsSameAs(Mesh, FDynamicMesh3::FSameAsOptions()));

	FSparseMatrixD SystemMatrix, MatrixB, MatrixE;
	ConstructSystemMatrices(SystemMatrix, MatrixB, MatrixE);

	FColumnVectorD Solution;
	const bool bSolveOK = SolveParameterization(SystemMatrix, MatrixB, MatrixE, Solution);
	
	OutUVBuffer.SetNum(Mesh.MaxVertexID());
	if (ensure(bSolveOK)) 
	{
		const int32 NumVerts = VtxLinearization.NumVerts();
		for (int32 Idx = 0; Idx < VtxLinearization.NumVerts(); ++Idx)
		{
			const int32 VertexID = VtxLinearization.GetId(Idx);
			OutUVBuffer[VertexID] = FVector2d(Solution(Idx), Solution(Idx + NumVerts));
		}
	}
	else 
	{
		// TODO: This should be deleted, the caller of the method should handle the case if SolveUVs returns false.
		// Keeping this for now for the backwards compatibility
		for (int32 Idx = 0; Idx < VtxLinearization.NumVerts(); ++Idx)
		{
			const int32 VertexID = VtxLinearization.GetId(Idx);
			OutUVBuffer[VertexID] = FVector2d::Zero();
		}
	}	

	return bSolveOK;;
}

void FSpectralConformalMeshUVSolver::SetInitialSolution(const TArray<FVector2d>& InInitialUVs) 
{
	RandomStream.Reset();
	InitialUVs = InInitialUVs;
}

void FSpectralConformalMeshUVSolver::SetInitialSolution(const FRandomStream& InRandomStream) 
{
	InitialUVs.Reset();
	RandomStream = InRandomStream;
}