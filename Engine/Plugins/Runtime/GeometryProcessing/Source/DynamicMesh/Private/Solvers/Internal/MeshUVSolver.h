// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Solvers/MatrixInterfaces.h"
#include "Solvers/MeshLinearization.h"
#include "Solvers/ConstrainedMeshSolver.h"
#include "Math/RandomStream.h"
#include "MatrixBase.h"
#include "DenseMatrix.h"
#include "SparseMatrix.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * FConstrainedMeshUVSolver is an abstract class that implements methods for setting vertex constraints and their weights.
 */
class FConstrainedMeshUVSolver : public UE::Solvers::IConstrainedMeshUVSolver
{
public:
	typedef UE::Solvers::FUVConstraint  FUVConstraint;
	
	UE_API FConstrainedMeshUVSolver(const FDynamicMesh3& DynamicMesh);

	virtual ~FConstrainedMeshUVSolver() = default; 

	// Add constraint associated with given vertex id.
	UE_API void AddConstraint(const int32 VtxId, const double Weight, const FVector2d& Pos, const bool bPostFix) override;

	// Update the position of an existing constraint. Bool return if a corresponding constraint weight exists.
	UE_API bool UpdateConstraintPosition(const int32 VtxId, const FVector2d& Position, const bool bPostFix) override;

	// The underlying solver will have to refactor the matrix if this is done. Bool return if a corresponding constraint position exists.
	UE_API bool UpdateConstraintWeight(const int32 VtxId, const double Weight) override;

	// Clear all constraints associated with this uv solver.
	UE_API void ClearConstraints() override;

	// Test if for constraint associated with given vertex id. 
	UE_API bool IsConstrained(const int32 VtxId) const override;

protected:

	// The Key (int32) here is the vertex index not vertex ID
	// making it the same as the matrix row
	TMap<int32, FUVConstraint> ConstraintMap;

	// currently unused
	bool bConstraintPositionsDirty = true;
	bool bConstraintWeightsDirty = true;
	
	// Used to map between VtxId and vertex Index in linear vector.
	FVertexLinearization VtxLinearization;

	// The Mesh for which we are computing the UV mapping
	const FDynamicMesh3& Mesh;
};


/**
 * FConformalMeshUVSolver is an abstract class that implements shared methods for the constrained conformal UV solvers.
 */
class FConformalMeshUVSolver : public FConstrainedMeshUVSolver
{
public:
	
	FConformalMeshUVSolver(const FDynamicMesh3& InMesh)
	:
	FConstrainedMeshUVSolver(InMesh)
	{
	};

	virtual ~FConformalMeshUVSolver() = default; 

	/**
	 * Construct the 2Vx2V (V is the number of vertices) conformal energy matrix C = L - A, which is equal to the 
	 * difference between the (possibly area-weighted) cotangent Laplacian (L) and the vector area matrices (A). 
	 * 
	 * @param OutConfromalEnergy 	Conformal energy matrix.
	 * @param bInAreaWeighted  		Should the entries of the cotangent Laplacian and the vector area matrices be  
	 * 								weighted by the areas of the triangles, they are computed on.
	 * @param InSmallTriangleArea   The "floor" for the triangle area. Triangles with areas below this value are  
	 * 						   	    considered to be triangles with SmallTriangleArea area. Optional and only used
	 * 								if bAreaWeighted == true.
	 * @param OutCotangentMatrix 	(optional) If not nullptr, then store the cotangent Laplacian matrix.
	 * @param OutAreaMatrix 		(optional) If not nullptr, then store the vector area matrix.
	 */
	UE_API void ConstructConformalEnergyMatrix(FSparseMatrixD& OutConfromalEnergy,
										const bool bInAreaWeighted = false, 
										const double InSmallTriangleArea = SMALL_NUMBER,
										FSparseMatrixD* OutCotangentMatrix = nullptr,
										FSparseMatrixD* OutAreaMatrix = nullptr);
protected:

	/**
	 * Construct the 2Vx2V (V is the number of vertices) symmetric vector area matrix A. Given the vector X = [u;v] of 
	 * the stacked UV coordinates, we can compute the signed 2D area of the parameterization using the quadratic form 
	 * (1/4)*X'*A*X.
	 * 
	 * The area is computed using only the coordinates of the boundary vertices as the contributions from internal edges 
	 * cancel out. Given a border edge ij:
	 * A_{Ui,Vj} =  1, where Ui is the u index (into X) of vert_i and Vj is the v index of vert_j.
	 * A_{Vi,Uj} = -1, where Vi is the v index of vert_i and Uj is the u index of vert_j.
	 * 
	 * To ensure the symmetry of A, we additionally set A_{Vj,Ui} = 1 and A_{Uj,Vi} = -1 since the area matrix A 
	 * will be used as a quadratic coefficients matrix.
	 * 
	 * @param OutAreaMatrix The signed vector area matrix.
	 */
	UE_API void ConstructVectorAreaMatrix(FSparseMatrixD& OutAreaMatrix);

	
	/**
	 * Construct the 2Vx2V (V is the number of vertices) symmetric weighted area matrix A. This is similar to 
	 * ConstructVectorAreaMatrix() method but the matrix coefficients are now scaled by the areas of the triangle on
	 * which they are computed. This formulation is used to enforce insensitivity to sampling irregularity of the meshes.
	 * 
	 * Because of area scaling, we no longer can only use border edges, but all the directed edges will need to be 
	 * considered. Given the directed edge ij of a triangle t:
	 * A_{Ui,Vj} = 1/Area(t), where Ui is the u index (into X) of vert_i and Vj is the v index of vert_j.
	 * A_{Vi,Uj} = -1/Area(t), where Vi is the v index of vert_i and Uj is the u index of vert_j.
	 * 
	 * To ensure the symmetry of A, we additionally set A_{Vj,Ui} = 1/Area(t) and A_{Uj,Vi} = -1/Area(t) since the area 
	 * matrix A will be used as a quadratic coefficients matrix.
	 *  
	 * @param OutAreaMatrix  	   The signed weighted vector area matrix.
	 * @param OutMaxArea		   The maximum area among all mesh triangles
	 * @param InSmallTriangleArea  The "floor" for the triangle area. Triangles with areas below this value are  
	 * 						   	   considered to be triangles with SmallTriangleArea area.
	 */
	UE_API void ConstructWeightedVectorAreaMatrix(FSparseMatrixD& OutAreaMatrix, 
										   double& OutMaxArea,
										   const double InSmallTriangleArea = SMALL_NUMBER); 

	
};

/**
 * FLeastSquaresConformalMeshUVSolver computes the conformal UV map using the Least squares conformal maps for automatic
 * texture atlas generation [Levy 2002] method. It's equivalent to Intrinsic parameterizations of surface meshes [Desburn 2002].
 * 
 * However, the energy is constructed using the formulation presented in the Spectral Conformal parameterization [Mullen 2008].
 */
class FLeastSquaresConformalMeshUVSolver : public FConformalMeshUVSolver
{
public:
		
	FLeastSquaresConformalMeshUVSolver(const FDynamicMesh3& InMesh)
	:
	FConformalMeshUVSolver(InMesh)
	{
	};

	virtual ~FLeastSquaresConformalMeshUVSolver() = default; 

	UE_API virtual bool SolveUVs(const FDynamicMesh3* InMesh, TArray<FVector2d>& OutUVBuffer) override;

protected:
	
	/** 
	 * Create the "left-hand side" system matrix used in our linear solve. The matrix is equal to the conformal energy 
	 * matrix (created with ConstructConformalEnergyMatrix()) but with the rows corresponding to the constraints 
	 * (defined by ConstraintMap) to only have one non-zero value along the diagonal (set to one).
	 */
	UE_API void ConstructSystemMatrices(FSparseMatrixD& OutSystemMatrix);
	
	/**
	 * Solve the natural/free-boundary conformal parameterization problem defined by the given system matrix
	 * (likely generated by ConstructSystemMatrices() above) using the specified linear solver.
	 * 
	 * Requires that FixedIndices/Positions pairs define at least two constraint points for the solution to be 
	 * well-defined. Assumes that these rows are also constrained in the System Matrix.
	 * 
	 * @param OutSolution Array of UV coordinates if parameterization succeeded. Empty if the solve failed.
	 * 
	 * @return true if the solve succeded, false otherwise.
	 */
	UE_API bool SolveParameterization(const FSparseMatrixD& InSystemMatrix, 
							   const EMatrixSolverType InMatrixSolverType, 
							   FColumnVectorD& OutSolution);
};


/**
 * FConformalMeshUVSolver computes the conformal UV map using the Spectral Conformal Parameterization [Mullen 2008] 
 * method including enforcing insensitivity to sampling irregularity extension.
 */
class FSpectralConformalMeshUVSolver : public FConformalMeshUVSolver
{
public:

	FSpectralConformalMeshUVSolver(const FDynamicMesh3& InMesh, const bool bInPreserveIrregularity = false)
	:
	FConformalMeshUVSolver(InMesh), bPreserveIrregularity(bInPreserveIrregularity)
	{
	};

	virtual ~FSpectralConformalMeshUVSolver() = default; 

	UE_API virtual bool SolveUVs(const FDynamicMesh3* InMesh, TArray<FVector2d>& OutUVBuffer) override;

	
	//
	// Debug utility functions
	//
public:

	/**
	 * Spectral conformal solver uses an iterative method to compute the final UVs. As most iterative methods 
	 * it requires the initial guess of the solution. By default, we randomly set UV values between [-1, 1] and 
	 * then converge to the correct solution. 
	 *  
	 * However, the solution is rotation invariant and it can be different given different initial guesses. Hence, 
	 * to help make this method more deterministic, you can manually specify the initial per-vertex UVs. 
	 * Same initial values will always produce the same solution.
	 * 
	 * @param InInitialUVs vector of UV coordinates for every vertex in the mesh.
	 * 
	 * @note resets RandomStream optional variable to avoid confusion as to which one to use.
	 */ 
	UE_API void SetInitialSolution(const TArray<FVector2d>& InInitialUVs);
 
	/**
	 * @param InRandomStream  The stream will be used to sample random values for the initial UV solution. 
	 * 					      You can set the desired seed value in the stream.
	 *
	 * @note resets InitialUVs optional variable to avoid confusion as to which one to use.
	 */
	UE_API void SetInitialSolution(const FRandomStream& InRandomStream);

protected: 

	/** 
	 * Construct the matrices needed for solving the general eigenvalue problem.
	 * 
	 * @param OutConformalEnergy Conformal energy matrix (created using ConstructConformalEnergyMatrix() method).
	 * @param OutMatrixB 		 2V×2V diagonal matrix with 1 at each diagonal element corresponding to the boundary  
	 *							 vertices (not including any of the internal boundaries) and 0 everywhere else.
	 * @param OutMatrixE 		 2V×2V matrix such that E(i,1) (resp., E(i,2)) is 1/sqrt(V_b) for each u-coordinate 
	 * 					 		 (resp.,v-coordinate) of the boundary vertices (not including any of the internal 
	 * 							 boundaries) and 0 otherwise. V_b is the number of the boundary vertices.
	 */
	UE_API void ConstructSystemMatrices(FSparseMatrixD& OutConformalEnergy, 
								 FSparseMatrixD& OutMatrixB, 
								 FSparseMatrixD& OutMatrixE);

	/** 
	 * Solve the general eigenvalue problem to compute the UV mapping using the matrices computed in 
	 * ConstructSystemMatrices().
	 */
	UE_API bool SolveParameterization(const FSparseMatrixD& InConformalEnergy,
							   const FSparseMatrixD& InMatrixB,
							   const FSparseMatrixD& InMatrixE,
							   FColumnVectorD& OutSolution);
	
protected: 

	// The mesh can contain triangles of variable sizes. By default this difference in size is not considered during 
	// the UV solve. Enabling this allows to preserve the triangle size irregularity in the resulting parameterization.
	bool bPreserveIrregularity = false;

	// Optional debug varibles used for setting the initial UV values. 
	TOptional<TArray<FVector2d>> InitialUVs; 
	TOptional<FRandomStream> RandomStream;
};



} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
