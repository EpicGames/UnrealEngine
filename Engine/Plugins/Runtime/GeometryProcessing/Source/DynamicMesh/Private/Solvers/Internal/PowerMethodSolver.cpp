// Copyright Epic Games, Inc. All Rights Reserved.

#include "PowerMethodSolver.h"
#include "DenseMatrix.h"
#include "Math/NumericLimits.h"

using namespace UE::Geometry;

FPowerMethod::ESolveStatus FPowerMethod::Solve(ScalarType& OutEigenValue, RealVectorType& OutEigenVector, bool bComputeLargest)
{
    if (VerifyUserParameters(bComputeLargest) == false) 
    {
        return ESolveStatus::InvalidParameters;
    }

    // If needed, prefactorize matrices A or B to speed up the per-iteration computations
    if (this->SetupSolver(bComputeLargest) == false) 
    {
        return ESolveStatus::NumericalIssue;
    }

    // Initial guess for the eigenvector
    if (Parms.InitialSolution.IsSet())
    {
        OutEigenVector = Parms.InitialSolution.GetValue();
    }
    else 
    {
        const int32 Rows = OpMatrixA.Rows();
        FRandomStream Stream;
        Stream.GenerateNewSeed();
        RandomDenseMatrix(OutEigenVector, Rows, 1, Stream); 
    }

    bool bConverged = false;
    for (IterationNum = 0; IterationNum < Parms.MaxIterations; ++IterationNum) 
    {	
        // Update the eigenvector with the next best guess
        if (this->Iteration(OutEigenVector, bComputeLargest) == false) 
        {
            return ESolveStatus::NumericalIssue;
        }

        // Check that we didn't produce an invalid vector
        ScalarType SquaredMagnitude = OutEigenVector.squaredNorm();
        if (FMath::IsFinite(SquaredMagnitude) == false || FMath::IsNearlyZero(SquaredMagnitude))
        {
            return ESolveStatus::NumericalIssue;
        } 

        // Normalize the vector to avoid the values getting too large
        this->Normalize(OutEigenVector);

        // Compute the eigenvalue based on the current eigenvector using Rayleigh Quotient formula
        OutEigenValue = this->RayleighQuotient(OutEigenVector);  
        if (FMath::IsFinite(OutEigenValue) == false) 
        {
            return ESolveStatus::NumericalIssue;
        }

        if (this->Converged(OutEigenValue, OutEigenVector)) 
        {   
            bConverged = true;
            break;
        }
    } 

    if (bConverged) 
    {
        return ESolveStatus::Success;
    }
    else 
    {
        return ESolveStatus::NotConverged;
    }
}

bool FPowerMethod::Converged(ScalarType Lambda, const RealVectorType& EVector)
{	
    RealVectorType AEVector;
    OpMatrixA.Product(EVector, AEVector);

    if (IsGeneralProblem()) 
    {
        RealVectorType BEVector;
        OpMatrixB.GetValue().Product(EVector, BEVector);

        IterationTol = (AEVector- Lambda * BEVector).array().abs().maxCoeff();		
    }
    else 
    {
        IterationTol = (AEVector - Lambda * EVector).array().abs().maxCoeff();
    }

    checkSlow(FMath::IsFinite(IterationTol)); 
    return IterationTol < Parms.Tolerance;
}

FPowerMethod::ScalarType FPowerMethod::RayleighQuotient(const RealVectorType& EVector) const
{
    ScalarType Divisor; 
    if(IsGeneralProblem()) 
    { 
        RealVectorType BEVector;
        OpMatrixB.GetValue().Product(EVector, BEVector);
        Divisor = EVector.dot(BEVector);
    }
    else 
    { 
        Divisor = EVector.dot(EVector);
    }

    if (FMath::IsNearlyZero(Divisor)) 
    {
        checkSlow(false);
        Divisor = 1.0;
    } 

    RealVectorType AEVector;
    OpMatrixA.Product(EVector, AEVector);
    
    return EVector.dot(AEVector)/Divisor;
}

bool FPowerMethod::SetupSolver(bool bComputeLargest)
{
    Solver = nullptr; // release previous solver

    if (bComputeLargest && IsGeneralProblem()) 
    {
        Solver = OpMatrixB.GetValue().Factorize();
        return Solver != nullptr;
    } 
    
	if (bComputeLargest == false)
    {
        Solver = OpMatrixA.Factorize();
        return Solver != nullptr;
    }
    
    return true;
}

bool FPowerMethod::Iteration(RealVectorType& EVector, bool bComputeLargest) 
{
    if (IsGeneralProblem()) 
    {
        if (bComputeLargest)
        {
            OpMatrixA.Product(EVector, EVector);
        }
        else 
        {
            OpMatrixB.GetValue().Product(EVector, EVector);
        }

        // Solver already contains pre-factorization of the correct matrix
        RealVectorType EVectorSolve;
        Solver->Solve(EVector, EVectorSolve);
        if (Solver->bSucceeded() == false) 
        {
            return false;
        }
        
        EVector = EVectorSolve;
    }
    else 
    {
        if (bComputeLargest) 
        {
            // x = A*x power iteration step 
            OpMatrixA.Product(EVector, EVector); 
        }
        else 
        {
            // x = A^(-1)*x inverse power iteration step
            RealVectorType EVectorSolve;
            Solver->Solve(EVector, EVectorSolve);
            if (Solver->bSucceeded() == false) 
            {
                return false;
            } 

            EVector = EVectorSolve;
        }
    }
   
    return true;
}

bool FPowerMethod::VerifyUserParameters(bool bComputeLargest) const
{	
    if (OpMatrixA.Product == nullptr || OpMatrixA.Rows == nullptr) 
    {
        return false;
    }

    if (IsGeneralProblem()) 
    {   
        if (OpMatrixB.GetValue().Product == nullptr || (bComputeLargest && OpMatrixB.GetValue().Factorize == nullptr))
        {
            return false; 
        }
    }
    else 
    {
        if (bComputeLargest == false && OpMatrixA.Factorize == nullptr) 
        {
            return false;
        }
    }
	
    if (Parms.MaxIterations <= 1 || Parms.Tolerance < 0) 
    {
        return false;
    } 

    // check that the initial vector (if set) has the correct dimensions
    if (Parms.InitialSolution.IsSet() && Parms.InitialSolution.GetValue().rows() != OpMatrixA.Rows())
    {
        return false; 
    }

    return true;
}

void FSparsePowerMethod::CreateSparseMatrixOperator(const FSparseMatrixD& InMatrix, 
                                                    const FMatrixHints& InMatrixHints,
                                                    MatrixOperator& OutMatrixOp) 
{
	checkSlow(InMatrix.rows() <= MAX_int32);

    EMatrixSolverType MatrixSolverType = InMatrixHints.bIsPSD ? EMatrixSolverType::FastestPSD : EMatrixSolverType::LU;
    bool bIsSymmetric = InMatrixHints.bIsSymmetric || InMatrixHints.bIsPSD;
    
    OutMatrixOp.Product = [&InMatrix](const RealVectorType& InVector, RealVectorType& OutVector) 
    {
        OutVector = InMatrix * InVector; 
    };

    OutMatrixOp.Factorize = [&InMatrix, MatrixSolverType, bIsSymmetric]() 
    {	
        TUniquePtr<IMatrixSolverBase> Solver = ConstructMatrixSolver(MatrixSolverType);
        Solver->SetUp(InMatrix, bIsSymmetric);

        if (Solver->bSucceeded() == false) 
        {
            Solver = nullptr;
        }

        return Solver;
    };

    OutMatrixOp.Rows = [&InMatrix]() 
    {
        return (int32)InMatrix.rows();
    };
}