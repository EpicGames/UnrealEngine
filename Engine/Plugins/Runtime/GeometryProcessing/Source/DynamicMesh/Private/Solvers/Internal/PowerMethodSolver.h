// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "MatrixSolver.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{
	
/**
 * Power iteration solver for computing the largest or smallest eigenvalue/eigenvector pair of a generalized 
 * eigenvalue problem A*x = lambda*B*x. 
 * 
 * A, B are real square matrices. B is optional.
 * 
 * If the matrices A and B are not symmetric, this power method might fail due to the fact that the target eigenvalue 
 * could be complex. Additionally, with symmetric matrices, the method may still fail to converge in the case that 
 * the largest/smallest eigenvalue is not distinct.
 * 
 * For sparse matrices see FSparsePowerMethod.
 */
class FPowerMethod
{
public:
	using ScalarType = typename IMatrixSolverBase::ScalarType;
	using RealVectorType = typename IMatrixSolverBase::RealVectorType;

	// User-specified parameters
	struct Parameters
	{
		ScalarType Tolerance = 1e-6; // Convergence parameter, see Converged()
		int32 MaxIterations = 1000;  // Maximum number of times the power iteration is run 
		TOptional<RealVectorType> InitialSolution; // Optionally specify the starting vector for the first iteration.
	};

	// The result of the solve
	enum class ESolveStatus 
	{
		Success,          // Solve was successful and converged within the maximum number of allowed iterations.

		NotConverged,     // Solve was successful but did not converge to the user-specified tolerance within the 
					      // maximum number of the iterations. In this case we can solve again by increasing the 
					      // maximum number of iterations or increasing the tolerance.
 
		NumericalIssue,   // Encountered a numerical issues. Most likely causes are a failed linear solve 
						  // or encounterd a non-finite number during calculations.

		InvalidParameters // User specified parameters are invalid or the matrix operators are not properly set.
	};

	// Instead of providing matrices we specify matrix operations via function handles
	struct MatrixOperator
	{ 
		// Matrix vector product
		TFunction<void(const RealVectorType& VecIn, RealVectorType& VecOut)> Product;
		
		// Creates the IMatrixSolverBase instance and calls SetUp to prefactorize the matrix which is used to compute 
		// the matrix inverse vector product. 
		// This is optional and is only needed when the problem requires it:
		// 		Matrix A needs this when computing the smallest eigenpair.
		// 		Matrix B needs this when computing the largest eigenpair.
		TFunction<TUniquePtr<IMatrixSolverBase>()> Factorize;
		
		// The number of matrix rows. Can omit for B.
		TFunction<int32()> Rows;
	};

	FPowerMethod(const Parameters& InParms) 
	:
	Parms(InParms)
	{
	}

	FPowerMethod(const MatrixOperator& InMatrixA, const Parameters& InParms) 
	:
	OpMatrixA(InMatrixA), Parms(InParms)
	{
	}

	FPowerMethod(const MatrixOperator& InMatrixA, const MatrixOperator& InMatrixB, const Parameters& InParms) 
	:
	OpMatrixA(InMatrixA), OpMatrixB(InMatrixB), Parms(InParms)
	{
	}

	virtual ~FPowerMethod() = default; 
	
	/**
	 * Compute the eigen pair corresponding to the largest (smallest) eigenvalue.
	 * 
	 * @param OutEigenValue Computed largest (smallest) eigenvalue.
	 * @param OutEigenVector Computed eigenvector corresponding to the largest (smallest) eigen value.
	 * @param bComputeLargest If true (false) will compute the eigen pair corresponding to the largest (smallest) eigenvalue. 
	 */
	UE_API virtual ESolveStatus Solve(ScalarType& OutEigenValue, RealVectorType& OutEigenVector, bool bComputeLargest = false);

	/** The tolerance value upon reaching convergence or upon hitting the max iteration. */
	inline ScalarType GetConvergedTolerance() 
	{
		return IterationTol;
	}

	/** How many iterations did it take for the solver to converge. */
	inline int32 GetConvergedIterations() 
	{
		return IterationNum;
	}

protected:
	/** Checks if we reached the convergence condition ||A*x - lambda*B*x||_Inf < Tolerance */
	UE_API virtual bool Converged(ScalarType Lambda, const RealVectorType& EVector);
	
	/** Compute the eigenvalue corresponding to the given eigenvector using the Rayleigh Quotient: lambda = x'*A*x/x'*B*x */
	UE_API virtual ScalarType RayleighQuotient(const RealVectorType& NormalizedEVector) const;
	
	/** Normalize the input vector. Default is L2 normalization. */
	virtual void Normalize(RealVectorType& EVector) const 
	{
		 EVector.normalize();
	}

	/** Single power iteration that computes the next approximation for the eigenvector.*/
	UE_API virtual bool Iteration(RealVectorType& EVector, bool bComputeLargest);

	/** 
	 * In some cases we need to solve a linear system on every iteration. This method is called outside of the loop 
	 * to pre-factorize any matrices used in the solve.
	 */
	UE_API virtual bool SetupSolver(bool bComputeLargest);

	/** Check that the input user parameters are valid and matrix operators are set */
	UE_API virtual bool VerifyUserParameters(bool bComputeLargest) const;

	/** Is matrix B not an identity matrix*/
	inline bool IsGeneralProblem() const 
	{
		return OpMatrixB.IsSet();
	}

public:
	MatrixOperator OpMatrixA; // The A matrix in the A*x = lambda*B*x formula
	TOptional<MatrixOperator> OpMatrixB; // The optional B matrix in the A*x = lambda*B*x formula

protected:
	TUniquePtr<IMatrixSolverBase> Solver; // Will either contain pre-factorization of A or B

	Parameters Parms; // User specified parameters
	
	ScalarType IterationTol = 0.0;  // Current iteration tolerance
	int32 IterationNum = -1; // Current iteration number
};

/**
 * Convenience class that handles the power iteration solver where A and B are sparse matrices. 
 * Allows passing FSparseMatrixD matrices which are used to construct the corresponding matrix operators with correct 
 * solvers.
 * 
 * Example usage:
 * 
 *		FSparseMatrixD L = ... // setup PSD laplacian matrix
 *	
 *		// Make sure we use the best solver for the job
 *		FMatrixHints LHints;
 *		LHints.IsPSD = true;
 *	
 *		FSparsePowerMethod Solver(L, LHints);
 *
 *		RealVectorType OutVector;
 *		ScalarType OutValue;
 *		ESolveStatus Status = Solver.Solve(OutValue, OutVector, true); // compute the largest eigenvalue pair
 *
 *		if (Status == ESolveStatus::Success) 
 *		{
 *			// OutValue, OutVector is a valid eigenpair
 *		}
 *
 * @note The MatrixOperator TFunctions capture the passed-in matrices by reference, so the lifetime of these matrices 
 * must extend through the use of the FSparsePowerMethod class methods (specifically the Solve() method).
 */
class FSparsePowerMethod : public FPowerMethod
{
public:

	// Hints to help choose the best solver for the problem
	struct FMatrixHints 
	{
		FMatrixHints()
		{}

		FMatrixHints(bool bSetAll)
		:
		bIsSymmetric(bSetAll), bIsPSD(bSetAll)
		{}

		FMatrixHints(bool bIsSymmetric, bool bIsPSD) 
		:
	 	bIsSymmetric(bIsSymmetric), bIsPSD(bIsPSD)
		{}

		bool bIsSymmetric = false;
		bool bIsPSD = false;
	};

	FSparsePowerMethod(const FSparseMatrixD& InMatrixA,
					   const FMatrixHints& InHintsA = FMatrixHints(), 
					   const Parameters& InParms = Parameters()) 
	:
	FPowerMethod(InParms)
	{
		CreateSparseMatrixOperator(InMatrixA, InHintsA, OpMatrixA);
	}

	FSparsePowerMethod(const FSparseMatrixD& InMatrixA, 
					   const FSparseMatrixD& InMatrixB, 
					   const FMatrixHints& InHintsA = FMatrixHints(), 
					   const FMatrixHints& InHintsB = FMatrixHints(), 
					   const Parameters& InParms = Parameters()) 
	:
	FPowerMethod(InParms)
	{
		CreateSparseMatrixOperator(InMatrixA, InHintsA, OpMatrixA);
		
		OpMatrixB = MatrixOperator();
		CreateSparseMatrixOperator(InMatrixB, InHintsB, OpMatrixB.GetValue());
	}

	virtual ~FSparsePowerMethod() = default;

    /** 
     * Convenience method to set up the Product(), Factorize() and Rows() TFunctions for the MatrixOperator.
     * 
     * @note InMatrix is captured by reference by all TFunctions and hence we assume that the lifetime of the
     * referenced matrix extends past the use of the OutMatrixOp.
     */
	static UE_API void CreateSparseMatrixOperator(const FSparseMatrixD& InMatrix, 
										   const FMatrixHints& InHints, 
										   MatrixOperator& OutMatrixOp);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
