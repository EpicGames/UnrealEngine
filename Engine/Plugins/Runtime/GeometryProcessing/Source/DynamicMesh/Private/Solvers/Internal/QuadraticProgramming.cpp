// Copyright Epic Games, Inc. All Rights Reserved.

#include "Solvers/Internal/QuadraticProgramming.h"
#include "MatrixSolver.h"
#include "Async/ParallelFor.h"
#include <vector>

using namespace UE::Geometry;

FQuadraticProgramming::FQuadraticProgramming(const FSparseMatrixD* InMatrixQ, const FColumnVectorD* InVectorF)
:
MatrixQ(InMatrixQ),
VectorF(InVectorF)
{
    checkSlow(InMatrixQ);
}

void FQuadraticProgramming::SetEnableParallelization(const bool bEnable)
{
    bUseParallel = bEnable;
}

bool FQuadraticProgramming::SetFixedConstraints(const TArray<int>* InFixedRowIndices, const FSparseMatrixD* InFixedValues)
{
    if (InFixedRowIndices && InFixedValues && InFixedRowIndices->Num() == InFixedValues->rows())
    {
        if (InFixedRowIndices->Num() <= MatrixQ->rows() && InFixedValues->rows() <= MatrixQ->rows())
        {
            FixedRowIndices = InFixedRowIndices;
            FixedValuesSparse = InFixedValues; // dense and sparse constraints are mutually exclusive
            FixedValuesDense = nullptr;
            bFixedConstraintsSet = true; 
            
			return true;
        }
    }

    return false;
}

bool FQuadraticProgramming::SetFixedConstraints(const TArray<int>* InFixedRowIndices, const FDenseMatrixD* InFixedValues)
{
    if (InFixedRowIndices && InFixedValues && InFixedRowIndices->Num() == InFixedValues->rows())
    {
        if (InFixedRowIndices->Num() <= MatrixQ->rows() && InFixedValues->rows() <= MatrixQ->rows())
        {
            FixedRowIndices = InFixedRowIndices;
            FixedValuesDense = InFixedValues; // dense and sparse constraints are mutually exclusive
            FixedValuesSparse = nullptr;
            bFixedConstraintsSet = true; 
            
			return true;
        }
    }

    return false;
}

bool FQuadraticProgramming::PreFactorize()
{   
    //
    // Setup the solver depending on which types of constraints are set
    //
    if (bFixedConstraintsSet)
    { 
        /**
         * With fixed constraints only, we can rewrite the original optimization problem 
         *
         *      trace(0.5*X^t*Q*X + X^t*f)
         *      
         * as a sum of sub-problems acting on each column x of X
         * 
         *                | Q_vv Q_vf |   |x_v|                    |f_v|
         *  |x_v x_f|^t * | Q_fv Q_ff | * |x_f|   +  |x_v x_f|^t * |f_f|
         * 
         * where: 
         *      x is a single column of X.
         *      f is a set of all indices (rows) that are fixed to known values.
         *      v is a set of the leftover variables we are optimizing for.
         * 
         * So Q_vf is a submatrix that is a result of slicing the rows of the original matrix Q by the indices in "v",
         * and the columns by the indices in "f".
         * 
         * We can rewrite our constrained optimization problem as an unconstrained optimization 
         * problem where we are optimizing for the x_v only:
         * 
         *      min (0.5*x_v^t*Q_vv*x_v  +  x_v^t * (f_v + Q_vf)) 
         *  
         * Taking the gradient with respect to x_v and setting the result to zero, we get:
         * 
         *      x_v = inverse(Q_vv) * -(f_v + Q_vf * x_f);
         */  
        const int32 NumParameters = static_cast<int32>(MatrixQ->rows()); // parameters are variable plus fixed values
        const int32 NumFixed = FixedRowIndices->Num();
        const int32 NumVariables = NumParameters - NumFixed;

        
        // Compute the row indices of the variable parameters based on the input fixed parameters
        TBitArray<FDefaultBitArrayAllocator> IsParameterVariable(true, NumParameters);
        for (const int FixedRowIdx : *FixedRowIndices)
        {
            IsParameterVariable[FixedRowIdx] = false;
        }
        
		// The variable "v" set
		VariableRowIndices.SetNumUninitialized(NumVariables);
        for (int ParameterRowIdx = 0, VariableRowIdx = 0; ParameterRowIdx < NumParameters; ++ParameterRowIdx)
        {
            if (IsParameterVariable[ParameterRowIdx])
            {
				VariableRowIndices[VariableRowIdx++] = ParameterRowIdx;
            }
        }

        // The Q_vv matrix
        FSparseMatrixD VariablesQ;
		SliceSparseMatrix(*MatrixQ, VariableRowIndices, VariableRowIndices, VariablesQ);

        // Construct a linear solver for a symmetric positive (semi-)definite matrix
        const EMatrixSolverType MatrixSolverType = EMatrixSolverType::FastestPSD;
        const bool bIsSymmetric = true;
        Solver = ConstructMatrixSolver(MatrixSolverType);
        Solver->SetUp(VariablesQ, bIsSymmetric);
        if (!ensure(Solver->bSucceeded())) 
        {
            Solver = nullptr;
            return false;
        }

		return true;
    }

    return false;
}

bool FQuadraticProgramming::Solve(FDenseMatrixD& Solution, const bool bVariablesOnly)
{
    FDateTime StartTime = FDateTime::UtcNow();
    if (bFixedConstraintsSet)
    {   
        if (!ensureMsgf(Solver, TEXT("Solver was not setup. Call the SetUp() method first.")))
        {
            return false;
        }

        if (!ensureMsgf((FixedValuesSparse == nullptr && FixedValuesDense != nullptr) || 
                        (FixedValuesSparse != nullptr && FixedValuesDense == nullptr),
                        TEXT("Either the fixed constraints were not setup or both sparse and dense constraints are setup.")))
        {
            return false;
        }

        const int32 FixedValuesCols = FixedValuesSparse ? (int32)FixedValuesSparse->cols() : (int32)FixedValuesDense->cols();

        // The Q_vf matrix
        FSparseMatrixD VariablesFixedQ;
		SliceSparseMatrix(*MatrixQ, VariableRowIndices, *FixedRowIndices, VariablesFixedQ);

		// The f_v column vector 
		FDenseMatrixD VariablesF;
		if (VectorF)
		{ 
			SliceDenseMatrix(*VectorF, VariableRowIndices, VariablesF);
		}
		
        // Matrix that will hold the final solution values. Optionally, omit fixed rows and only store variables.
        const FDenseMatrixD::Index SolutionRows = bVariablesOnly ? static_cast<FDenseMatrixD::Index>(VariableRowIndices.Num()) : MatrixQ->rows();
        Solution.resize(SolutionRows, FixedValuesCols);
        Solution.setZero();

		// If FixedValues is a sparse matrix then we can check which columns are non-zero and 
		// skip the solve for zero columns when VectorF is null
        TArray<int32> NonZeroFixedValuesCols;
        if (FixedValuesSparse)
        {
            for (int32 ColdIdx = 0; ColdIdx < FixedValuesSparse->outerSize(); ++ColdIdx)
            {
                if (!VectorF && FixedValuesSparse->innerVector(ColdIdx).nonZeros() > 0)
                {
                    NonZeroFixedValuesCols.Add(ColdIdx);
                }
            }
        }

		// Solve for each non-zero column in parallel (if bUseParallel == true)
        TArray<bool> SolveSucceeded; // track for which columns we failed to solve
        SolveSucceeded.Init(true, static_cast<int32>(Solution.cols()));
        const int32 NumIterations = FixedValuesSparse ? NonZeroFixedValuesCols.Num() : FixedValuesCols;
        ParallelFor(NumIterations, [&](int32 IterIdx)
        {
            const int32 ColIdx = FixedValuesSparse ? NonZeroFixedValuesCols[IterIdx] : IterIdx;
			FColumnVectorD FixedVector;
			if (FixedValuesSparse)
			{
				FixedVector = FixedValuesSparse->col(ColIdx);
			}
			else
			{
				FixedVector = FixedValuesDense->col(ColIdx);
			}


			// The -(f_v + Q_vf * x_f) vector. If the linear coefficients were not specified, skip f_v.
            FColumnVectorD VectorB;
            if (VectorF) 
            {
				VectorB = -1 * (VariablesF + VariablesFixedQ * FixedVector);
            } 
            else 
            {
                VectorB = -1 * VariablesFixedQ * FixedVector;
            }

            FColumnVectorD VariablesVector;
            Solver->Solve(VectorB, VariablesVector);
            if (ensure(Solver->bSucceeded()))
            {   
                if (bVariablesOnly)
                {
                    // Copy over only the variable values to the solution matrix
                    Solution.col(ColIdx) = VariablesVector;
                }
                else
                {
                    // Copy over the fixed and variable values to the solution matrix
                    for (int RowIdx = 0; RowIdx < FixedRowIndices->Num(); ++RowIdx)
                    {
                        Solution((*FixedRowIndices)[RowIdx], ColIdx) = FixedVector(RowIdx);
                    }

                    for (int RowIdx = 0; RowIdx < VariableRowIndices.Num(); ++RowIdx)
                    {
                        Solution(VariableRowIndices[RowIdx], ColIdx) = VariablesVector(RowIdx);
                    }
                }
            }
			else 
			{
				SolveSucceeded[ColIdx] = false;
			}
        }, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

        // Check if all solves were successful
        for (const bool bSuccess : SolveSucceeded)
        {
            if (!bSuccess)
            {
                return false;
            }
        }
    }

    SolveTimeElapsedInSec = (FDateTime::UtcNow() - StartTime).GetTotalSeconds();

	return true;
}

bool FQuadraticProgramming::Solve(FSparseMatrixD& Solution, const bool bVariablesOnly, const double Tolerance)
{
    // Eigen library has a sparseView() function but it prunes all values below a Tolerance, even negative values. 
    // We want to keep the values whose absolute value is above the Tolerance. 

    FDenseMatrixD SolutionDense;
    if (!Solve(SolutionDense, bVariablesOnly))
    {
        return false;
    }

    std::vector<Eigen::Triplet<FSparseMatrixD::Scalar>> Triplets;
    for (int32 ColIdx = 0; ColIdx < SolutionDense.cols(); ++ColIdx)
    {
        for (int32 RowIdx = 0; RowIdx < SolutionDense.rows(); ++RowIdx)
        {
            FDenseMatrixD::Scalar Value = SolutionDense(RowIdx, ColIdx);
            if (FMath::Abs<FDenseMatrixD::Scalar>(Value) > (FDenseMatrixD::Scalar)Tolerance)
            {
                Triplets.emplace_back(RowIdx, ColIdx, Value);
            }
        }
    }

    Solution.resize(SolutionDense.rows(), SolutionDense.cols());
    Solution.setFromTriplets(Triplets.begin(), Triplets.end());

    return true;
}


// 
// Helper "one-function call" methods for setting up and solving the QP problems
//

bool FQuadraticProgramming::SolveWithFixedConstraints(const FSparseMatrixD& MatrixQ, 
                                                      const FColumnVectorD* VectorF, 
                                                      const TArray<int>& FixedRowIndices, 
                                                      const FSparseMatrixD& FixedValues, 
                                                      FDenseMatrixD& Solution,
                                                      const bool bVariablesOnly,
                                                      TArray<int>* VariableRowIndices)
{
    FQuadraticProgramming QP(&MatrixQ, VectorF);
    if (!QP.SetFixedConstraints(&FixedRowIndices, &FixedValues))
    {
        return false;
    }
    
    if (!QP.PreFactorize())
    {
        return false;
    }

    *VariableRowIndices = QP.GetVariableRowIndices();
    return QP.Solve(Solution, bVariablesOnly);
}

bool FQuadraticProgramming::SolveWithFixedConstraints(const FSparseMatrixD& MatrixQ,
                                                      const FColumnVectorD* VectorF,
                                                      const TArray<int>& FixedRowIndices,
                                                      const FSparseMatrixD& FixedValues,
                                                      FSparseMatrixD& Solution,
                                                      const bool bVariablesOnly,
                                                      const double Tolerance,
                                                      TArray<int>* VariableRowIndices)
{
    FQuadraticProgramming QP(&MatrixQ, VectorF);
    if (!QP.SetFixedConstraints(&FixedRowIndices, &FixedValues))
    {
        return false;
    }
    
    if (!QP.PreFactorize())
    {
        return false;
    }

    *VariableRowIndices = QP.GetVariableRowIndices();

    return QP.Solve(Solution, bVariablesOnly, Tolerance);
}


//
// Solver stats
//

double FQuadraticProgramming::GetSolveTimeElapsedInSec() const 
{ 
    return SolveTimeElapsedInSec; 
}

TArray<int> FQuadraticProgramming::GetVariableRowIndices() const
{
    return VariableRowIndices;
}