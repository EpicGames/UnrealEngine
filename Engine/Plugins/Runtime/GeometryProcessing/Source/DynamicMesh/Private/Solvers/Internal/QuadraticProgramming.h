// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/UniquePtr.h"
#include "SparseMatrix.h"
#include "DenseMatrix.h"
#include "MatrixBase.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{
/**
 * This class provides functionality to solve quadratic programming problems.
 * 
 * The goal is to minimize the function:
 *
 *      trace(0.5*X^t*Q*X + X^t*f)
 *
 *      where:
 *          Q \in R^(n x n) is a symmetric positive (semi-)definite matrix of quadratic coefficients.
 *          X \in R^(n x m) is a matrix of parameters we optimize for.
 *          f \in R^(n x 1) is a column vector of linear coefficients.
 *
 *      subject to:
 *
 *          Fixed Constraints: 
 *              X[FixedRowIndices, :] = FixedValues
 *                  where FixedRowIndices \in Z+^(k x 1), FixedValues \in R^(k x m) and k is the number of fixed rows.
 *
 *          TODO:
 *          Linear Equality Constraints:
 *          Linear Inequality Constraints:
 * 
 * Example usage:
 *      - The solver is initialized with the quadratic Q and linear coefficients f:
 *          FSparseMatrixD Q = ...
 *          FColumnVectorD f = ...
 *          FQuadraticProgramming QP(&Q, &f);
 * 
 *      - Then constraints are set by calling Set[ConstraintType]Constraints methods (only the fixed constraints are supported at the moment).
 *          TArray<int> FixedIndices = ...
 *          FSparseMatrix FixedValues = ...
 *          QP.SetFixedConstraints(&FixedIndices, &FixedValues);
 * 
 *      - Then PreFactorize() method needs to be called to pre-factorize the matrices and to set up the internal linear solver. 
 *           QP.PreFactorize(Solution);
 * 
 *      - Finally call Solve():
 *          FDenseMatrixD Solution;
 *          QP.Solve(Solution);
 * 
 *       Calling the PreFactorize() method can be skipped if: 
 *          Fixed constraints:
 *              If only the values of the fixed constraints changed from the last time the PreFactorize() was called.
 *          
 *          TODO:
 *          Linear Equality Constraints:
 *          Linear Inequality Constraints:
 */

class FQuadraticProgramming
{
public:

    UE_API FQuadraticProgramming(const FSparseMatrixD* InMatrixQ, const FColumnVectorD* InVectorF = nullptr);

    /** If true, will solve for each column x \in X in parallel. By default, parallelization is enabled. */
    UE_API void SetEnableParallelization(const bool bEnable);

    /** Set the fixed constraints either via a sparse or a dense matrix. Calls are mutually exclusive. */
    UE_API bool SetFixedConstraints(const TArray<int>* InFixedRowIndices, const FSparseMatrixD* InFixedValues);
    UE_API bool SetFixedConstraints(const TArray<int>* InFixedRowIndices, const FDenseMatrixD* InFixedValues);

    /** Pre-factorizes the matrices and sets up the solver. */
    UE_API bool PreFactorize();
    
    /**
     * @param Solution The result of the solve as a dense matrix.
	 * @param bVariablesOnly If true, the Solution matrix only contains the rows for non-fixed (variable) rows (set via SetFixedConstraints function).
     */
    UE_API bool Solve(FDenseMatrixD& Solution, const bool bVariablesOnly = false);

    /**
     * @param Solution The result of the solve as a sparse matrix.
	 * @param bVariablesOnly If true, the Solution matrix only contains the rows for non-fixed (variable) rows (set via SetFixedConstraints function).
     * @param Tolerance Convert dense solution to sparse by pruning any values with their absolute value below the Tolerance.
     */
    UE_API bool Solve(FSparseMatrixD& Solution, const bool bVariablesOnly = false, const double Tolerance = SMALL_NUMBER);


    // 
    // Helper "one-function call" methods for setting up and solving the QP problems
    //

    static UE_API bool SolveWithFixedConstraints(const FSparseMatrixD& MatrixQ, const FColumnVectorD* VectorF, const TArray<int>& FixedRowIndices, const FSparseMatrixD& FixedValues, FDenseMatrixD& Solution, const bool bVariablesOnly = false, TArray<int>* VariableRowIndices = nullptr);
    static UE_API bool SolveWithFixedConstraints(const FSparseMatrixD& MatrixQ, const FColumnVectorD* VectorF, const TArray<int>& FixedRowIndices, const FSparseMatrixD& FixedValues, FSparseMatrixD& Solution, const bool bVariablesOnly = false, const double Tolerance = SMALL_NUMBER, TArray<int>* VariableRowIndices = nullptr);


    //
    // Solver stats
    // 

    /** How long did the last call to Solve() take in seconds. */
    UE_API double GetSolveTimeElapsedInSec() const;

    /** Array of row indices into MatrixQ representing variables (non-fixed). */
    UE_API TArray<int> GetVariableRowIndices() const;

protected:

    const FSparseMatrixD* MatrixQ = nullptr; // Matrix of quadratic coefficient
    const FColumnVectorD* VectorF = nullptr; // Column vector of linear coefficients

    // Fixed constraints
    bool bFixedConstraintsSet = false;                  // set to true if SetFixedConstraints is called and successful
    const TArray<int>* FixedRowIndices = nullptr;       // row indices of the fixed parameters, set by the user when calling SetFixedConstraints
    const FSparseMatrixD* FixedValuesSparse = nullptr;  // matrix of fixed values, set by the user when calling SetFixedConstraints
    const FDenseMatrixD* FixedValuesDense = nullptr;    // matrix of fixed values, set by the user when calling SetFixedConstraints

    TArray<int> VariableRowIndices;  // row indices of the variable parameters, computed internally by the call to PreFactorize()

    TUniquePtr<IMatrixSolverBase> Solver = nullptr; // PreFactorize() method pre-factorizes the matrices and setups the solver

    bool bUseParallel = true; // if true when solving for X, will solve for each column x \in X in parallel


    //
    // Debug solver stats
    //
    
    double SolveTimeElapsedInSec = 0.0; // Set every time Solve() is called
};

}
}

#undef UE_API
