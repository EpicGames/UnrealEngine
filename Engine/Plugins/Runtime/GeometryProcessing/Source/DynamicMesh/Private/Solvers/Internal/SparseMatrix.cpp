// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseMatrix.h"

namespace UE
{
namespace Geometry 
{

bool SliceSparseMatrix(const FSparseMatrixD& InMatrix, 
                       const TArray<int>& InRowsToSlice,
                       const TArray<int>& InColsToSlice,
                       FSparseMatrixD& OutSlicedMatrix)
{
    const int32 NumRows = static_cast<int32>(InMatrix.rows());
    const int32 NumCols = static_cast<int32>(InMatrix.cols());
    const int32 NumNonZeros = static_cast<int32>(InMatrix.nonZeros());
    const int32 NumRowsSliced = InRowsToSlice.Num();
    const int32 NumColsSliced = InColsToSlice.Num();

    if (NumRowsSliced == 0 && NumColsSliced == 0)
    {
        // if both slice masks are empty then simply return the copy
        OutSlicedMatrix = InMatrix;
        return true;
    }
    
    if (NumRows == 0 || NumCols == 0)
    {
        return true; // empty matrix, nothing to do
    }

    bool bUseAllRows = false;
    if (NumRowsSliced == 0)
    {
        bUseAllRows = true;
    }

    bool bUseAllCols = false;
    if (NumColsSliced == 0)
    {
        bUseAllCols = true;
    }

    // Map row/col indices in the input matrix to their row/col indices in the sliced matrix
    TMap<int, int> ToSlicedRowIndexMap;
    ToSlicedRowIndexMap.Reserve(NumRowsSliced);

    TMap<int, int> ToSlicedColIndexMap;
    ToSlicedColIndexMap.Reserve(NumColsSliced);

    for (int Idx = 0; Idx < NumRowsSliced; ++Idx)
    {
        const int Row = InRowsToSlice[Idx];
        if (Row < 0 || Row >= NumRows)
        {
            return false;
        }
        ToSlicedRowIndexMap.Add(Row, Idx);
    }

    for (int Idx = 0; Idx < NumColsSliced; ++Idx)
    {	
        const int Col = InColsToSlice[Idx];
        if (Col < 0 || Col >= NumCols)
        {
            return false;
        }
        ToSlicedColIndexMap.Add(Col, Idx);
    }

    // Take a guess at the number of the nonzero elements in the sliced matrix
    std::vector<Eigen::Triplet<FSparseMatrixD::Scalar>> Triplets;
    const float NonZeroProbabilityPerEntry = static_cast<float>(NumNonZeros) / static_cast<float>(NumRows) / static_cast<float>(NumCols);	// ~= NumNonZeros / (NumRows*NumCols) but 
																																			// won't hit int32 overflow on large systems

    const float SlicedNumEntries = static_cast<float>(NumRowsSliced) * static_cast<float>(NumColsSliced);
    const int SlicedNonZeroEstimate = static_cast<int>(NonZeroProbabilityPerEntry * SlicedNumEntries);
	if (!ensure(SlicedNonZeroEstimate >= 0))
	{
		return false;
	}

    Triplets.reserve(SlicedNonZeroEstimate);

    // Iterate over all of the non-zero elements and check if they are part of the new sliced matrix.
    for (int OuterIdx = 0; OuterIdx < InMatrix.outerSize(); ++OuterIdx)
    {
        for (typename FSparseMatrixD::InnerIterator It(InMatrix, OuterIdx); It; ++It)
        {	
            const int Row = static_cast<int>(It.row());
            const int Col = static_cast<int>(It.col());
            if ((bUseAllRows || ToSlicedRowIndexMap.Contains(Row)) && (bUseAllCols || ToSlicedColIndexMap.Contains(Col)))
            {	
                // Remap the row and col indices to the sliced matrix row and col indices
                const int SlicedRow = ToSlicedRowIndexMap[Row];
                const int SlicedCol = ToSlicedColIndexMap[Col];
                checkSlow(SlicedRow < NumRowsSliced && SlicedCol < NumColsSliced);
                Triplets.emplace_back(SlicedRow, SlicedCol, It.value());
            }
        }
    }

    OutSlicedMatrix.resize(NumRowsSliced, NumColsSliced);
    OutSlicedMatrix.setFromTriplets(Triplets.begin(), Triplets.end());

    return true;
}

}
}