// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Util/ElementLinearization.h"   // TVector3Arrays

#include "Solvers/MatrixInterfaces.h"    // TSparseMatrixAssembler

#include "Containers/Array.h"

// According to http://eigen.tuxfamily.org/index.php?title=Main_Page 
// SimplicialCholesky, AMD ordering, and constrained_cg are disabled.

#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif

#if defined(__clang__) && __clang_major__ > 9
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wimplicit-int-float-conversion"
#endif

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Sparse>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

#if defined(__clang__) && __clang_major__ > 9
#pragma clang diagnostic pop
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif


namespace UE
{
namespace Geometry
{


// NB: The LU solver likes ColMajor but the CG sovler likes RowMajor
//     Also, to change everything to float / double just change the scalar type here

typedef Eigen::SparseMatrix<double, Eigen::ColMajor>  FSparseMatrixD;




//
// Extension of TSparseMatrixAssembler suitable for eigen sparse matrix
//
class FEigenSparseMatrixAssembler : public UE::Solvers::TSparseMatrixAssembler<double>
{
public:
	typedef FSparseMatrixD::Scalar    ScalarT;
	typedef Eigen::Triplet<ScalarT>  MatrixTripletT;

	TUniquePtr<FSparseMatrixD> Matrix;
	std::vector<MatrixTripletT> EntryTriplets;

	FEigenSparseMatrixAssembler(int32 RowsI, int32 ColsJ)
	{
		Matrix = MakeUnique<FSparseMatrixD>(RowsI, ColsJ);

		ReserveEntriesFunc = [this](int32 NumElements)
		{
			EntryTriplets.reserve(NumElements);
		};

		AddEntryFunc = [this](int32 i, int32 j, double Value)
		{
			EntryTriplets.push_back(MatrixTripletT(i, j, Value));
		};

		AddEntriesFunc = [this](const FTupleData* Data, int32 NumData)
		{
			const MatrixTripletT* MatrixTriplet = reinterpret_cast<const MatrixTripletT*>(Data);
			static_assert(sizeof(MatrixTripletT) == sizeof(FTupleData));

			EntryTriplets.insert(EntryTriplets.end(), MatrixTriplet, MatrixTriplet + NumData);
		};
	}

	void ExtractResult(FSparseMatrixD& Result)
	{
		Matrix->setFromTriplets(EntryTriplets.begin(), EntryTriplets.end());
		Matrix->makeCompressed();

		Result.swap(*Matrix);
	}
};



/**
* A struct of arrays representation used to hold vertex positions
* in three vectors that can interface with the eigen library
*/
class FSOAPositions : public UE::Geometry::TVector3Arrays<double>
{
public:
	typedef typename FSparseMatrixD::Scalar ScalarType;
	typedef Eigen::Matrix<ScalarType, Eigen::Dynamic, 1>  RealVectorType;
	typedef Eigen::Map<RealVectorType> VectorType;
	typedef Eigen::Map<const RealVectorType> ConstVectorType;

	FSOAPositions(int32 Size) : UE::Geometry::TVector3Arrays<double>(Size)
	{ }

	FSOAPositions()
	{}

	VectorType Array(int32 i)
	{
		TArray<ScalarType>& Column = (i == 0) ? XVector : (i == 1) ? YVector : ZVector;
		return (Column.Num() > 0) ? VectorType(&Column[0], Column.Num()) : VectorType(nullptr, 0);
	}
	ConstVectorType Array(int32 i) const
	{
		const TArray<ScalarType>& Column = (i == 0) ? XVector : (i == 1) ? YVector : ZVector;
		return (Column.Num() > 0) ? ConstVectorType(&Column[0], Column.Num()) : ConstVectorType(nullptr, 0);
	}

};

/** 
 * Slice the matrix such that OutSlicedMatrix = InMatrix[InRowsToSlice, InColsToSlice].
 * 
 * @param InMatrix The matrix we are slicing.
 * @param InRowsToSlice Rows slice mask. If empty all rows will be used. 
 * @param InColsToSlice Columns slice mask. If empty all columns will be used.
 * @param OutSlicedMatrix The sliced matrix.
 * 
 * @return true if slicing succeeds, false otherwise. Slicing can fail if one of the input parameters is invalid.
 */
bool SliceSparseMatrix(const FSparseMatrixD& InMatrix, 
					   const TArray<int>& InRowsToSlice,
					   const TArray<int>& InColsToSlice,
					   FSparseMatrixD& OutSlicedMatrix);

}
}