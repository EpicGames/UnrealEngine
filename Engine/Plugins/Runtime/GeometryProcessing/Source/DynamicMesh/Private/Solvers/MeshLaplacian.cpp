// Copyright Epic Games, Inc. All Rights Reserved.

#include "Solvers/MeshLaplacian.h"



FString LaplacianSchemeName(const ELaplacianWeightScheme Scheme)
{
	FString LaplacianName;
	switch (Scheme)
	{
	case ELaplacianWeightScheme::ClampedCotangent:
		LaplacianName = FString(TEXT("Clamped Cotangent Laplacian"));
		break;
	case ELaplacianWeightScheme::Cotangent:
		LaplacianName = FString(TEXT("Cotangent Laplacian"));
		break;
	case ELaplacianWeightScheme::Umbrella:
		LaplacianName = FString(TEXT("Umbrella Laplacian"));
		break;
	case ELaplacianWeightScheme::MeanValue:
		LaplacianName = FString(TEXT("MeanValue Laplacian"));
		break;
	case ELaplacianWeightScheme::Uniform:
		LaplacianName = FString(TEXT("Uniform Laplacian"));
		break;
	case ELaplacianWeightScheme::Valence:
		LaplacianName = FString(TEXT("Valence Laplacian"));
		break;
	case ELaplacianWeightScheme::IDTCotanget:
		LaplacianName = FString(TEXT("Intrinsic Delaunay Triangulation Contangent Laplacian"));
		break;
	default:
		check(0 && "Unknown Laplacian Weight Scheme Enum");
	}

	return LaplacianName;
}




bool bIsSymmetricLaplacian(const ELaplacianWeightScheme Scheme)
{
	bool bSymmetric = false;
	switch (Scheme)
	{
	case ELaplacianWeightScheme::ClampedCotangent:
		bSymmetric = false;
		break;
	case ELaplacianWeightScheme::Cotangent:
		bSymmetric = false;
		break;
	case ELaplacianWeightScheme::Umbrella:
		bSymmetric = false;
		break;
	case ELaplacianWeightScheme::MeanValue:
		bSymmetric = false;
		break;
	case ELaplacianWeightScheme::Uniform:
		bSymmetric = true;
		break;
	case ELaplacianWeightScheme::Valence:
		bSymmetric = true;
		break;
	case ELaplacianWeightScheme::IDTCotanget:
		bSymmetric = false;
		break;
	default:
		check(0);
	}
	return bSymmetric;
}