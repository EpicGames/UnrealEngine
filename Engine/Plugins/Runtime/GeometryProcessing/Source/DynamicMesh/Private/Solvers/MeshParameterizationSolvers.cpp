// Copyright Epic Games, Inc. All Rights Reserved.

#include "Solvers/MeshParameterizationSolvers.h"
#include "Solvers/Internal/MeshUVSolver.h"


TUniquePtr<UE::Solvers::IConstrainedMeshUVSolver> UE::MeshDeformation::ConstructNaturalConformalParamSolver(const FDynamicMesh3& DynamicMesh)
{
	TUniquePtr<UE::Solvers::IConstrainedMeshUVSolver> Solver(new FLeastSquaresConformalMeshUVSolver(DynamicMesh));
	return Solver;
}

TUniquePtr<UE::Solvers::IConstrainedMeshUVSolver> UE::MeshDeformation::ConstructSpectralConformalParamSolver(const FDynamicMesh3& DynamicMesh, bool bPreserveIrregularity)
{
	TUniquePtr<UE::Solvers::IConstrainedMeshUVSolver> Solver(new FSpectralConformalMeshUVSolver(DynamicMesh, bPreserveIrregularity));
	return Solver;
}