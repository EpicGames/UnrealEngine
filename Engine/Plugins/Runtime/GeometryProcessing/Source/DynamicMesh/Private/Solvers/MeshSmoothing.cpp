// Copyright Epic Games, Inc. All Rights Reserved.

#include "Solvers/MeshSmoothing.h"
#include "Solvers/Internal/MeshDiffusionSmoothing.h"
#include "Solvers/Internal/ConstrainedMeshSmoothers.h"
#include "MeshWeights.h"
#include "Util/ProgressCancel.h"

using namespace UE::Geometry;

void UE::MeshDeformation::ComputeSmoothing_BiHarmonic(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& OriginalMesh,
	const double Speed, const double Intensity, const int32 NumIterations, TArray<FVector3d>& PositionArray, FProgressCancel* Progress)
{
	// This is equivalent to taking a single backward Euler time step of bi-harmonic diffusion
	// where L is the Laplacian (Del^2) , and L^T L is an approximation of the Del^4.
	// 
	// dp/dt = - k*k L^T L[p]
	// with 
	// weight = 1 / (k * Sqrt[dt] )
	//
	// p^{n+1} + dt * k * k L^TL [p^{n+1}] = p^{n}
	//
	// re-write as
	// L^TL[p^{n+1}] + weight * weight p^{n+1} = weight * weight p^{n}

#ifndef EIGEN_MPL2_ONLY
	const EMatrixSolverType MatrixSolverType = EMatrixSolverType::LTL;
#else
	// const EMatrixSolverType MatrixSolverType = EMatrixSolverType::LU;
	// const EMatrixSolverType MatrixSolverType = EMatrixSolverType::PCG;

	// The Symmetric Laplacians are SPD, and so are the LtL Operators
	const EMatrixSolverType MatrixSolverType = (bIsSymmetricLaplacian(WeightScheme)) ? EMatrixSolverType::PCG : EMatrixSolverType::LU;

#endif


#ifdef TIME_LAPLACIAN_SMOOTHERS
	FString DebugLogString = FString::Printf(TEXT("Biharmonic Smoothing of mesh with %d verts "), OriginalMesh.VertexCount()) + LaplacianSchemeName(WeightScheme) + MatrixSolverName(MatrixSolverType);

	FScopedDurationTimeLogger Timer(DebugLogString);
#endif

	const double TimeStep = Speed * FMath::Min(Intensity, 1.e6);

	FBiHarmonicDiffusionMeshSmoother BiHarmonicDiffusionSmoother(OriginalMesh, WeightScheme, Progress);

	BiHarmonicDiffusionSmoother.Integrate_BackwardEuler(MatrixSolverType, NumIterations, TimeStep);

	BiHarmonicDiffusionSmoother.GetPositions(PositionArray);

}

void UE::MeshDeformation::ComputeSmoothing_ImplicitBiHarmonicPCG(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& OriginalMesh,
	const double Speed, const double Weight, const int32 MaxIterations, TArray<FVector3d>& PositionArray)
{
	// This is equivalent to taking a single backward Euler time step of bi-harmonic diffusion
	// where L is the Laplacian (Del^2) , and L^T L is an approximation of the Del^4.
	// 
	// dp/dt = - k*k L^T L[p]
	// with 
	// weight = 1 / (k * Sqrt[dt] )
	//
	// p^{n+1} + dt * k * k L^TL [p^{n+1}] = p^{n}
	//
	// re-write as
	// L^TL[p^{n+1}] + weight * weight p^{n+1} = weight * weight p^{n}
#ifdef TIME_LAPLACIAN_SMOOTHERS
	FString DebugLogString = FString::Printf(TEXT("PCG Biharmonic Smoothing of mesh with %d verts "), OriginalMesh.VertexCount()) + LaplacianSchemeName(WeightScheme);

	FScopedDurationTimeLogger Timer(DebugLogString);
#endif 
	if (MaxIterations < 1) return;

	FCGBiHarmonicMeshSmoother Smoother(OriginalMesh, WeightScheme);

	// Treat all vertices as constraints with the same weight
	const bool bPostFix = false;

	for (int32 VertId : OriginalMesh.VertexIndicesItr())
	{
		FVector3d Pos = OriginalMesh.GetVertex(VertId);

		Smoother.AddConstraint(VertId, Weight, Pos, bPostFix);
	}

	Smoother.SetMaxIterations(MaxIterations);
	Smoother.SetTolerance(1.e-4);

	bool bSuccess = Smoother.ComputeSmoothedMeshPositions(PositionArray);

}

void UE::MeshDeformation::ComputeSmoothing_Diffusion(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& OriginalMesh, bool bForwardEuler,
	const double Speed, const double Intensity, const int32 IterationCount, TArray<FVector3d>& PositionArray, FProgressCancel* Progress)
{
#ifndef EIGEN_MPL2_ONLY
	const EMatrixSolverType MatrixSolverType = EMatrixSolverType::LTL;
#else
	const EMatrixSolverType MatrixSolverType = EMatrixSolverType::LU;
	//const EMatrixSolverType MatrixSolverType = EMatrixSolverType::PCG;
	//const EMatrixSolverType MatrixSolverType = EMatrixSolverType::BICGSTAB;
#endif

#ifdef TIME_LAPLACIAN_SMOOTHERS
	FString DebugLogString = FString::Printf(TEXT("Diffusion Smoothing of mesh with %d verts"), OriginalMesh.VertexCount());
	if (!bForwardEuler)
	{
		DebugLogString += MatrixSolverName(MatrixSolverType);
	}

	FScopedDurationTimeLogger Timer(DebugLogString);
#endif
	if (IterationCount < 1) return;

	FLaplacianDiffusionMeshSmoother Smoother(OriginalMesh, WeightScheme, Progress);

	if (bForwardEuler)
	{
		Smoother.Integrate_ForwardEuler(IterationCount, Speed);
	}
	else
	{
		const double TimeStep = Speed * FMath::Min(Intensity, 1.e6);
		Smoother.Integrate_BackwardEuler(MatrixSolverType, IterationCount, TimeStep);
	}

	Smoother.GetPositions(PositionArray);
};


void UE::MeshDeformation::ComputeSmoothing_Forward(bool bUniformWeightScheme, bool bSmoothBoundary, const FDynamicMesh3& OriginalMesh,
	TFunctionRef<double(int VID, bool bOnBoundary)> GetSmoothingAlpha, const int32 NumIterations, TArray<FVector3d>& PositionArray, FProgressCancel* Progress)
{
	int32 NV = OriginalMesh.MaxVertexID();

	// cache boundary verts info
	TArray<bool> IsBoundary;
	TArray<int32> BoundaryVerts;
	IsBoundary.SetNum(NV);
	ParallelFor(NV, [&OriginalMesh, &IsBoundary](int32 VID)
	{
		IsBoundary[VID] = OriginalMesh.IsBoundaryVertex(VID) && OriginalMesh.IsReferencedVertex(VID);
	});

	for (int32 VID = 0; VID < NV; ++VID)
	{
		if (IsBoundary[VID])
		{
			BoundaryVerts.Add(VID);
		}
	}

	TArray<FVector3d> SecondBuffer;
	SecondBuffer.SetNumUninitialized(NV);

	TArray<FVector3d>* CurrentPositions = &PositionArray, *NextPositions = &SecondBuffer;

	for (int32 Iter = 0; Iter < NumIterations; ++Iter)
	{
		if (Progress && Progress->Cancelled()) return;

		// calculate smoothed positions of interior vertices
		ParallelFor(NV, [bUniformWeightScheme, &OriginalMesh, &GetSmoothingAlpha, &IsBoundary, &CurrentPositions, &NextPositions](int32 VID)
		{
			if (OriginalMesh.IsReferencedVertex(VID) == false || IsBoundary[VID])
			{
				(*NextPositions)[VID] = (*CurrentPositions)[VID];
				return;
			}

			FVector3d Centroid;
			if (bUniformWeightScheme)
			{
				Centroid = FMeshWeights::UniformCentroid(OriginalMesh, VID, [&CurrentPositions](int32 NbrVID) { return (*CurrentPositions)[NbrVID]; });
			}
			else
			{
				Centroid = FMeshWeights::CotanCentroidSafe(OriginalMesh, VID, [&CurrentPositions](int32 NbrVID) { return (*CurrentPositions)[NbrVID]; }, 1.0);

				// This code does not work because the mean curvature increases as things get smaller. Need to normalize it,
				// however that *also* doesn't really work because there is a maximum step size based on edge length, and as edges
				// collapse to nearly zero length, progress stops. Need to refine while smoothing.

				//FVector3d Uniform = FMeshWeights::UniformCentroid(*ResultMesh, vid, [&](int32 nbrvid) { return PositionBuffer[nbrvid]; });
				//FVector3d MeanCurvNorm = UE::MeshCurvature::MeanCurvatureNormal(*ResultMesh, vid, [&](int32 nbrvid) { return PositionBuffer[nbrvid]; });
				//Centroid = PositionBuffer[vid] - 0.5*MeanCurvNorm;
				//if (Centroid.DistanceSquared(PositionBuffer[vid]) > Uniform.DistanceSquared(PositionBuffer[vid]))
				//{
				//	Centroid = Uniform;
				//}
			}

			double UseAlpha = GetSmoothingAlpha(VID, false);
			(*NextPositions)[VID] = Lerp((*CurrentPositions)[VID], Centroid, UseAlpha);

		});

		// calculate boundary vertices
		if (bSmoothBoundary)
		{
			ParallelFor(BoundaryVerts.Num(),
				[&OriginalMesh, &GetSmoothingAlpha, &CurrentPositions, &NextPositions, &IsBoundary, &BoundaryVerts](int32 BdryIdx)
			{
				int32 VID = BoundaryVerts[BdryIdx];
				FVector3d Centroid = FMeshWeights::FilteredUniformCentroid(OriginalMesh, VID,
					[&CurrentPositions](int32 NbrVID) { return (*CurrentPositions)[NbrVID]; },
					[&IsBoundary](int32 NbrVID) { return IsBoundary[NbrVID]; });
				double UseAlpha = GetSmoothingAlpha(VID, true);
				(*NextPositions)[VID] = Lerp((*CurrentPositions)[VID], Centroid, UseAlpha);
			});
		}

		Swap(CurrentPositions, NextPositions);
	}

	if (CurrentPositions != &PositionArray)
	{
		PositionArray = MoveTemp(*CurrentPositions);
	}
}