// Copyright Epic Games, Inc. All Rights Reserved.

#include "Solvers/PrecomputedMeshWeightData.h"

using namespace UE::Geometry;


void UE::MeshDeformation::CotanTriangleData::Initialize(const FDynamicMesh3& DynamicMesh, int32 SrcTriId)
{
	TriId = SrcTriId;

	// edges: ab, bc, ca
	FIndex3i EdgeIds = DynamicMesh.GetTriEdges(TriId);

	FVector3d VertA, VertB, VertC;
	DynamicMesh.GetTriVertices(TriId, VertA, VertB, VertC);

	const FVector3d EdgeAB(VertB - VertA);
	const FVector3d EdgeAC(VertC - VertA);
	const FVector3d EdgeBC(VertC - VertB);


	OppositeEdge[0] = EdgeIds[1]; // EdgeBC is opposite vert A
	OppositeEdge[1] = EdgeIds[2]; // EdgeAC is opposite vert B
	OppositeEdge[2] = EdgeIds[0]; // EdgeAB is opposite vert C


	// NB: didn't use VectorUtil::Area() so we can re-use the Edges.
	//     also this formulation of area is always positive.

	const double TwiceArea = EdgeAB.Cross(EdgeAC).Length();

	// NB: Area = 1/2 || EdgeA X EdgeB ||  where EdgeA and EdgeB are any two edges in the triangle.

	// Compute the Voronoi areas
	// 
	// From Discrete Differential-Geometry Operators for Triangulated 2-Manifolds (Meyer, Desbrun, Schroder, Barr)
	// http://www.geometry.caltech.edu/pubs/DMSB_III.pdf
	//    Given triangle P,Q, R the voronoi area at P is given by
	//    Area = (1/8) * ( |PR|**2 Cot<Q  + |PQ|**2 Cot<R ) 

	if (TwiceArea > 2. * SmallTriangleArea)
	{



		// Compute the cotangent of the angle between V1 and V2 
		// as the ratio  V1.Dot.V2 / || V1 Cross V2 ||

		// Cotangent[i] is cos(theta)/sin(theta) at the i'th vertex.

		Cotangent[0] = EdgeAB.Dot(EdgeAC) / TwiceArea;
		Cotangent[1] = -EdgeAB.Dot(EdgeBC) / TwiceArea;
		Cotangent[2] = EdgeAC.Dot(EdgeBC) / TwiceArea;

		Area = 0.5 * TwiceArea;

		if (bIsObtuse())
		{
			// Voronoi inappropriate case.  Instead use Area(T)/2 at obtuse corner
			// and Area(T) / 4 at the other corners.

			VoronoiArea[0] = 0.25 * Area;
			VoronoiArea[1] = 0.25 * Area;
			VoronoiArea[2] = 0.25 * Area;

			for (int i = 0; i < 3; ++i)
			{
				if (Cotangent[i] < 0.)
				{
					VoronoiArea[i] = 0.5 * Area;
				}
			}
		}
		else
		{
			// If T is non-obtuse.

			const double EdgeABSqLength = EdgeAB.SquaredLength();
			const double EdgeACSqLength = EdgeAC.SquaredLength();
			const double EdgeBCSqLength = EdgeBC.SquaredLength();

			VoronoiArea[0] = EdgeABSqLength * Cotangent[2] + EdgeACSqLength * Cotangent[1];
			VoronoiArea[1] = EdgeABSqLength * Cotangent[2] + EdgeBCSqLength * Cotangent[0];
			VoronoiArea[2] = EdgeACSqLength * Cotangent[1] + EdgeBCSqLength * Cotangent[0];

			const double Inv8 = .125; // 1/8

			VoronoiArea[0] *= Inv8;
			VoronoiArea[1] *= Inv8;
			VoronoiArea[2] *= Inv8;
		}
	}
	else
	{
		// default small triangle - equilateral 
		double CotOf60 = 1. / FMathd::Sqrt(3.f);
		Cotangent[0] = CotOf60;
		Cotangent[1] = CotOf60;
		Cotangent[2] = CotOf60;

		VoronoiArea[0] = SmallTriangleArea / 3.;
		VoronoiArea[1] = SmallTriangleArea / 3.;
		VoronoiArea[2] = SmallTriangleArea / 3.;

		Area = SmallTriangleArea;
	}
}


void UE::MeshDeformation::MeanValueTriangleData::Initialize(const FDynamicMesh3& DynamicMesh, int32 SrcTriId)
{
	TriId = SrcTriId;

	// VertAId, VertBId, VertCId
	TriVtxIds = DynamicMesh.GetTriangle(TriId);
	TriEdgeIds = DynamicMesh.GetTriEdges(TriId);

	FVector3d VertA, VertB, VertC;
	DynamicMesh.GetTriVertices(TriId, VertA, VertB, VertC);

	const FVector3d EdgeAB(VertB - VertA);
	const FVector3d EdgeAC(VertC - VertA);
	const FVector3d EdgeBC(VertC - VertB);

	EdgeLength[0] = EdgeAB.Length();
	EdgeLength[1] = EdgeAC.Length();
	EdgeLength[2] = EdgeBC.Length();

	constexpr double SmallEdge = 1e-4;

	bDegenerate = (EdgeLength[0] < SmallEdge || EdgeLength[1] < SmallEdge || EdgeLength[2] < SmallEdge);

	// Compute tan(angle/2) = Sqrt[ (1-cos) / (1 + cos)]

	const double ABdotAC = EdgeAB.Dot(EdgeAC);
	const double BCdotBA = -EdgeBC.Dot(EdgeAB);
	const double CAdotCB = EdgeAC.Dot(EdgeBC);


	const double RegularizingConst = 1.e-6; // keeps us from dividing by zero when making tan[180/2] = sin[90]/cos[90] = inf
	TanHalfAngle[0] = (EdgeLength[0] * EdgeLength[1] - ABdotAC) / (EdgeLength[0] * EdgeLength[1] + ABdotAC + RegularizingConst);
	TanHalfAngle[1] = (EdgeLength[0] * EdgeLength[2] - BCdotBA) / (EdgeLength[0] * EdgeLength[2] + BCdotBA + RegularizingConst);
	TanHalfAngle[2] = (EdgeLength[1] * EdgeLength[2] - CAdotCB) / (EdgeLength[1] * EdgeLength[2] + CAdotCB + RegularizingConst);

	// The ABS is just a precaution.. mathematically these should all be positive, but very small angles may result in negative values.

	TanHalfAngle[0] = FMathd::Sqrt(FMathd::Abs(TanHalfAngle[0])); // at vertA
	TanHalfAngle[1] = FMathd::Sqrt(FMathd::Abs(TanHalfAngle[1])); // at vertB
	TanHalfAngle[2] = FMathd::Sqrt(FMathd::Abs(TanHalfAngle[2])); // at vertC
#if 0
// testing
	double totalAngle = 2. * (std::atan(TanHalfAngle[0]) + std::atan(TanHalfAngle[1]) + std::atan(TanHalfAngle[2]));

	double angleError = M_PI - totalAngle;
#endif
}

void UE::MeshDeformation::ConstructEdgeCotanWeightsDataArray(const FDynamicMesh3& Mesh, TArray<double>& EdgeWeightsDataArray, double ClampMin, double ClampMax)
{
	TArray<UE::MeshDeformation::CotanTriangleData> CotangentTriangleDataArray;
	CotangentTriangleDataArray.SetNumUninitialized(Mesh.MaxTriangleID());

	for (const int32 TriID : Mesh.TriangleIndicesItr())
	{
		CotangentTriangleDataArray[TriID].Initialize(Mesh, TriID);
	}

	EdgeWeightsDataArray.SetNumUninitialized(Mesh.MaxEdgeID());
	for (const int32 EdgeId : Mesh.EdgeIndicesItr())
	{   
		const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeId);

		const double CotanAlpha = CotangentTriangleDataArray[Edge.Tri[0]].GetOpposingCotangent(EdgeId);

		const double CotanBeta = (Edge.Tri[1] != FDynamicMesh3::InvalidID) ? CotangentTriangleDataArray[Edge.Tri[1]].GetOpposingCotangent(EdgeId) : 0.0;

		EdgeWeightsDataArray[EdgeId] = FMathd::Clamp(CotanAlpha + CotanBeta, ClampMin, ClampMax);
	}
}