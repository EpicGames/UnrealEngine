// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tessellation/AdaptiveDisplacement.h"
#include "Image/DisplacementMap.h"
#include "Tessellation/Affine.h"
#include "Tessellation/FeatureAdaptive.h"
#include "Tessellation/AdaptiveMetrics.h"
#include "Tessellation/Regularization.h"
#include "MeshConstraints.h"
#include "MeshConstraintsUtil.h"
#include "Tessellation/DynamicMeshPolicy.h"
#include "Containers/BitArray.h"

namespace UE {
namespace Geometry {
namespace DisplacementInternals {
	
struct FDynamicMeshDisplacementPolicy
{
	using FIndex3i = Geometry::FIndex3i;

	FDynamicMeshDisplacementPolicy(
		const FDynamicMesh3& InMesh,
		const FMeshConstraints& InMeshConstraints,
		const TBitArray<>& InActiveTriangles,
		const FDisplacementMap& InDisplacementMap,
		const double InSampleRate,
		const double InTargetError,
		const float InCenter,
		const float InMagnitude,
		const float InFeatureSensitivity,
		const FDynamicMeshUVOverlay* InUVOverlay) // 
		: Mesh(InMesh)
		, MeshConstraints(InMeshConstraints)
		, ActiveTriangles(InActiveTriangles)
		, DisplacementMap(InDisplacementMap)
		, SampleRate(InSampleRate)
		, TargetError(InTargetError)
		, Center(InCenter)
		, Magnitude(InMagnitude)
		, FeatureSensitivity(InFeatureSensitivity)
		, UVOverlay(InUVOverlay)
		, MaxRefinementLevel(10)
		, DisplacementBoundsRefinements(1)
		, LongestEdgeRatioThreshold(0.95f)
	{
		check(Mesh.HasVertexNormals());
	}

	[[nodiscard]] inline FVector2f TransformUVs(const FVector2f UV) const
	{
		// We don't arbitary UV transforms here, it would require cutting the mesh along the internal seams, 
		// to introduce edge constraints.

		return UV;
	}

	[[nodiscard]] inline FVector3d GetVertexDisplacement(int32 VertexIndex, const int TriIndex) const
	{
		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);
		const int LocalVtxIdx = IndexUtil::FindTriIndex(VertexIndex, Triangle);

		const FIndex3i UVTri = UVOverlay->GetTriangle(TriIndex);
		const FVector2f UV = TransformUVs(UVOverlay->GetElement(UVTri[LocalVtxIdx]));

		if (UV[0] < 0.f || UV[0] > 1.f || UV[1] < 0.f || UV[1] > 1.f)
			return FVector3d(0.f);

		return FVector3d(Mesh.GetVertexNormal(VertexIndex) * (DisplacementMap.Sample(UV) - Center) * Magnitude);
	}

	inline void GetTransformedTriUVs(const int TriIndex, FVector2f UVs[3]) const
	{
		UVOverlay->GetTriElements(TriIndex, UVs[0], UVs[1], UVs[2]);

		UVs[0] = TransformUVs(UVs[0]);
		UVs[1] = TransformUVs(UVs[1]);
		UVs[2] = TransformUVs(UVs[2]);
	}

	[[nodiscard]] inline FVector3d GetDisplacement(const FVector3d Barycentrics, const int TriIndex) const
	{
		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

		FVector3f TriNormals[3] = {	Mesh.GetVertexNormal(Triangle[0]),
			Mesh.GetVertexNormal(Triangle[1]),
			Mesh.GetVertexNormal(Triangle[2]) };

		FVector3f Normal = TriNormals[0] * static_cast<float>(Barycentrics.X) +
						   TriNormals[1] * static_cast<float>(Barycentrics.Y) +
						   TriNormals[2] * static_cast<float>(Barycentrics.Z);
		Normal.Normalize();

		FVector2f UVs[3];
		GetTransformedTriUVs(TriIndex, UVs);
		
		const FVector2f UV = UVs[0] * static_cast<float>(Barycentrics.X) +
							 UVs[1] * static_cast<float>(Barycentrics.Y) +
							 UVs[2] * static_cast<float>(Barycentrics.Z);

		if (UV[0] < 0.f || UV[0] > 1.f || UV[1] < 0.f || UV[1] > 1.f)
			return FVector3d(0.);

		return FVector3d(Normal) * (DisplacementMap.Sample(UV) - Center) * Magnitude;
	}

	[[nodiscard]] inline FVector2d GetErrorBounds( const FVector3d* const Barycentrics,
									 const FVector3d Displacement0,
									 const FVector3d Displacement1,
									 const FVector3d Displacement2,
									 const int TriIndex ) const
	{
		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

		FVector3f VertexNormals[3] = { 
			Mesh.GetVertexNormal(Triangle[0]),
			Mesh.GetVertexNormal(Triangle[1]),
			Mesh.GetVertexNormal(Triangle[2]) };

		FVector3d VertexNormals64[3] = { FVector3d(VertexNormals[0]), FVector3d(VertexNormals[1]), FVector3d(VertexNormals[2]) };

		FVector2f UVs[3];
		GetTransformedTriUVs(TriIndex, UVs);

		const FVector3d Displacements[] = { Displacement0, Displacement1, Displacement2 };

		FVector3f Barycentrics3f[3] = { FVector3f(Barycentrics[0]), FVector3f(Barycentrics[1]), FVector3f(Barycentrics[2]) };
	
		FVector2f UVBounds[2];
		UE::Geometry::TessellationUtil::CalculateUVBounds(Barycentrics3f, UVs, UVBounds);
			
		FVector2f MagnitudeScaleBounds(Magnitude);
			
		return UE::Geometry::TessellationUtil::CalculateErrorBounds<double>(Barycentrics, VertexNormals64, UVBounds, Displacements, DisplacementMap, Center, MagnitudeScaleBounds, DisplacementBoundsRefinements);
	}

	[[nodiscard]] inline int32 GetNumSamples(const FVector3d* const Barycentrics, const FIndex3i& Triangle, const int TriIndex) const
	{
		FVector2f UVs[3];
		GetTransformedTriUVs(TriIndex, UVs);

		return TessellationUtil::GetNumSamples( Barycentrics, UVs, FVector2f(static_cast<float>(DisplacementMap.GetSizeX()), static_cast<float>(DisplacementMap.GetSizeY())) );
	}

	[[nodiscard]] inline bool ShouldRefine(const int32 TriIndex, const TArray<FVector3d>& Displacements, FVector3d& SplitVertexBary, const int32 Level) const
	{
		if (Level >= MaxRefinementLevel) 
		{
			return false;
		}

		if (TriIndex < ActiveTriangles.Num() && !ActiveTriangles[TriIndex])
		{
			// ActiveTriangles may be empty -> no constraint, or if the the index is larger, it is a newly added triangle, that is
			// implicitly active
			return false;
		}

		FVector3d P0, P1, P2;
		Mesh.GetTriVertices(TriIndex, P0, P1, P2);

		const FVector3d Edge01 = P1 - P0;
		const FVector3d Edge12 = P2 - P1;
		const FVector3d Edge20 = P0 - P2;

		const FVector3d EdgeLengths( Edge01.Length(), Edge12.Length(), Edge20.Length() );

		SplitVertexBary = FVector3d( 1./3., 1./3., 1./3. );

		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);
		
		FVector2f UVs[3];
		GetTransformedTriUVs(TriIndex, UVs);
		
		// check for padding region
		const FVector2f MinUV(UVs[0].ComponentMin(UVs[1]).ComponentMin(UVs[2]));
		const FVector2f MaxUV(UVs[0].ComponentMax(UVs[1]).ComponentMax(UVs[2]));
		
		if (MaxUV[0] < 0.f || MaxUV[1] < 0.f || MinUV[0] > 1.f || MinUV[1] > 1.f)
		{
			return false;
		}

		const FVector3d Displacement0 = Displacements[Triangle.A];
		const FVector3d Displacement1 = Displacements[Triangle.B];
		const FVector3d Displacement2 = Displacements[Triangle.C];

		if (FeatureSensitivity > 0.f)
		{
			TFeatureSensitivityRefinementCheck<double> RefinementCheck(FeatureSensitivity, SampleRate, P0 + Displacement0, P1 + Displacement1, P2 + Displacement2);

			if (RefinementCheck.NeedsCheck())
			{
				bool bShouldSplit = false;
				for (int32 EdgeIdx = 0; EdgeIdx < 3; ++EdgeIdx)
				{
					const int EID = Mesh.GetTriEdge(TriIndex, EdgeIdx);
					
					UE::Geometry::FEdgeConstraint EdgeConstraint = MeshConstraints.GetEdgeConstraint(EID);

					// we only apply the feature criterion if all of the edges are unconstrained
					if (!EdgeConstraint.IsUnconstrained())
					{
						bShouldSplit = false;
						break;
					}

					const Geometry::FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EID);

					if (bShouldSplit == false && Edge.Tri[1] != IndexConstants::InvalidID)
					{
						const int32 AdjTriIdx = (Edge.Tri[0] == TriIndex) ? Edge.Tri[1] : Edge.Tri[0];

						const FIndex3i AdjTri = Mesh.GetTriangle(AdjTriIdx);
						const int32 FarLocalIndex = IndexUtil::FindTriOtherIndex(Edge.Vert[0], Edge.Vert[1], AdjTri);
						const int32 FarVID = AdjTri[FarLocalIndex];

						check(AdjTri[(FarLocalIndex+1)%3] == Edge.Vert[0] || AdjTri[(FarLocalIndex+1)%3] == Edge.Vert[1]);
						check(AdjTri[(FarLocalIndex+2)%3] == Edge.Vert[0] || AdjTri[(FarLocalIndex+2)%3] == Edge.Vert[1]);

						const FVector3d P = Mesh.GetVertex(FarVID) + Displacements[FarVID];

						if (RefinementCheck.CheckCriterion(EdgeIdx, P))
						{
							bShouldSplit = true;
						}
					}
				}

				if (bShouldSplit)
				{
					return true;
				}
			}
		}

		if( EdgeLengths[0] < SampleRate &&
			EdgeLengths[1] < SampleRate &&
			EdgeLengths[2] < SampleRate )
		{
			return false;
		}
		
		// avoid skinny triangles
		const int32 LongestEdgeIdx = FMath::Max3Index( EdgeLengths[0], EdgeLengths[1], EdgeLengths[2] );
		const double LongestEdge = EdgeLengths[LongestEdgeIdx];
		const double RemainingEdgeLen = EdgeLengths[(LongestEdgeIdx+1)%3] + EdgeLengths[(LongestEdgeIdx+2)%3];
			
		if (LongestEdge > LongestEdgeRatioThreshold * RemainingEdgeLen)
		{
			// try edge split
			if (!Mesh.IsBoundaryEdge(Mesh.GetTriEdge(TriIndex, LongestEdgeIdx)))
			{
				SplitVertexBary[LongestEdgeIdx]       = 0.5f;
				SplitVertexBary[(LongestEdgeIdx+1)%3] = 0.5f;
				SplitVertexBary[(LongestEdgeIdx+2)%3] = 0.0f;
				
				return true;
			}

			return false;
		}
	
		FVector3d Barycentrics[3] =
		{
			{ 1.0f, 0.0f, 0.0f },
			{ 0.0f, 1.0f, 0.0f },
			{ 0.0f, 0.0f, 1.0f },
		};

		const FVector2d ErrorBounds = GetErrorBounds(
			Barycentrics,
			Displacement0,
			Displacement1,
			Displacement2,
			TriIndex);

		if (ErrorBounds[1] > TargetError)
		{
			return true;
		}

		return false;
	}

private:
	const FDynamicMesh3&           Mesh;
	const FMeshConstraints&        MeshConstraints;
	const TBitArray<>&             ActiveTriangles;
	const FDisplacementMap&        DisplacementMap;
	const double                   SampleRate;
	const double                   TargetError;
	const float                    Center;
	const float                    Magnitude;
	const float                    FeatureSensitivity;
		
	const FDynamicMeshUVOverlay*   UVOverlay        { nullptr };
	
	const int32                    MaxRefinementLevel;
	const int32                    DisplacementBoundsRefinements;
	const float                    LongestEdgeRatioThreshold;
};

} // namespace DisplacementInternals

bool ApplyAdaptiveDisplacement(
	FDynamicMesh3& Mesh,
	FDynamicMeshUVOverlay* UVOverlay,
	const FDisplacementMap& DisplacementMap,
	const FAdaptiveTessellatorOptions& Options,
	const float FeatureSensitivity,
	FMeshConstraints& MeshConstraints,
	TBitArray<>& ActiveTriangles,
	const bool bApplyPostOptimization )
{
	if (!Mesh.HasVertexNormals())
	{
		return false;
	}

	FDynamicMesh3Policy DynamicMesh3Policy(Mesh, MeshConstraints);

	DisplacementInternals::FDynamicMeshDisplacementPolicy DisplacementPolicy(
		Mesh, MeshConstraints, ActiveTriangles, DisplacementMap, 
		Options.SampleRate, Options.TargetError, 
		0.f, 1.f, 
		FeatureSensitivity, 
		UVOverlay );

	using TessellatorT = TAdaptiveTessellator<FDynamicMesh3Policy, DisplacementInternals::FDynamicMeshDisplacementPolicy>;

	TessellatorT Tessellator(DynamicMesh3Policy, DisplacementPolicy, Options);

	if (bApplyPostOptimization)
	{
		SplitLongEdges(Mesh);

		IntrinsicDelaunayFlipEdge(Mesh);

		TBitArray<> IsInteriorVertex;
		IsInteriorVertex.Init(true, Mesh.MaxVertexID()); 
		for (const int32 EID : Mesh.BoundaryEdgeIndicesItr())
		{
			FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EID);
			IsInteriorVertex[Edge.Vert[0]] = false;
			IsInteriorVertex[Edge.Vert[1]] = false;
		}

		RegularizeVolumePreserving(Mesh, true, IsInteriorVertex);
	}

	return true;
}
	
} // namespace Geometry
} // namespace UE
