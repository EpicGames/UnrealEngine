// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3sharp FaceGroupUtil

#pragma once

#include "MathUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "Polygroups/PolygroupSet.h"
PREDECLARE_USE_GEOMETRY_CLASS(ITriangleGroupSetAdapter);

#define UE_API DYNAMICMESH_API

/**
 * Utility functions for dealing with Mesh FaceGroups
 */
namespace FaceGroupUtil
{
	using namespace UE::Geometry;

	/**
	 * Set group ID of all triangles in Mesh
	 */
	DYNAMICMESH_API void SetGroupID(FDynamicMesh3& Mesh, int32 to);

	/**
	 * Set group id of subset of triangles in Mesh
	 */
	DYNAMICMESH_API void SetGroupID(FDynamicMesh3& Mesh, const TArrayView<const int32>& triangles, int32 to);

	/**
	 * replace group id in Mesh
	 */
	DYNAMICMESH_API void SetGroupToGroup(FDynamicMesh3& Mesh, int32 from, int32 to);

	/**
	 * @return true if mesh has multiple face groups
	 */
	DYNAMICMESH_API bool HasMultipleGroups(const FDynamicMesh3& Mesh);

	/**
	 * find the set of group ids used in Mesh
	 */
	DYNAMICMESH_API void FindAllGroups(const FDynamicMesh3& Mesh, TSet<int32>& GroupsOut);

	/**
	 * count number of tris in each group in Mesh; TODO: does this need sparse storage?
	 */
	DYNAMICMESH_API void CountAllGroups(const FDynamicMesh3& Mesh, TArray<int32>& GroupCountsOut);

	/**
	 * collect triangles by group id. Returns array of triangle lists (stored as arrays).
	 * This requires 2 passes over Mesh, but each pass is linear
	 */
	DYNAMICMESH_API void FindTriangleSetsByGroup(const FDynamicMesh3& Mesh, TArray<TArray<int32>>& GroupTrisOut, int32 IgnoreGID = -1);

	/**
	 * find list of triangles in Mesh with specific group id
	 */
	DYNAMICMESH_API bool FindTrianglesByGroup(FDynamicMesh3& Mesh, int32 FindGroupID, TArray<int32>& TrianglesOut);

	/**
	* split input Mesh into submeshes based on group ID
	* **does not** separate disconnected components w/ same group ID
	*/
	UE_DEPRECATED(5.8, "Use TArray<TUniquePtr<FDynamicMesh3>> version")
	DYNAMICMESH_API void SeparateMeshByGroups(FDynamicMesh3& Mesh, TArray<FDynamicMesh3>& SplitMeshes);
	/**
	 * split input Mesh into submeshes based on group ID
	 * **does not** separate disconnected components w/ same group ID
	 */
	DYNAMICMESH_API void SeparateMeshByGroups(FDynamicMesh3& Mesh, TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes);

	/**
	* split input Mesh into submeshes based on group ID
	* **does not** separate disconnected components w/ same group ID
	*/
	UE_DEPRECATED(5.8, "Use TArray<TUniquePtr<FDynamicMesh3>> version")
	DYNAMICMESH_API void SeparateMeshByGroups(FDynamicMesh3& Mesh, TArray<FDynamicMesh3>& SplitMeshes, TArray<int32>& GroupIDs);
	/**
	 * split input Mesh into submeshes based on group ID
	 * **does not** separate disconnected components w/ same group ID
	 */
	DYNAMICMESH_API void SeparateMeshByGroups(FDynamicMesh3& Mesh, TArray<TUniquePtr<FDynamicMesh3>>& SplitMeshes, TArray<int32>& GroupIDs);

}


namespace UE
{
namespace Geometry
{


/**
 * FGroupVisualizationCache is intended to be used as a cache for per-group information that 
 * would be used to visualize PolyGroups, in particular drawing the Polygroup ID for each group.
 */
class FGroupVisualizationCache
{
public:

	struct FGroupInfo
	{
		int32 GroupID;
		FAxisAlignedBox3d Bounds;		// bounding box of the triangles in the group

		FVector3d Center;				// "center point" of the group - may or may not actually be on the mesh surface
		FIndex2i CenterTris;			// "center triangles" of the group - one or two triangles, may be invalid

		TArray<int32> TriangleIDs;		// list of triangles in the group, only initialized if bStorePerGroupTriangleIDs is true
	};

	// if enabled, each FGroupInfo keeps track of the TriangleIDs that belong to it. Can have significant memory overhead.
	bool bStorePerGroupTriangleIDs = false;

	/**
	 * Computed Per-Group Info, for each group found during the Update functions below
	 */
	TArray<FGroupInfo> GroupInfo;


	/**
	 * Update the GroupInfo array by finding group-connected components.
	 * Note that this may result in multiple GroupInfo items with the same GroupID, if they are not connected-components.
	 * Computing connected components is somewhat expensive. In addition, for each component, if it is more
	 * than 2 triangles it will be "eroded" down a single triangle to determine the group "center", also somewhat expensive.
	 * This step can be computed in parallel if bParallel is true.
	 */
	UE_API void UpdateGroupInfo_ConnectedComponents(
		const FDynamicMesh3& SourceMesh,
		const FPolygroupSet& GroupSet,
		bool bParallel = true);
	
	/**
	 * Update the GroupInfo array by finding group-connected components.
	 * Note that this may result in multiple GroupInfo items with the same GroupID, if they are not connected-components.
	 * Computing connected components is somewhat expensive. In addition, for each component, if it is more
	 * than 2 triangles it will be "eroded" down a single triangle to determine the group "center", also somewhat expensive.
	 * This step can be computed in parallel if bParallel is true.
	 */
	UE_API void UpdateGroupInfo_ConnectedComponents(
		const FDynamicMesh3& SourceMesh,
		const ITriangleGroupSetAdapter& Adapter,
		bool bParallel = true);

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable ranged-based for loop support
	 */
	auto begin() { return GroupInfo.begin(); }
	auto begin() const { return GroupInfo.begin(); }
	auto end() { return GroupInfo.end(); }
	auto end() const { return GroupInfo.end(); }
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
