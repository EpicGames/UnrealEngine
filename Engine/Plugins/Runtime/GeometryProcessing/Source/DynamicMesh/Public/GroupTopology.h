// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "EdgeSpan.h"
#include "Util/IndexUtil.h"
#include "Containers/BitArray.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FGroupTopology;

/**
 * FGroupTopologySelection represents a set of selected elements of a FGroupTopology
 */
struct FGroupTopologySelection
{
	TSet<int32> SelectedGroupIDs;
	TSet<int32> SelectedCornerIDs;
	TSet<int32> SelectedEdgeIDs;

	/**
	 * These are helper methods to get out a value when you know you only have one, or you don't
	 * care which one of the existing ones you grab. 
	 * They only exist because TSet doesn't have a clean way to grab one arbitrary element
	 * besides *TSet<int32>::TConstIterator(MySet). The hope is that these are slightly cleaner.
	 */
	int32 GetASelectedGroupID() const
	{
		return SelectedGroupIDs.Num() > 0 ? *TSet<int32>::TConstIterator(SelectedGroupIDs) : IndexConstants::InvalidID;
	}
	int32 GetASelectedCornerID() const
	{
		return SelectedCornerIDs.Num() > 0 ? *TSet<int32>::TConstIterator(SelectedCornerIDs) : IndexConstants::InvalidID;
	}
	int32 GetASelectedEdgeID() const
	{
		return SelectedEdgeIDs.Num() > 0 ? *TSet<int32>::TConstIterator(SelectedEdgeIDs) : IndexConstants::InvalidID;
	}

	FGroupTopologySelection() { Clear(); }
	
	void Clear()
	{
		SelectedGroupIDs.Reset();
		SelectedCornerIDs.Reset();
		SelectedEdgeIDs.Reset();
	}

	bool IsEmpty() const
	{
		return SelectedCornerIDs.Num() == 0 && SelectedEdgeIDs.Num() == 0 && SelectedGroupIDs.Num() == 0;
	}

	void Append(const FGroupTopologySelection& Selection)
	{
		SelectedCornerIDs.Append(Selection.SelectedCornerIDs);
		SelectedEdgeIDs.Append(Selection.SelectedEdgeIDs);
		SelectedGroupIDs.Append(Selection.SelectedGroupIDs);
	}


	void Remove(const FGroupTopologySelection& Selection)
	{
		// Note: difference assembles a new set by inserting everything that's not
		// removed. We could do removals in a loop instead, which would be faster
		// for large selections, esp when removing few things.
		SelectedCornerIDs = SelectedCornerIDs.Difference(Selection.SelectedCornerIDs);
		SelectedEdgeIDs = SelectedEdgeIDs.Difference(Selection.SelectedEdgeIDs);
		SelectedGroupIDs = SelectedGroupIDs.Difference(Selection.SelectedGroupIDs);
	}


	void Toggle(const FGroupTopologySelection& Selection)
	{
		for (int32 i : Selection.SelectedCornerIDs)
		{
			ToggleItem(SelectedCornerIDs, i);
		}
		for (int32 i : Selection.SelectedEdgeIDs)
		{
			ToggleItem(SelectedEdgeIDs, i);
		}
		for (int32 i : Selection.SelectedGroupIDs)
		{
			ToggleItem(SelectedGroupIDs, i);
		}
	}

	/** Returns true if this selection contains every element in the passed in selection */
	bool Contains(const FGroupTopologySelection& Selection) const
	{
		return SelectedCornerIDs.Includes(Selection.SelectedCornerIDs)
			&& SelectedEdgeIDs.Includes(Selection.SelectedEdgeIDs)
			&& SelectedGroupIDs.Includes(Selection.SelectedGroupIDs);
	}

	bool operator==(const FGroupTopologySelection& Selection) const
	{
		return SelectedCornerIDs.Num() == Selection.SelectedCornerIDs.Num()
			&& SelectedEdgeIDs.Num() == Selection.SelectedEdgeIDs.Num()
			&& SelectedGroupIDs.Num() == Selection.SelectedGroupIDs.Num()
			// No need to also check Selection.Contains(*this) here since counts match and sets contain unique elements
			&& Contains(Selection);
	}

	bool operator!=(const FGroupTopologySelection& Selection) const
	{
		return !(*this == Selection);
	}

	inline bool IsSelectedTriangle(const FDynamicMesh3* Mesh, const FGroupTopology* Topology, int32 TriangleID) const;


private:
	void ToggleItem(TSet<int32>& Set, int32 Item)
	{
		if (Set.Remove(Item) == 0)
		{
			Set.Add(Item);
		}
	}
};




/**
 * Given a per-triangle integer ("group"), FGroupTopology extracts a
 * group-level topological graph from an input Mesh. The graph consists
 * of three elements:
 *   Corners: there is a corner at each vertex where 3 or more groups meet (ie 3 or more groups in one-ring)
 *   Edges: a group edge is a list of one or more connected edges that have the same pair of different groups on either side
 *   Group: a group is a set of connected faces with the same GroupID
 *   
 * By default, the triangle group attribute of the input Mesh is used.
 * You can override GetGroupID to provide your own grouping.
 * 
 * Various query functions are provided to allow group topology to be interrogated.
 * Note that when functions refer to "CornerID", and "GroupEdgeID", these are simply
 * indices into the internal Corners and Edges arrays. However references to "GroupID"
 * refer to the actual group IDs in FDynamicMesh (not an index into Groups).
 */
class FGroupTopology
{
public:
	FGroupTopology() {}
	UE_API FGroupTopology(const FDynamicMesh3* Mesh, bool bAutoBuild);
	UE_API FGroupTopology(const FDynamicMesh3* Mesh, const FDynamicMeshPolygroupAttribute* GroupLayer, bool bAutoBuild);

	//~ If we allow deprecation warnings, we'll get a warning in destructor and assignment operator 
	//~ about CornerVerticesFlags (which we deprecated here) whenever it's copied or destroyed.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	FGroupTopology(const FGroupTopology&) = default;
	FGroupTopology& operator=(const FGroupTopology&) = default;
	virtual ~FGroupTopology() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;

	/**
	 * Keeps the structures of topology in place but points the mesh pointers to
	 * a new cloned mesh. If the given mesh is not cloned, the topology will no
	 * longer be consistent.
	 */
	UE_API void RetargetOnClonedMesh(const FDynamicMesh3* Mesh);

	const FDynamicMesh3* GetMesh() const 
	{ 
		return Mesh; 
	}

	/**
	 * Build the group topology graph.
	 * @return true on success, false if there was an error
	 */
	UE_API virtual bool RebuildTopology();

	/**
	 * Optional function that will add extra corners (in addition to the ones that are always placed at junctures
	 * of three or more group edges) during RebuildTopology whenever the function returns true for a vertex that
	 * is part of a group edge.
	 * For instance, this allows corners to be placed at sharp group edge bends even if the given vertex doesn't
	 * connect 3 or more group edges.
	 */
	TFunction<bool(const FGroupTopology& GroupTopology, int32 Vid, const FIndex2i& AttachedGroupEdgeEids)> ShouldAddExtraCornerAtVert = nullptr;

	/**
	 * @return Set of vertex ID's that are currently considered corners despite not originally having 3 or more 
	 *  attached group edges (due to the effects of ShouldAddExtraCornerAtVert or RebuildTopologyWithSpecificExtraCorners).
	 */
	const TSet<int32>& GetCurrentExtraCornerVids() const { return CurrentExtraCornerVids; }

	/**
	 * Rebuilds the topology graph while ignoring ShouldAddExtraCornerAtVert and only determining whether
	 * an extra corner should be forced by seeing whether the vertex is in the provided set. This is used to rebuild
	 * the topology with a particular group of extra corners during client undo/redo operations where the behavior
	 * of ShouldAddExtraCornerAtVert may have changed in the interim.
	 */
	UE_API virtual bool RebuildTopologyWithSpecificExtraCorners(const TSet<int32>& ExtraCornerVids);

	/** 
	 * Adjacency of Per-Triangle integers are what define the triangle groups.
	 * Override this function to provide an alternate group definition.
	 * Note: By convention, group IDs should not be negative
	 * @return group id integer for given TriangleID 
	 */
	virtual int GetGroupID(int TriangleID) const
	{
		return FMath::Max(0, (GroupLayer != nullptr) ? GroupLayer->GetValue(TriangleID) : Mesh->GetTriangleGroup(TriangleID));
	}


	/**
	 * FCorner is a "corner" in the group topology, IE a vertex where 3+ groups meet.
	 */
	struct FCorner
	{
		/** Mesh Vertex ID */
		int VertexID;		
		/** List of IDs of groups connected to this Corner */
		TArray<int> NeighbourGroupIDs;
	};

	/** List of Corners in the topology graph (ie these are the nodes/vertices of the graph) */
	TArray<FCorner> Corners;

	/** A Group is bounded by closed loops of FGroupEdge elements. A FGroupBoundary is one such loop. */
	struct FGroupBoundary
	{
		/** Ordered list of edges forming this boundary */
		TArray<int> GroupEdges;
		/** List of IDs of groups on the "other side" of this boundary (this GroupBoundary is owned by a particular FGroup) */
		TArray<int> NeighbourGroupIDs;
		/** true if one or more edges in GroupEdges is on the mesh boundary */
		bool bIsOnBoundary;
	};

	/** FGroup is a set of connected triangles with the same GroupID */
	struct FGroup
	{
		/** GroupID for this group */
		int GroupID;
		/** List of triangles forming this group */
		TArray<int> Triangles;
		
		/** List of boundaries of this group (may be empty, eg on a closed component with only one group) */
		TArray<FGroupBoundary> Boundaries;
		/** List of groups that are adjacent to this group */
		TArray<int> NeighbourGroupIDs;
	};

	/** List of Groups in the topology graph (ie faces) */
	TArray<FGroup> Groups;


	/**
	 * FGroupEdge is a sequence of group-boundary-edges where the two groups on either side of each edge
	 * are the same. The sequence is stored an FEdgeSpan.
	 * 
	 * FGroupEdge instances are *shared* between the two FGroups on either side, via FGroupBoundary.
	 */
	struct FGroupEdge
	{
		/** Groups adjoinging this edge, sorted by increasing value if both present, otherwise the second element may be InvalidID (if boundary edge). */
		FIndex2i Groups;
		/** Edge span for this edge */
		FEdgeSpan Span;

		/** Index of corners at either end of span, if they exist, otherwise both InvalidID */
		FIndex2i EndpointCorners;

		/** @return the member of .Groups that is not GroupID */
		int OtherGroupID(int GroupID) const 
		{ 
			check(Groups.A == GroupID || Groups.B == GroupID);
			return (Groups.A == GroupID) ? Groups.B : Groups.A; 
		}

		/** @return true if any vertex in the Span is in the Vertices set*/
		UE_API bool IsConnectedToVertices(const TSet<int>& Vertices) const;
	};

	/** List of Edges in the topology graph (each edge connects two corners/nodes) */
	TArray<FGroupEdge> Edges;



	/** @return the mesh vertex ID for the given Corner ID */
	UE_API int GetCornerVertexID(int CornerID) const;

	/** @return the Group ID for a given Vertex ID, or InvalidID if vertex is not a group corner */
	UE_API int32 GetCornerIDFromVertexID(int32 VertexID) const;

	/** @return the FGroup for the given GroupID, or nullptr if not found */
	UE_API const FGroup* FindGroupByID(int GroupID) const;
	/** @return the list of triangles in the given GroupID, or empty list if not found */
	const TArray<int>& GetGroupFaces(int GroupID) const { return GetGroupTriangles(GroupID); }
	/** @return the list of triangles in the given GroupID, or empty list if not found */
	UE_API const TArray<int>& GetGroupTriangles(int GroupID) const;
	/** @return the list of neighbour GroupIDs for the given GroupID, or empty list if not found */
	UE_API const TArray<int>& GetGroupNbrGroups(int GroupID) const;

	/** @return true if the mesh edge identified by FMeshTriEdgeID is a group boundary edge */
	UE_API bool IsGroupEdge(FMeshTriEdgeID TriEdgeID, bool bIncludeMeshBoundary = false) const; 
	/** @return the ID of the FGroupEdge that contains the given Mesh Edge ID*/
	UE_API int FindGroupEdgeID(int MeshEdgeID) const;
	/** @return the ID of the FGroupEdge that contains the given Mesh Edge ID */
	UE_API int FindGroupEdgeID(FMeshTriEdgeID FMeshTriEdgeID) const;
	/** @return the list of vertices of a FGroupEdge identified by the GroupEdgeID */
	UE_API const TArray<int>& GetGroupEdgeVertices(int GroupEdgeID) const;
	/** @return the list of edges of a FGroupEdge identified by the GroupEdgeID */
	UE_API const TArray<int>& GetGroupEdgeEdges(int GroupEdgeID) const;

	/** Add the groups connected to the given GroupEdgeID to the GroupsOut list. This is not the either-side pair, but the set of groups on the one-ring of each connected corner. */
	UE_API void FindEdgeNbrGroups(int GroupEdgeID, TArray<int>& GroupsOut) const;
	/** Add the groups connected to all the GroupEdgeIDs to the GroupsOut list. This is not the either-side pair, but the set of groups on the one-ring of each connected corner. */
	UE_API void FindEdgeNbrGroups(const TArray<int>& GroupEdgeIDs, TArray<int>& GroupsOut) const;

	/** Add the edges connected to the given GroupEdgeID to the EdgesOut list. This is somewhat expensive. */
	UE_API void FindEdgeNbrEdges(int GroupEdgeID, TArray<int>& EdgesOut) const;

	/** @return true if the FGroupEdge identified by the GroupEdgeID is on the mesh boundary */
	UE_API bool IsBoundaryEdge(int32 GroupEdgeID) const;
	/** @return true if group edge is a "simple" edge, meaning it only has one mesh edge */
	UE_API bool IsSimpleGroupEdge(int32 GroupEdgeID) const;

	/** @return true if the FGroupEdge identified by the GroupEdgeID is an isolated loop (ie not connected to any other edges) */
	UE_API bool IsIsolatedLoop(int32 GroupEdgeID) const;

	/** @return arc length of edge, and optionally accumulated arclength distances for each edge vertex */
	UE_API double GetEdgeArcLength(int32 GroupEdgeID, TArray<double>* PerVertexLengthsOut = nullptr) const;

	/** @return arclength midpoint of edge, and optionally arclength info */
	UE_API FVector3d GetEdgeMidpoint(int32 GroupEdgeID, double* ArcLengthOut = nullptr, TArray<double>* PerVertexLengthsOut = nullptr) const;

	/** @return average position of edge (may not be on edge for curved edges) */
	UE_API FVector3d GetEdgeAveragePosition(int32 GroupEdgeID) const;

	/** Add all the groups connected to the given Corner to the GroupsOut list */
	UE_API void FindCornerNbrGroups(int CornerID, TArray<int>& GroupsOut) const;
	/** Add all the groups connected to the given Corners to the GroupsOut list */
	UE_API void FindCornerNbrGroups(const TArray<int>& CornerIDs, TArray<int>& GroupsOut) const;

	/**
	 * Helper that iterates over the group edges neighboring a particular corner.
	 * @param ReturnTrueToContinue Function to call on all the group edges, which should return
	 *  true if it wants iteration to continue, and false if it wants to exit early.
	 */
	UE_API void ForCornerNbrEdges(int CornerID, TFunctionRef<bool(int32 EdgeID)> ReturnTrueToContinue) const;

	/** Add all the Edges connected to the given Corner to the EdgesOut list. This is somewhat expensive as there is no connection from Corners to Edges, must iterate over adjacent faces */
	UE_API void FindCornerNbrEdges(int CornerID, TArray<int>& EdgesOut) const;
	/** Add all the Corners connected to the given Corner to the EdgesOut list. This is somewhat expensive as there is no connection from Corners to Corners, must iterate over adjacent faces/edges */
	UE_API void FindCornerNbrCorners(int CornerID, TArray<int>& CornersOut) const;

	/** Add all the groups connected to the given Mesh Vertex to the GroupsOut list */
	UE_API void FindVertexNbrGroups(int VertexID, TArray<int>& GroupsOut ) const;
	/** Add all the groups connected to the given Mesh Vertices to the GroupsOut list */
	UE_API void FindVertexNbrGroups(const TArray<int>& VertexIDs, TArray<int>& GroupsOut) const;

	/**
	 * Call EdgeFunc for each boundary edge of the given Group  (no order defined). 
	 * Templated so it works with both lambdas or TFunctions. Defined here to avoid
	 * dll explicit instantiation woes.
	 */
	template <typename FuncTakingFGroupEdgeAndInt>
	void ForGroupEdges(int GroupID, const FuncTakingFGroupEdgeAndInt& EdgeFunc) const
	{
		const FGroup* Group = FindGroupByID(GroupID);
		ensure(Group != nullptr);
		if (Group != nullptr)
		{
			for (const FGroupBoundary& Boundary : Group->Boundaries)
			{
				for (int EdgeIndex : Boundary.GroupEdges)
				{
					EdgeFunc(Edges[EdgeIndex], EdgeIndex);
				}
			}
		}
	}

	/** 
	 * Call EdgeFunc for each boundary edge of each of the given Groups (no order defined). 
	 * Templated so that it can be used with either sets (such as selections) or lists, and
	 * works with both lambdas or TFunctions. Defined here to avoid dll explicit instantiation woes.
	 */
	template <typename IntIterableContainerType, typename FuncTakingFGroupEdgeAndInt>
	void ForGroupSetEdges(const IntIterableContainerType& GroupIDs,
		const FuncTakingFGroupEdgeAndInt& EdgeFunc) const
	{
		TArray<int> DoneEdges;

		for (int GroupID : GroupIDs)
		{
			const FGroup* Group = FindGroupByID(GroupID);
			ensure(Group != nullptr);
			if (Group != nullptr)
			{
				for (const FGroupBoundary& Boundary : Group->Boundaries)
				{
					for (int EdgeIndex : Boundary.GroupEdges)
					{
						if (DoneEdges.Contains(EdgeIndex) == false)
						{
							EdgeFunc(Edges[EdgeIndex], EdgeIndex);
							DoneEdges.Add(EdgeIndex);
						}
					}
				}
			}
		}
	}

	/** Add all the vertices of the given GroupID to the Vertices set */
	UE_API void CollectGroupVertices(int GroupID, TSet<int>& Vertices) const;
	/** Add all the group boundary vertices of the given GroupID to the Vertices set */
	UE_API void CollectGroupBoundaryVertices(int GroupID, TSet<int>& Vertices) const;

	/** 
	 * Calculate tangent of group edge, as direction from start to end endpoints
	 * @return false if edge is degenerate or start == end 
	 */
	UE_API bool GetGroupEdgeTangent(int GroupEdgeID, FVector3d& TangentOut) const;

	/**
	 * @return a 3D frame for the given group. Based on centroid of triangle centroids, so does not necessarily lie on group surface.
	 */
	UE_API FFrame3d GetGroupFrame(int32 GroupID) const;

	/**
	 * @return a 3D frame for the given selection
	 */
	UE_API FFrame3d GetSelectionFrame(const FGroupTopologySelection& Selection, FFrame3d* InitialLocalFrame = nullptr) const;

	/**
	 * @return A Bounding Box for the vertices of the current selection, with TransformFunc applied
	 */
	UE_API FAxisAlignedBox3d GetSelectionBounds(const FGroupTopologySelection& Selection, TFunctionRef<FVector3d(const FVector3d&)> TransformFunc) const;


	/**
	 * Get the set of selected triangles for a given GroupTopologySelection
	 */
	UE_API void GetSelectedTriangles(const FGroupTopologySelection& Selection, TArray<int32>& Triangles) const;

	/**
	 * Helper function for use with ShouldAddExtraCornerAtVert. Checks that the dot product of the two mesh edge
	 * unit vectors starting from shared vertex is greater than or equal to the given threshold. Assumes that SharedVid 
	 * actually is shared by the two edges, and returns false if either edge is degenerate.
	 * 
	 * @param DotThreshold Threshold usually set equal to FMathd::Cos(ThresholdDegrees * FMathd::DegToRad). The
	 *   parameter is not degrees just to avoid that calculation on each invocation.
	 */ 
	static UE_API bool IsEdgeAngleSharp(const FDynamicMesh3* Mesh, int32 SharedVid, const FIndex2i& AttachedGroupEdgeEids, double DotThreshold);

protected:
	const FDynamicMesh3* Mesh = nullptr;
	const FDynamicMeshPolygroupAttribute* GroupLayer = nullptr;

	TArray<int> GroupIDToGroupIndexMap;		// allow fast lookup of index in .Groups, given GroupID

	UE_DEPRECATED(5.1, "Use VertexIDToCornerIDMap instead.")
	TBitArray<> CornerVerticesFlags;		// bit array of corners for fast testing in ExtractGroupEdges  (redundant w/ VertexIDToCornerIDMap?)

	TArray<int> EmptyArray;
	TMap<int32, int32> VertexIDToCornerIDMap;

	TSet<int32> CurrentExtraCornerVids;

	/**
	 * @return true if given mesh vertex should be a Corner vertex. This looks at the actual mesh- it doesn't
	 *  use the topology structures.
	 */
	UE_DEPRECATED(5.1, "Use ShouldVertBeCorner instead.")
	UE_API virtual bool IsCornerVertex(int VertexID) const;

	/**
	 * Return true if the given vertex should be a corner based on the number of adjoining group edges or based on
	 * ShouldAddExtraCornerAtVert, if provided.
	 * 
	 * @param bIsExtraCornerOut Set to true if the vertex didn't have 3 or more group adjoining edges, but was forced
	 *  to be a corner by ShouldAddExtraCornerAtVert.
	 */
	UE_API virtual bool ShouldVertBeCorner(int VertexID, bool* bIsExtraCornerOut = nullptr) const;

	/** Return false if we had a problem finding group boundaries */
	UE_DEPRECATED(5.1, "Use GenerateBoundaryAndGroupEdges instead.")
	bool ExtractGroupEdges(FGroup& Group) { return false; }

	// Generates group edges and boundary structs for given group. Returns false if failed to find group boundaries.
	UE_API bool GenerateBoundaryAndGroupEdges(FGroup& Group, TMap<int32, int32>& GroupEdgeMinEidToGroupEdgeID, TBitArray<>& VertCheckedForCorner);

	UE_DEPRECATED(5.1, "Use MakeEdgeGroupsPair instead. The returned pair cannot uniquely identify a group edge.")
	inline FIndex2i MakeEdgeID(int MeshEdgeID) const { return MakeEdgeGroupsPair(MeshEdgeID); }
	UE_DEPRECATED(5.1, "Use MakeEdgeGroupsPair instead. The returned pair cannot uniquely identify a group edge.")
	inline FIndex2i MakeEdgeID(int Group1, int Group2) const { return MakeEdgeGroupsPair(Group1, Group2); }

	inline FIndex2i MakeEdgeGroupsPair(int MeshEdgeID) const;
	inline FIndex2i MakeEdgeGroupsPair(int Group1, int Group2) const;

	UE_DEPRECATED(5.1, "Group edges cannot be uniquely identified by a group pair.")
	int FindExistingGroupEdge(int GroupID, int OtherGroupID, int FirstVertexID, int SecondVertexID) { return 0; }

	UE_API void GetAllVertexGroups(int32 VertexID, TArray<int32>& GroupsOut) const;
};

FIndex2i FGroupTopology::MakeEdgeGroupsPair(int MeshEdgeID) const
{
	FIndex2i EdgeTris = Mesh->GetEdgeT(MeshEdgeID);

	if (EdgeTris.B == FDynamicMesh3::InvalidID)
	{
		return FIndex2i(GetGroupID(EdgeTris.A), FDynamicMesh3::InvalidID);
	}
	else
	{
		return MakeEdgeGroupsPair(GetGroupID(EdgeTris.A), GetGroupID(EdgeTris.B));
	}
}

FIndex2i FGroupTopology::MakeEdgeGroupsPair(int Group1, int Group2) const
{
	check(Group1 != Group2);
	check(Group1 >= 0 && Group2 >= 0);
	return (Group1 < Group2) ? FIndex2i(Group1, Group2) : FIndex2i(Group2, Group1);
}

// Defined here instead of the class declaration because it requires FGroupTopology to be defined.
bool FGroupTopologySelection::IsSelectedTriangle(const FDynamicMesh3* Mesh, const FGroupTopology* Topology, int32 TriangleID) const
{
	int GroupID = Topology->GetGroupID(TriangleID);
	return SelectedGroupIDs.Contains(GroupID);
}




/**
 * FTriangleGroupTopology is a simplification of FGroupTopology that just represents a normal mesh.
 * This allows algorithms to be written against FGroupTopology that will also work per-triangle.
 * (However, there is enormous overhead to doing it this way!)
 */
class FTriangleGroupTopology : public FGroupTopology
{
public:
	FTriangleGroupTopology() {}
	UE_API FTriangleGroupTopology(const FDynamicMesh3* Mesh, bool bAutoBuild);

	UE_API virtual bool RebuildTopology() override;

	virtual int GetGroupID(int TriangleID) const override
	{
		return TriangleID;
	}
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
