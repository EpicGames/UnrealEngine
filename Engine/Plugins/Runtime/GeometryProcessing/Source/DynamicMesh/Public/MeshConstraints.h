// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp MeshConstraints

#pragma once

#include "Spatial/SpatialInterfaces.h"   // for projection target
#include "Polyline3.h"

namespace UE
{
namespace Geometry
{

/**
 * EEdgeRefineFlags indicate constraints on triangle mesh edges
 */
enum class EEdgeRefineFlags 
{
	/** Edge is unconstrained */
	NoConstraint = 0,
	/** Edge cannot be flipped */
	NoFlip = 1,
	/** Edge cannot be split */
	NoSplit = 2,
	/** Edge cannot be collapsed */
	NoCollapse = 4,
	/** Topology of constraints should not be changed, i.e. disconnected constraint edges should not be connected via an adjacent edge collapse */
	NoTopologyMerge = 8,
	/** Edge cannot be flipped, split, collapsed, or topologically-merged */
	FullyConstrained = NoFlip | NoSplit | NoCollapse | NoTopologyMerge,
	/** Edge can only be split */
	SplitsOnly = NoFlip | NoCollapse,
	/** Edge can only flip*/
	FlipOnly = NoSplit | NoCollapse,
	/** Edge can only collapse*/
	CollapseOnly = NoFlip | NoSplit
};


/**
 * FEdgeConstraint is a constraint on a triangle mesh edge
 */
struct FEdgeConstraint
{
public:
	/** Constraint flags on this edge */
	EEdgeRefineFlags RefineFlags;
	/** Edge is associated with this projection target. */
	IProjectionTarget * Target;

	/** This ID is not a constraint, but can be used to find descendants of a constrained input edge after splits */
	int TrackingSetID;

	/** Squared tolerance distance: if positive, edge should not move farther than this distance from Target (if set, and feature is enabled; see TMeshSimplification::bLimitConstrainedSeamMovement) */
	float DistanceToleranceSq = -1.f;

	FEdgeConstraint()
	{
		RefineFlags = EEdgeRefineFlags::NoConstraint;
		Target = nullptr;
		TrackingSetID = -1;
		DistanceToleranceSq = -1.f;
	}

	explicit FEdgeConstraint(EEdgeRefineFlags ConstraintFlags)
	{
		RefineFlags = ConstraintFlags;
		Target = nullptr;
		TrackingSetID = -1;
	}

	explicit FEdgeConstraint(EEdgeRefineFlags ConstraintFlags, IProjectionTarget* TargetIn)
	{
		RefineFlags = ConstraintFlags;
		Target = TargetIn;
		TrackingSetID = -1;
	}

	/** @return true if edge can be flipped */
	bool CanFlip() const
	{
		return ((int)RefineFlags & (int)EEdgeRefineFlags::NoFlip) == 0;
	}

	/** @return true if edge can be split */
	bool CanSplit() const
	{
		return CanSplit(RefineFlags);
	}

	/** @return true if edge can be collapsed */
	bool CanCollapse() const
	{
		return CanCollapse(RefineFlags);
	}

	bool CanMergeTopology() const
	{
		return CanMergeTopology(RefineFlags);
	}

	/** @return true if edge cannot be modified at all */
	bool NoModifications() const
	{
		return NoModifications(RefineFlags);
	}

	/** @return true if edge is unconstrained */
	bool IsUnconstrained() const
	{
		return RefineFlags == EEdgeRefineFlags::NoConstraint && Target == nullptr;
	}

	/** @return an unconstrained edge constraint (ie not constrained at all) */
	static FEdgeConstraint Unconstrained() { return FEdgeConstraint(EEdgeRefineFlags::NoConstraint); }

	/** @return a no-flip edge constraint */
	static FEdgeConstraint NoFlips() { return FEdgeConstraint(EEdgeRefineFlags::NoFlip); }

	/** @return a splits-only edge constraint */
	static FEdgeConstraint SplitsOnly() { return FEdgeConstraint(EEdgeRefineFlags::SplitsOnly); }

	/** @return a fully constrained edge constraint */
	static FEdgeConstraint FullyConstrained() { return FEdgeConstraint(EEdgeRefineFlags::FullyConstrained); }

	/** @return true if flags permit flip */
	static inline bool CanFlip(const EEdgeRefineFlags Flags)
	{
		return ((int)Flags & (int)EEdgeRefineFlags::NoFlip) == 0;
	}

	/** @return true if flags permit split */
	static inline bool CanSplit(const EEdgeRefineFlags Flags)
	{
		return ((int)Flags & (int)EEdgeRefineFlags::NoSplit) == 0;
	}

	/** @return true if flags permit collapsed */
	static inline bool CanCollapse(const EEdgeRefineFlags Flags)
	{
		return ((int)Flags & (int)EEdgeRefineFlags::NoCollapse) == 0;
	}

	static inline bool CanMergeTopology(const EEdgeRefineFlags Flags)
	{
		return ((int)Flags & (int)EEdgeRefineFlags::NoTopologyMerge) == 0;
	}

	/** @return true if flags permit no edge modification */
	static inline bool NoModifications(const EEdgeRefineFlags Flags)
	{
		return ((int)Flags & (int)EEdgeRefineFlags::FullyConstrained) == (int)EEdgeRefineFlags::FullyConstrained;
	}
};



/**
 * FVertexConstraint is a constraint on a triangle mesh vertex
 */
struct FVertexConstraint
{
public:

	/** Is this vertex permanent (i.e. cannot be removed by topology-change operations). */
	bool bCannotDelete;

	/** Can this vertex be moved */
	bool bCanMove;

	/** Vertices marked as "CannotDelete" but sharing the same FixedSetID can optionally be collapsed together (ie in Remesher) */
	int FixedSetID;

	/** value for FixedSetID that is treated as not-a-fixed-set-ID by various functions (ie don't use this value yourself) */
	static constexpr int InvalidSetID = -1;

	/** Vertex is associated with this ProjectionTarget, and should be projected onto it (ie in Remesher) */
	IProjectionTarget* Target;

	FVertexConstraint() :
		bCannotDelete(false),
		bCanMove(false),
		FixedSetID(InvalidSetID),
		Target(nullptr)
	{}

	explicit FVertexConstraint(bool bCannotDelete, bool bCanMove, int FixedSetID = InvalidSetID) :
		bCannotDelete(bCannotDelete),
		bCanMove(bCanMove),
		FixedSetID(FixedSetID),
		Target(nullptr)
	{}

	explicit FVertexConstraint(IProjectionTarget* Target, int FixedSetID = InvalidSetID) :
		bCannotDelete(false),
		bCanMove(true),
		FixedSetID(FixedSetID),
		Target(Target)
	{}

	bool IsUnconstrained() const 
	{
		return ( (bCanMove == true) && (bCannotDelete == false) && (Target == nullptr) );
	}

	/** @return an unconstrained vertex constraint (ie not constrained at all) */
	static FVertexConstraint Unconstrained() { return FVertexConstraint(false, true); }

	/** @return a fully constrained vertex: can't delete and can't move */
	static FVertexConstraint FullyConstrained() { return FVertexConstraint(true, false); }

	/** @return a movable but not "delete-able" vertex constraint */
	static FVertexConstraint PermanentMovable() { return FVertexConstraint(true, true); }

	/** Combine the incoming constraint with this constraint. The combination cannot be less restrictive than the
	    existing constraint. */
	void CombineConstraint(const FVertexConstraint& OtherConstraint)
	{
		bCannotDelete = bCannotDelete || OtherConstraint.bCannotDelete;
		bCanMove = bCanMove && OtherConstraint.bCanMove;
		// TODO: What should happen if both constraints have projection targets?
		Target = OtherConstraint.Target;
	}

};


class FMeshConstraintCurve : public FPolyline3d, public IProjectionTarget
{
public:

	virtual FVector3d Project(const FVector3d& Point, int /*Identifier*/) override
	{
		int32 NearestSegmentIdx_Unused;
		double DistSq_Unused, UnitParam_Unused;
		FVector3d ClosestPoint;
		if (CurveUtil::ProjectToCurve<double, FVector3d>(Vertices, false, Point, DistSq_Unused, NearestSegmentIdx_Unused, ClosestPoint, UnitParam_Unused, nullptr))
		{
			return ClosestPoint;
		}
		else
		{
			// Just return the input if the projection fails (i.e., if the constraint curve is empty)
			return Point;
		}
	}
};

struct FConstraintProjectionData
{
	TArray<TSharedPtr<FMeshConstraintCurve>> ProjectionCurves;
};


/**
 * FMeshConstraints is a set of Edge and Vertex constraints for a Triangle Mesh
 */
class FMeshConstraints
{
protected:

	/** Map of mesh edge IDs to active EdgeConstraints */
	TMap<int, FEdgeConstraint> Edges;

	/** Map of mesh vertex IDs to active VertexConstraints */
	TMap<int, FVertexConstraint> Vertices;

	/** internal counter used to allocate new FixedSetIDs */
	int FixedSetIDCounter;

public:

	/** Additional data for constraint curve projection */
	FConstraintProjectionData ProjectionData;


	FMeshConstraints()
	{
		FixedSetIDCounter = 0;
	}

	/** @return an unused Fixed Set ID */
	int AllocateSetID() 
	{
		return FixedSetIDCounter++;
	}

	/** @return map of active edge constraints */
	const TMap<int, FEdgeConstraint>& GetEdgeConstraints() const
	{
		return Edges;
	}

	/** @return map of active vertex constraints */
	const TMap<int, FVertexConstraint>& GetVertexConstraints() const
	{
		return Vertices;
	}


	/** @return true if any edges or vertices are constrained */
	bool HasConstraints() const
	{
		return Edges.Num() > 0 || Vertices.Num() > 0;
	}



	/** @return true if given EdgeID has active constraint */
	bool HasEdgeConstraint(int EdgeID) const
	{
		return Edges.Find(EdgeID) != nullptr;
	}

	/** @return EdgeConstraint for give EdgeID, may be Unconstrained */
	FEdgeConstraint GetEdgeConstraint(int EdgeID) const
	{
		const FEdgeConstraint* Found = Edges.Find(EdgeID);
		if (Found == nullptr)
		{
			return FEdgeConstraint::Unconstrained();
		}
		else
		{
			return *Found;
		}
	}

	/** @return true if EdgeID is constrained and ConstraintOut is filled */
	bool GetEdgeConstraint(int EdgeID, FEdgeConstraint& ConstraintOut) const
	{
		const FEdgeConstraint* Found = Edges.Find(EdgeID);
		if (Found == nullptr)
		{
			return false;
		}
		else
		{
			ConstraintOut = *Found;
			return true;
		}
	}

	/** Set the constraint on the given EdgeID */
	void SetOrUpdateEdgeConstraint(int EdgeID, const FEdgeConstraint& ec)
	{
		Edges.Add(EdgeID, ec);
	}

	/** Clear the constraint on the given EdgeID */
	void ClearEdgeConstraint(int EdgeID)
	{
		Edges.Remove(EdgeID);
	}


	/** Find all constraint edges for the given SetID, and return via FoundListOut */
	void FindConstrainedEdgesBySetID(int SetID, TArray<int>& FoundListOut) const
	{
		for (const TPair<int, FEdgeConstraint>& pair : Edges)
		{
			if (pair.Value.TrackingSetID == SetID)
			{
				FoundListOut.Add(pair.Key);
			}
		}
	}



	/** @return true if given VertexID has active constraint */
	bool HasVertexConstraint(int VertexID) const
	{
		return Vertices.Find(VertexID) != nullptr;
	}


	/** @return VertexConstraint for give VertexID, may be Unconstrained */
	FVertexConstraint GetVertexConstraint(int VertexID) const
	{
		const FVertexConstraint* Found = Vertices.Find(VertexID);
		if (Found == nullptr)
		{
			return FVertexConstraint::Unconstrained();
		}
		else
		{
			return *Found;
		}
	}

	/** @return true if VetexID  is constrained and ConstraintOut is filled */
	bool GetVertexConstraint(int VertexID, FVertexConstraint& ConstraintOut)const
	{
		const FVertexConstraint* Found = Vertices.Find(VertexID);
		if (Found == nullptr)
		{
			return false;
		}
		else
		{
			ConstraintOut = *Found;
			return true;
		}
	}

	/** Set the constraint on the given VertexID */
	void SetOrUpdateVertexConstraint(int VertexID, const FVertexConstraint& vc)
	{
		Vertices.Add(VertexID, vc);
	}

	void SetOrCombineVertexConstraint(int VertexID, const FVertexConstraint& vc)
	{
		FVertexConstraint* Found = Vertices.Find(VertexID);
		if (Found != nullptr)
		{
			Found->CombineConstraint(vc);
		}
		else
		{
			Vertices.Add(VertexID, vc);
		}
	}

	/** Clear the constraint on the given VertexID */
	void ClearVertexConstraint(int VertexID)
	{
		Vertices.Remove(VertexID);
	}

};


} // end namespace UE::Geometry
} // end namespace UE