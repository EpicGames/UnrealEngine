// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp MeshConstraintsUtil

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "MeshConstraints.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

/**
 * Utility functions for configuring a FMeshConstraints instance
 */
class FMeshConstraintsUtil
{
public:

	
	/**
	 * Constrain all attribute seams for all overlays of a mesh
	 * @param Constraints the set of constraints to add to
	 * @param Mesh the mesh to constrain
	 * @param bAllowSplits should we allow constrained edges to be split
	 * @param bAllowSmoothing should we allow constrained vertices to be smoothed
	 * @param bParallel should we run the algo in parallel
	 */
	static UE_API void ConstrainAllSeams(FMeshConstraints& Constraints, const FDynamicMesh3& Mesh, bool bAllowSplits, bool bAllowSmoothing, bool bParallel = true);

	/**
	 * Constrain all attribute seams for all overlays of a mesh and different types of boundaries on the mesh
	 * @param Constraints the set of constraints to add to
	 * @param Mesh the mesh to constrain
	 * @param MeshBoundaryConstraint the constraints to place on Mesh boundaries
	 * @param GroupBoundaryConstraint the constraints to place on boundaries between polygon groups
	 * @param MaterialBoundaryConstraint the constraints to place on boundaries between different materials
	 * @param bAllowSplits should we allow constrained edges to be split
	 * @param bAllowSmoothing should we allow constrained vertices to be smoothed
	 * @param bAllowSeamCollapse should we allow seam edges to collapse
	 * @param bParallel should we run the algo in parallel
	 */
	static UE_API void ConstrainAllBoundariesAndSeams(FMeshConstraints& Constraints,
											   const FDynamicMesh3& Mesh,
											   EEdgeRefineFlags MeshBoundaryConstraint,
											   EEdgeRefineFlags GroupBoundaryConstraint,
											   EEdgeRefineFlags MaterialBoundaryConstraint,
											   bool bAllowSeamSplits, bool bAllowSeamSmoothing, bool bAllowSeamCollpase = false,
											   bool bParallel = true);

	/**
	 * Constrain all attribute seams for all overlays of a mesh, for edges in the edge array. This includes edges in 
s	 * EdgeROI that are seams, as well as vertices incident to edges in EdgeROI that are on seams.
	 * @param Constraints the set of constraints to add to
	 * @param Mesh the mesh to constrain
	 * @param EdgeROI list of edges to try to constrain
	 * @param bAllowSplits should we allow constrained edges to be split
	 * @param bAllowSmoothing should we allow constrained vertices to be smoothed
	 */
	static UE_API void ConstrainSeamsInEdgeROI(FMeshConstraints& Constraints, const FDynamicMesh3& Mesh, const TArray<int>& EdgeROI, bool bAllowSplits, bool bAllowSmoothing, bool bParallel = true);


	/**
	 * Constrain each edge in the EdgeROI set that is incident on exactly one triangle in the TriangleROI set
	 * @param Constraints the set of constraints to add to
	 * @param Mesh the mesh to constrain
	 * @param EdgeROI set of edges to try to constrain
	 * @param TriangleROI set of triangles that determine whether an edge is a "boundary"
	 * @param bAllowSplits should we allow constrained edges to be split
	 * @param bAllowSmoothing should we allow constrained vertices to be smoothed
	 */
	static UE_API void ConstrainROIBoundariesInEdgeROI(FMeshConstraints& Constraints, 
		const FDynamicMesh3& Mesh, 
		const TSet<int>& EdgeROI, 
		const TSet<int>& TriangleROI,
		bool bAllowSplits, 
		bool bAllowSmoothing);

	/**
	 * Constrain attribute seams of the given overlay
	 * @param Constraints the set of constraints to add to
	 * @param Mesh the mesh to constrain
	 * @param Overlay the attribute overlay to find seams in
	 */
	template<typename RealType, int ElementSize>
	static void ConstrainSeams(FMeshConstraints& Constraints, const FDynamicMesh3& Mesh, const TDynamicMeshOverlay<RealType, ElementSize>& Overlay)
	{
		for (int EdgeID : Mesh.EdgeIndicesItr())
		{
			if (Overlay.IsSeamEdge(EdgeID))
			{
				Constraints.SetOrUpdateEdgeConstraint(EdgeID, FEdgeConstraint::FullyConstrained());
				FIndex2i EdgeVerts = Mesh.GetEdgeV(EdgeID);
				Constraints.SetOrUpdateVertexConstraint(EdgeVerts.A, FVertexConstraint::FullyConstrained());
				Constraints.SetOrUpdateVertexConstraint(EdgeVerts.B, FVertexConstraint::FullyConstrained());
			}
		}
	}



	/** 
	 * For all edges, disable flip/split/collapse. For all vertices, pin in current position.
	 * @param Constraints the set of constraints to add to
	 * @param Mesh the mesh to constrain
	 */
	template<typename iter>
	static void FullyConstrainEdges(FMeshConstraints& Constraints, const FDynamicMesh3& Mesh, iter BeginEdges, iter EndEdges)
	{
		while (BeginEdges != EndEdges)
		{
			int EdgeID = *BeginEdges;
			if (Mesh.IsEdge(EdgeID))
			{
				Constraints.SetOrUpdateEdgeConstraint(EdgeID, FEdgeConstraint::FullyConstrained());
				FIndex2i EdgeVerts = Mesh.GetEdgeV(EdgeID);
				Constraints.SetOrUpdateVertexConstraint(EdgeVerts.A, FVertexConstraint::FullyConstrained());
				Constraints.SetOrUpdateVertexConstraint(EdgeVerts.B, FVertexConstraint::FullyConstrained());
			}
			BeginEdges++;
		}
	}


	/**
	 * For all edges, disable flip/split/collapse. For all vertices, pin in current position.
	 * @param Constraints the set of constraints to add to
	 * @param Mesh the mesh to constrain
	 * @param Enumerable object that can be passed to a range-based for loop
	 */
	template<typename EnumerableType>
	static void FullyConstrainEdges(FMeshConstraints& Constraints, const FDynamicMesh3& Mesh, EnumerableType Enumerable)
	{
		for ( int EdgeID : Enumerable)
		{
			if (Mesh.IsEdge(EdgeID))
			{
				Constraints.SetOrUpdateEdgeConstraint(EdgeID, FEdgeConstraint::FullyConstrained());
				FIndex2i EdgeVerts = Mesh.GetEdgeV(EdgeID);
				Constraints.SetOrUpdateVertexConstraint(EdgeVerts.A, FVertexConstraint::FullyConstrained());
				Constraints.SetOrUpdateVertexConstraint(EdgeVerts.B, FVertexConstraint::FullyConstrained());
			}
		}
	}

	/**
	 * Constrain all attribute seams for all overlays of a mesh and different types of boundaries on the mesh
	 * @param EdgeID the edge in Mesh to constrain
	 * @param Mesh the mesh to constrain
	 * @param MeshBoundaryConstraint the constraint policy for Mesh boundaries
	 * @param GroupBoundaryConstraint the constraint policy for boundaries between polygon groups
	 * @param MaterialBoundaryConstraint the constraint policy for boundaries between different materials
	 * @param SeamEdgeConstraint  the constraint policy for attribute (e.g. uv, normals ) seams
	 * @param bAllowSmoothing should we allow constrained vertices to be smoothed
	 * @param EdgeConstraint [out] edge constraint to be updated
	 * @param VertexConstraintA [out] vertex constraint to be updated on return
	 * @param VertexConstraintB [out] vertex constraint to be updated on return
	 * @return false if no constraint is needed for this edge
	 */
	static UE_API bool ConstrainEdgeBoundariesAndSeams(const int EdgeID,
											    const FDynamicMesh3& Mesh,
			                                    const EEdgeRefineFlags MeshBoundaryConstraint,
			                                    const EEdgeRefineFlags GroupBoundaryConstraint,
			                                    const EEdgeRefineFlags MaterialBoundaryConstraint,
			                                    const EEdgeRefineFlags SeamEdgeConstraint,
			                                    const bool bAllowSeamSmoothing,
			                                    FEdgeConstraint& EdgeConstraint,
			                                    FVertexConstraint& VertexConstraintA,
			                                    FVertexConstraint& VertexConstraintB);

	enum class EBoundaryType
	{
		Mesh = 0,
		Group = 1,
		MaterialID = 2
	};

	/**
	 * Create polycurve projection targets at group boundaries. 
	 * Vertices on multiple boundaries are fixed, as well as vertices at boundary "corners" 
	 * (where the angle formed between adjacent boundary edges is above a given threshold.)
	 * 
	 * @param Constraints (out) constraint data to add new constraints to
	 * @param Mesh the mesh to constrain
	 * @param CornerAngleThreshold angle threshold (in degrees) above which adjacent edges are considered a corner
	 * @param CurveConstraintToleranceDistance threshold distance used to limit how far edges should be allowed to deviate from their constraint curves
	 */
	static UE_API void SetBoundaryConstraintsWithProjection(
		FMeshConstraints& Constraints,
		const EBoundaryType BoundaryToConstrain,
		const FDynamicMesh3& Mesh,
		double CornerAngleThreshold = 45.0,
		float CurveConstraintToleranceDistance = -1.f);

private:
	FMeshConstraintsUtil() = delete;		// this class is not constructible


};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
