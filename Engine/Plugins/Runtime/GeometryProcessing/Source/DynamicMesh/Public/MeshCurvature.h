// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Sampling/SampleSetStatistics.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
	/**
	 * Implementations of various mesh curvatures and curvature-related metrics
	 */
	namespace MeshCurvature
	{
		using namespace UE::Geometry;

		/**
		 * Calculate the Discrete Mean-Curvature Normal at a vertex as defined by discrete differential geometry.
		 * Based on Eq 8 from "Discrete Differential-Geometry Operators for Triangulated 2-Manifolds", Meyer et al 2002
		 * The Discrete Mean-Curvature Normal is (2.0 * MeanCurvature * SurfaceNormal)    ((unclear why it has the 2.0))
		 */
		DYNAMICMESH_API FVector3d MeanCurvatureNormal(const FDynamicMesh3& Mesh, int32 VertexIndex);


		/**
		 * Calculate the Discrete Mean-Curvature Normal at a vertex as defined by discrete differential geometry.
		 * Based on Eq 8 from "Discrete Differential-Geometry Operators for Triangulated 2-Manifolds", Meyer et al 2002
		 * The Discrete Mean-Curvature Normal is (2.0 * MeanCurvature * SurfaceNormal)    ((unclear why it has the 2.0))
		 * @param VertexPositionFunc use positions returned by this function instead of mesh positions
		 */
		DYNAMICMESH_API FVector3d MeanCurvatureNormal(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc);



		/**
		 * Calculate the Discrete Gaussian Curvature at a vertex as defined by discrete differential geometry
		 * Based on Eq 9 from "Discrete Differential-Geometry Operators for Triangulated 2-Manifolds", Meyer et al 2002
		 */
		DYNAMICMESH_API double GaussianCurvature(const FDynamicMesh3& Mesh, int32 VertexIndex);

		/**
		 * Calculate the Discrete Gaussian Curvature at a vertex as defined by discrete differential geometry
		 * Based on Eq 9 from "Discrete Differential-Geometry Operators for Triangulated 2-Manifolds", Meyer et al 2002
		 * @param VertexPositionFunc use positions returned by this function instead of mesh positions
		 */
		DYNAMICMESH_API double GaussianCurvature(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc);

	}
}


namespace UE
{
namespace Geometry
{


/**
 * FMeshVertexCurvatureCache calculates and stores various per-vertex Curvature types
 * for a Mesh, as well as some statistics for those values.
 */
class FMeshVertexCurvatureCache
{
public:

	struct FVertexCurvature
	{
		double Mean = 0;
		double Gaussian = 0;
		double MaxPrincipal = 0;
		double MinPrincipal = 0;
	};

	TArray<FVertexCurvature> Curvatures;

	FSampleSetStatisticsd MeanStats;
	FSampleSetStatisticsd GaussianStats;
	FSampleSetStatisticsd MaxPrincipalStats;
	FSampleSetStatisticsd MinPrincipalStats;

	int32 Num() const { return Curvatures.Num(); }
	const FVertexCurvature& operator[](int32 Index) const { return Curvatures[Index]; }

	UE_API void BuildAll(const FDynamicMesh3& Mesh);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
