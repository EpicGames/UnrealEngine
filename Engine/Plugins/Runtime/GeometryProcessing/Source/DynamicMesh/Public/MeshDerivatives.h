// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MatrixTypes.h"

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * First derivatives of some mesh quantities with respect to particular vertex positions
 */
class FMeshDerivatives
{
public:

	/**
	 * Gradient of the interior angle of a vertex within a triangle
	 * 
	 * @param Mesh			 Surface mesh to consider
	 * @param TriangleIndex  The triangle whose interior angle is considered
	 * @param VertexIndex	 The vertex whose interior angle gradient is computed (note this a global mesh vertex index, not [0,1,2])
	 * @param WRTVertexIndex The gradient is computed with respect to this vertex's position
	 * @return 3x1 gradient of interior angle with respect to the specified vertex position
	 */
	static FVector3d InteriorAngleGradient(const FDynamicMesh3& Mesh, int32 TriangleIndex, int32 VertexIndex, int32 WRTVertexIndex);

	/**
	 * Gradient of the unit-length triangle normal
	 * 
	 * @param Mesh			 Surface mesh to consider
	 * @param TriangleIndex  Which triangle to compute the normal gradient for
	 * @param WRTVertexIndex The gradient is computed with respect to this vertex's position
	 * @return 3x3 Jacobian of the the triangle normal with respect to the specified vertex position
	 */
	static FMatrix3d TriangleNormalGradient(const FDynamicMesh3& Mesh, int32 TriangleIndex, int32 WRTVertexIndex);

	/**
	 * Gradient of the triangle normal scaled by triangle area
	 *
	 * @param Mesh           Surface mesh to consider
	 * @param TriangleIndex  Triangle for which to compute the gradient of the area-scaled normal
	 * @param WRTVertexIndex The gradient is computed with respect to this vertex's position
	 * @return 3x3 Jacobian of the the area-scaled triangle normal with respect to the specified vertex position
	 */
	static FMatrix3d TriangleAreaScaledNormalGradient(const FDynamicMesh3& Mesh, int32 TriangleIndex, int32 WRTVertexIndex);

};

} // end namespace UE::Geometry
} // end namespace UE
