// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "EdgeSpan.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/** 
  * Given a mesh and a set of edges, walk connected edges to form a set of paths.
  * We do not distinguish betweens spans and loops -- if the input set of edges forms a loop, it will create an FEdgeSpan with those edges.
  * If the input edges do not form loops, then paths will begin/end at vertices incident on 1 or >2 of the given edges (e.g. "dead ends" or "intersections" of paths.)
  */
class FMeshPaths
{
public:

	FMeshPaths(const FDynamicMesh3* Mesh, const TSet<int>& Edges ) :
		Mesh(Mesh),
		Edges(Edges)
	{}

	// Inputs
	const FDynamicMesh3* Mesh = nullptr;
	const TSet<int> Edges;

	// Outputs
	TArray<FEdgeSpan> Spans;

	// Find connected simple paths made up of edges in the input Edges structure
	UE_API bool Compute();

private:

	TSet<int> UnvisitedEdges;

	// Helper for the Compute function

	/*
	 * Find an edge adjacent to CurrentEdge which is in both the Edges and UnvisitedEdges set.
	 * Consider only edges incident on CrossedVertex, which is the vertex belonging to CurrentEdge that is not FromVertex.
	 * Do not consider an adjacent edge if CrossedVertex is incident on more than 2 input Edges (i.e. we've hit a corner).
	 */
	UE_API int GetNextEdge(int CurrentEdge, int FromVertex, int& CrossedVertex) const;

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
