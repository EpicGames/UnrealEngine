// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp FMeshRefinerBase

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "MeshConstraints.h"
#include "Util/ProgressCancel.h"

namespace UE
{
namespace Geometry
{

class FDynamicMeshChangeTracker;

// Hacky base class to avoid 8 bytes of padding after the vtable
class FMeshRefinerBaseFixLayout
{
public:
	virtual ~FMeshRefinerBaseFixLayout() = default;
};

/**
 * This is a base class that implements common functionality for various triangle mesh resampling strategies
 * (ie FRemesher and FReducer). You probably should not use this class directly.
 */
class FMeshRefinerBase : public FMeshRefinerBaseFixLayout
{
protected:
	/** Mesh that will be refined */
	FDynamicMesh3* Mesh = nullptr;

	/** Constraints are used to control how certain edges and vertices can be modified */
	TOptional<FMeshConstraints> Constraints;

	/** Vertices can be projected onto this surface when they are modified */
	IProjectionTarget* ProjTarget = nullptr;

	FDynamicMeshChangeTracker* ActiveChangeTracker = nullptr;

	FMeshRefinerBase(FDynamicMesh3* MeshIn)
	{
		this->Mesh = MeshIn;
	}

	FMeshRefinerBase()
	{
	}


public:

	enum class EVertexControl : uint8
	{
		AllowAll = 0,
		NoSmooth = 1,
		NoProject = 2,
		NoMovement = NoSmooth | NoProject
	};
	/**
	 * This function allows client to specify fine-grained control over what happens to specific vertices.
	 * Somewhat redundant w/ FVertexConstraints, but simpler to code and has the option to be dynamic during remesh pass.
	 */
	TFunction<EVertexControl(int)> VertexControlF = nullptr;

	/** Set this to be able to cancel running Remesher/Reducer*/
	FProgressCancel* Progress = nullptr;

	/** This is a debugging aid, will break to debugger if these edges are touched, in debug builds */
	TArray<int> DebugEdges;

	/** Options for projecting vertices onto target surface */
	enum class ETargetProjectionMode : uint8
	{
		NoProjection = 0,		// disable projection
		AfterRefinement = 1,	// do all projection after the refine/smooth pass
		Inline = 2				// project after each vertex update. Better results but more
								// expensive because eg we might create a vertex with
								// split, then project, then smooth, then project again.
	};

	/** Method to use to project vertices onto target surface. Default is no projection. */
	ETargetProjectionMode ProjectionMode = ETargetProjectionMode::NoProjection;


	/**
	* If true, then when two Fixed vertices have the same non-invalid SetID,
	* we treat them as not fixed and allow collapse
	*/
	bool AllowCollapseFixedVertsWithSameSetID = true;

	/** Set to true to profile various passes @todo re-enable this! */
	bool ENABLE_PROFILING = false;

	/** 0 = no checking, 1 = check constraints each pass, 2 = and check validity each pass, 3 = and check validity after every mesh change (v slow but best for debugging) */
	int DEBUG_CHECK_LEVEL = 0;


public:
	FMeshRefinerBase(const FMeshRefinerBase&) = delete;
	FMeshRefinerBase(FMeshRefinerBase&&) = delete;
	FMeshRefinerBase& operator=(const FMeshRefinerBase&) = delete;
	FMeshRefinerBase& operator=(FMeshRefinerBase&&) = delete;
	virtual ~FMeshRefinerBase() = default;

	/** Get the current mesh we are operating on */
	FDynamicMesh3* GetMesh() { return Mesh; }

	/** Get the current mesh constraints */
	const TOptional<FMeshConstraints>& GetConstraints() { return Constraints; }

	/**
	 * Set external constraints.
	 * Note that this object will be updated during computation.
	 */
	void SetExternalConstraints(TOptional<FMeshConstraints> ConstraintsIn) { Constraints = MoveTemp(ConstraintsIn); }


	/** Get the current Projection Target */
	IProjectionTarget* ProjectionTarget() { return this->ProjTarget; }

	/** Set a Projection Target */
	void SetProjectionTarget(IProjectionTarget* TargetIn) { this->ProjTarget = TargetIn; }





	/** @return edge flip tolerance */
	double GetEdgeFlipTolerance() { return EdgeFlipTolerance; }

	/** Set edge flip tolerance. Value is clamped to range [-1,1] */
	void SetEdgeFlipTolerance(double NewTolerance) { EdgeFlipTolerance = VectorUtil::Clamp(NewTolerance, -1.0, 1.0); }





	/** If this returns true, abort computation.  */
	virtual bool Cancelled()
	{
		return (Progress == nullptr) ? false : Progress->Cancelled();
	}



protected:

	/** If normals dot product is less than this, we consider it a normal flip. default = 0 */
	double EdgeFlipTolerance = 0.0f;

	/**
	 * @return edge-flip dotproduct metric in range [-1,1] measured between two possibly-not-normalized normal directions. if EdgeFlipTolerance is 0, only the sign of the returned value is valid
	 */
	inline double ComputeEdgeFlipMetric(const FVector3d& Direction0, const FVector3d& Direction1) const
	{
		if (EdgeFlipTolerance == 0)
		{
			return Direction0.Dot(Direction1);
		}
		else
		{
			double ZeroTolerance = FMathd::ZeroTolerance;
			return Normalized(Direction0, ZeroTolerance).Dot(Normalized(Direction1, ZeroTolerance));
		}
	}


	/**
	 * Check if edge collapse will create a face-normal flip.
	 * Also checks if collapse would violate link condition, since we are iterating over one-ring anyway.
	 * This only checks one-ring of vid, so you have to call it twice, with vid and vother reversed, to check both one-rings
	 * @param vid first vertex of edge
	 * @param vother other vertex of edge
	 * @param newv new vertex position after collapse
	 * @param tc triangle on one side of edge
	 * @param td triangle on other side of edge
	 */
	DYNAMICMESH_API bool CheckIfCollapseCreatesFlipOrInvalid(int vid, int vother, const FVector3d& newv, int tc, int td) const;

	/** Avoid creation of triangles smaller than this value. 
	 * We compare the triangle cross product's norm squared to this value since it's cheaper to compute than the triangle area. So the actual
	 * area threshold is sqrt(TinyTriangleThreshold)/2.
	 * Default is based on the threshold for creating triangles in a simulation mesh 
	 */
	double TinyTriangleThreshold = SMALL_NUMBER;

	/**
	 * Check if edge collapse will create a triangle with small area (either all vertices are close together, or in a sliver configuration)
	 * This only checks one-ring of vid, so you have to call it twice, with vid and vother reversed, to check both one-rings
	 * @param vid first vertex of edge
	 * @param vother other vertex of edge
	 * @param newv new vertex position after collapse
	 * @param tc triangle on one side of edge
	 * @param td triangle on other side of edge
	 */
	DYNAMICMESH_API bool CheckIfCollapseCreatesTinyTriangle(int vid, int vother, const FVector3d& newv, int tc, int td) const;

	/**
	 * Check if edge flip might reverse normal direction.
	 * Not entirely clear on how to best implement this test. Currently checking if any normal-pairs are reversed.
	 * @param a first vertex of edge
	 * @param b second vertex of edge
	 * @param c opposing vertex 1
	 * @param d opposing vertex 2
	 * @param t0 index of triangle containing [a,b,c]
	 */
	DYNAMICMESH_API bool CheckIfFlipInvertsNormals(int a, int b, int c, int d, int t0) const;

	/**
	 * Check if edge flip might create a triangle with small area (either all vertices are close together, or in a sliver configuration)
	 * @param a first vertex of edge
	 * @param b second vertex of edge
	 * @param c opposing vertex 1
	 * @param d opposing vertex 2
	 * @param t0 index of triangle containing [a,b,c]
	 */
	DYNAMICMESH_API bool CheckIfFlipCreatesTinyTriangle(int OriginalEdgeVertexA, int OriginalEdgeVertexB, int OppositeEdgeVertexC, int OppositeEdgeVertexD, int OriginalTriangleIndex) const;

	/**
	 * Figure out if we can collapse edge eid=[a,b] under current constraint set.
	 * First we resolve vertex constraints using CanCollapseVertex(). However this
	 * does not catch some topological cases at the edge-constraint level, which
	 * which we will only be able to detect once we know if we are losing a or b.
	 * See comments on CanCollapseVertex() for what collapse_to is for.
	 * @param a first vertex of edge
	 * @param b second vertex of edge
	 * @param c opposing vertex 1
	 * @param d opposing vertex 2
	 * @param tc index of triangle [a,b,c]
	 * @param td index of triangle [a,b,d]
	 * @param collapse_to either a or b if we should collapse to one of those, or -1 if either is acceptable
	 */
	DYNAMICMESH_API bool CanCollapseEdge(int eid, int a, int b, int c, int d, int tc, int td, int& collapse_to) const;

	/**
	 * Resolve vertex constraints for collapsing edge eid=[a,b]. Generally we would
	 * collapse a to b, and set the new position as 0.5*(v_a+v_b). However if a *or* b
	 * are constrained, then we want to keep that vertex and collapse to its position.
	 * This vertex (a or b) will be returned in collapse_to, which is -1 otherwise.
	 * If a *and* b are constrained, then things are complicated (and documented below).
	 * @param eid edge ID
	 * @param a first vertex of edge
	 * @param b second vertex of edge*
	 * @param collapse_to either a or b if we should collapse to one of those, or -1 if either is acceptable
	 */
	DYNAMICMESH_API bool CanCollapseVertex(int eid, int a, int b, int& collapse_to) const;


	/**
	 * @return true if given vertex can't move, or has a projection target
	 */
	inline bool IsVertexPositionConstrained(int VertexID)
	{
		if (Constraints)
		{
			FVertexConstraint vc = Constraints->GetVertexConstraint(VertexID);
			return (!vc.bCanMove || vc.Target != nullptr);
		}
		return false;
	}

	/**
	 * @return constraint for given vertex
	 */
	inline FVertexConstraint GetVertexConstraint(int VertexID)
	{
		if (Constraints)
		{
			return Constraints->GetVertexConstraint(VertexID);
		}
		return FVertexConstraint::Unconstrained();
	}

	/**
	 * @return true if vertex has constraint
	 */
	inline bool GetVertexConstraint(int VertexID, FVertexConstraint& OutConstraint)
	{
		return Constraints &&
			Constraints->GetVertexConstraint(VertexID, OutConstraint);
	}



public:
	//
	// Mesh Change Tracking support
	//

	DYNAMICMESH_API void SetMeshChangeTracker(FDynamicMeshChangeTracker* Tracker);

protected:
	DYNAMICMESH_API virtual void SaveTriangleBeforeModify(int32 TriangleID);
	DYNAMICMESH_API virtual void SaveEdgeBeforeModify(int32 EdgeID);
	DYNAMICMESH_API virtual void SaveVertexTrianglesBeforeModify(int32 VertexID);


protected:
	/*
	 * testing/debug/profiling stuff
	 */
	DYNAMICMESH_API void RuntimeDebugCheck(int EdgeID);
	DYNAMICMESH_API virtual void DoDebugChecks(bool bEndOfPass = false);
	DYNAMICMESH_API void DebugCheckUVSeamConstraints();
	DYNAMICMESH_API void DebugCheckVertexConstraints();

};


} // end namespace UE::Geometry
} // end namespace UE
