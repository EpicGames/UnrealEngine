// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp Reducer

#pragma once

#include "MeshRefinerBase.h"
#include "QuadricError.h"
#include "Util/IndexPriorityQueue.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"

namespace UE
{
namespace Geometry
{

enum class ESimplificationResult
{
	Ok_Collapsed = 0,
	Ignored_CannotCollapse = 1,
	Ignored_EdgeIsFullyConstrained = 2,
	Ignored_EdgeTooLong = 3,
	Ignored_Constrained = 4,
	Ignored_CreatesFlip = 5,
	Failed_OpNotSuccessful = 6,
	Failed_NotAnEdge = 7,
	Failed_IsolatedTriangle = 8,
	Failed_GeometricDeviation = 9,
	Ignored_CreatesTinyTriangle = 10,
	Failed_ConstraintDeviation = 11
};

enum class ERegularizeSimplificationMethod
{
	// Regularize with a distance-from-original-vertex-position term
	// Simplest / fastest method, stable even for very large weights
	Position,
	// Regularize with a distance-from-line-through-vertex-along-normal term.
	// Tends to give slightly better results for small weights; can be less stable for very large weights
	NormalLine
};


template <typename T, typename = void>
struct TQuadricOptionsHelper
{ 
	 struct FOptions {}; 
};

// Partial specialization (kicks in when T::FOptions exists)
template <typename T>
struct TQuadricOptionsHelper<T, std::void_t<typename T::FOptions>>
{
    using FOptions = typename T::FOptions;
};

/**
 * Implementation of Garland & Heckbert Quadric Error Metric (QEM) Triangle Mesh Simplification
 */
template <typename QuadricErrorType>
class TMeshSimplification : public FMeshRefinerBase
{
public:

	typedef QuadricErrorType                       FQuadricErrorType;
	typedef typename FQuadricErrorType::ScalarType RealType;
	typedef TQuadricError<RealType>                FSeamQuadricType;

	using FQuadricOptions = TQuadricOptionsHelper<QuadricErrorType>::FOptions;
	
	enum class ESimplificationCollapseModes
	{
		MinimalQuadricPositionError = 0,
		MinimalExistingVertexError = 1,
		AverageVertexPosition = 2,
	};

	/**
	 * Controls the method used when selecting the position the results from an edge collapse. Note some
	 * of the simpler methods (such as average) may be significantly slower as they result in many more
	 * points that are rejected because the would cause a triangle flip.
	 *
	 * MinimalQuadricPositionError  Try to find position for collapsed vertices that minimizes quadric error.
	 *                              If false we just use midpoints, which is actually significantly slower, because it results
	 *                              in many more points that would cause a triangle flip, which are then rejected.
	 *
	 * MinimalExistingVertexError   Use one of the existing vertex position with smallest error for the collapse point.
	 *
	 * AverageVertexPosition        Use the midpoint of the two vertex positions.
	 */
	ESimplificationCollapseModes CollapseMode = ESimplificationCollapseModes::MinimalQuadricPositionError;

	/** if false, face and vertex quadrics are recomputed in the neighborhood of each collapse, definitely slower but maybe higher quality*/
	bool bRetainQuadricMemory = false;

	/** if true, we try to keep boundary vertices on boundary. 
	* You probably want this if you aren't using seam quadrics.
	*/
	bool bPreserveBoundaryShape = true;

	/** if true, we allow UV and Normal seams to collapse during simplification.*/
	bool bAllowSeamCollapse = true;

	/** Controls whether we disallow creation of triangles with small areas inside edge operations.
	 * This is moderately expensive and in some cases can result in lower-quality meshes. Disabled by default.
	 */
	bool bPreventTinyTriangles = false;

	/**
	 * If true, we prevent collapses that would move edge midpoints away from their constraint target by more than the edge constraint's threshold distance (if any)
	 */
	bool bLimitConstrainedSeamMovement = false;

	/** 
	 * If greater than 0, we will add a regularization term with this weight.
	 * This can help give a better triangulation for example in flat regions where the error metric is under-determined
	 */
	double RegularizeWeight = 0.;

	/** Method to use to regularize, if RegularizeWeight > 0 */
	ERegularizeSimplificationMethod RegularizeMethod = ERegularizeSimplificationMethod::NormalLine;

	/** When using the constraint system, these options will apply to the appropriate boundaries. */
	EEdgeRefineFlags MeshBoundaryConstraint, GroupBoundaryConstraint, MaterialBoundaryConstraint;


	/** Ways to measure geometric error */
	enum class EGeometricErrorCriteria
	{
		/** No geometric error checking */
		None = 0,
		/** Fixed envelope around projection target */
		PredictedPointToProjectionTarget = 1
	};

	/** Geometric error measurement/check to perform before allowing an edge collapse */
	EGeometricErrorCriteria GeometricErrorConstraint = EGeometricErrorCriteria::None;

	/** Tolerance to use in geometric error checking */
	double GeometricErrorTolerance = 0.0;

	/**
	 * If > 0, this is an absolute upper bound on the result triangle count: while the mesh has more
	 * triangles than this, the simplifier bypasses its stopping criteria (geometric error tolerance,
	 * edge-too-long rejection in SimplifyToEdgeLength, and the SimplifyToMaxError loop break) and keeps
	 * collapsing. Once the count drops to this value or below, the configured stopping criteria resume.
	 * Default 0 = stopping criteria always apply (unchanged behavior).
	 *
	 * Useful in combination with SimplifyToTriangleCount + GeometricErrorTolerance: simplification will
	 * always reach this ceiling, then continue toward the SimplifyToTriangleCount target only while
	 * collapses stay within tolerance.
	 */
	int32 MaxResultTriangleCount = 0;

	FQuadricOptions QuadricOptions;

	// Custom scales to apply to *measured* errors and edge lengths. Note this is in effect the inverse of scaling the tolerances / target lengths.
	TFunction<double(const FDynamicMesh3&, int VertexA, int VertexB)> CustomGeometricErrorScaleF;
	TFunction<double(const FDynamicMesh3&, int VertexA, int VertexB)> CustomEdgeLengthScaleF;
	// Custom per-vertex quadric error scaling. Values should be greater than zero.
	TFunction<double(const FDynamicMesh3&, int Vertex)> CustomQuadricErrorScaleF;

	TMeshSimplification(FDynamicMesh3* m) : FMeshRefinerBase(m)
	{
		NormalOverlay = nullptr;
		if (FDynamicMeshAttributeSet* Attributes = m->Attributes())
		{
			NormalOverlay = Attributes->PrimaryNormals();
		}
	}


	/**
	 * Simplify mesh until we reach a specific triangle count, unless geometric error criterion stops
	 * the decimation earlier.
	 * 
	 * Note that because triangles are removed in pairs, the resulting count may be TriangleCount-1.
	 * @param TriangleCount the target triangle count
	 */
	DYNAMICMESH_API virtual void SimplifyToTriangleCount(int TriangleCount);

	/**
	 * Simplify mesh until it has a specific vertex count
	 * @param VertexCount the target vertex count
	 */
	DYNAMICMESH_API virtual void SimplifyToVertexCount(int VertexCount);

	/**
	 * Simplify mesh until no edges smaller than min length remain. This is not a great criteria.
	 * @param MinEdgeLength collapse any edge longer than this
	 */
	DYNAMICMESH_API virtual void SimplifyToEdgeLength(double MinEdgeLength);

	/**
	 * Simplify mesh until the quadric error of an edge collapse exceeds the specified criteria.
	 * @param MaxError collapse an edge if the corresponding quadric error exceeds this 
	 */
	DYNAMICMESH_API virtual void SimplifyToMaxError(double MaxError);

	/**
	 * Maximally collapse mesh in a way that does not change shape at all.
	 * This process does not involve quadric error at all.
	 * @param AngleTolDeg two triangles are considered coplanar if their normals are within this angle tolerance
	 * @param EdgeFilterPredicate only edges that pass this predicate will be considered for collapse. Default all true.
	 */
	DYNAMICMESH_API virtual void SimplifyToMinimalPlanar(
		double CoplanarAngleTolDeg = 0.001,
		TFunctionRef<bool(int32 EdgeID)> EdgeFilterPredicate = [](int32) { return true; }
		);


	/** 
	 * Does N rounds of collapsing edges longer than fMinEdgeLength. Does not use Quadrics or priority queue.
	 * This is a quick way to get triangle count down on huge meshes (eg like marching cubes output). 
	 * @param MinEdgeLength collapse any edge longer than this
	 * @param Rounds number of collapse rounds
	 * @param MeshIsClosedHint if you know the mesh is closed, this pass this true to avoid some precomputes
	 * @param MinTriangleCount halt fast collapse if mesh falls below this triangle count
	 */
	DYNAMICMESH_API virtual void FastCollapsePass(double MinEdgeLength, int Rounds = 1, bool bMeshIsClosedHint = false, uint32 MinTriangleCount = 0);

	

protected:

	TMeshSimplification()		// for subclasses that extend our behavior
	{
	}

	// this just lets us write more concise code
	bool EnableInlineProjection() const { return ProjectionMode == ETargetProjectionMode::Inline; }

	// Returns true while the mesh is still above MaxResultTriangleCount and we should therefore
	// override stopping criteria (geometric error, edge-too-long, max-error loop break) to keep
	// collapsing. When MaxResultTriangleCount is 0 (default) this is always false.
	bool ShouldIgnoreStoppingCriteria() const
	{
		return MaxResultTriangleCount > 0 && Mesh->TriangleCount() > MaxResultTriangleCount;
	}

	float MaxErrorAllowed = FLT_MAX;
	double MinEdgeLength = FMathd::MaxReal;
	int TargetCount = INT_MAX;
	enum class ETargetModes
	{
		TriangleCount = 0,
		VertexCount = 1,
		MinEdgeLength = 2,
		MaxError  = 3
	};
	ETargetModes SimplifyMode = ETargetModes::TriangleCount;



	/** Top-level function that does the simplification */
	DYNAMICMESH_API virtual void DoSimplify();



	// StartEdges() and GetNextEdge() control the iteration over edges that will be refined.
	// Default here is to iterate over entire mesh->
	// Subclasses can override these two functions to restrict the affected edges (eg EdgeLoopRemesher)

	// We are using a modulo-index loop to break symmetry/pathological conditions. 
	uint64 MaxEdgeID = 0;
	virtual int StartEdges() 
	{
		MaxEdgeID = static_cast<uint64>(Mesh->MaxEdgeID());
		return 0;
	}

	virtual int GetNextEdge(int CurEdgeID, bool& bDoneOut) 
	{
		constexpr uint64 ModuloPrime = 4294967311ull;     // choose prime > max uint32, to always be co-prime with MaxEdgeID
		int new_eid = static_cast<int>((static_cast<uint64>(CurEdgeID) + ModuloPrime) % MaxEdgeID);
		bDoneOut = (new_eid == 0);
		return new_eid;
	}


	
	double SeamEdgeWeight =  256.;

	TArray<FQuadricErrorType> vertQuadrics;
	DYNAMICMESH_API virtual void InitializeVertexQuadrics();

	TMap<int, FSeamQuadricType> seamQuadrics;
	DYNAMICMESH_API virtual void InitializeSeamQuadrics();

	TArray<double> triAreas;
	TArray<FVector3d> triNormals; // this array is only populate if needed; i.e. if RegularizeUsesNormals() returns true
	TArray<FQuadricErrorType> triQuadrics;
	TArray<double> AppliedVertexQuadricScales; // needed to properly update scaled vertex quadrics in the memoryless update case
	DYNAMICMESH_API virtual void InitializeTriQuadrics();

	bool RegularizeUsesNormals() const
	{
		return RegularizeWeight > 0 && RegularizeMethod == ERegularizeSimplificationMethod::NormalLine;
	}

	FDynamicMeshNormalOverlay* NormalOverlay;

	FQuadricErrorType ComputeFaceQuadric(const int tid, FVector3d& nface, FVector3d& c, double& Area) const;
	
	// uses pre-computed vertex, face and seam quadrics to construct the edge quadric.
	FQuadricErrorType AssembleEdgeQuadric(const FDynamicMesh3::FEdge& edge) const;

	FQuadricErrorType MergeVertexQuadricsToEdgeQuadric(const FDynamicMesh3::FEdge& edge) const;
		
	void UpdateVertexAttributes(int edgeID, int va, int vb );
	
	// internal class for priority queue
	struct QEdge 
	{
		int eid;
		FQuadricErrorType q;
		FVector3d collapse_pt;

		QEdge() { eid = 0; }

		QEdge(int edge_id, const FQuadricErrorType& qin, const FVector3d& pt) 
		{
			eid = edge_id;
			q = qin;
			collapse_pt = pt;
		}
	};

	TArray<QEdge> EdgeQuadrics;
	FIndexPriorityQueue EdgeQueue;

	struct FEdgeError
	{
		float error;
		int eid;
		bool operator<(const FEdgeError& e2) const
		{
			return error < e2.error;
		}
	};

	DYNAMICMESH_API virtual void InitializeQueue();

	// return point that minimizes quadric error for edge [ea,eb]
	FVector3d OptimalPoint(int eid, const FQuadricErrorType& q, int ea, int eb);
	
	FVector3d GetProjectedPoint(const FVector3d& pos)
	{
		if (EnableInlineProjection() && ProjTarget != nullptr)
		{
			return ProjTarget->Project(pos);
		}
		return pos;
	}


	// update queue weight for each edge in vertex one-ring and rebuild and quadrics necessary 
	DYNAMICMESH_API virtual void UpdateNeighborhood(const FDynamicMesh3::FEdgeCollapseInfo& collapseInfo);


	virtual void Reproject() 
	{
		ProfileBeginProject();
		if (ProjTarget != nullptr && ProjectionMode == ETargetProjectionMode::AfterRefinement)
		{
			FullProjectionPass();
			DoDebugChecks();
		}
		ProfileEndProject();
	}





	bool bHaveBoundary;
	TArray<bool> IsBoundaryVtxCache;
	void Precompute(bool bMeshIsClosed = false);

	inline bool IsBoundaryVertex(int vid) const
	{
		return IsBoundaryVtxCache[vid];
	}

	// Optional additional info to support edge collapses in more configurations
	// i.e. for tri ABC, if edge AB is not constrained, but AC and BC are, 
	// it is *sometimes* safe to collapse AB *if* we also remove constraints from 
	// vertex B and edges B(AC) after the collapse. (As long as B is not otherwise constrained)
	struct FCollapseInfo
	{
		TArray<int32, TInlineAllocator<2>> UnconstrainVIDs;
		TArray<int32, TInlineAllocator<2>> UnconstrainEIDs;
	};

	/**
	* Figure out if we can collapse edge eid=[a,b] under current constraint set
	* Similar to the base class FMeshRefinerBase::CanCollapseEdge, but with
	* simplifier-specific modifications (some more flexibility and restrictions depending on CollapseMode)
	* 
	* @param OutCollapseInfo If non-null, we can allow more edge collapses that require some additional post-collapse updates
	* See the similar base class for description of the other parameters.
	*/
	bool CanCollapseEdge(int edgeID, int a, int b, int c, int d, int t0, int t1, int& collapse_to, FCollapseInfo* OutCollapseInfo = nullptr) const;


	/**
	 * Collapse given edge. 
	 * @param RequireCollapseToVert if >= 0 and after constraints/etc the vertex we will collapse "to" cannot be this vertex, do not collapse
	 */
	ESimplificationResult CollapseEdge(int edgeID, FVector3d vNewPos, FDynamicMesh3::FEdgeCollapseInfo& collapseInfo, int32 RequireKeepVert = -1);

	/**
	* Remove an isolated triangle.
	* @return false if the triangle shares a vertex with another triangle
	*/
	bool RemoveIsolatedTriangle(int tID);


	// subclasses can override this to implement custom behavior...
	DYNAMICMESH_API virtual void OnEdgeCollapse(int edgeID, int va, int vb, const FDynamicMesh3::FEdgeCollapseInfo& collapseInfo);

	// subclasses can override this to implement custom behavior...
	DYNAMICMESH_API virtual void OnRemoveIsolatedTriangle(int tId);

	// Project vertices onto projection target. 
	DYNAMICMESH_API virtual void FullProjectionPass();

	DYNAMICMESH_API virtual void ProjectVertex(int vID, IProjectionTarget* targetIn);

	// used by collapse-edge to get projected position for new vertex
	DYNAMICMESH_API virtual FVector3d GetProjectedCollapsePosition(int vid, const FVector3d& vNewPos);

	DYNAMICMESH_API virtual void ApplyToProjectVertices(const TFunction<void(int)>& apply_f);


	/**
	 * Check if edge collapse would violate geometric error criteria
	 * @param vid first vertex of edge
	 * @param vother other vertex of edge
	 * @param newv new vertex position after collapse
	 * @param tc triangle on one side of edge
	 * @param td triangle on other side of edge
	 */
	bool CheckIfCollapseWithinGeometricTolerance(int vid, int vother, const FVector3d& newv, int tc, int td);

	/**
	 * Check if edge collapse would violate edge constraint tolerance
	 * @param VKeep First edge vertex
	 * @param VRemove Other edge vertex
	 * @param NewPosition New vertex position after collapse
	 * @param TC triangle on one side of edge
	 * @return true if collapsing the edge won't move constrained edge midpoints too far from their projection targets
	 */
	bool CheckIfCollapseWithinEdgeConstraintsTolerance(int VKeep, int VRemove, const FVector3d& NewPosition, int TC);
	


	/*
	 * testing/debug/profiling stuff
	 */
protected:


	//
	// profiling functions, turn on ENABLE_PROFILING to see output in console
	// 
	int COUNT_COLLAPSES;
	int COUNT_ITERATIONS;
	//Stopwatch AllOpsW, SetupW, ProjectW, CollapseW;

	virtual void ProfileBeginPass() 
	{
		if (ENABLE_PROFILING) 
		{
			COUNT_COLLAPSES = 0;
			COUNT_ITERATIONS = 0;
			//AllOpsW = new Stopwatch();
			//SetupW = new Stopwatch();
			//ProjectW = new Stopwatch();
			//CollapseW = new Stopwatch();
		}
	}

	virtual void ProfileEndPass()
	{
		if (ENABLE_PROFILING) 
		{
			//System.Console.WriteLine(string.Format(
			//	"ReducePass: T {0} V {1} collapses {2}  iterations {3}", mesh->TriangleCount, mesh->VertexCount, COUNT_COLLAPSES, COUNT_ITERATIONS
			//));
			//System.Console.WriteLine(string.Format(
			//	"           Timing1: setup {0} ops {1} project {2}", Util.ToSecMilli(SetupW.Elapsed), Util.ToSecMilli(AllOpsW.Elapsed), Util.ToSecMilli(ProjectW.Elapsed)
			//));
		}
	}

	virtual void ProfileBeginOps()
	{
		//if (ENABLE_PROFILING) AllOpsW.Start();
	}
	virtual void ProfileEndOps()
	{
		//if (ENABLE_PROFILING) AllOpsW.Stop();
	}
	virtual void ProfileBeginSetup()
	{
		//if (ENABLE_PROFILING) SetupW.Start();
	}
	virtual void ProfileEndSetup()
	{
		//if (ENABLE_PROFILING) SetupW.Stop();
	}

	virtual void ProfileBeginProject()
	{
		//if (ENABLE_PROFILING) ProjectW.Start();
	}
	virtual void ProfileEndProject() 
	{
		//if (ENABLE_PROFILING) ProjectW.Stop();
	}

	virtual void ProfileBeginCollapse() 
	{
		//if (ENABLE_PROFILING) CollapseW.Start();
	}
	virtual void ProfileEndCollapse() 
	{
		//if (ENABLE_PROFILING) CollapseW.Stop();
	}

private:

	void ApplyScalingAndRegularizationToVertexQuadric(FQuadricErrorType& Q, int32 VID, double AreaSum, FVector3d Normal);

};

// The simplifier
typedef TMeshSimplification< FAttrBasedQuadricErrord >  FAttrMeshSimplification;
typedef TMeshSimplification< FVolPresQuadricErrord >    FVolPresMeshSimplification;
typedef TMeshSimplification< FQuadricErrord >           FQEMSimplification;


} // end namespace UE::Geometry
} // end namespace UE
