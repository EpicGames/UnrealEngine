// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "IndexTypes.h"
#include "MatrixTypes.h"
#include "VectorTypes.h"
#include "Math/UnrealMathUtility.h"
#include "QuadricError.h"
#include "VectorUtil.h"

namespace UE
{
namespace Geometry
{

using namespace UE::Math;

// Corresponds to the additional, attribute-related terms added to the main quadrics (A, b, c)
// Tracking this allows computing the geometry quadric separately from the attribute quadric.
//
template <typename RealType>
struct TQuadricAttributeData
{
	RealType Axx{0}, Axy{0}, Axz{0}, Ayy{0}, Ayz{0}, Azz{0};
	RealType bx{0}, by{0}, bz{0};
	RealType c{0};

	void Add(RealType Weight, const TQuadricAttributeData& Other)
	{
		Axx += Weight * Other.Axx;
		Axy += Weight * Other.Axy;
		Axz += Weight * Other.Axz;
		Ayy += Weight * Other.Ayy;
		Ayz += Weight * Other.Ayz;
		Azz += Weight * Other.Azz;
		bx += Weight * Other.bx;
		by += Weight * Other.by;
		bz += Weight * Other.bz;
		c += Weight * Other.c;
	}

	// Accumulate from wedge data during position quadric construction
	void AccumulateFromWedge(RealType AreaTerm, const TVector<RealType>& G, RealType D)
	{
		Axx += AreaTerm * G.X * G.X;
		Axy += AreaTerm * G.X * G.Y;
		Axz += AreaTerm * G.X * G.Z;
		Ayy += AreaTerm * G.Y * G.Y;
		Ayz += AreaTerm * G.Y * G.Z;
		Azz += AreaTerm * G.Z * G.Z;
		bx += AreaTerm * G.X * D;
		by += AreaTerm * G.Y * D;
		bz += AreaTerm * G.Z * D;
		c += AreaTerm * D * D;
	}

	// Compute the debug quadric correction term (to be subtracted from base evaluation)
	RealType ComputeCorrection(const TVector<RealType>& P) const
	{
		TVector<RealType> AxP(
			Axx * P.X + Axy * P.Y + Axz * P.Z, 
			Axy * P.X + Ayy * P.Y + Ayz * P.Z, 
			Axz * P.X + Ayz * P.Y + Azz * P.Z);

		RealType QuadraticTerm = P.Dot(AxP);
		RealType LinearTerm = RealType(2.) * (bx * P.X + by * P.Y + bz * P.Z);
		RealType ConstantTerm = c;

		return QuadraticTerm + LinearTerm + ConstantTerm;
	}
};

// Empty policy for release builds - all operations compile away to nothing
template <typename RealType>
struct TQuadricNoAttributeData
{
	void Add(RealType, const TQuadricNoAttributeData&) { }
	void AccumulateFromWedge(RealType, const TVector<RealType>&, RealType) { }

	RealType ComputeCorrection(const TVector<RealType>&) const 
	{ 
		ensureMsgf(false, TEXT("No attribute quadric data available. Correction term not available."));
		return RealType(0); 
	}
};

// This needs to be enabled to allow computing the geometry quadric separately from the attribute quadric.
// Functionally not required for the simplification, so disabled by default.
// #define UE_GEOMETRY_QEM_EXTRA_ATTRIBUTE_DATA
#ifdef UE_GEOMETRY_QEM_EXTRA_ATTRIBUTE_DATA
	template<typename RealType>
	using TQuadricAttributePolicy = TQuadricAttributeData<RealType>;
#else
	template<typename RealType>
	using TQuadricAttributePolicy = TQuadricNoAttributeData<RealType>;
#endif

// Configurable attributes with separate topology (wedges)
// Volume preservation
// Plane (Garland/Heckbert) and triangle quadrics (Lindstrom/Turk), including probabilistic versions (Trettner/Kobbelt)
//
template<typename RealType>
class TAttrBasedQuadricErrorV2 : public TQuadricError<RealType>
{
protected:
	using BaseStruct = TQuadricError<RealType>;

	TQuadricAttributePolicy<RealType> AttributeData; // compiles away if UE_GEOMETRY_QEM_EXTRA_ATTRIBUTE_DATA isn't defined

	struct FAttrWedgeQuadricData
	{
		// g and d terms, as described in http://hhoppe.com/newqem.pdf
		// NOTE: they are scaled by the area term for face quadrics and accumulated accordingly
		// for vertex and edge quadrics.
		TVector<RealType> G[3]; 
		TVector<RealType> D;

		FIndex3i          AttrTri;              // attribute layer topology
		RealType          TotalAreaTerm { 0. }; // sum of triangles contributing to wedge, or sum squared
		RealType          AttrWeight    { 0. };

		void Scale(const RealType Wt)
		{
			G[0] *= Wt;
			G[1] *= Wt;
			G[2] *= Wt;
			D    *= Wt;
		}
		
		void AccumulateQuadricData(const FAttrWedgeQuadricData& FaceQuadricData)
		{
			ensure(FaceQuadricData.TotalAreaTerm >= 0.);

			G[0] += FaceQuadricData.G[0];
			G[1] += FaceQuadricData.G[1];
			G[2] += FaceQuadricData.G[2];
			D    += FaceQuadricData.D;
			TotalAreaTerm += FaceQuadricData.TotalAreaTerm;
			AttrWeight = FaceQuadricData.AttrWeight;
		}

		void RemoveQuadricData(const FAttrWedgeQuadricData& FaceQuadricData)
		{
			G[0] -= FaceQuadricData.G[0];
			G[1] -= FaceQuadricData.G[1];
			G[2] -= FaceQuadricData.G[2];
			D    -= FaceQuadricData.D;
			TotalAreaTerm -= FaceQuadricData.TotalAreaTerm;
			AttrWeight = FaceQuadricData.AttrWeight;

			TotalAreaTerm = FMath::Max(0., TotalAreaTerm);
		}

		void AccumulateToPositionQuadric(TAttrBasedQuadricErrorV2& FaceQuadric) const
		{
			const RealType AreaTerm = FaceQuadric.TotalAreaTerm;

			ensure(AreaTerm >= 0.);

			for (int i=0; i<3; ++i)
			{
				FaceQuadric.Axx += AreaTerm * G[i].X * G[i].X;
				FaceQuadric.Axy += AreaTerm * G[i].X * G[i].Y;
				FaceQuadric.Axz += AreaTerm * G[i].X * G[i].Z;

				FaceQuadric.Ayy += AreaTerm * G[i].Y * G[i].Y;
				FaceQuadric.Ayz += AreaTerm * G[i].Y * G[i].Z;

				FaceQuadric.Azz += AreaTerm * G[i].Z * G[i].Z;

				FaceQuadric.bx  += AreaTerm * G[i].X * D[i];
				FaceQuadric.by  += AreaTerm * G[i].Y * D[i];
				FaceQuadric.bz  += AreaTerm * G[i].Z * D[i];

				FaceQuadric.c   += AreaTerm * D[i] * D[i];

				// Accumulate debug data
				FaceQuadric.AttributeData.AccumulateFromWedge(AreaTerm, G[i], D[i]);
			}
		}
	};

	constexpr static uint32 InlineAttrElements = 6; 
	
	// 6 enough to hold Normals, Tangents (2x), VertexColor, 2 x UV
	TArray<FAttrWedgeQuadricData, TInlineAllocator<InlineAttrElements>> AttrWedgeQuadricDatas;
	TArray<int, TInlineAllocator<InlineAttrElements>>                   WedgeCount;

public:

	using FAttrArray = TArray<TVector<RealType>, TInlineAllocator<InlineAttrElements>>;

	enum class EQuadricVariant
	{
		PlaneQuadric,      // classic Garland-Heckbert squared distance to plane
		TriangleQuadric    // Lindstrom-Turk volumetric error 
	};

	// Helper class to register attributes to the quadric on construction
	class FScopedAttributeDataBuilder
	{
	private:
		void CalcGandD(const FVector3f& W0, const FVector3f& W1, const FVector3f& W2, TVector<RealType> G[3], TVector<RealType>& D) const
		{
			if (IsValid) 
			{
				for (int i = 0; i < 3; ++i)
				{
					const FVector3d Attr = FVector3d(W0[i], W1[i], W2[i]);
					const FVector3d DataVec(Attr[2] - Attr[0], Attr[1] - Attr[0], 0.);
					G[i] = CoBasisMatrix * DataVec;
					D[i] = Attr[0] - P0.Dot(G[i]);
				}
			}
			else
			{
				for (int i = 0; i < 3; ++i)
				{
					G[i] = TVector<RealType>::Zero();
					D[i] = (W0[i] + W1[i] + W2[i]) / 3.; 
				}
			}
		}

	public:
		FScopedAttributeDataBuilder(
			TAttrBasedQuadricErrorV2& InQuadric,
			const TVector<RealType>& InP0, 
		    const TVector<RealType>& InP1, 
			const TVector<RealType>& InP2,
			const TVector<RealType>& InN,
			const RealType ScaleCorrection)
			: Quadric(InQuadric), P0(InP0)
		{
			// position quadric scales with TotalAreaTerm * GeomerticError^2
			// attribute scales with TotalAreaTerm only
			// GeometricError scales squared in both cases, the attribute term also scales quadratically.

			AttributeScaling = ScaleCorrection;

			const TMatrix3<RealType> BasisMatrix(InP2-InP0, InP1-InP0, InN, true);
			const RealType Det = BasisMatrix.Determinant();
			if (FMath::Abs(Det) > 1.e-8)
			{
				CoBasisMatrix = BasisMatrix.Inverse();
				IsValid = true;
			}
		}

		void AddWedgeAttribute(const FIndex3i& AttrTri, const FVector3f& W0, const FVector3f& W1, const FVector3f& W2, const RealType AttrWeight)
		{
			FAttrWedgeQuadricData AttrWedgeQuadricData;
			CalcGandD(W0, W1, W2, AttrWedgeQuadricData.G, AttrWedgeQuadricData.D);

			AttrWedgeQuadricData.TotalAreaTerm = Quadric.TotalAreaTerm;
			AttrWedgeQuadricData.AttrWeight = AttrWeight * AttributeScaling;

			AttrWedgeQuadricData.G[0] *= AttrWeight * AttributeScaling;
			AttrWedgeQuadricData.G[1] *= AttrWeight * AttributeScaling;
			AttrWedgeQuadricData.G[2] *= AttrWeight * AttributeScaling;
			AttrWedgeQuadricData.D    *= AttrWeight * AttributeScaling;

			AttrWedgeQuadricData.AccumulateToPositionQuadric(Quadric);

			AttrWedgeQuadricData.G[0] *= Quadric.TotalAreaTerm;
			AttrWedgeQuadricData.G[1] *= Quadric.TotalAreaTerm;
			AttrWedgeQuadricData.G[2] *= Quadric.TotalAreaTerm;
			AttrWedgeQuadricData.D    *= Quadric.TotalAreaTerm;

			AttrWedgeQuadricData.AttrTri = AttrTri;
			Quadric.AttrWedgeQuadricDatas.Add(AttrWedgeQuadricData);
			Quadric.WedgeCount.Add(1);
		}

		void AddMissingWedgeAttribute()
		{
			Quadric.WedgeCount.Add(0);
		}

	private:
		TAttrBasedQuadricErrorV2& Quadric; // face quadric
		const TVector<RealType>& P0; 
		FMatrix3d CoBasisMatrix;
		bool IsValid { false };
		RealType AttributeScaling { 1. };
	};

	struct FOptions
	{
		EQuadricVariant QuadricVariant { EQuadricVariant::PlaneQuadric };

		// When positive, will trigger the probabilistic quadrics, leading to regularized
		// results that are less sensitive to local noise and improve behavior in flat regions.
		//
		RealType SigmaN                { 0. }; 

		// Used to rebalance the optimization weight for geometry and attributes
		// when scaling objects. The behavior of the simplifier is not scale
		// invariant. Scaling an object by X will impact the geometry terms
		// X^2 times more than it will impact the attributes.
		// 
		// Setting this value to X, should then give the same behavior as if the
		// the object was scaled down by a factor of X and rescaled back up after simplification.
		//
		// It is also applied to edge quadric scaling which is weighted by edge-length instead of area. 
		// 
		RealType ScaleCorrection { 1. };

		// currently mimicking FQuadricSimplifierMeshReduction, see ReduceMeshDescription.
		//
		RealType NormalAttributeWeight   { 16.0 };
		RealType TangentAttributeWeight  { 0.1 };
		RealType ColorAttributeWeight    { 0.1 };
		RealType TexCoordAttributeWeight { 0.5 };
		RealType WeightLayerWeight       { 0.0 };
	};

	// empty quadric
	TAttrBasedQuadricErrorV2() 
		: BaseStruct()
	{
	}

	// initialize a face quadric
	TAttrBasedQuadricErrorV2(const TVector<RealType>& P0, 
		                     const TVector<RealType>& P1, 
	 						 const TVector<RealType>& P2,
							 const FOptions& Options)
							 : BaseStruct()
	{
		switch (Options.QuadricVariant)
		{
			case EQuadricVariant::PlaneQuadric:
				MakeProbabilisticPlaneQuadric(P0, P1, P2, Options.SigmaN );
				break;
			case EQuadricVariant::TriangleQuadric:
				MakeProbabilisticTriangleQuadric(P0, P1, P2, Options.SigmaN );
				break;
			default:
				ensureMsgf(false, TEXT("Invalid quadric variant. Must be either plane or triangle quadric."));
		}

		const TVector<RealType> Normal = VectorUtil::Normal(P0, P1, P2);
		GVol = Normal;
		DVol = ( Normal | P0 );
	}

	// this should never be called. addition of vertex quadrics is not supported because it doesn't work for attribute wedges
	TAttrBasedQuadricErrorV2(const TAttrBasedQuadricErrorV2& Q0, const TAttrBasedQuadricErrorV2& Q1)
		: BaseStruct(Q0, Q1)
	{
		check(false);
		Area = Q0.Area + Q1.Area;
		GVol = Q0.GVol + Q1.GVol;
		DVol = Q0.DVol + Q1.DVol;
		TotalAreaTerm = Q0.TotalAreaTerm + Q1.TotalAreaTerm;
	}

	// initialize to an edge quadric given the two vertex quadrics at the end points of the edge
	// and (up to two) face quadrics corresponding to the overlap of both
	// 
	// the local edge indices correspond to the local edge index [0..3) of the shared edge within A and B respectively.
	// 
	TAttrBasedQuadricErrorV2(const TAttrBasedQuadricErrorV2& VertexQuadric0, 
		                     const TAttrBasedQuadricErrorV2& VertexQuadric1,
							 const TAttrBasedQuadricErrorV2* TriangleQuadricA,
							 const TAttrBasedQuadricErrorV2* TriangleQuadricB,
							 const int LocalEdgeIndexA,
							 const int LocalEdgeIndexB,
							 const RealType RemoveScale)

		: BaseStruct(VertexQuadric0, VertexQuadric1)
	{
		Area = VertexQuadric0.Area + VertexQuadric1.Area;
		GVol = VertexQuadric0.GVol + VertexQuadric1.GVol;
		DVol = VertexQuadric0.DVol + VertexQuadric1.DVol;
		TotalAreaTerm = VertexQuadric0.TotalAreaTerm + VertexQuadric1.TotalAreaTerm;
 
		checkSlow(VertexQuadric0.WedgeCount.Num() == VertexQuadric1.WedgeCount.Num());

		bCanCollapse = MergeVertexQuadricsToEdgeQuadric(VertexQuadric0, VertexQuadric1, TriangleQuadricA, TriangleQuadricB, LocalEdgeIndexA, LocalEdgeIndexB, RemoveScale);
		checkSlow(!bCanCollapse || (WedgeCount.Num() == VertexQuadric0.WedgeCount.Num()));

		AttributeData.Add(1.0, VertexQuadric0.AttributeData);
		AttributeData.Add(1.0, VertexQuadric1.AttributeData);
	}

	inline static TAttrBasedQuadricErrorV2 Zero() { return TAttrBasedQuadricErrorV2(); };

	// If this is a vertex, accumulate a face quadric on to it.
	// \param CornerIdx local vertex index [0..3) of the center vertex within the triangle of the face quadric 
	// 
	void AccumulateFaceQuadric(const int CornerIdx, RealType TriArea, RealType Weight, const TAttrBasedQuadricErrorV2& FaceQuadric);
	
	// If this is a vertex, remove a face quadric from it (undoing accumulate face quadric)
	// \param CornerIdx local vertex index [0..3) of the center vertex within the triangle of the face quadric 
	//
	void RemoveFaceQuadric(const int CornerIdx, RealType TriArea, RealType RemoveScale, const TAttrBasedQuadricErrorV2& FaceQuadric);
	
	void Scale(RealType Wt)
	{
		BaseStruct::Scale(Wt);

		for (FAttrWedgeQuadricData& AttrWedgeQuadricData : AttrWedgeQuadricDatas)
		{
			AttrWedgeQuadricData.Scale(Wt);
		}
	}
	
	/**
	* The optimal point minimizing the quadric error with respect to a volume conserving constraint
	*/
	bool OptimalPoint(UE::Math::TVector<RealType>& OutResult, RealType MinThresh = 1000.0*TMathUtil<RealType>::Epsilon) const;
	
	// Scale-correct attributes 
	FAttrArray ComputeAttributes(const TVector<RealType>& P) const;

	// combination of geometry and attribute quadrics
	RealType Evaluate(const TVector<RealType>& P) const
	{
		FAttrArray WedgeAttr = ComputeScaledAttributes(P);
		return Evaluate(P, WedgeAttr);
	}

	// geometry quadrics only
	RealType EvaluateGeometryOnly(const TVector<RealType>& P) const;
	
	// For each attribute, checks whether the area of all wedges adds up to the total area of the quadric.
	bool ValidateArea() const;

	// actual area, not the scaling term
	RealType GetArea() const
	{
		return Area;
	}

	// is this a valid edge-collapse quadric?
	bool CanCollapse() const { return bCanCollapse; }
	
	int GetAttributeCount() const 
	{ 
		return WedgeCount.Num(); 
	}

	int GetWedgeCount(int AttributeIndex) const
	{
		return WedgeCount[AttributeIndex];
	}
 
	bool IsWedgeActive(int WedgeIndex) const
	{
		return AttrWedgeQuadricDatas[WedgeIndex].TotalAreaTerm > 0.;
	}

	const FIndex3i& GetWedgeAttributeIndex(int WedgeIndex) const
	{
		return AttrWedgeQuadricDatas[WedgeIndex].AttrTri;
	}

protected:

	template <typename SolverType>
	bool OptimalPointImpl(TVector<RealType>& OutResult) const;
	
	FAttrArray ComputeScaledAttributes(const TVector<RealType>& P) const;

	RealType Evaluate(const TVector<RealType>& P, const FAttrArray& ScaledWedgeAttrs) const;

	bool MergeVertexQuadricsToEdgeQuadric(const TAttrBasedQuadricErrorV2& VertexQuadric0, 
		const TAttrBasedQuadricErrorV2& VertexQuadric1,
		const TAttrBasedQuadricErrorV2* TriangleQuadricA,
		const TAttrBasedQuadricErrorV2* TriangleQuadricB,
		const int LocalEdgeIndexA,
		const int LocalEdgeIndexB,
		const RealType RemoveScale);

		// Set A to rank-1 matrix V \otimes V = V V^T 
	void AAddOuterProduct(const TVector<RealType>& V)
	{
		BaseStruct::Axx += V.X * V.X;
		BaseStruct::Axy += V.X * V.Y;
		BaseStruct::Axz += V.X * V.Z;
		BaseStruct::Ayy += V.Y * V.Y;
		BaseStruct::Ayz += V.Y * V.Z;
		BaseStruct::Azz += V.Z * V.Z;
	}

    void AAddCrossProductSquaredTranspose(const TVector<RealType>& V)
    {
		const RealType V0Sqr = V.X * V.X;
		const RealType V1Sqr = V.Y * V.Y;
		const RealType V2Sqr = V.Z * V.Z;

		BaseStruct::Axx +=  V1Sqr + V2Sqr;
        BaseStruct::Ayy +=  V0Sqr + V2Sqr;
        BaseStruct::Azz +=  V0Sqr + V1Sqr;
        
		BaseStruct::Axy -= V.X * V.Y;
		BaseStruct::Axz -= V.X * V.Z;
		BaseStruct::Ayz -= V.Y * V.Z;
    }

    void AScale(const RealType Alpha)
    {
		BaseStruct::Axx *= Alpha;
		BaseStruct::Ayy *= Alpha;
		BaseStruct::Azz *= Alpha;
        
		BaseStruct::Axy *= Alpha;
		BaseStruct::Axz *= Alpha;
		BaseStruct::Ayz *= Alpha;
    }

	void MakeProbabilisticPlaneQuadric(
		const TVector<RealType>& P0, 
		const TVector<RealType>& P1, 
		const TVector<RealType>& P2,
		const RealType SigmaN);

	void MakeTriangleQuadric(
		const TVector<RealType>& P0, 
		const TVector<RealType>& P1, 
		const TVector<RealType>& P2);
    	
    void MakeProbabilisticTriangleQuadric(
		const TVector<RealType>& P0, 
		const TVector<RealType>& P1,
		const TVector<RealType>& P2,
		const RealType SigmaN);

	// See: http://hhoppe.com/newqem.pdf or https://www.cc.gatech.edu/~turk/my_papers/memless_vis98.pdf
	// volume constraint: dot(g_vol,x) = DVol
	// 
	// The volume of the tetrahedron spanned by the base triangle (v0, v1, v2) and x is 1/6 * det(v1-v0 v2-v0 x-v0) = 1/6 dot((v1-v0) ^ (v2-v0), x-v0)
	// Summed over all triangles and setting to 0: dot(sum_t N, x) = sum(dot(N, v0)) with N the scaled normal (v1-v0) ^ (v2-v0)
	// 
	TVector<RealType> GVol { 0. };
	RealType          DVol { 0. };
	RealType          Area  { 0. };         // accumulated area, this is always used for scaling the volume constraint.
	RealType          TotalAreaTerm { 0. }; // accumulated area or area^2, depending on which variant is used. This is used for scaling the quadric.
	bool              bCanCollapse { false };
};

template <typename RealType>
inline bool TAttrBasedQuadricErrorV2<RealType>::MergeVertexQuadricsToEdgeQuadric(
	const TAttrBasedQuadricErrorV2& VertexQuadric0,
	const TAttrBasedQuadricErrorV2& VertexQuadric1,
	const TAttrBasedQuadricErrorV2* TriangleQuadricA,
	const TAttrBasedQuadricErrorV2* TriangleQuadricB,
	const int LocalEdgeIndexA,
	const int LocalEdgeIndexB,
	const RealType RemoveScale)
{	
	WedgeCount.Init(0, VertexQuadric0.WedgeCount.Num());
	checkSlow(VertexQuadric0.WedgeCount.Num() == VertexQuadric1.WedgeCount.Num());

	// Walk through WedgeCount

	int Src0WedgeBegin = 0;
	int Src1WedgeBegin = 0;
	int TriAWedgeBegin = 0;
	int TriBWedgeBegin = 0;

	int MergedWedgeBegin = 0;

	const int NextLocalEdgeIndexA = (LocalEdgeIndexA + 1) % 3; 
	const int NextLocalEdgeIndexB = (LocalEdgeIndexB + 1) % 3; 

	TArray<FAttrWedgeQuadricData, TInlineAllocator<InlineAttrElements>> MergedAttrWedgeQuadricDatas;
	TArray<int, TInlineAllocator<InlineAttrElements>> MergedWedgeCount;

	// loop over all attributes

	// check whether a wedge on one vertex would be merged with two distinct wedges on the other vertex.
	// this could happen on bowtie-like topology of the attributes. we can't handle this and don't 
	// allow the collapse.
	bool CheckFailed = false;

	for (int i=0; i<VertexQuadric0.WedgeCount.Num(); ++i)
	{
		const int NSrc0 = VertexQuadric0.WedgeCount[i]; 
		const int NSrc1 = VertexQuadric1.WedgeCount[i]; 

		const int NTriA = TriangleQuadricA ? TriangleQuadricA->WedgeCount[i] : 0; 
		const int NTriB = TriangleQuadricB ? TriangleQuadricB->WedgeCount[i] : 0; 
		
		// copy NDst over
		for (int j=0; j<NSrc0; ++j)
		{
			MergedAttrWedgeQuadricDatas.Add(VertexQuadric0.AttrWedgeQuadricDatas[Src0WedgeBegin + j]);
		}
		int NMerged = NSrc0;

		// for all source wedges, search the ones in the merged list
		for (int j=0; j<NSrc1; ++j)
		{
			// the element ID we associate the wedge with
			const FIndex3i& AttrTri1 = VertexQuadric1.AttrWedgeQuadricDatas[Src1WedgeBegin + j].AttrTri;
			checkSlow(AttrTri1.B == IndexConstants::InvalidID && AttrTri1.C == IndexConstants::InvalidID);
			const int EID1 = AttrTri1.A;

			int MergeIdx = IndexConstants::InvalidID;
			for (int k=0; k<NMerged; ++k)
			{
				const FIndex3i& AttrTri0 = MergedAttrWedgeQuadricDatas[MergedWedgeBegin + k].AttrTri;
				checkSlow(AttrTri0.C == IndexConstants::InvalidID);
				const int EID0 = AttrTri0.A;

				if (NTriA > 0)
				{
					checkSlow(NTriA == 1);
					const FIndex3i AttrTriA = TriangleQuadricA->AttrWedgeQuadricDatas[TriAWedgeBegin].AttrTri;

					const int EIDA_0 = AttrTriA[LocalEdgeIndexA];
					const int EIDA_1 = AttrTriA[NextLocalEdgeIndexA];

					if ((EIDA_0 == EID0 && EIDA_1 == EID1) || (EIDA_0 == EID1 && EIDA_1 == EID0))
					{
						if (!(MergeIdx == IndexConstants::InvalidID || MergeIdx == MergedWedgeBegin + k)) {
							CheckFailed = true;
						}
						MergeIdx = MergedWedgeBegin + k;
					}
				}

				if (NTriB > 0)
				{
					checkSlow(NTriB == 1);
					const FIndex3i AttrTriB = TriangleQuadricB->AttrWedgeQuadricDatas[TriBWedgeBegin].AttrTri;

					const int EIDB_0 = AttrTriB[LocalEdgeIndexB];
					const int EIDB_1 = AttrTriB[NextLocalEdgeIndexB];

					if ((EIDB_0 == EID0 && EIDB_1 == EID1) || (EIDB_0 == EID1 && EIDB_1 == EID0))
					{
						if (!(MergeIdx == IndexConstants::InvalidID || MergeIdx == MergedWedgeBegin + k)) {
							CheckFailed = true;
						}
						MergeIdx = MergedWedgeBegin + k;
					}
				}
			}
			
			const int EID = VertexQuadric1.AttrWedgeQuadricDatas[Src1WedgeBegin + j].AttrTri[0];
			if (MergeIdx == IndexConstants::InvalidID)
			{
				// add the entry
				MergeIdx = MergedAttrWedgeQuadricDatas.AddZeroed();
				MergedAttrWedgeQuadricDatas[MergeIdx].AttrTri = FIndex3i( EID, IndexConstants::InvalidID, IndexConstants::InvalidID );
				NMerged++;
			} 
			else
			{
				MergedAttrWedgeQuadricDatas[MergeIdx].AttrTri[1] = EID;
			} 

			// accumulate the whole wedge (these are vertex attribute quadrics, so already weighted by area)
			MergedAttrWedgeQuadricDatas[MergeIdx].AccumulateQuadricData(VertexQuadric1.AttrWedgeQuadricDatas[Src1WedgeBegin + j]);

		} // end walking over all attributes wedges from vertex1 for this attribute

		MergedWedgeBegin += NMerged;
		MergedWedgeCount.Add(NMerged);

		Src0WedgeBegin += NSrc0;
		Src1WedgeBegin += NSrc1;
		TriAWedgeBegin += NTriA;
		TriBWedgeBegin += NTriB;
		
	} // end walking over all attribute slots

	AttrWedgeQuadricDatas = MoveTemp(MergedAttrWedgeQuadricDatas);
	WedgeCount = MoveTemp(MergedWedgeCount);

	return !CheckFailed;
}

template <typename RealType>
inline void TAttrBasedQuadricErrorV2<RealType>::AccumulateFaceQuadric(
	const int CornerIdx, RealType TriArea, RealType Weight, const TAttrBasedQuadricErrorV2& FaceQuadric)
{
	ensure(TriArea > -UE_DOUBLE_KINDA_SMALL_NUMBER);

	GVol += TriArea * FaceQuadric.GVol;
	DVol += TriArea * FaceQuadric.DVol;
	Area += TriArea;
	TotalAreaTerm += FaceQuadric.TotalAreaTerm;

	BaseStruct::Add(Weight, FaceQuadric);
	AttributeData.Add(Weight, FaceQuadric.AttributeData);

	if (WedgeCount.IsEmpty())
	{
		WedgeCount.Init(0, FaceQuadric.WedgeCount.Num());
	}

	int SrcWedgeBegin = 0;
	int DstWedgeBegin = 0;
	int MergedWedgeBegin = 0;

	TArray<FAttrWedgeQuadricData, TInlineAllocator<InlineAttrElements>> MergedAttrWedgeQuadricDatas;
	TArray<int, TInlineAllocator<InlineAttrElements>> MergedWedgeCount;

	// loop over all attributes
	for (int i=0; i<FaceQuadric.WedgeCount.Num(); ++i)
	{
		const int NSrc = FaceQuadric.WedgeCount[i]; // we have N wedges for this attribute in Other
		const int NDst = WedgeCount[i];     
		
		check(NSrc < 2); // Other should be a tri quadric and not have multiple wedges per attribute
		
		// copy NDst over
		for (int j=0; j<NDst; ++j)
		{
			MergedAttrWedgeQuadricDatas.Add(AttrWedgeQuadricDatas[DstWedgeBegin + j]);
		}
		int NMerged = NDst;

		// for all source wedges, search the ones in the merged list
		for (int j=0; j<NSrc; ++j) //-V1008
		{
			// the element ID we associate the wedge with
			int EID = FaceQuadric.AttrWedgeQuadricDatas[SrcWedgeBegin + j].AttrTri[CornerIdx];

			int MergeIdx = IndexConstants::InvalidID;
			for (int k=0; k<NMerged; ++k)
			{
				const FIndex3i AttrTri = MergedAttrWedgeQuadricDatas[MergedWedgeBegin + k].AttrTri;
				check(AttrTri.C == IndexConstants::InvalidID);
				if (AttrTri.A == EID || AttrTri.B == EID)
				{
					// found the wedge -> accumulate
					MergeIdx = MergedWedgeBegin + k;
					break;
				}
			}
			if (MergeIdx == IndexConstants::InvalidID)
			{
				// add the entry
				MergeIdx = MergedAttrWedgeQuadricDatas.AddZeroed();
				NMerged++;
				MergedAttrWedgeQuadricDatas[MergeIdx].AttrTri = FIndex3i(EID, IndexConstants::InvalidID, IndexConstants::InvalidID);
			}
			
			MergedAttrWedgeQuadricDatas[MergeIdx].AccumulateQuadricData(FaceQuadric.AttrWedgeQuadricDatas[SrcWedgeBegin + j]);
		}

		MergedWedgeBegin += NMerged;
		MergedWedgeCount.Add(NMerged);
		check(MergedAttrWedgeQuadricDatas.Num() == MergedWedgeBegin);

		DstWedgeBegin += NDst;
		SrcWedgeBegin += NSrc;
	}

#if !UE_BUILD_SHIPPING
	{
		// Sanity check
		check(MergedWedgeCount.Num() == FaceQuadric.WedgeCount.Num());

		int MergedWedgeSum = 0;
		for (int N : MergedWedgeCount)
		{
			MergedWedgeSum += N;
		}
		check(MergedWedgeSum == MergedAttrWedgeQuadricDatas.Num());
	}
#endif

	AttrWedgeQuadricDatas = MoveTemp(MergedAttrWedgeQuadricDatas);
	WedgeCount = MoveTemp(MergedWedgeCount);
}

template <typename RealType>
inline void TAttrBasedQuadricErrorV2<RealType>::RemoveFaceQuadric(
	const int CornerIdx, RealType TriArea, RealType RemoveScale, const TAttrBasedQuadricErrorV2& FaceQuadric)
{
	ensure(TriArea >= -UE_DOUBLE_KINDA_SMALL_NUMBER);
	GVol -= TriArea * FaceQuadric.GVol;
	DVol -= TriArea * FaceQuadric.DVol;
	Area -= TriArea;
	
	TotalAreaTerm -= FaceQuadric.TotalAreaTerm;

	// the face quadric contains the position and the attribute terms in A, b, c. These correspond to the 
	// accumulation of all the wedges. The B and d terms are per-wedge and aggregated below.
	BaseStruct::Add(-RemoveScale, FaceQuadric);
	AttributeData.Add(-RemoveScale, FaceQuadric.AttributeData);

	int SrcWedgeBegin = 0;
	int DstWedgeBegin = 0;
	
	check(FaceQuadric.WedgeCount.Num() == WedgeCount.Num());

	// loop over all attributes, each attribute has a separate list of wedges
	for (int i=0; i<FaceQuadric.WedgeCount.Num(); ++i)
	{
		const int NSrc = FaceQuadric.WedgeCount[i]; // we have N wedges for this attribute in Other
		const int NDst = WedgeCount[i];     
		
		checkSlow(NSrc < 2); // should be a face quadric and not have multiple wedges per attribute
		
		bool Found = false;

		// for all source wedges, search the ones in the merged list
		for (int j=0; j<NSrc; ++j) //-V1008
		{
			// the element ID we associate the wedge with
			const int EID = FaceQuadric.AttrWedgeQuadricDatas[SrcWedgeBegin + j].AttrTri[CornerIdx];

			for (int k=0; k<NDst; ++k)
			{
				const FIndex3i AttrTri = AttrWedgeQuadricDatas[DstWedgeBegin + k].AttrTri;
				checkSlow(AttrTri.C == IndexConstants::InvalidID);
				
				if (AttrTri.A == EID || AttrTri.B == EID)
				{
					// found the wedge -> remove
					Found = true;

					AttrWedgeQuadricDatas[DstWedgeBegin + k].RemoveQuadricData(FaceQuadric.AttrWedgeQuadricDatas[SrcWedgeBegin + j]);
					break;
				}
			}
			checkSlow(Found);
		}

		DstWedgeBegin += NDst;
		SrcWedgeBegin += NSrc;
	}
}

template <typename RealType>
inline void TAttrBasedQuadricErrorV2<RealType>::MakeProbabilisticPlaneQuadric(
	const TVector<RealType>& P0, 
	const TVector<RealType>& P1, 
	const TVector<RealType>& P2,
	const RealType SigmaN)
{
	const TVector<RealType> Normal { VectorUtil::NormalArea(P0, P1, P2, Area) };
	const TVector<RealType> P = (P0 + P1 + P2) * (1./3.);

	TotalAreaTerm = Area;
	
	BaseStruct::Axx = TotalAreaTerm * Normal.X * Normal.X;
	BaseStruct::Axy = TotalAreaTerm * Normal.X * Normal.Y;
	BaseStruct::Axz = TotalAreaTerm * Normal.X * Normal.Z;
	BaseStruct::Ayy = TotalAreaTerm * Normal.Y * Normal.Y;
	BaseStruct::Ayz = TotalAreaTerm * Normal.Y * Normal.Z;
	BaseStruct::Azz = TotalAreaTerm * Normal.Z * Normal.Z;
	
	RealType dotNP = Normal | P;

	BaseStruct::bx = TotalAreaTerm * -dotNP * Normal.X; 
	BaseStruct::by = TotalAreaTerm * -dotNP * Normal.Y; 
	BaseStruct::bz = TotalAreaTerm * -dotNP * Normal.Z; 

	BaseStruct::c  = TotalAreaTerm * dotNP * dotNP;

	const RealType SigmaN2 = SigmaN * SigmaN;
	if (SigmaN > 0.)
	{
		BaseStruct::Axx += TotalAreaTerm * SigmaN2;
		BaseStruct::Ayy += TotalAreaTerm * SigmaN2;
		BaseStruct::Azz += TotalAreaTerm * SigmaN2;

		BaseStruct::bx -= TotalAreaTerm * SigmaN2 * P.X;
		BaseStruct::by -= TotalAreaTerm * SigmaN2 * P.Y;
		BaseStruct::bz -= TotalAreaTerm * SigmaN2 * P.Z;

		BaseStruct::c += TotalAreaTerm * SigmaN2 * P.Dot(P);
	}
}

template <typename RealType>
inline void TAttrBasedQuadricErrorV2<RealType>::MakeTriangleQuadric(
	const TVector<RealType>& P0, 
	const TVector<RealType>& P1, 
	const TVector<RealType>& P2)
{
	Area = VectorUtil::Area(P0, P1, P2);
	TotalAreaTerm = Area * Area;

	const TVector<RealType> P0xP1 = P0 ^ P1;
	const TVector<RealType> P1xP2 = P1 ^ P2;
	const TVector<RealType> P2xP0 = P2 ^ P0;

	const TVector<RealType> S = P0xP1 + P1xP2 + P2xP0;
	const RealType Det = P0xP1.Dot(P2); // scalar triple product

	AAddOuterProduct(S);
	BaseStruct::bx = -S.X * Det;
	BaseStruct::by = -S.Y * Det;
	BaseStruct::bz = -S.Z * Det;

	BaseStruct::c = Det * Det;
}

template <typename RealType>
inline void TAttrBasedQuadricErrorV2<RealType>::MakeProbabilisticTriangleQuadric(
	const TVector<RealType>& P0,
	const TVector<RealType>& P1,
	const TVector<RealType>& P2,
	const RealType Sigma)
{
	Area = VectorUtil::Area(P0, P1, P2);
	TotalAreaTerm = Area * Area;

	const RealType SigmaSqr = Sigma * Sigma;

	const TVector<RealType> P0xP1 = P0 ^ P1;
	const TVector<RealType> P1xP2 = P1 ^ P2;
	const TVector<RealType> P2xP0 = P2 ^ P0;

	const RealType Det = P0xP1.Dot(P2); // scalar triple product

	const TVector<RealType> S = P0xP1 + P1xP2 + P2xP0;

	const TVector<RealType> P1P0 = P0 - P1;
	const TVector<RealType> P2P1 = P1 - P2;
	const TVector<RealType> P0P2 = P2 - P0;

	AAddCrossProductSquaredTranspose(P1P0);
	AAddCrossProductSquaredTranspose(P2P1);
	AAddCrossProductSquaredTranspose(P0P2);
	AScale(SigmaSqr);

	AAddOuterProduct(S);

	const RealType SigmaQ = SigmaSqr * SigmaSqr;

	BaseStruct::Axx += 6. * SigmaQ;
	BaseStruct::Ayy += 6. * SigmaQ;
	BaseStruct::Azz += 6. * SigmaQ;
	
	TVector<RealType> B = S * Det
		- ((P1P0 ^ P0xP1) + (P2P1 ^ P1xP2) + (P0P2 ^ P2xP0)) * SigmaSqr
		+ (P0 + P1 + P2) * (2.0 * SigmaQ);

	BaseStruct::bx = -B.X;
	BaseStruct::by = -B.Y;
	BaseStruct::bz = -B.Z;

	BaseStruct::c = Det * Det
		+ SigmaSqr * (P0xP1.Dot(P0xP1) + P1xP2.Dot(P1xP2) + P2xP0.Dot(P2xP0))
		+ 2.0 * SigmaQ * (P0.Dot(P0) + P1.Dot(P1) + P2.Dot(P2))
		+ 6.0 * SigmaQ * SigmaSqr;
}

template <typename RealType>
inline TAttrBasedQuadricErrorV2<RealType>::FAttrArray TAttrBasedQuadricErrorV2<RealType>::ComputeScaledAttributes(
	const TVector<RealType>& P) const
{
	FAttrArray WedgeAttrs;
	WedgeAttrs.SetNum(AttrWedgeQuadricDatas.Num());

	for (int i=0; i<AttrWedgeQuadricDatas.Num(); ++i)
	{
		TVector<RealType>& Attr = WedgeAttrs[i];
		if (AttrWedgeQuadricDatas[i].TotalAreaTerm > UE_DOUBLE_KINDA_SMALL_NUMBER)
		{
			Attr.X = AttrWedgeQuadricDatas[i].G[0].Dot(P) + AttrWedgeQuadricDatas[i].D[0];
			Attr.Y = AttrWedgeQuadricDatas[i].G[1].Dot(P) + AttrWedgeQuadricDatas[i].D[1];
			Attr.Z = AttrWedgeQuadricDatas[i].G[2].Dot(P) + AttrWedgeQuadricDatas[i].D[2];
			Attr /= AttrWedgeQuadricDatas[i].TotalAreaTerm;
		}
		else
		{
			Attr = TVector<RealType>(0.);
		}
	} 

	return WedgeAttrs;
}

template <typename RealType>
inline TAttrBasedQuadricErrorV2<RealType>::FAttrArray TAttrBasedQuadricErrorV2<RealType>::ComputeAttributes(const TVector<RealType>& P) const
{
	FAttrArray ScaledAttributes = ComputeScaledAttributes(P);

	for (int i=0; i<AttrWedgeQuadricDatas.Num(); ++i)
	{
		ScaledAttributes[i] /= AttrWedgeQuadricDatas[i].AttrWeight;
	}
	return ScaledAttributes;
}

template <typename RealType>
inline RealType TAttrBasedQuadricErrorV2<RealType>::Evaluate(
	const TVector<RealType>& P, const FAttrArray& ScaledWedgeAttrs) const
{
	// Extended quadric system is
	//
	//         ( A + \sum g_i g_i^T  = C | B                 )
	// A_ext = ( - - - - - - - - - - - - - - - - - - - - - - )
	//         ( B^T                     | aI                )
	// 
	// 
	// b_ext = ( b + \sum d_i g_i    )
	//         ( - - - - - - - - - - )
	//         ( - (d_1, ... d_m)^t  )
	// 
	// c_ext = c + \sum_j d_i^2 
	//
	// (P^T, S^T) A_ext (P, S) = P^T C P + 2 <P, B S> + a<S, S>
	
	const RealType ptCp = P.Dot(BaseStruct::MultiplyA(P));

	RealType attrDotAttr = 0.;
	TVector<RealType> BS { 0. };

	ensure(ScaledWedgeAttrs.Num() == AttrWedgeQuadricDatas.Num());

	int AttrIdx = 0;
	for (const FAttrWedgeQuadricData& AttrWedgeQuadricData : AttrWedgeQuadricDatas)
	{
		const TVector<RealType> (&G)[3] = AttrWedgeQuadricData.G;
		const TVector<RealType>& V = ScaledWedgeAttrs[AttrIdx];
		attrDotAttr += V.Dot(V) * AttrWedgeQuadricData.TotalAreaTerm;

		BS -= G[0] * V[0];
		BS -= G[1] * V[1];
		BS -= G[2] * V[2];

		++AttrIdx;
	}
	RealType PBS = P.Dot(BS);
	
	const RealType QuadraticTerm = ptCp + 2.0 * PBS + attrDotAttr;
	RealType LinearTerm = RealType(2.) * (BaseStruct::bx * P.X + BaseStruct::by * P.Y + BaseStruct::bz * P.Z);
	RealType ConstantTerm = BaseStruct::c;

	AttrIdx = 0;
	for (const FAttrWedgeQuadricData& AttrWedgeQuadricData : AttrWedgeQuadricDatas)
	{
		const TVector<RealType>& D = AttrWedgeQuadricData.D;
		LinearTerm -= RealType(2.) * D.Dot(ScaledWedgeAttrs[AttrIdx]);
		++AttrIdx;
	}

	return QuadraticTerm + LinearTerm + ConstantTerm;
}


template <typename RealType>
inline RealType TAttrBasedQuadricErrorV2<RealType>::EvaluateGeometryOnly(
	const TVector<RealType>& P) const
{
	// Anew = A + \sum g_i g_i^T
	// bnew = b + \sum d_i g_i
	// cnew = c + \sum d_i^2

	RealType QuadraticTerm = P.Dot(BaseStruct::MultiplyA(P));
	RealType LinearTerm = RealType(2.) * (BaseStruct::bx * P.X + BaseStruct::by * P.Y + BaseStruct::bz * P.Z);
	RealType ConstantTerm = BaseStruct::c;

	// Subtract the extra, attribute-related contributions to recover the pure geometry quadric.
	return QuadraticTerm + LinearTerm + ConstantTerm - AttributeData.ComputeCorrection(P);
}

template <typename RealType>
inline bool TAttrBasedQuadricErrorV2<RealType>::ValidateArea() const
{
	int WedgeBegin = 0;
	for (int i=0; i<WedgeCount.Num(); ++i)
	{
		RealType AttrTotalAreaTerm = 0.;
		const int N = WedgeCount[i]; 

		if (N == 0)
		{
			continue;
		}

		for (int j=0; j<N; ++j)
		{
			AttrTotalAreaTerm += AttrWedgeQuadricDatas[WedgeBegin + j].TotalAreaTerm;
		}
		WedgeBegin += N;

		const RealType RelError = FMath::Abs(AttrTotalAreaTerm - TotalAreaTerm) / TotalAreaTerm;
		if (RelError > 1.e-12)
		{
			return false;
		}
	}
	return true;
}

using FAttrBasedQuadricErrorV2d = TAttrBasedQuadricErrorV2<double>;

} // namespace Geometry
} // namespace UE