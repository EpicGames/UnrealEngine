// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp MeshWeights

#pragma once

#include "DynamicMesh/DynamicMesh3.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * FMeshWeights implements various techniques for computing local weights of a mesh,
 * for example one-ring weights like Cotangent or Mean-Value.
 */
class FMeshWeights
{
public:

	/**
	 * Compute uniform centroid of a vertex one-ring.
	 * These weights are strictly positive and all equal to 1 / valence
	 */
	static UE_API FVector3d UniformCentroid(const FDynamicMesh3& Mesh, int32 VertexIndex);

	/**
	 * Compute uniform centroid of a vertex one-ring.
	 * These weights are strictly positive and all equal to 1 / valence
	 */
	static UE_API FVector3d UniformCentroid(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc );

	/**
	 * Compute uniform centroid of a subset of vertex one-ring (eg boundary vertices)
	 * These weights are strictly positive and all equal to 1 / valence
	 */
	static UE_API FVector3d FilteredUniformCentroid(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc, TFunctionRef<bool(int32)> VertexFilterFunc);

	/**
	 * Compute mean-value centroid of a vertex one-ring.
	 * These weights are strictly positive.
	 */
	static UE_API FVector3d MeanValueCentroid(const FDynamicMesh3& Mesh, int32 VertexIndex, double WeightClamp = FMathf::MaxReal);

	/**
	 * Compute mean-value centroid of a vertex one-ring.
	 * These weights are strictly positive.
	 */
	static UE_API FVector3d MeanValueCentroid(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc, double WeightClamp = FMathf::MaxReal);

	/**
	 * Compute cotan-weighted centroid of a vertex one-ring.
	 * These weights are numerically unstable if any of the triangles are degenerate.
	 * We catch these problems and return input vertex as centroid
	 */
	static UE_API FVector3d CotanCentroid(const FDynamicMesh3& Mesh, int32 VertexIndex);

	/**
	 * Compute cotan-weighted centroid of a vertex one-ring.
	 * These weights are numerically unstable if any of the triangles are degenerate.
	 * We catch these problems and return input vertex as centroid
	 */
	static UE_API FVector3d CotanCentroid(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc);



	/**
	 * Compute cotan-weighted centroid of a vertex one-ring, with some weight analysis/clamping to avoid vertices getting "stuck"
	 * in explicit integration/iterations. If failure is detected, Uniform centroid is returned, which does cause some tangential flow
	 * @param DegenerateTol if any weights are larger than this value, return uniform weights instead. Should be > 1.
	 * @param bFailedToUniform will be set to true if non-null and result was clamped
	 */
	static UE_API FVector3d CotanCentroidSafe(const FDynamicMesh3& Mesh, int32 VertexIndex, double DegenerateTol = 100.0, bool* bFailedToUniform = nullptr);

	/**
	 * Compute cotan-weighted centroid of a vertex one-ring, with some weight analysis/clamping to avoid vertices getting "stuck"
	 * in explicit integration/iterations. If failure is detected, Uniform centroid is returned, which does cause some tangential flow
	 * @param DegenerateTol if any weights are larger than this value, return uniform weights instead. Should be > 1.
	 * @param bFailedToUniform will be set to true if non-null and result was clamped
	 */
	static UE_API FVector3d CotanCentroidSafe(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc, double DegenerateTol = 100.0, bool* bFailedToUniform = nullptr);


	/**
	 * Compute cotan-weighted blend for a vertex one-ring, with some weight analysis/clamping to avoid vertices getting "stuck"
	 * in explicit integration/iterations. If failure is detected, Uniform centroid is returned, which does cause some tangential flow.
	 * Equivalent to CotanCentroidSafe() if the weights are used to blend vertex positions, but can be used to weight other properties.
	 * @param BlendingFunc BlendingFunc(NbrVertexID, Weight) will be called for every one-ring neighbour vertex
	 * @param DegenerateTol if any weights are larger than this value, return uniform weights instead. Should be > 1.
	 * @param bFailedToUniform will be set to true if non-null and result was clamped
	 */
	static UE_API void CotanWeightsBlendSafe(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<void(int32,double)> BlendingFunc, double DegenerateTol = 100.0, bool* bFailedToUniform = nullptr);



	/**
	 * Compute the Mixed Voronoi Area associated with a vertex.
	 * Based on Fig 4 from "Discrete Differential-Geometry Operators for Triangulated 2-Manifolds", Meyer et al 2002,
	 */
	static UE_API double VoronoiArea(const FDynamicMesh3& Mesh, int32 VertexIndex);

	/**
	 * Compute the Mixed Voronoi Area associated with a vertex.
	 * Based on Fig 4 from "Discrete Differential-Geometry Operators for Triangulated 2-Manifolds", Meyer et al 2002,
	 */
	static UE_API double VoronoiArea(const FDynamicMesh3& Mesh, int32 VertexIndex, TFunctionRef<FVector3d(int32)> VertexPositionFunc);


protected:
	FMeshWeights() = delete;
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
