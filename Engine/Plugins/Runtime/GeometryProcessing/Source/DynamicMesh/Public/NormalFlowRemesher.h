// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QueueRemesher.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * Remeshing with "face aligned projection". This approach to projection attempts to preserve sharp features in the 
 * mesh by aligning triangle normals with normals from the original mesh during the projection pass.
 * This class also performs a pass of edge flips aimed at further aligning triangle normals to their originals. This
 * can be helpful in fixing the occasional single "bad edge" along a sequence of feature edges.
 */
class FNormalFlowRemesher : public FQueueRemesher
{
public:

	FNormalFlowRemesher(FDynamicMesh3* MeshIn) : FQueueRemesher(MeshIn) {}

	/// During each call to RemeshIteration, do this many passes of face-aligned projection
	int FaceProjectionPassesPerRemeshIteration = 1;

	/// Additional projection iterations after the usual remesh step
	int NumExtraProjectionIterations = 5;

	/// drag on surface projection
	double SurfaceProjectionSpeed = 0.2;

	/// drag on normal alignment
	double NormalAlignmentSpeed = 0.2;

	/// Control whether or not we want to apply mesh smoothing in "free" areas that have not projected to target surface.
	/// This smoothing is on applied in the ExtraProjections Iterations
	bool bSmoothInFillAreas = true;

	/// This is used as a multiplier on MaxEdgeLength to determine when we identify points as being in "free" areas
	float FillAreaDistanceMultiplier = 0.25;

	/// This is used as a multiplier on the Remesher smoothing rate, applied to points identified as being in "free" areas
	float FillAreaSmoothMultiplier = 0.25;

	/// cap on triangle count, to prevent runaway mesh messes. Ignored if zero.
	int MaxTriangleCount = 0;

	/// "Outer loop" for all remeshing operations
	void BasicRemeshPass() override
	{
		RemeshWithFaceProjection();
	}

	/// Gather candidates for edge flipping in parallel, then do the actual flips sequentially
	bool bEnableParallelEdgeFlipPass = true;

protected:

	// Do remeshing with face-aligned projection. During the Projection step of RemeshIteration, perform N passes of 
	// face-aligned projection (N = FaceProjectionPassesPerRemeshIteration).
	// Additionally, do M passes of face-aligned projection after remeshing has finished (M = NumExtraProjectionIterations).
	UE_API void RemeshWithFaceProjection();

	// Perform face-aligned projection onto the target mesh. Queue edges whose lengths change because of it.
	UE_API void TrackedFaceProjectionPass(double& MaxDistanceMoved, bool bIsTuningIteration);
	UE_API void TrackedFaceProjectionPass_Serial(double& MaxDistanceMoved, bool bIsTuningIteration);

	// This is called during RemeshIteration 
	void TrackedFullProjectionPass(bool bParallel) override
	{
		for (int i = 0; i < FaceProjectionPassesPerRemeshIteration; ++i)
		{
			double ProjectionDistance;
			constexpr bool bIsTuningIteration = false;
			if (bParallel)
			{
				TrackedFaceProjectionPass(ProjectionDistance, bIsTuningIteration);
			}
			else
			{
				TrackedFaceProjectionPass_Serial(ProjectionDistance, bIsTuningIteration);
			}
		}
	}


	TArray<double> TempWeightBuffer;

	// Per triangle-vertex weights for face projection pass. (Size >= 3 * MaxTriangleID)
	TArray<double> ProjectionWeightBuffer;

	// Per triangle-vertex projection positions for face projection pass. (Size >= 3 * MaxTriangleID)
	TArray<FVector3d> ProjectionVertexBuffer;


	// Similar to InitializeVertexBufferForPass, but also initialize the additional per-vertex weight buffer
	void InitializeVertexBufferForFacePass()
	{
		FQueueRemesher::InitializeVertexBufferForPass();

		if (TempWeightBuffer.Num() < Mesh->MaxVertexID())
		{
			TempWeightBuffer.SetNum(2 * Mesh->MaxVertexID());
		}
		TempWeightBuffer.Init(0.0, TempWeightBuffer.Num());

		int NumTriangleVertices = 3 * Mesh->MaxTriangleID();

		if (ProjectionWeightBuffer.Num() < NumTriangleVertices)
		{
			ProjectionWeightBuffer.SetNum(2 * NumTriangleVertices);
		}
		ProjectionWeightBuffer.Init(0.0, ProjectionWeightBuffer.Num());

		if (ProjectionVertexBuffer.Num() < NumTriangleVertices)
		{
			ProjectionVertexBuffer.SetNum(2 * NumTriangleVertices);
		}
		ProjectionVertexBuffer.Init(FVector3d{ 0,0,0 }, ProjectionWeightBuffer.Num());

	}


	/** 
	Test to see if flipping an edge would improve "normal error". Normal error for a triangle is:
		0.5 * (1.0 - TriangleNormal \cdot ProjectedNormal),
	where ProjectedNormal is the normal of the triangle returned by finding the closest point on ProjTarget. The normal 
	error for an edge is the sum of errors for the edge's incident triangles.

	@param EdgeID Edge to consider.
	@param BadEdgeErrorThreshold Only edges with this error or greater are candidates for flipping.
	@param ImprovementRatioThreshold New edge must have error less than old edge error times this ratio.

	@return True iff (CurrentEdgeError > BadEdgeErrorThreshold) and (NewEdgeError < ImprovementRatioThreshold * CurrentEdgeError)
	*/
	UE_API bool EdgeFlipWouldReduceNormalError(int EdgeID, double BadEdgeErrorThreshold = 0.01, double ImprovementRatioThreshold = 0.75) const;

	/** Flip all edges for which EdgeFlipWouldReduceNormalError returns true */
	UE_API void TrackedEdgeFlipPass();
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
