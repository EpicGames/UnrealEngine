// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

#define UE_API DYNAMICMESH_API

// Forward declarations
namespace UE::Geometry 
{ 
	class FDynamicMesh3; 
    template<typename RealType, int ElementSize>
    class TDynamicMeshOverlay; 
}

namespace UE
{
namespace Geometry
{

/** 
 * Compute the per-vertex values which are the weighted average of all vertex element values stored in the overlay. 
 */
class FAverageOverlayToVertices
{
public:

	//
	// Optional Inputs
	//
	
    /** Enable/disable multi-threading. */
    bool bUseParallel = true;

    /** Array of vertex IDs whose vertices will be affected. */
    TArray<int32> Selection;
        
    /** The type of weight used for each element. */
    enum class ETriangleWeight : uint8
    {
        Uniform,      // all elements are weighted equally
        Angle = 0,    // use the angle between the edges that share the element
        Area = 1,     // use the area of the triangle containing the element
        AngleArea = 2 // use the angle weighted by the area
    };
    ETriangleWeight TriangleWeight = ETriangleWeight::AngleArea;

protected:
		
	const FDynamicMesh3* Mesh;

public:

	UE_API FAverageOverlayToVertices(const FDynamicMesh3& InMesh);

	virtual ~FAverageOverlayToVertices() = default;

    /** 
     * Compute the per-vertex values which are the weighted average of all vertex element values stored in the overlay. 
     * 
     * @param Overlay      Overlay containing the element values to be averaged.
     * @param VertexValues Array of per-vertex averaged element values. The size is either equal to the 
     *                     ElementSize*Mesh.MaxVertexID() or ElementSize*Selection.Num() if Selection is not empty. 
     * @param HasElements  Array of per-vertex flags, indicating if the vertex had at least one element set in the  
     *                     overlay and contains correct averaged value.
     * 
     * @return true on success, false otherwise
     */
    template<typename RealType, int ElementSize>
	bool AverageOverlay(const TDynamicMeshOverlay<RealType, ElementSize>* Overlay,
                        TArray<RealType>& VertexValues,
                        TArray<bool>& HasElements);
};

}
}

#undef UE_API
