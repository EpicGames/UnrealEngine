// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "TransformSequence.h"

namespace UE::Geometry
{

class FDynamicMesh3;

// Parameters controlling how exterior visiblity is tested
struct FExteriorVisibilitySampling
{
	// Approximate spacing between triangle samples used for visibility tests
	// Set a smaller value to increase the number of visibility samples per triangle, which can give more accurate visibility results
	double SamplingDensity = 1.0;

	// Whether to treat faces as double-sided
	bool bDoubleSided = false;

	// Number of directions to test for visibility
	int32 NumSearchDirections = 128;

	// Whether to mark degenerate tris as visible. If bPerPolyGroup is true, this setting controls the classification of PolyGroups that *only* have degenerate tris.
	bool bMarkDegenerateAsVisible = true;

	// Compute per-triangle visibility array for a triangle mesh
	// @param OutTriOccluded		For each valid triangle ID, OutTriOccluded[ID] will be true if that triangle is hidden, false if it is visible
	// @param bPerPolyGroup			If true, visiblity will be determined on a per group basis: If any triangle in a group is visible, the whole group will be marked as not-occluded
	// @param SamplingParameters	Parameters controlling how exterior visiblity is tested
	// @param SkipTris				True for triangles do not need visibility testing (will be marked as not occluded). If empty, no triangles will be skipped.
	DYNAMICMESH_API static void ComputePerTriangleOcclusion(const FDynamicMesh3& Mesh, TArray<bool>& OutTriOccluded, bool bPerPolyGroup, const FExteriorVisibilitySampling& SamplingParameters, 
		TArrayView<const bool> SkipTris = TArrayView<const bool>());

	// Compute per-triangle visibility array for a triangle mesh, with support for transparent (non-occluding) triangles
	// @param IsTriTransparent		Indicator function returns true if a triangle is 'transparent' (should not occlude, but can still be occluded)
	// @param OutTriOccluded		For each valid triangle ID, OutTriOccluded[ID] will be true if that triangle is hidden, false if it is visible
	// @param bPerPolyGroup			If true, visiblity will be determined on a per group basis: If any triangle in a group is visible, the whole group will be marked as not-occluded
	// @param SamplingParameters	Parameters controlling how exterior visiblity is tested
	// @param SkipTris				True for triangles do not need visibility testing (will be marked as not occluded). If empty, no triangles will be skipped.
	DYNAMICMESH_API static void ComputePerTriangleOcclusion(const FDynamicMesh3& Mesh, TFunctionRef<bool(int32)> IsTriTransparent, TArray<bool>& OutTriOccluded, bool bPerPolyGroup, const FExteriorVisibilitySampling& SamplingParameters,
		TArrayView<const bool> SkipTris = TArrayView<const bool>());

	// Compute per-triangle visibility array for a triangle mesh, with support for transparent (non-occluding) triangles, and with custom sampling directions (ignoring the NumSearchDirections parameter)
	// @param IsTriTransparent		Indicator function returns true if a triangle is 'transparent' (should not occlude, but can still be occluded)
	// @param OutTriOccluded		For each valid triangle ID, OutTriOccluded[ID] will be true if that triangle is hidden, false if it is visible
	// @param InitialViewTestDirections Initial directions to test for triangle visibility/occlusion. Note: directions are from mesh surface toward outside.
	//									Use this to specify a small set of directions (e.g., cardinal directions) to test per triangle before using the full set, which can help improve the algorithm performance.
	// @param FullViewTestDirections Full set of directions to test for triangle visibility/occlusion. Note: directions are from mesh surface toward outside
	// @param bAlsoSampleTriangleNormalDirections Whether to also sample in the normal direction for each triangle. Can be disabled for greater control over the sampling directions.
	// @param bPerPolyGroup			If true, visiblity will be determined on a per group basis: If any triangle in a group is visible, the whole group will be marked as not-occluded
	// @param SamplingParameters	Parameters controlling how exterior visiblity is tested
	// @param SkipTris				True for triangles do not need visibility testing (will be marked as not occluded). If empty, no triangles will be skipped.
	DYNAMICMESH_API static void ComputePerTriangleOcclusionFromDirections(const FDynamicMesh3& Mesh, TFunctionRef<bool(int32)> IsTriTransparent, TArray<bool>& OutTriOccluded, bool bPerPolyGroup, 
		TConstArrayView<FVector3d> InitialTestDirections, TConstArrayView<FVector3d> FullTestDirections, bool bAlsoSampleTriangleNormalDirections,
		const FExteriorVisibilitySampling& SamplingParameters,
		TArrayView<const bool> SkipTris = TArrayView<const bool>());

	DYNAMICMESH_API static TArray<FVector3d> GetRestrictedViewDirections(TConstArrayView<FVector3d> RestrictViewDirections, double RestrictViewAngle, int32 NumSphereSamples);
};

// Determine which meshes are visible from exterior views of a list of meshes
class FDetectPerDynamicMeshExteriorVisibility
{
public:

	struct FDynamicMeshInstance
	{
		const FDynamicMesh3* SourceMesh;
		FTransformSequence3d Transforms;
		FDynamicMeshInstance() = default;
		FDynamicMeshInstance(FDynamicMesh3* Mesh, const FTransform& InTransform) : SourceMesh(Mesh)
		{
			Transforms.Append(InTransform);
		}
	};

	//
	// Inputs
	//

	// Instances for which we need to determine visibility
	TArray<FDynamicMeshInstance> Instances;
	
	// Instances which occlude visibility, but for which we don't need to determine visibility
	TArray<FDynamicMeshInstance> OccludeInstances;

	// Instances which do not occlude visibility, but for which we do need to determine visibility
	TArray<FDynamicMeshInstance> TransparentInstances;

	// Parameters controlling how the triangles are sampled for visibility
	FExteriorVisibilitySampling SamplingParameters;

	// Compute which source meshes are hidden
	// @param OutIsOccluded				Indicate if each mesh in Instances is hidden (true) or visible (false)
	// @param OutTransparentIsOccluded	If provided, indicate if each mesh in TransparentInstances is hidden (true) or visible (false). If not provided, TransparentInstances are not tested.
	DYNAMICMESH_API void ComputeHidden(TArray<bool>& OutIsOccluded, TArray<bool>* OutTransparentIsOccluded = nullptr);

};


} // end namespace UE::Geometry
