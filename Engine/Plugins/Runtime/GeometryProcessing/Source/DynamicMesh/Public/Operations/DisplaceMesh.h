// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "DynamicMesh/MeshNormals.h"
#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{

class FDisplaceMesh
{

public:

	inline static void DisplaceMeshWithVertexWeights(FDynamicMesh3& Mesh,
													 const FMeshNormals& Normals,
													 const TArray<float>& VertexWeights,
													 float Intensity)
	{
		check(VertexWeights.Num() >= Mesh.MaxVertexID());
		check(Normals.GetNormals().Num() >= Mesh.MaxVertexID());

		int NumVertices = Mesh.MaxVertexID();
		
		ParallelFor(NumVertices, [&Mesh, &VertexWeights, &Normals, Intensity](int32 VertexID)
		{
			if (!Mesh.IsVertex(VertexID)) 
			{ 
				return; 
			}

			float Offset = VertexWeights[VertexID];
			FVector3d NewPosition = Mesh.GetVertexRef(VertexID) + (Offset * Intensity * Normals[VertexID]);
			Mesh.SetVertex(VertexID, NewPosition, false);
		});
		Mesh.UpdateChangeStamps(true, false);
	}


	// TODO: Refactor DisplaceMeshTool to use this class

};


} // end namespace UE::Geometry
} // end namespace UE