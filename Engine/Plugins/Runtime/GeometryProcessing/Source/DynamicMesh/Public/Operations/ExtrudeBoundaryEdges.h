// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "FrameTypes.h"
#include "HAL/Platform.h" //int32
#include "Misc/Optional.h"
#include "VectorTypes.h"

#define UE_API DYNAMICMESH_API

class FProgressCancel;

namespace UE::Geometry
{

class FDynamicMesh3;

// Hacky base class to avoid 8 bytes of padding after the vtable
class FExtrudeBoundaryEdgesFixLayout
{
public:
	virtual ~FExtrudeBoundaryEdgesFixLayout() = default;
};

class FExtrudeBoundaryEdges : public FExtrudeBoundaryEdgesFixLayout
{
public:
	using FFrame3d = UE::Geometry::FFrame3d;
	
	// Represents a frame where the axes might not be unit scaled (but are still orthogonal).
	// Allows vertices to adjust the extrusion distance along one of their extrusion frame
	//  axes when trying to keep edges parallel.
	// When using extrusion frames, vertices will be moved in the XZ plane, usually along X.
	struct FExtrudeFrame
	{
		FExtrudeFrame() {}

		FExtrudeFrame(const FFrame3d& FrameIn)
			: Frame(FrameIn)
		{}
		FExtrudeFrame(const FFrame3d& FrameIn, const FVector3d& InFrameScaleDirection, double ScalingIn)
			: Frame(FrameIn)
			, InFrameScaleDirection(InFrameScaleDirection)
			, Scaling(ScalingIn)
		{}

		UE_API FVector3d FromFramePoint(FVector3d FramePoint) const;

		UE_API FVector3d ToFramePoint(FVector3d WorldPoint) const;

		FFrame3d Frame;

		// Extra scaling direction used to support adjusting extruded vertices when trying to keep
		//  edges parallel. The input is scaled along this axis in frame space. See comment inside
		//  ExtrudeBoundaryEdgesLocals::GetExtrudeFrameAtVertex.
		TOptional<FVector3d> InFrameScaleDirection;
		double Scaling = 1.0;
	};

	// Data needed to create a new vert and its extrude frame
	struct FNewVertSourceData
	{
		int32 SourceVid = IndexConstants::InvalidID;
		// Neighboring edges, ordered by (incoming, outgoing)
		FIndex2i SourceEidPair = FIndex2i::Invalid();
	};

	// Inputs:

	/** The mesh that we are modifying */
	FDynamicMesh3* Mesh;

	/** The edges we're extruding */
	TArray<int32> InputEids;

	/** Whether to calculate local extrude frames and supply them to OffsetPositionFunc */
	bool bUsePerVertexExtrudeFrames = true;

	/** When generating extrude frames, whether to use unselected neighbors for setting the frame. */
	bool bAssignAnyBoundaryNeighborToUnmatched = false;

	/**
	 * Function queried for new vertex positions. ExtrudeFrame origin is Position, unless it is not initialized
	 *  due to bUsePerVertexExtrudeFrames being false.
	 * The default given here assumes that bUsePerVertexExtrudeFrames is true, and extrudes along the X axis of the extrude frame.
	 */
	TFunction<FVector3d(const FVector3d& Position, const FExtrudeFrame& ExtrudeFrame, int32 SourceVid)> OffsetPositionFunc =
		[this](const FVector3d& Position, const FExtrudeFrame& ExtrudeFrame, int32 SourceVid)
	{
		return ExtrudeFrame.FromFramePoint(FVector3d(this->DefaultOffsetDistance, 0, 0));
	};

	/** 
	 * If greater than 1, maximal amount by which a vertex can be moved in an attempt to keep edges parallel to
	 *  original edges while extruding. This "movement" is done by scaling the X axis of the extrude frame.
	 */
	double ScalingAdjustmentLimit = 1.0;

	/** 
	 * Optional mapping, 1:1 with Eids, that gives the group id to use for each generated quad. Otherwise all generated
	 *  triangles will be given the same new group id.
	 */
	TOptional<TArray<int32>> GroupsToSetPerEid;

	/** Used in the default OffsetPositionFunc. */
	double DefaultOffsetDistance = 1.0;


	// Outputs:
	TArray<int32> NewTids;
	TArray<int32> NewExtrudedEids;

public:
	UE_API FExtrudeBoundaryEdges(FDynamicMesh3* mesh);

	virtual ~FExtrudeBoundaryEdges() = default;

	/**
	 * Apply the operation to the input mesh.
	 * @return true if the algorithm succeeds
	 */
	UE_API virtual bool Apply(FProgressCancel* Progress);

	/** 
	 * Pairs up edges across vertices to help in extrusion frame calculation. Public because it is used by
	 * the extrude edges activity to find an operational space for the gizmos used to set extrude distance.
	 * 
	 * @return false if there is an error.
	 */
	static UE_API bool GetInputEdgePairings(
		const FDynamicMesh3& Mesh, TArray<int32>& InputEids, bool bAssignAnyBoundaryNeighborToUnmatched,
		TArray<FNewVertSourceData>& NewVertDataOut, TMap<int32, FIndex2i>& EidToIndicesIntoNewVertsOut);

	/**
	 * Gets an extrude frame given a vert and its neighboring boundary edges. Like GetInputEdgePairings,
	 * public because it is used to set up UX for setting extrude distance.
	 * 
	 @return false if there is an error.
	 */
	static UE_API bool GetExtrudeFrame(const FDynamicMesh3& Mesh, int32 Vid,
		int32 IncomingEid, int32 OutgoingEid, FExtrudeFrame& ExtrudeFrameOut,
		double ScalingLimit);
};

} // end namespace UE::Geometry

#undef UE_API
