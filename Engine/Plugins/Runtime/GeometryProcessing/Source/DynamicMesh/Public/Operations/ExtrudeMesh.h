// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MergeCoincidentEdges

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "MeshRegionBoundaryLoops.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
class FMeshNormals;

// Hacky base class to avoid 8 bytes of padding after the vtable
class FExtrudeMeshFixLayout
{
public:
	virtual ~FExtrudeMeshFixLayout() = default;
};

/**
 * Note: FExtrudeMesh might someday be removed. Consider using FOffsetMeshRegion instead,
 * which can extrude all or part of a mesh.
 * 
 * FExtrudeMesh implements a full-mesh extrusion of a mesh. This happens in two stages:
 * 1) all triangles of input mesh are duplicated and offset
 * 2) base and offset border loops are stitched together with triangulated quads
 * Step 2 does not occur if there are no boundary loops (ie for a closed input mesh)
 * 
 * Each quad of the border loop is assigned it's own normal and UVs (ie each is a separate UV-island)
 */
class FExtrudeMesh : public FExtrudeMeshFixLayout
{
public:

	//
	// Inputs
	//
	
	/** The mesh that we are modifying */
	FDynamicMesh3* Mesh;

	/** This function is called to generate the offset vertex position. Default returns (Position + DefaultExtrudeDistance * Normal) */
	TFunction<FVector3d (const FVector3d&, const FVector3f&, int)> ExtrudedPositionFunc;

	/** If no Extrude function is set, we will displace by DefaultExtrudeDistance*Normal */
	double DefaultExtrudeDistance = 1.0;

	/** if Extrusion is "negative" (ie negative distance, inset, etc) then this value must be set to false or the output will have incorrect winding orientation */
	bool IsPositiveOffset = true;

	/** If true, skip closed components */
	bool bSkipClosedComponents = false;

	/** quads on the stitch loop are planar-projected and scaled by this amount */
	float UVScaleFactor = 1.0f;

	//
	// Outputs
	//

	/**
	 * FExtrusionInfo stores info about extrusion for a single mesh connected component. 
	 * Note that this may involve multiple boundary loops (eg if a region has holes)
	 */
	struct FExtrusionInfo
	{
		/** Initial boundary loops on the mesh (may be empty) */
		FMeshRegionBoundaryLoops InitialLoops;
		/** set of triangles that were extruded */
		TArray<int> InitialTriangles;
		/** set of vertices that were extruded */
		TArray<int> InitialVertices;
		/** Map from initial vertices to new offset vertices */
		TMap<int, int> InitialToOffsetMapV;
		/** list of triangles on offset surface, in correspondence with InitialTriangles (note: can get vertices via InitialToOffsetMapV(InitialVertices) */
		TArray<int> OffsetTriangles;
		/** list of new groups of triangles on offset surface */
		TArray<int> OffsetTriGroups;

		/** New edge loops on borders of offset patches (1-1 correspondence w/ InitialLoops.Loops) */
		TArray<FEdgeLoop> NewLoops;
		/** Lists of triangle-strip "tubes" that connect each loop-pair */
		TArray<TArray<int>> StitchTriangles;
		/** List of group ids / polygon ids on each triangle-strip "tube" */
		TArray<TArray<int>> StitchPolygonIDs;
	};

	/**
	 * List of extrusion regions, one per connected component of input
	 */
	TArray<FExtrusionInfo> Extrusions;

public:
	UE_API FExtrudeMesh(FDynamicMesh3* mesh);

	virtual ~FExtrudeMesh() = default;


	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		// @todo calculate MeshBoundaryLoops and make sure it is valid

		// is there any reason we couldn't do this??

		return EOperationValidationResult::Ok;
	}


	/**
	 * Apply the Extrude operation to the input mesh.
	 * @return true if the algorithm succeeds
	 */
	UE_API virtual bool Apply();



protected:

	UE_API bool ApplyExtrude(FExtrusionInfo& Region, FMeshNormals* UseNormals = nullptr);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
