// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "IntVectorTypes.h"
#include "BoxTypes.h"
#include "Containers/StaticArray.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#define UE_API DYNAMICMESH_API

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

enum class ELatticeInterpolation : uint8
{
	Linear = 0,
	Cubic = 1
};

struct FLatticeExecutionInfo
{
	bool bParallel = true;
	int CancelCheckSize = 100000;		// Number of vertices to process before checking for cancel
};

/** 
 * Free-form deformation lattice. Initialize it with a mesh and desired resolution, get the initial lattice points out.
 * Then pass in deformed lattice points and it will compute deformed mesh vertex positions.
 */
class FFFDLattice
{
public:

	/// Linear interpolation information. There should be one of these per mesh vertex.
	struct FEmbedding
	{
		FVector3i LatticeCell = { -1, -1, -1 };		// Original point is in the hexahedron whose Min corner is LatticeCell
		FVector3d CellWeighting = { 0,0,0 };	// Original linear interpolation weights of the point in the cell
	};

	/// Create a lattice that fits the given mesh and has the given resolution along each dimension. Also precompute
	/// the weight of each mesh vertex inside the appropriate lattice cell.
	UE_API FFFDLattice(const FVector3i& InDims, const FDynamicMesh3& InMesh, float Padding);

	/// Create a lattice that fits the oriented box defined by the AABB and Transform, and has the given resolution along each dimension. Also precompute
	/// the weight of each mesh vertex inside the appropriate lattice cell. Mesh vertices outside of the oriented bounding box should have a LatticeCell value of {-1, -1, -1}.
	UE_API FFFDLattice(const FVector3i& InDims, const FDynamicMesh3* Mesh, float Padding, const FAxisAlignedBox3d& InInitialBounds, const FTransformSRT3d& LatticeTransform = FTransformSRT3d::Identity());

	/// Whether the given vertex has a valid embedding, i.e. whether the original vertex location was inside a lattice cell and it would be affected by lattice deformation
	UE_API bool VertexHasValidEmbedding(int32 VertexIndex) const;

	/// Using the Lattice's current control point positions and the original embedding information, compute the
	/// deformed mesh vertex positions.
	UE_API void GetDeformedMeshVertexPositions(const TArray<FVector3d>& LatticeControlPoints,
										TArray<FVector3d>& OutVertexPositions, 
										ELatticeInterpolation Interpolation = ELatticeInterpolation::Linear,
										FLatticeExecutionInfo ExecutionInfo = FLatticeExecutionInfo(),
										FProgressCancel* Progress = nullptr) const;

	/// Using the Lattice's current control point positions, the original embedding information, and the original 
	/// normals contained in an overlay, compute the new rotated normals
	UE_API void GetRotatedOverlayNormals(const TArray<FVector3d>& LatticeControlPoints,
								  const FDynamicMeshNormalOverlay* NormalOverlay,
								  TArray<FVector3f>& OutNormals,
								  ELatticeInterpolation Interpolation = ELatticeInterpolation::Linear,
								  FLatticeExecutionInfo ExecutionInfo = FLatticeExecutionInfo(),
								  FProgressCancel* Progress = nullptr) const;

	/// Using the Lattice's current control point positions, the original embedding information, and the original 
	/// normals, compute the new rotated normals
	UE_API void GetRotatedMeshVertexNormals(const TArray<FVector3d>& LatticeControlPoints,
									 const TArray<FVector3f>& OriginalNormals,
									 TArray<FVector3f>& OutNormals,
									 ELatticeInterpolation Interpolation,
									 FLatticeExecutionInfo ExecutionInfo,
									 FProgressCancel* Progress) const;

	/// Return the set of lattice corner positions in undeformed state
	UE_API void GenerateInitialLatticePositions(TArray<FVector3d>& OutLatticePositions) const;

	/// Return the lattice edges by index. The indices refer to the array of lattice positions returned by GenerateInitialLatticePositions
	UE_API void GenerateLatticeEdges(TArray<FVector2i>& OutLatticeEdges) const;

	/// Number of lattice points in each dimension
	const FVector3i& GetDimensions() const
	{
		return Dimensions;
	}

	/// 3D size of a lattice cell
	const FVector3d& GetCellSize() const
	{
		return CellSize;
	}

	/// Extents of the lattice before it is deformed
	const FAxisAlignedBox3d& GetInitialBounds() const
	{
		return InitialBounds;
	}

	/// Get the index into the flat TArray of positions given the (i,j,k) coordinates in the lattice
	int ControlPointIndexFromCoordinates(int i, int j, int k) const
	{
		int Idx = k + Dimensions.Z * (j + Dimensions.Y * i);
		return Idx;
	}

	/// Get the index into the flat TArray of positions given the (i,j,k) coordinates in the lattice
	int ControlPointIndexFromCoordinates(const FVector3i& Index) const
	{
		return ControlPointIndexFromCoordinates(Index.X, Index.Y, Index.Z);
	}

protected:

	// Lattice transform
	FTransformSRT3d Transform;

	// Number of lattice points in each dimension
	FVector3i Dimensions;

	// Extents of the lattice before it is deformed
	FAxisAlignedBox3d InitialBounds;

	// 3D size of a lattice cell
	FVector3d CellSize;

	// Interpolation weights and cell indices per vertex of the input FDynamicMesh3
	TArray<FEmbedding> VertexEmbeddings;

	// Ccnstructor helper function for setting up lattice data
	void InitLattice(float Padding);

	/// For each vertex in Mesh, compute the lattice cell it resides in and its weighting
	UE_API void ComputeInitialEmbedding(const FDynamicMesh3& Mesh, FLatticeExecutionInfo ExecutionInfo = FLatticeExecutionInfo());

	/// Compute cell index and linear interpolation weights for a given point
	UE_API FVector3d ComputeTrilinearWeights(const FVector3d& Pt, FVector3i& GridCoordinates) const;

	/// Helper for linear interpolation. Returns lattice points at (I,J,K) and (I+1,J,K).
	UE_API void GetValuePair(int I, int J, int K, FVector3d& A, FVector3d& B, const TArray<FVector3d>& LatticeControlPoints) const;

	/// Use the given cell index and weights to compute the interpolated position
	UE_API FVector3d InterpolatedPosition(const FEmbedding& VertexEmbedding, const TArray<FVector3d>& LatticeControlPoints) const;
	UE_API FVector3d InterpolatedPositionCubic(const FEmbedding& VertexEmbedding, const TArray<FVector3d>& LatticeControlPoints) const;

	UE_API FMatrix3d LinearInterpolationJacobian(const FEmbedding& VertexEmbedding, const TArray<FVector3d>& LatticeControlPoints) const;
	UE_API FMatrix3d CubicInterpolationJacobian(const FEmbedding& VertexEmbedding, const TArray<FVector3d>& LatticeControlPoints) const;

	/// Clamp the given index to [0, Dims] and return the current lattice point position at the clamped index
	UE_API FVector3d ClosestLatticePosition(const FVector3i& VirtualControlPointIndex, const TArray<FVector3d>& LatticeControlPoints) const;

	/// Compute the extrapolated position for a "virtual" lattice control point outside of the actual lattice
	UE_API FVector3d ExtrapolatedLatticePosition(const FVector3i& VirtualPointIndex, const TArray<FVector3d>& LatticeControlPoints) const;

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
