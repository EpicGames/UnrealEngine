// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/Array.h"
#include "Containers/BinaryHeap.h"
#include "Containers/List.h"
#include "Containers/Map.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "IndexTypes.h"
#include "IntrinsicTriangulationMesh.h"
#include "Util/DynamicVector.h"
#include "Util/RefCountVector.h"
#include "MathUtil.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

/**
* FEdgePath is is little more than a double link-list that exposes IDs instead of node pointers.
* The ID of a removed segment may be reused for a subsequently appended or inserted segment.
* When used in conjunction with a mesh, it represents a directed, edge-aligned path.
* 
* Note: this exists primarily for use with FDeformableEdgePath and 
*       no checking is done to ensure the path is continuous, in fact it could 
*       be used to represent multiple disconnected paths.
*/
class FEdgePath
{
public:

	
	constexpr static int InvalidID = IndexConstants::InvalidID;
	struct FDirectedSegment
	{
		int32 EID  = InvalidID;         // expected to be a mesh edge ID
		int32 HeadIndex = InvalidID;    // 0 or 1.  GetEdgeV()[HeadIndex] is the 'to' vertex,  GetEdgeV()[1-HeadIndex] is the 'from' vertex
	};
	
	FEdgePath() = default;
	FEdgePath(const FEdgePath&) = delete;

	/** @return number of segments in path */
	inline int32 NumSegments() const;

	/** @return upper bound on the segment ID used in path.  i.e. all segment IDs < MaxSegmentID */
	inline int32 MaxSegmentID() const;

	/** @return true if the specified segment ID corresponds to a valid segment in path */
	inline bool IsSegment(int32 SID) const;
	
	/** @return requested segment if it is part of the path, otherwise return invalid segment (with EID = InvalidID) */
	UE_API FDirectedSegment GetSegment(int32 SID) const;

	/** @return the segment ID of the {first, last} segment in the path, or InvalidID if the path is empty */
	UE_API int32 GetHeadSegmentID() const;
	UE_API int32 GetTailSegmentID() const;

	/** @return segment ID of {previous, next} segment in the path, or InvalidID if if no {previous, next} exists or if SID itself is not in the path  */
	UE_API int32 GetPrevSegmentID(int32 SID) const;
	UE_API int32 GetNextSegmentID(int32 SID) const;

	/** @return segment ID of a new segment appended to the tail of the path */
	UE_API int32 AppendSegment(const FDirectedSegment& Segment);

	/** @return segment ID of a new segment inserted before the specified existing segment. If SegmentIDToInsertBefore is InvalidID, then insert at head  */
	UE_API int32 InsertSegmentBefore(const FDirectedSegment& DirectedSegment, int32 SegmentIDToInsertBefore = InvalidID);
	
	/** remove specified segment from path. Note SID is added to an internal free list and maybe reused  */
	UE_API void RemoveSegment(int32 SID);


protected:

	struct FSegmentAndSID : FDirectedSegment
	{
		int32 SID = InvalidID; // segment id
	};

	typedef typename TDoubleLinkedList<FSegmentAndSID>::TDoubleLinkedListNode NodeType;
	
	TDoubleLinkedList<FSegmentAndSID>  PathLinkList;                // The path is represented by Segments in a double link list.
	
	TDynamicVector<NodeType*>          SIDtoNode;                   // Nodes per segment indexed by SegmentID (SID)
	FRefCountVector                    DirectedSegmentRefCounts{};  // SID list.
};


/**
* Given a single path initially defined by edges on a FDynamicMesh (the SurfaceMesh), this class uses an intrinsic mesh with 
* the same vertex set to minimize the length of the path on the surface mesh.  This is done iteratively by a succession 
* of local path deformations comprised of (local) intrisic edge flips and path re-routing, each deformation locally reduces the path length.  
* The resulting path is comprised of intrinsic mesh edges.
* 
* This is an implementation of the ideas in 
*       "You Can Find Geodesic Paths in Triangle Meshes by Just Flipping Edges" by Nicholas Sharp and Keenan Crane
*        ACM Trans. Graph. Vol. 39, No. 6, 2020
* 
* The original path is constructed as a sequence of mesh edges relative to the  SurfaceMesh ( the FDynamicMesh3 ),
* these edges are encoded with path direction as FDirectedSegments.
* Note: The current implementation does not support closed paths or multiple paths.
*/
class FDeformableEdgePath
{
public:
	
	constexpr static int InvalidID = IndexConstants::InvalidID;
	
	typedef FIntrinsicEdgeFlipMesh  IntrinsicMeshType;

	/**
	* Constructor assumes the directed segments are ordered from tail (at index 0) to head
	*/
	UE_API FDeformableEdgePath(const FDynamicMesh3& SurfaceMeshIn, const TArray<FEdgePath::FDirectedSegment>& OriginalPathAsDirectedSegments);
	
	virtual ~FDeformableEdgePath()= default;

	// only allow construction from a path and mesh.
	FDeformableEdgePath() = delete;
	FDeformableEdgePath(const FDeformableEdgePath& ) = delete;
	
	// information gathered during minimization
	struct FEdgePathDeformationInfo
	{
		int32 NumIterations   = 0;     // number of two-edge joints that were replaced by shorter paths (but perhaps more segments)
		int32 NumEdgeFlips    = 0;     // number of edge flips performed in minimizing the path
		double OriginalLength = 0.;    // path length prior to minimization
		double FinalLength    = 0.;    // path length after minimization
	};

	/**
	* Minimize the deformable edge path with respect to the total path length.  
	* This minimization finds a local minimum in path length but may not find the shortest over all distance between to points.
	* In this sense the operation can be thought of as a path-straightening. 
	* @param DeformedPathInfo  - on return this info struct is populated
	* @param MaxNumIterations  - the maximum number of two-edge joints that may be replaced by shorter paths.
	*/ 
	UE_API virtual void Minimize(FEdgePathDeformationInfo&  DeformedPathInfo, const int32 MaxNumIterations = TMathUtilConstants<int32>::MaxReal);
	
	/**
	* @return a const reference to the edge path.
	*/
	inline const FEdgePath& GetEdgePath() const;

	/**
	* @return Length of the path.
	*/
	inline double GetPathLength() const;

	/**
	* @return const reference to the intrinsic mesh on which the EdgePath is defined.
	*/ 
	inline const FIntrinsicEdgeFlipMesh& GetIntrinsicMesh() const;

	/**
	* struct that references a point on a mesh, by vertex, by edge-crossing, or barycentric coords
	*/ 
	using FSurfacePoint =  IntrinsicCorrespondenceUtils::FSurfacePoint;
	/**
	* @return an array of surface points relative to the SurfaceMesh that define this path.
	* Note the first and last surfacepoints correspond to the start and end vertex, but all 
	* other surface points may be either edge crossings or vertex points.
	* 
	* @param CoalesceThreshold - In barycentric units [0,1], edge-crossings within this threshold are snapped to the nearest vertex
	*                            and any resulting repitition of vertex surface points are replaced with a single vertex surface point.
	*                            Due to numerical precision issues, a path that 'should' intersect a surface mesh vertex may appear
	*                            as a sequence of vertex adjacent edge-crossings very close to the vertex.. the threshold is applied in
	*                            a post-process coalesce those crossings into a single vertex crossing.
	*/
	UE_API TArray<FSurfacePoint> AsSurfacePoints(double CoalesceThreshold = 0.) const;

protected:
	
	enum class ESide { Left, Right };      // indicates which wedge (Left or Right) has the smaller internal angle, where in 2d the path travels "up" 
	struct FPathJoint
	{
		int32  OutgoingSID = InvalidID;    // Outgoing segment.  EdgePath.GetPrevSegmentID(OutgoingSID) returns incoming segment
		ESide  InteriorSide;               // Side of the one-ring with the smaller angle.
		int32  VID = InvalidID;            // incoming and Outgoing edges meet at this vertex
	};


	/**
	* Replace the specified two connected edge segments (FPathJoint) in the main path with a new sub-path 
	* that connects the same end points. This new path is formed by flipping the adjacent edges to the 
	* left (or right) of the joint vertex and may have one, two, or more segments
	*/
	UE_API void DeformJoint(FPathJoint& PathJoint);

	/**
	* Edge flip the adjacent edges to the JointVID and collect the DeformedPath that can be used to replace the local incoming and outgoing edge path
	* @param IncomingEID  - ID of incoming edge of joint
	* @param OutgoingEID  - ID of outgoing edge of joint
	* @param JointVID     - ID of Vertex where incoming and outgoing edges meet
	* 
	* @param DeformedPath - On return, holds directed segments that can be used to replace the joint
	* @param bClockwise   - When true, the deformation and new path are on the "left" of the original path, otherwise on the "right"
	*                       where the path is locally viewed as traveling from bottom to top
	*/ 
	UE_API bool DeformJoint( const int32 IncomingEID, const int32 OutgoingEID, int32 JointVID, 
	                  TArray<FEdgePath::FDirectedSegment>& DeformedPath, const bool bClockwise );
	
	/**
	* @return the ID of the vertex at the "head" of the specified segment
	*/
	inline int32 SegmentHeadVID(int32 SID) const;
	
	/**
	* @return the ID of the vertex at the "tail" of the specified segment
	*/
	inline int32 SegmentTailVID(int32 SID) const;
	
	/**
	* @return true if no part of the global path lies in the wedge (partial one-ring) on PathJoint.Interior side
	*/ 
	UE_API bool IsJointFlexible(const FPathJoint& PathJoint) const;
	
	/**
	* Compute data associated with the joint formed by the specified segment (outgoing), and the preceding segment (incoming) in the path.
	* Also update the JointAngleQueue that tracks straightness of each joint.
	*/ 
	UE_API void UpdateJointAndQueue(int32 OutgoingSegmentID);
	
	/**
	* remove specified segment from the linked-list path
	* and clean up joint-related data for the joint that ends with this segment.
	* note: the joint data (e.g. joint angle queue) for the joint that started with this segment will be dirty.
	*/ 
	UE_API void RemoveSegment(int32 SID);

	/**
	* Splice in a path to replace the joint and update the joint angle queue.
	*/ 
	UE_API void ReplaceJointWithPath(const FPathJoint PathJoint, const TArray<FEdgePath::FDirectedSegment>& PathAsDirectedEdges);

	/**
	* Traveling either CCW or CW around vertex CenterVID, from edge StartEID to edge EndEID, flip edges adjacent to CVID.
	* return true on success, and false if one of the edges was a boundary edge.  
	* 
	* It is assumed the specified edges do in fact have vertex CVID in common, but this is not checked.
	*
	* Note: in the false case, some of the edges may have been flipped before encountering the boundary edge
	*       - no attempt is made to restore the mesh.
	*       
	*/
	UE_API bool OuterArcFlipEdges(int32 StartEID, int32 EndEID, int32 CenterVID, bool bClockwise = false);
	
	/**
	* Compute the two angles formed at the vertex CenterVID by the two wedges formed by splitting the local one-ring by local path
	* specified as incoming and outgoing edges.
	* 
	* It is assumed the specified edges do in fact have vertex CVID in common, but this is not checked.
	*/
	UE_API void ComputeWedgeAngles(int32 IncomingEID, int32 OutgoingEID, int32 CenterVID, 
	                        double& LeftSideAngle, double& RightSideAngle) const ;
	
protected:
     
	IntrinsicMeshType              EdgeFlipMesh;        // Intrinsic mesh with the same vertices as the original surface mesh. The geodesic path is comprised of edges in this mesh.

	double                         PathLength;          // Total length of the path
	int32                          NumFlips;            // Count of the number of edge flips performed in shortening the path.

	FEdgePath                      EdgePath;            // Path defined relative to the edges in the EdgeFlipMesh.
	TMap<int32, TArray<int32>>     EIDToSIDsMap;        // Map mesh edge ID to array of path segments.  A path could traverse the same mesh edge multiple times.
	TDynamicVector<FPathJoint>     PathJoints;          // List of directed segment joints in the path, indexed by SID of Outgoing segment.	
	FBinaryHeap<double>            JointAngleQueue;     // Priority queue that can automatically resize if needed.  Priority based on the smaller angle between the two edges in a joint
	
};

/**
* @return the length of the path, computed by suming the path segment lengths.  Primarily for testing
* as the result should match FDeformableEdgePath::GetPathLength()
*/ 
double SumPathLength(const FDeformableEdgePath& DeformableEdgePath);


const FEdgePath& FDeformableEdgePath::GetEdgePath() const
{
	return EdgePath;
}

double  FDeformableEdgePath::GetPathLength() const
{
	return PathLength;
}

const FIntrinsicEdgeFlipMesh& FDeformableEdgePath::GetIntrinsicMesh() const
{
	return EdgeFlipMesh;
}

int32 FDeformableEdgePath::SegmentHeadVID(int32 SID) const
{
	const auto Directed = EdgePath.GetSegment(SID);
	const FIndex2i EdgeV = EdgeFlipMesh.GetEdgeV(Directed.EID);
	return EdgeV[Directed.HeadIndex];
}
int32 FDeformableEdgePath::SegmentTailVID(int32 SID) const
{
	const auto Directed = EdgePath.GetSegment(SID);
	const FIndex2i EdgeV = EdgeFlipMesh.GetEdgeV(Directed.EID);
	return EdgeV[1 - Directed.HeadIndex];
}


int FEdgePath::NumSegments() const
{
	return PathLinkList.Num();
}

int32 FEdgePath::MaxSegmentID() const
{
	return DirectedSegmentRefCounts.GetMaxIndex();
}

bool FEdgePath::IsSegment(int32 SID) const
{
	return DirectedSegmentRefCounts.IsValid(SID);
}


}; // end namespace Geometry
}; // end namespace UE

#undef UE_API
