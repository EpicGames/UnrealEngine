// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GroupTopology.h"

#define UE_API DYNAMICMESH_API

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
class FGroupTopology;

/**
 * Used to insert group edges and group edge loops.
 */
class FGroupEdgeInserter
{
public:

	enum class EInsertionMode
	{
		Retriangulate,
		PlaneCut
	};

	/** Parameters for an InsertEdgeLoops() call. */
	struct FEdgeLoopInsertionParams
	{
		/** Both of these get updated in the operation */
		FDynamicMesh3* Mesh = nullptr;
		FGroupTopology* Topology = nullptr;

		/** Edge loops will be inserted perpendicular to this group edge */
		int32 GroupEdgeID = FDynamicMesh3::InvalidID;

		/** 
		 * Inputs can be proportions in the range (0,1), or absolute lengths. 
		 * As the name suggests, they must already be sorted. 
		 */
		const TArray<double>* SortedInputLengths = nullptr;
		bool bInputsAreProportions = true;

		/** 
		 * One of the endpoints of the group edge, from which the arc lengths
		 * or proportions should be measured
		 */
		int32 StartCornerID = FDynamicMesh3::InvalidID;

		/**
		 * When inserting edges, this is the distance that a desired new point can be to
		 * use a nearby vertex rather than splitting an edge to create a new one.
		 */
		double VertexTolerance = KINDA_SMALL_NUMBER * 10;

		/**
		 * When inserting edges, we can locally simplify to improve mesh quality along the edge path.
		 * Only does so when it should not change mesh shape, UVs, triangle groups, etc,
		 * and only applies when Insertion Mode is "Plane Cut"
		 */
		bool bSimplifyAlongPath = true;

		/** 
		 * Determines how the edge is inserted: by using a cutting plane to cut existing triangles
		 * along the path, or deleting triangles and retriangulating. Retriangulation will keep
		 * geometry nice but currently loses the UV's, and it obviously breaks for non-planar groups.
		 */
		EInsertionMode Mode = EInsertionMode::Retriangulate;
	};

	struct FOptionalOutputParams
	{
		// Some compilers (clang, at least) have issues with using a nested class with default initializers
		// and a default constructor as default argument in a function, hence this constructor.
		FOptionalOutputParams(){};

		/**
		 * Edge IDs of the edges composing the newly inserted group edges.
		 */
		TSet<int32>* NewEidsOut = nullptr;
		
		/** 
		 * Any triangle IDs whose triangles were deleted or changed by the operation (but not newly
		 * created tids). Useful for setting up undo. 
		 */
		TSet<int32>* ChangedTidsOut = nullptr;

		/**
		 * In loop insertion, the group edge IDs in the original topology that surround non-quad-like
		 * groups that stopped the loop.
		 */
		TSet<int32>* ProblemGroupEdgeIDsOut = nullptr;
	};

	UE_API bool InsertEdgeLoops(const FEdgeLoopInsertionParams& Params, FOptionalOutputParams OptionalOut = FOptionalOutputParams(), FProgressCancel* Progress = nullptr);


	/** Point along a group edge that is used as a start/endpoint for an inserted group edge. */
	struct FGroupEdgeSplitPoint
	{
		/** Either vertex ID or edge ID of the point. */
		int32 ElementID;

		/** Whether the point is an edge or vertex. */
		bool bIsVertex;

		/**
		 * When using a cutting plane, tangent that is used to help position the plane. For edges,
		 * it is just an edge vector, but for vertices, it is the average of the adjacent edge vectors
		 * of the group boundary.
		 */
		FVector3d Tangent;

		/**
		 * Only relevant for edges. The range (0,1) parameter that determines where the edge
		 * should be split (the order of endpoints is given by the mesh structure).
		 */
		double EdgeTValue;
	};

	/** Parameters for an InsertGroupEdge() call */
	struct FGroupEdgeInsertionParams
	{
		/** These are both modified in the operation */
		FDynamicMesh3* Mesh = nullptr;
		FGroupTopology* Topology = nullptr;

		/** Group across which the cut is inserted. */
		int32 GroupID = FDynamicMesh3::InvalidID;
		/** Index into the group's Boundary array that holds the boundary that the start and end points share. */
		int32 GroupBoundaryIndex = 0;

		FGroupEdgeSplitPoint StartPoint;
		FGroupEdgeSplitPoint EndPoint;

		/**
		 * When inserting edges, this is the distance that a desired new point can be to
		 * use a nearby vertex rather than splitting an edge to create a new one (used when
		 * inserting using plane cut).
		 */
		double VertexTolerance = KINDA_SMALL_NUMBER * 10;

		/**
		 * When inserting edges, we can locally simplify to improve mesh quality along the edge path.
		 * Only does so when it should not change mesh shape, UVs, triangle groups, etc,
		 * and only applies when Insertion Mode is "Plane Cut"
		 */
		bool bSimplifyAlongPath = true;

		EInsertionMode Mode = EInsertionMode::Retriangulate;
	};

	UE_API bool InsertGroupEdge(FGroupEdgeInsertionParams& Params, FOptionalOutputParams OptionalOut = FOptionalOutputParams(), FProgressCancel* Progress = nullptr);

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
