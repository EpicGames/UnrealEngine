// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdgeSpan.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
class FGroupTopology;
struct FGeometrySelection;
struct FGeometrySoftSelection;


/**
 * Basic cache of vertex positions.
 */
class FVertexPositionCache
{
public:
	TArray<int> Vertices;
	TArray<FVector3d> Positions;

	void Reset()
	{
		Vertices.Reset();
		Positions.Reset();
	}

	/** Save this vertex. Does not check that VertexID hasn't already been added. */
	void AddVertex(const FDynamicMesh3* Mesh, int VertexID)
	{
		Vertices.Add(VertexID);
		Positions.Add(Mesh->GetVertex(VertexID));
	}

	/** Apply saved positions to Mesh */
	void SetPositions(FDynamicMesh3* Mesh) const
	{
		const int Num = Vertices.Num();
		for (int k = 0; k < Num; ++k)
		{
			Mesh->SetVertex(Vertices[k], Positions[k]);
		}
	}
};



/**
 * FGroupTopologyDeformer supports deforming a Mesh based on an overlaid FGroupTopology.
 * First the client defines a set of "Handle" elements (Faces/Corners/Edges) using SetActiveHandleX(). 
 * The client will provide new vertex positions for the vertices of these elements via the 
 * HandleVertexDeformFunc argument to UpdateSolution(). Once the Handle vertices have been
 * updated, the deformer solves for updated vertex positions in the GroupTopology Faces
 * that are adjacent to the handles. This region is referred to as the "ROI" (Region-of-Interest)
 * 
 * The default deformation is to first solve for the updated edges, and then
 * solve for updated faces. This is done via linear encoding of the edge and face vertices
 * relative to their boundaries (edge boundary is endpoint corners, face boundary is edges).
 * 
 * Various functions can be overrided to customize behavior.
 * 
 */
class FGroupTopologyDeformer
{
public:
	virtual ~FGroupTopologyDeformer() = default;

	/** Set the Mesh and Topology to use for the deformation */
	UE_API void Initialize(const FDynamicMesh3* Mesh, const FGroupTopology* Topology);
	
	const FDynamicMesh3* GetMesh() const { return Mesh; }
	const FGroupTopology* GetTopology() const { return Topology; }

	//
	// Handle setup/configuraiton
	// 

	/**
	 * Set the active handle to the given Faces
	 */
	UE_API virtual void SetActiveHandleFaces(const TArray<int>& FaceGroupIDs);

	/**
	 * Set the active handle to the given Corners
	 */
	UE_API virtual void SetActiveHandleCorners(const TArray<int>& TopologyCornerIDs);

	/**
	 * Set the active handle to the given Edges
	 */
	UE_API virtual void SetActiveHandleEdges(const TArray<int>& TopologyEdgeIDs);


	/**
	 * Set the active handle from GeometrySelection
	 */
	UE_API virtual void SetActiveHandleFromSelection(const FGeometrySelection& Selection);

	
	/**
	 * Set the active handle to the given Faces, Edges, Corners
	 */
	UE_API virtual void SetActiveHandle(const TArray<int>& FaceGroupIDs, const TArray<int>& TopologyEdgeIDs, const TArray<int>& TopologyCornerIDs);
	
	//
	// Solving
	// 

	/**
	 * Update TargetMesh by first calling HandleVertexDeformFunc() to get new
	 * handle vertex positions, then solving for new ROI vertex positions.
	 */
	UE_API virtual void UpdateSolution(FDynamicMesh3* TargetMesh, const TFunction<FVector3d(FDynamicMesh3* Mesh, int)>& HandleVertexDeformFunc);

	/**
	 * Restore the Handle and ROI vertex positions to their initial state
	 */
	UE_API virtual void ClearSolution(FDynamicMesh3* TargetMesh);



	/** @return the set of handle vertices */
	const TSet<int>& GetHandleVertices() const { return HandleVertices; }
	/** @return the set of all vertices whose positions will be modified by the deformation*/
	const TSet<int>& GetModifiedVertices() const { return ModifiedVertices; }
	/** @return the set of all overlay normals that will be modified by the deformation*/
	const TSet<int>& GetModifiedOverlayNormals() const { return ModifiedOverlayNormals; }

	/** Call EdgeSpanFunc for every group FEdgeSpan in the modified-area ROI that will be modified by the solver (does not include edges that are encompassed by the Handle) */
	void EnumerateROIEdges(TFunctionRef<void(const FEdgeSpan& EdgeSpan)> EdgeSpanFunc) const
	{
		for (const FROIEdge& Edge : ROIEdges)
		{
			EdgeSpanFunc(Edge.Span);
		}
	}

protected:
	//
	// These are functions that subclasses may wish to override
	// 

	/** reset all internal data structures, eg when changing handle */
	UE_API virtual void Reset();

	/**
	 * Populate the internal ROI data structures based on the given HandleGroups and ROIGroups.
	 * HandleGroups will be empty if the Handle is a set of Vertices or Edges.
	 */
	UE_API virtual void CalculateROI(const TArray<int>& HandleGroups, const TArray<int>& ROIGroups);

	/**
	 * Save the positions of all vertices that will be modified
	 */
	UE_API virtual void SaveInitialPositions();

	/**
	 * Precompute the representation of the ROI vertices at the initial positions.
	 */
	UE_API virtual void ComputeEncoding();

protected:
	const FDynamicMesh3* Mesh = nullptr;
	const FGroupTopology* Topology = nullptr;

	FVertexPositionCache InitialPositions;
	TSet<int> ModifiedVertices;
	TSet<int> HandleVertices;
	TSet<int> HandleBoundaryVertices;
	TSet<int> FixedBoundaryVertices;
	TSet<int> ROIEdgeVertices;
	TSet<int> FaceVertsTemp;
	TSet<int> FaceBoundaryVertsTemp;
	TSet<int> ModifiedOverlayNormals;

	struct FROIEdge
	{
		int EdgeIndex;
		FEdgeSpan Span;
	};
	TArray<FROIEdge> ROIEdges;

	struct FROIFace
	{
		TArray<int> BoundaryVerts;
		TArray<int> InteriorVerts;
	};
	TArray<FROIFace> ROIFaces;


	//
	// Deformation strategy (should be in subclass?)
	// 

	struct FEdgeVertexEncoding
	{
		double T;
		FVector3d Delta;
		FEdgeVertexEncoding() { T = 0; Delta = FVector3d::Zero(); }
	};
	struct FEdgeEncoding
	{
		TArray<FEdgeVertexEncoding> Vertices;
	};
	TArray<FEdgeEncoding> EdgeEncodings;



	struct FFaceVertexEncoding
	{
		TArray<double> Weights;
		TArray<FVector3d> Deltas;
	};
	struct FFaceEncoding
	{
		TArray<FFaceVertexEncoding> Vertices;
	};
	TArray<FFaceEncoding> FaceEncodings;

};



} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
