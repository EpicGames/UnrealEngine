// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "EdgeLoop.h"

namespace UE
{
namespace Geometry
{


namespace HoleFillUtil
{
	/**
	 * Fill in overlay elements on new triangles, by copying values from provided edges.
	 * Used to fill in overlay values after a hole fill.
	 * Only supports fills where at most one new vertex is added inside the hole (e.g., does not support smooth hole fill)
	 * 
	 * @param Mesh Mesh holding the Overlay
	 * @param Overlay The dynamic mesh overlay to update
	 * @param PerEdgeIDFn Enumerator function, which will call a passed-in method per Edge ID on the hole border
	 * @param NewTriangles Triangles that fill the hole
	 * @param NewVertex If not invalid, the at-most one vertex created when filling the hole.
	 */
	template<typename OverlayType, typename EdgeEnumerateFn>
	void FillOverlayElements(const FDynamicMesh3& Mesh, OverlayType* Overlay, EdgeEnumerateFn PerEdgeIDFn, TConstArrayView<int32> NewTriangles, int32 NewVertex = IndexConstants::InvalidID)
	{
		TMap<int32, int32> VIDToElementID;
		bool bHasNewVertex = NewVertex != IndexConstants::InvalidID;


		PerEdgeIDFn([&Mesh, &Overlay, bHasNewVertex, &VIDToElementID](int32 EdgeID)
		{
			FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeID);
			int32 SourceTID = Overlay->IsSetTriangle(Edge.Tri.A) ? Edge.Tri.A : ((Mesh.IsTriangle(Edge.Tri.B) && Overlay->IsSetTriangle(Edge.Tri.B)) ? Edge.Tri.B : IndexConstants::InvalidID);
			if (SourceTID != IndexConstants::InvalidID)
			{
				FIndex3i SetTri = Mesh.GetTriangle(SourceTID);
				FIndex3i SetElTri = Overlay->GetTriangle(SourceTID);
				for (int32 SubIdx = 0; SubIdx < 2; ++SubIdx)
				{
					int32 SourceElementID = SetElTri[SetTri.IndexOf(Edge.Vert[SubIdx])];
					VIDToElementID.Add(Edge.Vert[SubIdx], SourceElementID);
				}
			}
		});
		typename OverlayType::VectorType AvgValue(0);
		if (bHasNewVertex && !VIDToElementID.IsEmpty())
		{
			for (TPair<int32, int32> VIDElemIDPair : VIDToElementID)
			{
				AvgValue += Overlay->GetElement(VIDElemIDPair.Value);
			}
			AvgValue /= (float)VIDToElementID.Num();
		}
		for (int32 NewTID : NewTriangles)
		{
			FIndex3i NewTri = Mesh.GetTriangle(NewTID);
			FIndex3i NewElTri;
			bool bAllFound = true;
			int32 AddElementSubIdx = IndexConstants::InvalidID;
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				int32 VID = NewTri[SubIdx];
				if (int32* FoundElementID = VIDToElementID.Find(VID))
				{
					NewElTri[SubIdx] = *FoundElementID;
				}
				else if (VID == NewVertex)
				{
					// we'll add this element below, if the triangle as a whole will be set
					AddElementSubIdx = SubIdx;
				}
				else
				{
					bAllFound = false;
					break;
				}
			}
			if (bAllFound)
			{
				if (AddElementSubIdx != IndexConstants::InvalidID)
				{
					int32 NewElementID = Overlay->AppendElement(AvgValue);
					VIDToElementID.Add(NewTri[AddElementSubIdx], NewElementID);
					NewElTri[AddElementSubIdx] = NewElementID;
				}
				Overlay->SetTriangle(NewTID, NewElTri);
			}
		}
	}

	/**
	 * Fill in color overlay elements on new triangles, by copying values from provided edges. If mesh doesn't have a color overlay, does nothing.
	 * Used to fill in overlay values after a hole fill.
	 * Only supports fills where at most one new vertex is added inside the hole (e.g., does not support smooth hole fill)
	 *
	 * @param Mesh The mesh to update
	 * @param HoleEdges The edges at the boundary of the hole
	 * @param NewTriangles Triangles that fill the hole
	 * @param NewVertex If not invalid, the at-most one vertex created when filling the hole.
	 */
	inline void FillColorOverlay(FDynamicMesh3& Mesh, TConstArrayView<int32> HoleEdges, TConstArrayView<int32> NewTriangles, int32 NewVertex = IndexConstants::InvalidID)
	{
		if (Mesh.HasAttributes() && Mesh.Attributes()->HasPrimaryColors())
		{
			FillOverlayElements<FDynamicMeshColorOverlay>(Mesh, Mesh.Attributes()->PrimaryColors(),
				[&Mesh, HoleEdges](TFunctionRef<void(int32)> ProcessEID)
				{
					for (int32 EID : HoleEdges)
					{
						if (Mesh.IsEdge(EID))
						{
							ProcessEID(EID);
						}
					}
				},
				NewTriangles, NewVertex);
		}
	}

	/**
	 * Fill in color overlay elements on new triangles, by copying values from edges along the provided vertex loops. If mesh doesn't have a color overlay, does nothing.
	 * Used to fill in overlay values after a hole fill.
	 * Only supports fills where at most one new vertex is added inside the hole (e.g., does not support smooth hole fill)
	 *
	 * @param Mesh The mesh to update
	 * @param HoleVertexLoops Vertex loops at the boundary of the hole
	 * @param NewTriangles Triangles that fill the hole
	 * @param NewVertex If not invalid, the at-most one vertex created when filling the hole.
	 */
	inline void FillColorOverlay(FDynamicMesh3& Mesh, const TArray<TArray<int32>>& HoleVertexLoops, TConstArrayView<int32> NewTriangles, int32 NewVertex = IndexConstants::InvalidID)
	{
		if (Mesh.HasAttributes() && Mesh.Attributes()->HasPrimaryColors())
		{
			FillOverlayElements<FDynamicMeshColorOverlay>(Mesh, Mesh.Attributes()->PrimaryColors(), 
				[&Mesh, &HoleVertexLoops](TFunctionRef<void(int32)> ProcessEID)
				{
					for (const TArray<int32>& Loop : HoleVertexLoops)
					{
						for (int32 Idx = 0, PrevIdx = Loop.Num() - 1; Idx < Loop.Num(); PrevIdx = Idx++)
						{
							int32 EID = Mesh.FindEdge(Loop[Idx], Loop[PrevIdx]);
							if (EID != IndexConstants::InvalidID)
							{
								ProcessEID(EID);
							}
						}
					}
				}, NewTriangles, NewVertex
			);
		}
	}
}


} // end namespace UE::Geometry
} // end namespace UE