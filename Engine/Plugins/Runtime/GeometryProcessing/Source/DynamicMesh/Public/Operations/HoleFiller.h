// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

namespace UE
{
namespace Geometry
{


class IHoleFiller
{
public:
	TArray<int32> NewTriangles;

	virtual ~IHoleFiller() = default;

	virtual bool Fill(int32 GroupID = -1) = 0;	
};


} // end namespace UE::Geometry
} // end namespace UE