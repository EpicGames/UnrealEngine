// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "MeshRegionBoundaryLoops.h"
#include "ProjectionTargets.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
class FMeshNormals;
class FDynamicMeshChangeTracker;

/**
 * FInsetMeshRegion implements local inset of a mesh region.
 * The selected triangles are separated and then stitched back together, creating
 * an new strip of triangles around their border (s). The boundary loop vertices
 * are inset by creating an offset line for each boundary loop edge, and then
 * finding closest-points between the sequential edge pairs. 
 *
 * Complex input regions are handled, eg it can be multiple disconnected components, donut-shaped, etc
 *
 * Each quad of the border loop is assigned it's own normal and UVs (ie each is a separate UV-island)
 */
class FInsetMeshRegion
{
public:

	//
	// Inputs
	//

	/** The mesh that we are modifying */
	FDynamicMesh3* Mesh;

	/** The triangle region we are modifying */
	TArray<int32> Triangles;

	/** Inset by this distance */
	double InsetDistance = 1.0;

	/** quads on the stitch loop are planar-projected and scaled by this amount */
	float UVScaleFactor = 1.0f;

	/** reproject positions onto input surface */
	bool bReproject = true;


	/** update positions of any non-boundary vertices in inset regions (via laplacian solve) */
	bool bSolveRegionInteriors = true;

	/** determines how strongly laplacian solve constraints are enforced. 0 means hard constraint. Valid range [0,1] */
	float Softness = 0.0;

	/** Linear attenuation of area correction factor, valid range [0,1], 0 means ignore area correction entirely */
	float AreaCorrection = 1.0;

	/** projection target */
	FMeshProjectionTarget* ProjectionTarget;

	/** If set, change tracker will be updated based on edit */
	TUniquePtr<FDynamicMeshChangeTracker> ChangeTracker;

	//
	// Outputs
	//

	/**
	 * Inset information for a single connected component
	 */
	struct FInsetInfo
	{
		/** Set of triangles for this region */
		TArray<int32> InitialTriangles;
		/** Initial loops on the mesh */
		TArray<FEdgeLoop> BaseLoops;
		/** 
		 * Inset loops on the mesh. If the region had bowtie vertices (in terms of selection, not
		 * even necessarily in terms of original mesh topology), then the number of loops here may
		 * no longer match number of loops in BaseLoops, StitchTriangles, etc (for example, imagine
		 * a circular hole tangent inside a circular region; when the tangent bowtie is split, this
		 * becomes one C-shaped loop instead of two nested ones)
		 */
		TArray<FEdgeLoop> InsetLoops;

		/** Lists of triangle-strip "tubes" that connect each loop-pair */
		TArray<TArray<int>> StitchTriangles;
		/** List of group ids / polygon ids on each triangle-strip "tube" */
		TArray<TArray<int>> StitchPolygonIDs;
	};

	/**
	 * List of Inset regions/components
	 */
	TArray<FInsetInfo> InsetRegions;

	/**
	 * List of all triangles created/modified by this operation
	 */
	TArray<int32> AllModifiedTriangles;


public:
	UE_API FInsetMeshRegion(FDynamicMesh3* mesh);

	UE_API virtual ~FInsetMeshRegion();


	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		// @todo calculate MeshBoundaryLoops and make sure it is valid

		// is there any reason we couldn't do this??

		return EOperationValidationResult::Ok;
	}


	/**
	 * Apply the Inset operation to the input mesh.
	 * @return true if the algorithm succeeds
	 */
	UE_API virtual bool Apply();


protected:

	UE_API virtual bool ApplyInset(FInsetInfo& Region, FMeshNormals* UseNormals = nullptr);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
