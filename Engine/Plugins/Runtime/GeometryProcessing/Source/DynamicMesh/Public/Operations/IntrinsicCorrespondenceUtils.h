// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/InfoTypes.h"
#include "Util/DynamicVector.h"
#include "VectorTypes.h"
#include "MathUtil.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

// Utilities (primarily structs) used to form a relationship between an intrinsic mesh and the surface mesh that it lives on.
namespace IntrinsicCorrespondenceUtils
{
	
	// Location on an extrinsic (surface) mesh stored as a union.
	struct FSurfacePoint
	{
		enum class EPositionType
		{
			Vertex,
			Edge,
			Triangle
		};

		// constructors respect the correct position type
		UE_API FSurfacePoint(); // defaults to invalid vertex position
		UE_API FSurfacePoint(const int32 VID);
		UE_API FSurfacePoint(const int32 EdgeID, const double Alpha);
		UE_API FSurfacePoint(const int32 TriID, const FVector3d& BaryCentrics);

		struct FVertexPosition
		{
			int32 VID;                      // Reference (extrinsic) vertex
		};
		struct FEdgePosition
		{
			int32 EdgeID;                  // Reference (extrinsic) edge
			double Alpha;                  // Lerp coordinate along referenced edge as Pos(Edge.A) (Alpha) + (1-Alpha) Pos(Edge.B)
		};
		struct FTrianglePosition
		{
			int32 TriID;                    // Reference (extrinsic) triangle
			FVector3d  BarycentricCoords;   // Barycentric coordinates within referenced triangle
		};
		union FSurfacePositionUnion
		{
			FSurfacePositionUnion() { VertexPosition.VID = -1;}
			FVertexPosition   VertexPosition;
			FEdgePosition     EdgePosition;
			FTrianglePosition TriPosition;
		};

		EPositionType PositionType;      // Position type, needed to correctly interpret the Position union
		FSurfacePositionUnion Position;  // Position relative to vertex, edge, or triangle. 
	};

	/**
	* @return the R3 position of this surface point as applied to the input Mesh
	* on return bValidPoint will be false if the edge / vertex / triangle ID referenced by this surface
	* point is not an element of the mesh in question
	* 
	* @param SurfacePoint - Position defined relative to the Mesh
	* @param Mesh         - Dynamic mesh that defines a surface
	* @param bValidPoint  - will be false if the vert, edge or tri that defines the surface point is not a member of Mesh
	*/
	FVector3d DYNAMICMESH_API AsR3Position(const FSurfacePoint& SurfacePoint, const FDynamicMesh3& Mesh, bool& bValidPoint);

	/**
	* @return true if the SurfacePoint corresponds to a vertex
	*/
	static inline bool IsVertexPoint(const FSurfacePoint& SurfacePoint)
	{
		return SurfacePoint.PositionType == FSurfacePoint::EPositionType::Vertex;
	}
	/**
	* @return true if the SurfacePoint corresponds to an edge
	*/
	static inline bool IsEdgePoint(const FSurfacePoint& SurfacePoint)
	{
		return SurfacePoint.PositionType == FSurfacePoint::EPositionType::Edge;
	}
	/**
	* @return true if the SurfacePoint corresponds to a face
	*/
	static inline bool IsFacePoint(const FSurfacePoint& SurfacePoint)
	{
		return SurfacePoint.PositionType == FSurfacePoint::EPositionType::Triangle;
	}

	// A Connection stores the information needed to define a local reference direction for each vertex and triangle.
	struct FMeshConnection
	{
		FMeshConnection() : SurfaceMesh(nullptr) {};

		UE_API FMeshConnection(const FDynamicMesh3& SurfMesh);

		/** Reset the connection to the given (not owned) surface mesh*/
		UE_API void Reset(const FDynamicMesh3& SurfMesh);     

		/** Reset the connection to used a internally owned surface mesh*/
		UE_API void Reset(TUniquePtr<FDynamicMesh3>& SurfMesh);
		

		

		const FDynamicMesh3* SurfaceMesh;                  // Pointer back the surface mesh used by the connection.  Will agree with OwnedSurfaceMesh if that is not null 
		TUniquePtr<FDynamicMesh3> OwnedSurfaceMesh;        // Optional, the connection can own a surface mesh.
		TArray<int32> VIDToReferenceEID;                   // Reference (original mesh) Edge per (original mesh) Vertex, defines local zero angle on tangent plane
		TArray<int32> TIDToReferenceEID;                   // Reference (original mesh) Edge per (original mesh) Triangle, defines local zero angle on tangent plane
	};

	/**
	* Normal Coordinates as defined by "Discrete Conformal Equivalence of Polyhedral Surfaces" - Gillespi, Springborn, Crane, TOG V40 No4, 2021
	* This structure assumes that the surface mesh is fixed, and the intrinsic mesh shares the same vertex set as the surface mesh and
	* is initialized with the same connectivity as the surface mesh. Note, it is assumed that the surface mesh has no bow-ties.
	*/
	struct FNormalCoordinates : public FMeshConnection
	{
		typedef FMeshConnection  MyBase;

		FNormalCoordinates(): MyBase() {}

		UE_API FNormalCoordinates(const FDynamicMesh3& SurfMesh);

		/** Reset the connection to the given (not owned) surface mesh*/
		UE_API void Reset(const FDynamicMesh3& SurfMesh);

		/** Reset the connection to used an internally owned surface mesh*/
		UE_API void Reset(TUniquePtr<FDynamicMesh3>& SurfMesh);



		struct FEdgeAndCrossingIdx
		{
			int32 TID;  // triangle traversed
			int32 EID;  // edge crossed.  This will be one of the three edges in GetTriEdges(TID)
			int32 CIdx; // index of curve relative to the edge crossed
		};

		/**
		* @return the number of surface edges that cross this intrinsic edge.  
		* Note: it is the responsibility of the caller to ensure IntrinsicEID is a valid edge on the intrinsic mesh.
		*/ 
		inline int32 NumEdgeCrossing(const int32 IntrinsicEID) const;

		/**
		* @return true if the specified intrinsic edge ( indicated by IntrinsicEID) is a segment of a surface edge.  
		* Note: By construction the initial intrinsic edges will have the same ID as the corresponding surface mesh edges, but edge splits 
		* will disrupt this.
		* Also note: it is the responsibility of the caller to ensure IntrinsicEID is a valid edge on the intrinsic mesh.
		*/ 
		inline bool IsSurfaceEdgeSegment(const int32 IntrinsicEID) const;

		/**
		* @return number of surface edges within the specified intrinsic triangle that are adjacent to the specified triangle corner.
		* This excludes any surface edges that are along the intrinsic triangle boundary edges.
		* note it is assumed that IntrinsicTriEIDs are edge ID of an intrinsic triangle, but no checking is done.
		*/
		inline int32 NumCornerEmanatingRefEdges(const FIndex3i& IntrinsicTriEIDs, const int32 TriCornerIndex) const;

		/**
		* @return number of surface edges that cross corner the specified corner of the intrinsic triangle defined by the given edges.
		* note, it is assumed that IntrinsicTriEIDs are edge ID of an intrinsic triangle, but no checking is done.
		*/
		inline int32 NumCornerCrossingRefEdges(const FIndex3i& IntrinsicTriEIDs, const int32 TriCornerIndex) const;

		/**
		* Update the NormalCoords and RoundaboutOrder after an intrinsic mesh edge flip. TriAEID, and TriBEID are the
		* triangle edges of the two affected triangles prior (!) to flipping the specified edge.
		* @return false if something failed
		*/
		UE_API bool  OnFlipEdge(const int32 T0ID, const FIndex3i& BeforeT0EIDs, const int32 T0OppVID,
			             const int32 T1ID, const FIndex3i& BeforeT1EIDs, const int32 T1OppVID,
			             const int32 FlippedEID);

		/**
		* @return the edgeID of the N-th out-going edge in the SurfaceMesh (counting CCW from the reference edge for this vertex).
		*/
		UE_API int32 GetNthEdgeID(const int32 VID, const int32 N) const;

		/**
		* @return the Order of the edge EID about VID (counting CCW from the surface reference edge for this vertex).
		* Note: this assumes that edge EID is adjacent to vertex VID - otherwise -1 is returned.
		*/
		UE_API int32 GetEdgeOrder(const int32 VID, const int32 SurfaceEID) const;

		TDynamicVector<int32>     NormalCoord;     // Per intrinsic edge: number of times src mesh edge crosses. By convention NormalCoord[EID] = -1 iff same as edge on surface mesh   
		TDynamicVector<FIndex3i>  RoundaboutOrder; // Per intrinsic Triangle, per (directed) edge: indicates the closest (when traveling ccw) reference mesh edge

		TArray<int32>             RefVertDegree;   // Per vertex in ref mesh, caches the result of counting the edges in RefMesh.VtxEdgeIter().  
		TArray<FIndex3i>          EdgeOrder;       // Per surface triangle, per directed edge: number of edges past reference edge traveling ccw.

	protected:
		/** rebuild the normal coordinate data in this connection. Assumes the base connection data is consistent with this surface mesh*/
		UE_API void RebuildNormalCoordinates(const FDynamicMesh3& SurfMesh);
	};


	/**
	* Signpost Coordinates inspired by "Navigating Intrinsic Triangulations" Sharp, Soliman and Crane [2019, ACM Transactions on Graphics]
	* This structure assumes that the surface mesh is fixed, but allows the intrinsic mesh to support operations like edge splits and triangle pokes.
	* is initialized with the same connectivity as the surface mesh.  Note, it is assumed that the surface mesh has no bow-ties.
	*/
	struct FSignpost : public FMeshConnection
	{
		typedef FMeshConnection  MyBase;
		UE_API FSignpost(const FDynamicMesh3& SurfaceMesh);

		/**
		* Update the signpost after an edge flip.  This requires information about the both the pre and post flip state.
		* @param EID                  - edge that was flipped
		* @param AdjTris              - IDs of the tris adjacent to flipped edge 
		* @param OpposingVerts        - verts opposite the edge, pre-flip.  i.e. EdgeFlipInfo.OpposingVerts
		* @param PreFlipIndexOf       - edge index in the EID in the pre-fliped tris. i.e. {GetTriEdges(Tris[0]).IndexOf(EID), GetTriEdges(Tris[1]).IndexOf(EID)}; 
		* @param Opp0NewInternalAngle - internal angle of opposing vert 0 after the flip. i.e. InternalAngles[Tris[1]][1]
		* @param Opp1NewInternalAngle - internal angle of opposing vert 1 after the flip. i.e  InternalAngles[Tris[0]][1];
		*/
		UE_API void OnFlipEdge(const int32 EID, const FIndex2i AdjTris, const FIndex2i OpposingVerts, const FIndex2i PreFlipIndexOf, 
		                const double Opp0NewInternalAngle, const double Opp1NewInternalAngle);

		struct FSurfaceTraceResult
		{
			FSurfacePoint SurfacePoint; // point on the surface mesh
			double Angle;               // polar angle relative to reference edge (on surface mesh) in the neighborhood of the surface point.
		};
	
		struct FGeometricInfo
		{
			double ToRadians = 1.;    // 2pi / sum of angles at this vertex.  Will be 1 if the Gaussian curvature is 0
			bool bIsInterior = false; // false if on boundary or at a bow-tie.
		};
		TDynamicVector<FVector3d>            IntrinsicEdgeAngles;             // Per-intrinsic Triangle - polar angles of each edge relative to the vertex it exits (i.e. edge 0 relative to vertex 0)
		TDynamicVector<FSurfacePoint>        IntrinsicVertexPositions;        // Per-intrinsic Vertex - Locates intrinsic mesh vertices relative to the surface defined by extrinsic mesh.
		TDynamicVector<FGeometricInfo>       GeometricVertexInfo;             // Per-intrinsic Vertex - Simple geometric information about the extrinsic surface in the neighborhood of intrinsic vert 	
	};

	int32 FNormalCoordinates::NumEdgeCrossing(const int32 IntrinsicEID) const
	{
		return TMathUtil<int32>::Max(0, NormalCoord[IntrinsicEID]);
	}

	bool FNormalCoordinates::IsSurfaceEdgeSegment(const int32 IntrinsicEID) const
	{
		return NormalCoord[IntrinsicEID] == -1;
	}



	int32 FNormalCoordinates::NumCornerEmanatingRefEdges(const FIndex3i& IntrinsicTriEIDs, const int32 TriCornerIndex) const
	{
		// see Gillespi et al, eqn 13
		const int32 k_index = TriCornerIndex;
		const int32 i_index = (TriCornerIndex + 1) % 3;
		const int32 j_index = (TriCornerIndex + 2) % 3;

		const int32 Edgeij = IntrinsicTriEIDs[i_index];
		const int32 Edgejk = IntrinsicTriEIDs[j_index];
		const int32 Edgeki = IntrinsicTriEIDs[k_index];

		const int32 N_ij = NumEdgeCrossing(Edgeij);
		const int32 N_jk = NumEdgeCrossing(Edgejk);
		const int32 N_ki = NumEdgeCrossing(Edgeki);



		return TMathUtil<int32>::Max(0, N_ij - N_jk - N_ki);
	}

	int32 FNormalCoordinates::NumCornerCrossingRefEdges(const FIndex3i& IntrinsicTriEIDs, const int32 TriCornerIndex) const
	{
		const int32 k_index = TriCornerIndex;
		const int32 i_index = (TriCornerIndex + 1) % 3;
		const int32 j_index = (TriCornerIndex + 2) % 3;

		const int32 Edgeij = IntrinsicTriEIDs[i_index];
		const int32 Edgejk = IntrinsicTriEIDs[j_index];
		const int32 Edgeki = IntrinsicTriEIDs[k_index];

		const int32 N_ij = NumEdgeCrossing(Edgeij);
		const int32 N_jk = NumEdgeCrossing(Edgejk);
		const int32 N_ki = NumEdgeCrossing(Edgeki);

		// see Gillespi et al, eqn 14

		const int32 Ejk_i = NumCornerEmanatingRefEdges(IntrinsicTriEIDs, i_index);
		const int32 Eki_j = NumCornerEmanatingRefEdges(IntrinsicTriEIDs, j_index);


		return  (TMathUtil<int32>::Max(0, N_jk + N_ki - N_ij) - Ejk_i - Eki_j) / 2;
	}


	// Utility to walk around a vertex (VID) visiting the adjacent edges and triangles.
	// Note: this isn't appropriate for bow-ties as it will terminate upon encountering a mesh edge.
	// If the vertex is on the mesh boundary, StartEID should correspond to the first boundary edge allowing 
	// for a ccw traversal of the vertex-adjacent faces.
	template <typename MeshType, typename FunctorType>
	bool VisitVertexAdjacentElements(const MeshType& Mesh, const int32 VID, const int32 StartEID, FunctorType& Functor)
	{
		constexpr static int InvalidID = IndexConstants::InvalidID;

		if (!Mesh.IsEdge(StartEID) || !Mesh.IsVertex(VID))
		{
			return false;
		}

		const int32 EndEID = [&]
			{
				int32 EEID = StartEID;
				for (int32 EID : Mesh.VtxEdgesItr(VID))
				{
					if (EID != StartEID && Mesh.IsBoundaryEdge(EID))
					{
						EEID = EID;
						break;
					}
				}
				return EEID;
			}();

		int32 CurEID        = StartEID;
		FIndex2i CurEdgeT   = Mesh.GetEdgeT(CurEID);
		int32 CurTID        = CurEdgeT[0];
		FIndex3i CurTriEIDs = Mesh.GetTriEdges(CurTID);
		int32 IndexOf       = CurTriEIDs.IndexOf(CurEID);

		if (Mesh.GetTriangle(CurTID)[IndexOf] != VID)
		{
			CurTID = CurEdgeT[1];
			if (CurTID == FDynamicMesh3::InvalidID)
			{
				checkSlow(0); // this shouldn't happen.  When the vertex is on a boundary, we should have selected the correct start edge. 
				return false;
			}
			CurTriEIDs = Mesh.GetTriEdges(CurTID);
			IndexOf = CurTriEIDs.IndexOf(StartEID);
		}

		const int32 NextEdgeOffset = 2;
		do
		{

			if (Functor(CurTID, CurEID, IndexOf))
			{
				break;
			}

			// advance to next edge
			CurEID   = CurTriEIDs[(NextEdgeOffset + IndexOf) % 3];
			CurEdgeT = Mesh.GetEdgeT(CurEID);
			CurTID   = (CurEdgeT[0] == CurTID) ? CurEdgeT[1] : CurEdgeT[0];

			if (CurTID == InvalidID)
			{
				checkSlow(CurEID == EndEID);
				break;
			}

			CurTriEIDs = Mesh.GetTriEdges(CurTID);
			IndexOf    = CurTriEIDs.IndexOf(CurEID);

			checkSlow(Mesh.GetTriangle(CurTID)[IndexOf] == VID);
		} while (CurEID != StartEID);

		return true;
	}
};
};
};

#undef UE_API
