// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/InfoTypes.h"
#include "Util/DynamicVector.h"
#include "IntrinsicCorrespondenceUtils.h"
#include "VectorTypes.h"
#include "MathUtil.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{
/**
* Intrinsic Meshes:
* 
* An Intrinsic Mesh can be thought of as a mesh that overlays another mesh, sharing
* the original surface.  The edges of the intrinsic mesh are constrained to be on the original mesh surface,
* but need not align with the original mesh edges. The lengths of the intrinsic edges are measured on 
* the surface of the original mesh (not the R3 distance).  The original mesh vertices are a subset of 
* the intrinsic mesh vertices and intrinsic edge splits and triangle pokes may introduce new intrinsic mesh vertices.
*
* Note: the implementation is a simple triangle-based mesh but there is no restriction that
* edge(a, b) is unique (e.g. multiple intrinsic edges may connect the same two vertices with different paths).
* In fact this mesh allows triangles and edges with repeated vertices. Such structures arise naturally
* with intrinsic triangulation, for example an edge that starts and ends at the
* same vertex may encircle a mesh.
* 
* The bookkeeping that manages the correspondence between locations on the intrinsic mesh and on the surface mesh
* is implemented either using integer-based "Normal Coordinates" 
* ( cf 'Integer Coordinates for Intrinsic Geometry Processing' M. Gillespie, N. Sharp, and K. Crane, TOG , Vol. 40,  December 2021. )
* as used by FIntrinsicEdgeFlipMesh and FIntrinsicMesh; 
* or alternately floating-point based generalized directions "Signpost data" 
* ( cf 'Signpost Coordinates inspired by "Navigating Intrinsic Triangulations' Sharp, Soliman and Crane [2019, ACM Transactions on Graphics])
* as used by FIntrinsicTriangulation.
*
*/

/**
* The FSimpleIntrinsicEdgeFlipMesh is designed to work with the function FlipToDelaunay() to make an
* Intrinsic Delaunay Triangulation (IDT) of the same surface as a given FDynamicMesh,
* allowing for more robust cotangent-Laplacians (see LaplacianMatrixAssembly.h)
* 
* 
* The vertices in this FSimpleIntrinsicEdgeFlipMesh are exactly those found in the surface mesh
* since it supports edge flips, but no other operations that change the mesh connectivity, 
* and no operations that affect the vertices.  
* 
* This mesh alone does not support computing the intersections of intrinsic edges with the surface mesh edges
* and as a result it can not be used to easily extract the path of an intrinsic edge.
* To do such computations FIntrinsicEdgeFlipMesh, FIntrinsicMesh or FIntrinsicTriangulation should be used.
*
* The API of FIntrinsicEdgeFlipMesh is similar to FDynamicMesh3, but only has a minimal subset of methods and some
* such as ::FlipEdge() and ::GetEdgeOpposingV() have very different implementations.
* 
* In addition to managing vertices, triangles and edges this class also tracks edge lengths (as measured on the surface mesh)
* and for convenience the internal angles for each triangle.
*/
class FSimpleIntrinsicEdgeFlipMesh
{
public:
	
	using FEdge = FDynamicMesh3::FEdge;
	
	// information collected when flipping an edge.
	struct FEdgeFlipInfo : DynamicMeshInfo::FEdgeFlipInfo
	{
		bool bHadSameOrientations = true; // will be false if the original triangles had different orientations.
	};
	
	/** InvalidID indicates that a vertex/edge/triangle ID is invalid */
	constexpr static int InvalidID = IndexConstants::InvalidID;

	FSimpleIntrinsicEdgeFlipMesh(){};

	/** Constructor does ID-matching deep copy the basic mesh topology (but no attributes or groups) */
	UE_API FSimpleIntrinsicEdgeFlipMesh(const FDynamicMesh3& SrcMesh);

	FSimpleIntrinsicEdgeFlipMesh(const FSimpleIntrinsicEdgeFlipMesh&) = delete;

	/** Discard all data */
	UE_API void Clear();

	/** Reset the mesh to an ID-matching deep copy the basic mesh topology (but no attributes or groups) */
	UE_API void Reset(const FDynamicMesh3& SrcMesh);
	/**
	* Flip a single edge in the Intrinsic Mesh.
	* Note: After a successful flip, both triangles will share the same orientation (matching that of the pre-flip Edge.Tri.A)
	* even if they had different orientations (e.g. cw & ccw) prior to flipping. 
	*       
	* @param EID indicates the edge to flip.
	* @param EdgeFlipInfo [out] populated on return.
	*
	* @return a success or failure info.
	*/
	UE_API EMeshResult FlipEdge(int32 EID, FEdgeFlipInfo& EdgeFlipInfo);


	/** @return the intrinsic edge length of specified edge */
	double GetEdgeLength(int32 EID) const
	{
		checkSlow(IsEdge(EID));
		return EdgeLengths[EID];
	}

	/**
	* @return the intrinsic edge lengths for specified tri
	* ordered to match the edges returned by ::GetEdges(TID)
	*/
	FVector3d GetTriEdgeLengths(const int32 TID) const
	{
		checkSlow(IsTriangle(TID));
		return GetEdgeLengthTriple(GetTriEdges(TID));
	}

	/**
	* @return the interior angle of specified triangle corner
	*
	* @param TID             - ID of containing triangle
	* @param IndexOf         - Index (i.e 0, 1, 2) of specified wedge of the triangle
	*/
	double GetTriInternalAngleR(const int32 TID, const int32 IndexOf) const
	{
		checkSlow(IndexOf == 0 || IndexOf == 1 || IndexOf == 2);
		return InternalAngles[TID][IndexOf];
	}


	/**
	* return FVector2( alpha_{ij} , beta_{ij} )
	* where alpha_{ij} and beta_{ij} are the two interior angles
	* opposite the specified edge, EID.
	*
	* Expected angles are in the [0,2pi) range
	*
	* Note: if EID is a mesh boundary beta_{ij} is replaced by -MAX_DOUBLE
	*/
	UE_API FVector2d EdgeOpposingAngles(int32 EID) const;


	/**
	* return 1/2 ( cot(alpha_{ij}) + cot(beta_{ij}) )
	* where alpha_{ij} and beta_{ij} are the to interior angles
	* opposite the specified edge, EID.
	*
	* Note: if EID is a mesh boundary, cot(alpha_ij) is returned
	*/
	UE_API double EdgeCotanWeight(int32 EID) const;

	// -- Minimal mesh interface --//


	/** @return enumerable object for valid edge indices suitable for use with range-based for, ie for ( int i : EdgeIndicesItr() ) */
	FRefCountVector::IndexEnumerable EdgeIndicesItr() const
	{
		return EdgeRefCounts.Indices();
	}

	/** @return enumerable object for one-ring edges of a vertex, suitable for use with range-based for, ie for ( int i : VtxEdgesItr(VertexID) ) */
	FSmallListSet::ValueEnumerable VtxEdgesItr(int32 VID) const
	{
		checkSlow(VertexRefCounts.IsValid(VID));
		return VertexEdgeLists.Values(VID);
	}

	/** @return the valence of a vertex (the number of connected edges) */
	int32 GetVtxEdgeCount(int32 VID) const
	{
		return VertexRefCounts.IsValid(VID) ? VertexEdgeLists.GetCount(VID) : -1;
	}

	/** @return number of triangles in the intrinsic mesh*/
	int32 TriangleCount() const
	{
		return (int32)TriangleRefCounts.GetCount();
	}

	/** @return number of edges in the intrinsic mesh*/
	int32 EdgeCount() const
	{
		return (int32)EdgeRefCounts.GetCount();
	}

	/** @return upper bound on the edge ID used in the mesh.  i.e. all edge IDs < MaxEdgeID*/
	int32 MaxEdgeID() const
	{
		return (int32)EdgeRefCounts.GetMaxIndex();
	}

	/** @return true if intrinsic mesh contains specified edge */
	inline bool IsEdge(int32 EdgeID) const
	{
		return EdgeRefCounts.IsValid(EdgeID);
	}

	/** @return true if specified edge is on the boundary of intrinsic mesh*/
	inline bool IsBoundaryEdge(int32 EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID].Tri[1] == InvalidID;
	}

	/** @return specified edge*/
	inline FEdge GetEdge(int32 EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID];
	}

	/** @return the vertex IDs that define the given edge in intrinsic mesh*/
	inline FIndex2i GetEdgeV(int32 EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID].Vert;
	}

	/** @return the triangle pair for an edge. The second triangle may be InvalidID */
	inline FIndex2i GetEdgeT(int32 EdgeID) const
	{
		checkSlow(IsEdge(EdgeID));
		return Edges[EdgeID].Tri;
	}

	/** @return the vertex pair opposite specified edge. The second vertex may be InvalidID */
	UE_API FIndex2i GetEdgeOpposingV(int32 EID) const;

	/** @return true if intrinsic mesh contains specified vertex */
	inline bool IsVertex(int32 VertexID) const
	{
		return VertexRefCounts.IsValid(VertexID);
	}

	/** @return true if the specified vertex is adjacent to a boundary edge*/
	inline bool IsBoundaryVertex(int32 VertexID) const
	{
		if (IsVertex(VertexID))
		{
			for (int32 EID : VertexEdgeLists.Values(VertexID))
			{
				if (Edges[EID].Tri[1] == InvalidID)
				{
					return true;
				}
			}
		}
		return false;
	}

	/** @return the r3 position of the specified vertex */
	inline FVector3d GetVertex(int32 VertexID) const
	{
		checkSlow(IsVertex(VertexID));
		return Vertices[VertexID];
	}

	/** @return upper bound on the vertex ID used in the mesh.  i.e. all edge IDs < MaxVertexID*/
	int32 MaxVertexID() const
	{
		return (int32)VertexRefCounts.GetMaxIndex();
	}

	/** @return upper bound on the triangle ID used in the mesh.  i.e. all triangle IDs < MaxTriangleID */
	int32 MaxTriangleID() const
	{
		return (int32)TriangleRefCounts.GetMaxIndex();
	}
	
	/** @return enumerable object for valid triangle indices suitable for use with range-based for, ie for ( int i : TriangleIndicesItr() ) */
	FRefCountVector::IndexEnumerable TriangleIndicesItr() const
	{
		return TriangleRefCounts.Indices();
	}

	/** @return true if intrinsic mesh contains specified triangle */
	inline bool IsTriangle(int32 TriangleID) const
	{
		return TriangleRefCounts.IsValid(TriangleID);
	}
	
	/** @return vertex ids for the three corners of the triangle (e.g. a, b, c)*/
	inline FIndex3i GetTriangle(int32 TriangleID) const
	{
		checkSlow(IsTriangle(TriangleID));
		return Triangles[TriangleID];
	}

	/** @return  reference to vertex ids for the three corners of the triangle (e.g. a, b, c)*/
	inline const FIndex3i& GetTriangleRef(int TriangleID) const
	{
		checkSlow(IsTriangle(TriangleID));
		return Triangles[TriangleID];
	}

	/** @return edge ids for the three sides of the triangle (e.g. ab, bc, ca)*/
	inline FIndex3i GetTriEdges(int32 TriangleID) const
	{
		checkSlow(IsTriangle(TriangleID));
		return TriangleEdges[TriangleID];
	}

	/** 
	* cyclic permutation of the src vector such that the 'Index' entry in the src is the first entry in the result
	* NB: Index must be 0, 1, or 2.
	*/ 
	template <typename Vector3Type>
	Vector3Type Permute(int32 Index, const Vector3Type& Src) const
	{
		checkSlow(Index == 0 || Index == 1 || Index == 2);
		return Vector3Type(Src[Index], Src[AddOneModThree[Index]], Src[AddTwoModThree[Index]]);
	}


	/**
	* @return false if the two triangles adjacent to the indicated edge have the different orientation (eg counter-clockwise and clockwise),
	* otherwise return true.
	* Note: by convention this will return true for boundary edges ( or invalid edges ).
	*/
	bool AreSameOrientation(int32 EID) const
	{
		if (!IsEdge(EID)) return true;

		const FSimpleIntrinsicEdgeFlipMesh::FEdge Edge = GetEdge(EID);

		// boundary edge is true by convention
		if (Edge.Tri.B == FSimpleIntrinsicEdgeFlipMesh::InvalidID) return true;

		const FIndex3i TriA = GetTriangle(Edge.Tri.A);
		const FIndex3i TriB = GetTriangle(Edge.Tri.B);
		const int V0 = Edge.Vert.A;
		const int V1 = Edge.Vert.B;
		const bool bOrderedAB = (IndexUtil::FindTriOrderedEdge(V0, V1, TriA) != FSimpleIntrinsicEdgeFlipMesh::InvalidID && IndexUtil::FindTriOrderedEdge(V1, V0, TriB) != FSimpleIntrinsicEdgeFlipMesh::InvalidID);
		const bool bOrderedBA = (IndexUtil::FindTriOrderedEdge(V1, V0, TriA) != FSimpleIntrinsicEdgeFlipMesh::InvalidID && IndexUtil::FindTriOrderedEdge(V0, V1, TriB) != FSimpleIntrinsicEdgeFlipMesh::InvalidID);

		// true if consistent orientation
		return bOrderedAB || bOrderedBA;
	}

protected:
	
	/** @return three requested edge lengths */
	FVector3d GetEdgeLengthTriple(const FIndex3i& EIDs) const
	{
		return FVector3d(EdgeLengths[EIDs.A], EdgeLengths[EIDs.B], EdgeLengths[EIDs.C]);
	}


	/**
	* computes the angles based on the edge lengths, so must have valid EdgeLengths for this tri before calling.
	* the order of the angles matches the order of the vertices that define the triangle, e.g. if (a, b, c) = GetTriangle()
	* the (angle at a, angle at b, angle at c) = ComputeTriInternalAnglesR
	*/  
	UE_API FVector3d ComputeTriInternalAnglesR(const int32 TID) const;

	/**
	* @return the intrinsic distance between the two vertices opposite the specified edge
	* NB: Will fail if EID is a boundary edge.
	*/
	UE_API double GetOpposingVerticesDistance(int32 EID) const;

protected:
	// basic mesh operations

	/**
	* update the mesh topology (e.g. triangles and edges) by doing an edge flip, but does not update the intrinsic edge lengths or angles.
	* Note: this requires precomputed FlipInfo.bHadSameOrientation. 
	*/
	UE_API EMeshResult FlipEdgeTopology(int32 eab, FEdgeFlipInfo& FlipInfo);

	/** update the existing edge to reference vertices 'a' and 'b'.*/
	void SetEdgeVerticesInternal(int32 EdgeID, int32 a, int32 b)
	{
		if (a > b)
		{
			Swap(a, b);
		}
		Edges[EdgeID].Vert[0] = a;
		Edges[EdgeID].Vert[1] = b;
	}
	/** update the existing edge to reference triangles t0 and t1*/
	inline void SetEdgeTrianglesInternal(int32 EdgeID, int32 t0, int32 t1)
	{
		Edges[EdgeID].Tri[0] = t0;
		Edges[EdgeID].Tri[1] = t1;
	}
	/** replace tOld with tNew in an existing edge */
	UE_API int32 ReplaceEdgeTriangle(int32 eID, int32 tOld, int32 tNew);
	/** replace eOld with eNew in and existing triangle */
	inline int32 ReplaceTriangleEdge(int32 tID, int32 eOld, int32 eNew)
	{
		FIndex3i& TriEdgeIDs = TriangleEdges[tID];
		for (int32 j = 0; j < 3; ++j)
		{
			if (TriEdgeIDs[j] == eOld)
			{
				TriEdgeIDs[j] = eNew;
				return j;
			}
		}
		return -1;
	}

	/**
	* @return vertex pair in specified edge, oriented by the given triangle.  
	* note: the vertices may be the same.
	* note: assumes EID is an edge in Triangle TID
	*/
	UE_API FIndex2i GetOrientedEdgeV(int32 EID, int32 TID) const;
protected:

	const int32 AddOneModThree[3] = { 1, 2, 0 };
	const int32 AddTwoModThree[3] = { 2, 0, 1 };

	// the basic mesh 

	TDynamicVector<FVector3d> Vertices{};   // list of vertex positions
	FRefCountVector VertexRefCounts{};      // Reference counts of vertex indices. For vertices that exist, the count is 1 + num_triangle_using_vertex. Iterate over this to find out which vertex indices are valid. 
	FSmallListSet VertexEdgeLists;          // List of per-vertex edge one-rings

	TDynamicVector<FIndex3i> Triangles;     // List of triangle vertex - index triplets[Vert0 Vert1 Vert2]
	FRefCountVector TriangleRefCounts{};    // Reference counts of triangle indices. Ref count is always 1 if the triangle exists. Iterate over this to find out which triangle indices are valid.
	TDynamicVector<FIndex3i> TriangleEdges; // List of triangle edge triplets [Edge0 Edge1 Edge2], note these correspond to [Vert0-Vert1, Vert1-Vert2, Vert2-Vert0] for a given triangle.

	TDynamicVector<FEdge> Edges;            // List of edge elements. An edge is four elements [VertA, VertB, Tri0, Tri1], where VertA < VertB, and Tri1 may be InvalidID (if the edge is a boundary edge)
	FRefCountVector EdgeRefCounts{};        // Reference counts of edge indices. Ref count is always 1 if the edge exists. Iterate over this to find out which edge indices are valid.

protected:
	
	// intrinsic quantities 
	
	TDynamicVector<double>            EdgeLengths;           // Length of each intrinsic edge in the Mesh: note these may differ from (Vert0 - Vert1).Lenght().
	TDynamicVector<FVector3d>         InternalAngles;        // Interior Angles for each triangle: note may differ from FDynamicMesh::GetTriInternalAnglesR(TID) as they are computed from intrinsic edge lengths

};


/**
* FSimpleIntrinsicMesh extends FSimpleIntrinsicEdgeFlipMesh to support edge splits and triangle pokes in addition to edge flips.
* The surface mesh vertices will be a subset of the intrinsic mesh vertex set. 
* 
* This is intended as a base class for more sophisticated intrinsic mesh classes, as it does not explicitly track
* the topology changes relative to the original surface mesh.  For example it does not record the locations on the surface mesh 
* of any new intrinsic vertices or the locations of intrinsic mesh edge intersections with the surface mesh edges.
*
* FIntrinsicMesh and FIntrinsicTriangulation are derived from this class and each maintain their own connection 
* with the surface mesh ( "normal coordinates" or "signpost data") that can track these surface mesh locations 
* and can be used to reconstruct paths (e.g. intrinsic edges) on the surface mesh.
*/
class FSimpleIntrinsicMesh : public FSimpleIntrinsicEdgeFlipMesh
{
public:
	typedef FSimpleIntrinsicEdgeFlipMesh  MyBase;
	using FEdge             = FSimpleIntrinsicEdgeFlipMesh::FEdge;
	using FEdgeFlipInfo     = FSimpleIntrinsicEdgeFlipMesh::FEdgeFlipInfo;
	using FPokeTriangleInfo = DynamicMeshInfo::FPokeTriangleInfo;
	using FEdgeSplitInfo    = DynamicMeshInfo::FEdgeSplitInfo;

	/** InvalidID indicates that a vertex/edge/triangle ID is invalid */
	constexpr static int InvalidID = IndexConstants::InvalidID;

	FSimpleIntrinsicMesh(){};

	/** Constructor does ID-matching deep copy the basic mesh topology (but no attributes or groups) */
	FSimpleIntrinsicMesh(const FDynamicMesh3& SrcMesh) : MyBase(SrcMesh)
	{};

	
	FSimpleIntrinsicMesh(const FSimpleIntrinsicMesh&) = delete;


	/**
	* Insert a new vertex inside an intrinsic triangle, ie do a 1 to 3 triangle split
	* @param TID             - index of triangle to poke
	* @param BaryCoordinates - barycentric coordinates of poke position
	* @param PokeInfo        - returned information about new and modified mesh elements
	* 
	* @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	*/
	UE_API EMeshResult PokeTriangle(int32 TID, const FVector3d& BaryCoordinates, FPokeTriangleInfo& PokeInfo);


	/**
	* Split an intrinsic edge of the mesh by inserting a vertex. This creates a new triangle on either side of the edge (ie a 2-4 split).
	* If the original edge had vertices [a,b], with triangles t0=[a,b,c] and t1=[b,a,d],  then the split inserts new vertex f.
	* After the split t0=[a,f,c] and t1=[f,a,d], and we have t2=[f,b,c] and t3=[f,d,b]  (it's best to draw it out on paper...)
	* SplitInfo.OriginalTriangles = {t0, t1} and SplitInfo.NewTriangles = {t2, t3}
	*
	* @param EdgeAB          - index of the edge to be split
	* @param SplitInfo       - returned information about new and modified mesh elements
	* @param SplitParameterT - defines the position along the edge that we split at, must be between 0 and 1, and is assumed to be based on the order of vertices in t0
	* 
	* @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	*/
	UE_API EMeshResult SplitEdge(int32 EdgeAB, FEdgeSplitInfo& SplitInfo, double SplitParameterT = 0.5);

protected:

	/**
	* Updates the mesh connectivity by adding a new vertex at the vertex-averaged location. Note: this does not update any of the intrinsic lengths, positions, angles, etc.
	* those quantities must be updated after.  See ::PokeTriangle()
	*/ 
	UE_API EMeshResult PokeTriangleTopology(int32 TID, FPokeTriangleInfo& PokeInfo);

	/**
	* Updates the mesh connectivity by adding a new vertex at the vertex-averaged location. Note: this does not update any of the intrinsic lengths, positions, angles, etc.
	* those quantities must be updated after.  See ::SplitEdge()
	*/
	UE_API EMeshResult SplitEdgeTopology(int32 EdgeAB, FEdgeSplitInfo& SplitInfo);

	/** allocate, or clear existing edge list for specified vertex*/
	inline void AllocateEdgesList(int32 VertexID)
	{
		if (VertexID < (int)VertexEdgeLists.Size())
		{
			VertexEdgeLists.Clear(VertexID);
		}
		VertexEdgeLists.AllocateAt(VertexID);
	}

	/** add new vertex to the mesh topology, does not update any of the intrinsic quantities*/
	inline int32 AppendVertex(const FVector3d& vPos)
	{
		int32 newVertID = VertexRefCounts.Allocate();
		Vertices.InsertAt(vPos, newVertID);
		AllocateEdgesList(newVertID);
		return newVertID;
	}

	/** add new edge to the mesh topology, does not update any of the intrinsic quantities*/
	inline int32 AddEdgeInternal(int32 vA, int32 vB, int32 tA, int32 tB = InvalidID)
	{
		if (vB < vA) {
			int32 t = vB; vB = vA; vA = t;
		}
		int32 eid = EdgeRefCounts.Allocate();
		Edges.InsertAt(FEdge{ {vA, vB},{tA, tB} }, eid);
		VertexEdgeLists.Insert(vA, eid);
		if (vA != vB)
		{
			VertexEdgeLists.Insert(vB, eid);
		}

		return eid;
	}
	/** add new triangle to the mesh topology, does not update any of the intrinsic quantities*/
	inline int32 AddTriangleInternal(int32 a, int32 b, int32 c, int32 e0, int32 e1, int32 e2)
	{
		int32 tid = TriangleRefCounts.Allocate();
		Triangles.InsertAt(FIndex3i(a, b, c), tid);
		TriangleEdges.InsertAt(FIndex3i(e0, e1, e2), tid);
		return tid;
	}
};

/**
* TEdgeCorrespondence allows intrinsic edges to be traced across the underlying surface mesh.
* When computing a large number intrinsic edge traces, this is to be preferred over the TraceEdge()
* intrinsic mesh methods because it avoids redundant computations.  But when tracing a small number
* of intrinsic edges the large up-front cost of computing information the entire mesh will make use
* of this class sub-optimal.
* 
* During construction a list of surface mesh edges crossed by each intrinsic edge is stored.
* The Trace operation for an intrinsic edge uses this information to unfold the crossed surface triangles
* into a 2d triangle strip upon which the actual trace takes place.
* 
* This is only for use with intrinsic mesh implementations that support  'normal coordinates' when doing flips/splits and pokes.
* such as the FIntrinsicMesh and FIntrinsicEdgeFlipMesh.
*/ 
template <typename IntrinsicMeshType>
struct TEdgeCorrespondence
{

	using FEdgeAndCrossingIdx = IntrinsicCorrespondenceUtils::FNormalCoordinates::FEdgeAndCrossingIdx;
	using FSurfacePoint = IntrinsicCorrespondenceUtils::FSurfacePoint;


	TEdgeCorrespondence(const IntrinsicMeshType& IntrinsicMesh)
	{
		Setup(IntrinsicMesh);
	}

	void Setup(const IntrinsicMeshType& IntrinsicMesh);
	/**
	* @return Array of FSurfacePoints relative to the ExtrinsicMesh that represent the specified intrinsic mesh edge.
	* This is directed relative to traversal of the first adjacent triangle (GetEdgeT().[0]), unless bReverse is true
	*
	* @param IntrinsicEID      - ID of the intrinsic mesh edge to be traced.
	* @param CoalesceThreshold - in barycentric units [0,1], edge crossings within this threshold are snapped to the nearest vertex.
	*                            and any resulting repetitions of vertex surface points are replaced with a single vertex surface point.
	* @param bReverse          - if true, trace edge direction is reversed
	*/
	TArray<UE::Geometry::IntrinsicCorrespondenceUtils::FSurfacePoint> TraceEdge(int32 IntrinsicEID, double CoalesceThreshold = 0., bool bReverse = false) const;


	const IntrinsicMeshType*               IntrinsicMesh;         // the intrinsic mesh
	const FDynamicMesh3*                   SurfaceMesh;           // the surface mesh 
	TArray<TArray<int32>>                  SurfaceEdgesCrossed;   // array of surface edge crossings per intrinsic edge.
};

/**
* FIntrinsicEdgeFlipMesh augments the base class FSimpleIntrinsicEdgeFlipMesh with
* the addition of a 'normal coordinate' based correspondence with the surface mesh.
*
* This class supports both the trace of intrinsic mesh edges across the surface mesh,
* and the trace of surface mesh edges across the intrinsic mesh.
* 
* Note, because only edge flips are supported (no edge split or triangle pokes), the
* vertex set of the intrinsic mesh and the surface (i.e. extrinsic) mesh will be the same.
* 
* The expected use
*		
*		// generate the implicit mesh from a surface mesh
*		FIntrinsicEdgeFlipMesh  ImplicitMesh( DynamicMesh );
*		
*			.. do some edge flips .. perhaps FlipToDelaunay( ImplicitMesh, ..)
* 
*      // if only a small percentage of the intrinsic edges are to be traced, this can be done directly as
*	   auto TracedIntrinsicEdge  =  ImplicitMesh.TraceEdge(IntrinsicEdgeID);
* 
*	   // alternately if a large number of edges will be traced, a correspondence will be more efficient. 
*      // create a correspondence that allows the edges of the implicit mesh to be traced on the surface mesh
*     
*      FIntrinsicEdgeFlipMesh::FEdgeCorrespondence  Correspondence = ImplicitMesh.ComputeEdgeCorrespondence();
*    
*	   for (.. doing multiple traces..  ) 
*      {
*          // trace desired edge across the surface mesh
*	       int32 IntrinsicEdgeID =  
*          auto TracedIntrinsicEdge  =  Correspondence.TraceEdge(IntrinsicEdgeID);
*      } 
*
* NB: The lifetime of this structure should not exceed that of the original
* FDynamicMesh as this class holds a pointer to that mesh to reference locations on its surface.
* Also, this class isn't currently expected to work with surface meshes that have bow-ties.
* 
* Note:  This assumes but does not check that the FDynamicMesh3 "Surface Mesh" has consistent orientation of all triangles.
*/
class FIntrinsicEdgeFlipMesh : public  FSimpleIntrinsicEdgeFlipMesh
{
public:
	typedef FSimpleIntrinsicEdgeFlipMesh MyBase;
	using FEdge               = FSimpleIntrinsicEdgeFlipMesh::FEdge;
	using FEdgeFlipInfo       = FSimpleIntrinsicEdgeFlipMesh::FEdgeFlipInfo;
	using FSurfacePoint       = IntrinsicCorrespondenceUtils::FSurfacePoint;
	using FEdgeAndCrossingIdx = IntrinsicCorrespondenceUtils::FNormalCoordinates::FEdgeAndCrossingIdx;

	UE_API FIntrinsicEdgeFlipMesh(const FDynamicMesh3& SurfaceMesh);

	FIntrinsicEdgeFlipMesh() = delete;
	FIntrinsicEdgeFlipMesh(const FIntrinsicEdgeFlipMesh&) = delete;

	/** @return pointer the reference mesh, note the FIntrinsicTriangulation does not manage the lifetime of this mesh */
	const FDynamicMesh3* GetExtrinsicMesh()  const
	{
		return NormalCoordinates.SurfaceMesh;
	}

	/**
	* @return Array of FSurfacePoints relative to the surface mesh that represent the specified intrinsic mesh edge.
	* This is directed relative to traversal of the first adjacent triangle (GetEdgeT().[0]), unless bReverse is true
	*
	* @param CoalesceThreshold - in barycentric units [0,1], edge crossings with in this threshold are snapped to the nearest vertex.
	*                            and any resulting repetitions of vertex surface points are replaced with a single vertex surface point.
	* @param bReverse          - if true, trace edge direction is reversed
	* 
	* Note: when tracing few edges, this method is preferred to using the FEdgeCorrespondence based tracing (see FEdgeCorrespondence below).  
	* But when tracing a large percentage of the mesh edges the FEdgeCorrespondence will be faster as it reuses intermediate results.
	*/
	UE_API TArray<FSurfacePoint> TraceEdge(int32 IntrinsicEID, double CoalesceThreshold = 0., bool bReverse = false) const;


	/**
	* Flip a single edge in the Intrinsic Mesh.
	* @param EdgeFlipInfo [out] populated on return.
	*
	* @return a success or failure info.
	*/
	UE_API EMeshResult FlipEdge(int32 EID, FEdgeFlipInfo& EdgeFlipInfo);


	typedef TEdgeCorrespondence< FIntrinsicEdgeFlipMesh >  FEdgeCorrespondence;

	FEdgeCorrespondence ComputeEdgeCorrespondence() const 
	{
		return FEdgeCorrespondence(*this);
	};

	/**
	* @return NormalCoordinate data structure that counts the number of regular surface (extrinsic) mesh edge-crossings for each intrinsic mesh edge. 
	*/
	const IntrinsicCorrespondenceUtils::FNormalCoordinates& GetNormalCoordinates() const
	{
		return NormalCoordinates;
	}

	/**
	* Trace the specified surface (extrinsic) mesh edge across the intrinsic mesh.  By default this traces from vertex EdgeV.A to vertex EdgeV.B
	* where EdgeV = SurfaceMesh.GetEdgeV(SurfaceEID).
	* @param SurfaceEID   - the surface edge to be traced
	* @param bReverse     - reverses the direction of the trace giving EdgeV.B -> EdgeV.A
	* 
	* @return array of intrinsic edges crossed between the specified vertices.  This can be used to compute the actual MeshPoints on the surface if desired.
	*/ 
	UE_API TArray<FEdgeAndCrossingIdx> GetImplicitEdgeCrossings(const int32 SurfaceEID, const bool bReverse = false) const;

	/**
	* Trace the specified surface (extrinsic) mesh edge across the intrinsic mesh.  By default this traces from vertex EdgeV.A to vertex EdgeV.B
	* where EdgeV = SurfaceMesh.GetEdgeV(SurfaceEID).
	* @param SurfaceEID   - the surface edge to be traced
	* @param bReverse     - reverses the direction of the trace giving EdgeV.B -> EdgeV.A
	* 
	* @return array of surface points defined relative to the intrinsic mesh.  In particular, the array will be a vertex followed by a sequence of (intrinsic) edge crossings.
	*/ 
	UE_API TArray<FSurfacePoint> TraceSurfaceEdge(int32 SurfaceEID, double CoalesceThreshold, bool bReverse = false) const;

	/**
	* @return true if the intrinsic vertex corresponds to a surface vertex.  
	*/ 
	bool IsSurfaceVertex(int VID) const 
	{
		return  (IsVertex(VID)) ? true : false;
	}

	/**
	* @return the position of the intrinsic vertex relative to the underlying surface mesh
	*/ 
	FSurfacePoint GetVertexSurfacePoint(int32 VID) const
	{
		// The FIntrinsicEdgeFlipMesh verts correspond to surface verts ( this mesh does not insert or delete verts).
		return FSurfacePoint(VID);
	}

protected:

	using FNormalCoordinates = IntrinsicCorrespondenceUtils::FNormalCoordinates;
	FNormalCoordinates  NormalCoordinates;      // connection and integer-based coordinates used in reconstructing {intrinsic, surface} edges on the {surface, intrinsic} mesh
};



/**
* FIntrinsicMesh extends the FSimpleIntrinsicMesh with the addition of a 'normal coordinate' 
* based correspondence with the surface mesh.
*
* This class supports both the trace of intrinsic mesh edges across the surface mesh,
* and the trace of surface mesh edges across the intrinsic mesh.
* 
* This supports mesh operations that: SplitEdge, PokeTriangle, FlipEdge.
*
* Both SplitEdge and PokeTriangle will generate new vertices and triangles in the
* intrinsic triangulation.  These vertices will live on the surface of the original mesh
* and their location (in R3 and relative to the original surface ) is computed by interpolation.
*
* NB: The lifetime of this structure should not exceed that of the original
* FDynamicMesh as this class holds a pointer to that mesh to reference locations on its surface.
* Also, this class isn't currently expected to work with surface meshes that have bow-ties.
* 
* Note:  This assumes but does not check that the FDynamicMesh3 "Surface Mesh" has consistent orientation of all triangles.
*/

class FIntrinsicMesh  : public  FSimpleIntrinsicMesh
{
public:
	typedef FSimpleIntrinsicMesh     MyBase;

	using FEdge = FSimpleIntrinsicMesh::FEdge;
	using FEdgeFlipInfo = FSimpleIntrinsicMesh::FEdgeFlipInfo;
	using FSurfacePoint = IntrinsicCorrespondenceUtils::FSurfacePoint;
	using FEdgeAndCrossingIdx = IntrinsicCorrespondenceUtils::FNormalCoordinates::FEdgeAndCrossingIdx;
	using FPokeTriangleInfo   = DynamicMeshInfo::FPokeTriangleInfo;
	using FEdgeSplitInfo      = DynamicMeshInfo::FEdgeSplitInfo;

	FIntrinsicMesh() = delete;
	FIntrinsicMesh(const FIntrinsicMesh&) = delete;

	UE_API FIntrinsicMesh(const FDynamicMesh3& SurfaceMesh);

	/** @return pointer the reference mesh, note the FIntrinsicTriangulation does not manage the lifetime of this mesh */
	const FDynamicMesh3* GetExtrinsicMesh()  const
	{
		return NormalCoordinates.SurfaceMesh;
	}


	/**
	* Flip a single edge in the Intrinsic Mesh.
	* @param EdgeFlipInfo [out] populated on return.
	*
	* @return a success or failure info.
	*/
	UE_API EMeshResult FlipEdge(int32 EID, FEdgeFlipInfo& EdgeFlipInfo);

	/**
	* Insert a new vertex inside an intrinsic triangle, ie do a 1 to 3 triangle split
	* @param TID             - index of triangle to poke
	* @param BaryCoordinates - barycentric coordinates of poke position
	* @param PokeInfo        - returned information about new and modified mesh elements
	* 
	* @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	*/
	UE_API EMeshResult PokeTriangle(int32 TID, const FVector3d& BaryCoordinates, FPokeTriangleInfo& PokeInfo);

	/**
	* Split an intrinsic edge of the mesh by inserting a vertex. This creates a new triangle on either side of the edge (ie a 2-4 split).
	* If the original edge had vertices [a,b], with triangles t0=[a,b,c] and t1=[b,a,d],  then the split inserts new vertex f.
	* After the split t0=[a,f,c] and t1=[f,a,d], and we have t2=[f,b,c] and t3=[f,d,b]  (it's best to draw it out on paper...)
	* SplitInfo.OriginalTriangles = {t0, t1} and SplitInfo.NewTriangles = {t2, t3}
	*
	* @param EdgeAB          - index of the edge to be split
	* @param SplitInfo       - returned information about new and modified mesh elements
	* @param SplitParameterT - defines the position along the edge that we split at, must be between 0 and 1, and is assumed to be based on the order of vertices in t0
	* 
	* @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	*/
	UE_API EMeshResult SplitEdge(int32 EdgeAB, FEdgeSplitInfo& SplitInfo, double SplitParameterT = 0.5);


	/**
	* @return NormalCoordinate data structure that counts regular surface (extrinsic) mesh edge-crossing for each intrinsic mesh edge. 
	*/
	const IntrinsicCorrespondenceUtils::FNormalCoordinates& GetNormalCoordinates() const
	{
		return NormalCoordinates;
	}

	/**
	* @return Array of FSurfacePoints relative to the surface mesh that represent the specified intrinsic mesh edge.
	* This is directed relative to traversal of the first adjacent triangle (GetEdgeT()[0]), unless bReverse is true
	*
	* @param CoalesceThreshold - in barycentric units [0,1], edge crossings with in this threshold are snapped to the nearest vertex.
	*                            and any resulting repetitions of vertex surface points are replaced with a single vertex surface point.
	* @param bReverse          - if true, trace edge direction is reversed
	* 
	* Note: when tracing few edges, this method is preferred to using the FEdgeCorrespondence based tracing (see FEdgeCorrespondence below).  
	* But when tracing a large percentage of the mesh edges the FEdgeCorrespondence will be faster as it reuses intermediate results.
	*/
	UE_API TArray<FSurfacePoint> TraceEdge(int32 IntrinsicEID, double CoalesceThreshold = 0., bool bReverse = false) const;

	/**
	* Trace the specified surface (extrinsic) mesh edge across the intrinsic mesh.  By default this traces from vertex EdgeV.A to vertex EdgeV.B
	* where EdgeV = SurfaceMesh.GetEdgeV(SurfaceEID).
	* @param SurfaceEID   - the surface edge to be traced
	* @param bReverse     - reverses the direction of the trace giving EdgeV.B -> EdgeV.A
	* 
	* @return array of intrinsic edges crossed between the specified vertices.  This can be used to compute the actual MeshPoints on the surface if desired.
	*/ 
	UE_API TArray<FEdgeAndCrossingIdx> GetImplicitEdgeCrossings(const int32 SurfaceEID, const bool bReverse = false) const;

	/**
	* Trace the specified surface (extrinsic) mesh edge across the intrinsic mesh.  By default this traces from vertex EdgeV.A to vertex EdgeV.B
	* where EdgeV = SurfaceMesh.GetEdgeV(SurfaceEID).
	* @param SurfaceEID   - the surface edge to be traced
	* @param bReverse     - reverses the direction of the trace giving EdgeV.B -> EdgeV.A
	* 
	* @return array of surface points defined relative to the intrinsic mesh.  In particular, the array will be a vertex followed by a sequence of (intrinsic) edge crossings.
	*/ 
	UE_API TArray<FSurfacePoint> TraceSurfaceEdge(int32 SurfaceEID, double CoalesceThreshold, bool bReverse = false) const;


	typedef TEdgeCorrespondence< FIntrinsicMesh >  FEdgeCorrespondence;

	FEdgeCorrespondence ComputeEdgeCorrespondence() const
	{
		return FEdgeCorrespondence(*this);
	};


	/**
	* @return position in r3 of an intrinsic vertex.
	*/
	FVector3d GetVertexPosition(int32 VID) const
	{
		return GetVertex(VID);
	}

	/**
	* @return the position of the intrinsic vertex relative to the underlying surface mesh
	*/ 
	FSurfacePoint GetVertexSurfacePoint(int32 VID) const
	{
		return IntrinsicVertexPositions[VID];
	}


protected:
	using FNormalCoordinates = IntrinsicCorrespondenceUtils::FNormalCoordinates;
	FNormalCoordinates  NormalCoordinates;      // connection and integer-based coordinates used in reconstructing {intrinsic, surface} edges on the {surface, intrinsic} mesh

	TDynamicVector<FSurfacePoint>        IntrinsicVertexPositions;        // Per-intrinsic Vertex - Locates intrinsic mesh vertices relative to the surface defined by extrinsic mesh.
};



/**
* Class that manages the intrinsic triangulation of a given FDynamicMesh.
*
* This supports mesh operations that: SplitEdge, PokeTriangle, FlipEdge.
*
*
* Both SplitEdge and PokeTriangle will generate new vertices and triangles in the
* intrinsic triangulation.  These vertices will live on the surface of the original mesh
* and their location (in R3 and relative to the original surface ) is computed by tracing
* a path on the original mesh.
*
* NB: The lifetime of this structure should not exceed that of the original
* FDynamicMesh as this class holds a pointer to that mesh to reference locations on its surface.
* Also, this class isn't currently expected to work with surface meshes that have bow-ties.
* 
* Note:  This assumes but does not check that the FDynamicMesh3 "Surface Mesh" has consistent orientation of all triangles.
*/ 
class FIntrinsicTriangulation : public  FSimpleIntrinsicMesh
{

public:

	typedef FSimpleIntrinsicMesh MyBase;

	using FEdgeFlipInfo = FSimpleIntrinsicMesh::FEdgeFlipInfo;
	using FEdgeSplitInfo = FSimpleIntrinsicMesh::FEdgeSplitInfo;
	using FPokeTriangleInfo = FSimpleIntrinsicMesh::FPokeTriangleInfo;
	using FSurfacePoint  = IntrinsicCorrespondenceUtils::FSurfacePoint;

	UE_API FIntrinsicTriangulation(const FDynamicMesh3& SrcMesh);

	FIntrinsicTriangulation() = delete;
	FIntrinsicTriangulation(const FIntrinsicTriangulation&) = delete;
	

	/** @return pointer the reference mesh, note the FIntrinsicTriangulation does not manage the lifetime of this mesh */
	const FDynamicMesh3* GetExtrinsicMesh()  const
	{
		return SignpostData.SurfaceMesh;
	}
	

	/**
	* @return Array of FSurfacePoints relative to the ExtrinsicMesh that represent the specified intrinsic mesh edge.
	* This is directed from vertex EdgeV.A to vertex EdgeV.B unless bReverse is true
	*
	* @param CoalesceThreshold - in barycentric units [0,1], edge crossings with in this threshold are snapped to the nearest vertex.
	*                            and any resulting repetitions of vertex surface points are replaced with a single vertex surface point.
	* @param bReverse          - if true, trace edge from EdgeV.B to EdgeV.A
	*/
	UE_API TArray<FIntrinsicTriangulation::FSurfacePoint> TraceEdge(int32 EID, double CoalesceThreshold = 0., bool bReverse = false) const;

	/**
	* @return FSurfacePoint for the specified intrinsic vertex.
	*/
	const FIntrinsicTriangulation::FSurfacePoint& GetVertexSurfacePoint(int32 VID) const
	{
		checkSlow(IsVertex(VID));
		return SignpostData.IntrinsicVertexPositions[VID];
	}

	/**
	* @return position in r3 of an intrinsic vertex.
	*/
	FVector3d GetVertexPosition(int32 VID) const
	{
		return GetVertex(VID);
	}
	
	/**
	* Flip a single edge in the Intrinsic Mesh.
	* @param EdgeFlipInfo [out] populated on return.
	*
	* @return a success or failure info.
	*/
	UE_API EMeshResult FlipEdge(int32 EID, FEdgeFlipInfo& EdgeFlipInfo);


	/**
	* Insert a new vertex inside an intrinsic triangle, ie do a 1 to 3 triangle split
	* @param TID             - index of triangle to poke
	* @param BaryCoordinates - barycentric coordinates of poke position
	* @param PokeInfo        - returned information about new and modified mesh elements
	* 
	* @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	*/
	UE_API EMeshResult PokeTriangle(int32 TID, const FVector3d& BaryCoordinates, FPokeTriangleInfo& PokeInfo);


	/**
	 * Split an intrinsic edge of the mesh by inserting a vertex. This creates a new triangle on either side of the edge (ie a 2-4 split).
	 * If the original edge had vertices [a,b], with triangles t0=[a,b,c] and t1=[b,a,d],  then the split inserts new vertex f.
	 * After the split t0=[a,f,c] and t1=[f,a,d], and we have t2=[f,b,c] and t3=[f,d,b]  (it's best to draw it out on paper...)
	 * SplitInfo.OriginalTriangles = {t0, t1} and SplitInfo.NewTriangles = {t2, t3}
	 *
	 * @param EdgeAB          - index of the edge to be split
	 * @param SplitInfo       - returned information about new and modified mesh elements
	 * @param SplitParameterT - defines the position along the edge that we split at, must be between 0 and 1, and is assumed to be based on the order of vertices in t0
	 * 
	 * @return Ok on success, or enum value indicates why operation cannot be applied. Mesh remains unmodified on error.
	 */
	UE_API EMeshResult SplitEdge(int32 EdgeAB, FEdgeSplitInfo& SplitInfo, double SplitParameterT = 0.5);

	

protected:

	

	/**
	* Given edge EID connects UpdateVID and StartVID, this update the SurfacePosition of the vertex UpdateVID by tracing the edge EID
	* a distance TraceDistance from StartVID.
	*
	* @return               - the polar angle of the tracing edge incident to VID ( relative to the local reference edge. )
	* @param UpdateVID      - ID of intrinsic vertex who's surface position is updated by this trace
	* @param StartVID       - ID of intrinsic vertex where the trace starts
	* @param TracePolarDir  - Local direction of the trace. 
	* @param TraceDistance  - Distance, measured on the extrinsic mesh surface to trace.
	*
	*/
	UE_API double UpdateVertexByEdgeTrace(const int32 UpdateVID, const int32 StartVID, const double TracePolarDir, const double TraceDistance);


protected:
	using FSignpost = IntrinsicCorrespondenceUtils::FSignpost;
	FSignpost SignpostData;     // connection and directional-based coordinates used in reconstructing {intrinsic, surface} edges on the {surface, intrinsic} mesh vert 		

};


/**
* Perform edge flips on intrinsic mesh until either the MaxFlipCount is reached or the mesh is fully Delaunay
*
* @param IntrinsicMesh - The mesh to be operated on.
* @param Uncorrected   - contains on return, all the edges that could not be flipped (e.g. edge shared by non-convex pair of triangles)
* 
* @return  the number of flips.
*/
int32 DYNAMICMESH_API FlipToDelaunay(FSimpleIntrinsicEdgeFlipMesh& IntrinsicMesh, TSet<int>& Uncorrected, const int32 MaxFlipCount = TMathUtilConstants<int>::MaxReal);
int32 DYNAMICMESH_API FlipToDelaunay(FIntrinsicEdgeFlipMesh& IntrinsicMesh, TSet<int>& Uncorrected, const int32 MaxFlipCount = TMathUtilConstants<int>::MaxReal);
int32 DYNAMICMESH_API FlipToDelaunay(FSimpleIntrinsicMesh& IntrinsicMesh, TSet<int>& Uncorrected, const int32 MaxFlipCount = TMathUtilConstants<int>::MaxReal);
int32 DYNAMICMESH_API FlipToDelaunay(FIntrinsicMesh& IntrinsicMesh, TSet<int>& Uncorrected, const int32 MaxFlipCount = TMathUtilConstants<int>::MaxReal);
int32 DYNAMICMESH_API FlipToDelaunay(FIntrinsicTriangulation& IntrinsicMesh, TSet<int>& Uncorrected, const int32 MaxFlipCount = TMathUtilConstants<int>::MaxReal);


} // end namespace Geometry
} // end namespace UE

#undef UE_API
