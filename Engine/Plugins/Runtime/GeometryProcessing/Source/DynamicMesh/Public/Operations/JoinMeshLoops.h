// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "MeshRegionBoundaryLoops.h"

#define UE_API DYNAMICMESH_API



namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
class FMeshNormals;

/**
 * FJoinMeshLoops connects two open loops of a mesh with a quad-strip.
 * A 1-1 match between the loop vertices is assumed.
 *
 * The fill strip currently is configured so that:
 *    - entire strip is assigned a new face group
 *    - the entire strip becomes a separate "normal island", ie hard normals at border with existing mesh
 *    - the strip is unwrapped into a UV rectangle starting at vertex 0 and ending back at 0 again
 */
class FJoinMeshLoops
{
public:

	//
	// Inputs
	//

	/** The mesh that we are modifying */
	FDynamicMesh3* Mesh;

	/** quads on the stitch loop are scaled by this amount */
	float UVScaleFactor = 1.0f;

	// first loop
	TArray<int32> LoopA;

	// second loop
	TArray<int32> LoopB;


	//
	// Outputs
	//

	// quads along the join strip, in order of loops. Each quad is two triangle indices
	TArray<FIndex2i> JoinQuads;
	// triangles of the quads flattend out (convenient), ie size = 2*NumQuads
	TArray<int32> JoinTriangles;

	// groups created by the operation (currently 1)
	TArray<int32> NewGroups;
	// group for each quad, ie size = NumQuads   (currently all have same value)
	TArray<int32> QuadGroups;

public:
	UE_API FJoinMeshLoops(FDynamicMesh3* Mesh);
	UE_API FJoinMeshLoops(FDynamicMesh3* Mesh, const TArray<int32>& LoopA, const TArray<int32>& LoopB);

	virtual ~FJoinMeshLoops() = default;


	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		if (LoopA.Num() != LoopB.Num())
		{
			return EOperationValidationResult::Failed_InvalidTopology;
		}
		return EOperationValidationResult::Ok;
	}

	/**
	 * Apply the Extrude operation to the input mesh.
	 * @return true if the algorithm succeeds
	 */
	UE_API virtual bool Apply();


protected:

	
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
