// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Operations/SkinWeightBinding.h"
#include "Skeletonization/MeshMedialAxisSampling.h"

#define UE_API DYNAMICMESH_API

namespace UE::Geometry::SkinBinding
{

struct FBindMedialSkeletonSettings
{
	// Search range in local medial skeleton to use when binding vertices --
	// i.e. a value of 1 will restrict vertices to bind to clusters within the 1-ring of their best-fit cluster
	int32 ClusterNeighborSearchRange = 1;

	// General skin weight binding settings
	FBindSettings BindSettings;

	// Options controlling conversion to an animation skeleton
	MedialAxis::FMedialSkeletonToTreeSkeletonOptions AnimationSkeletonOptions;
};

/**
 * Convert a medial skeleton + mesh to a skinned mesh, using the medial skeleton clusters 
 * to both define the animation skeleton and to help guide the skinning -- binding vertices
 * to bones within the local graph neighborhood of their corresponding medial skeleton cluster/medial sphere
 *
 * @param InMedialSkeleton Source medial skeleton
 * @param TargetMesh Mesh to receive skeleton attributes and skin weights
 * @param bOutMeshWasCompatible Reports whether the mesh was compatible with the medial skeleton (had a valid vertex->cluster mapping). If not, mesh can still be bound but will not respect some options.
 * @param WeightProfileName Skin weight profile name to use
 * @param Settings Options controlling the weight binding and skeleton conversion
 * @param RootIndex Specify the medial skeleton cluster to prefer as the animation skeleton's root bone. If INDEX_NONE, a default root will be selected automatically.
 * @return true on success, false if could not be bound (e.g. if medial skeleton was empty)
 */
UE_API bool CreateSkinWeightsFromMedialSkeleton(
	const MedialAxis::FMedialSkeleton& InMedialSkeleton,
	FDynamicMesh3& TargetMesh,
	bool& bOutMeshWasCompatible,
	FName WeightProfileName,
	const FBindMedialSkeletonSettings& Settings,
	int32 RootIndex = INDEX_NONE);

} // namespace UE::Geometry::SkinBinding

#undef UE_API
