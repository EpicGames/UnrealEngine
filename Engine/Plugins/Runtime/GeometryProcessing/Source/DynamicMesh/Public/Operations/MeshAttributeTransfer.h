// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

enum class EMeshAttributeTransferType
{
	MaterialID
};

enum class EMeshAttributeTransferError
{
	InvalidSource,
	InvalidTarget,
	SourceMissingAttribute,
	TargetMissingAttribute,
	InvalidOperationForInputs
};

/**
 * FMeshAttributeTransfer transfers attributes from a SourceMesh to a TargetMesh
 */
class FMeshAttributeTransfer
{
public:
	/** The mesh that we are transferring from */
	const FDynamicMesh3* SourceMesh;

	/** The mesh that we are transferring to */
	FDynamicMesh3* TargetMesh;

	/** What is being transferred */
	EMeshAttributeTransferType TransferType = EMeshAttributeTransferType::MaterialID;


	TArray<EMeshAttributeTransferError> Errors;

public:
	UE_API FMeshAttributeTransfer(const FDynamicMesh3* SourceMeshIn, FDynamicMesh3* TargetMeshIn);

	virtual ~FMeshAttributeTransfer() = default;

	/**
	 * Run the operation and modify .Mesh
	 * @return true if the algorithm succeeds
	 */
	UE_API virtual bool Apply();

protected:

	TUniquePtr<FDynamicMeshAABBTree3> SourceSpatial;


	UE_API bool Apply_MaterialID();
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
