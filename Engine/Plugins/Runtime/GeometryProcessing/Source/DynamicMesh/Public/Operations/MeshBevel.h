// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Util/ProgressCancel.h"
#include "Selections/QuadGridPatch.h"
#include "Math/InterpCurve.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FGroupTopology;
class FDynamicMeshChangeTracker;
class FEdgeLoop;


/**
 * FMeshBevel applies a "Bevel" operation to edges of a FDynamicMesh3. Bevel is not strictly well-defined,
 * there are a wide range of possible cases to handle and currently only some are supported.
 * See this website for a discussion of many interesting cases (most are not currently supported): https://wiki.blender.org/wiki/User:Howardt/Bevel
 * 
 * The bevel operation is applied in-place to the input mesh. The bevel is mesh-topological, ie implemented by
 * un-stitching mesh edges and inserting new triangles, rather than via booleans/etc. 
 * 
 * Currently supports:
 *   - Bevel an isolated closed loop of edges, that edge-loop becomes a quad-strip (simplest case)
 *   - Bevel a set of open edge spans that may meet a T-junctions
 *       - if incoming-span-valence at a vertex is >= 3, vertex is replaced by a polygon
 *       - if incoming-span-valence at a vertex is 1, bevel is "terminated" by expanding the vertex into an edge (this is the messiest case)
 *          - vertex-on-boundary is a special case that is simpler
 *       - edge spans are replaced by quad-strips
 * 
 * Currently does not support:
 *   - Beveling an isolated vertex
 *   - partial bevels of a GroupTopology Edge
 *   - multiple-segment bevel (eg to do rounds/etc)
 *   - ???
 * 
 * Generally this FMeshBevel is intended to be applied to group-boundary "edges" in a FGroupTopology. However the FGroupTopology
 * is currently only used in the initial setup, so other methods could be used to construct the bevel topology. 
 * 
 * Currently "updating" the bevel shape, once the topology is known, is not supported, but could be implemented
 * relatively easily, as all the relevant information is already tracked and stored.
 */
class FMeshBevel
{
public:

	//
	// Inputs
	// 

	/** Distance that bevel edges/vertices are inset from their initial position. Not guaranteed to hold for all vertices, though. */
	double InsetDistance = 5.0;
	
	/** Number of subdivisions inserted in each bevel strip */
	int32 NumSubdivisions = 0;

	/** 
	 * "Roundness" of the bevel profile. Ignored if Subdivisions = 0. Default=1 means try to be circular-ish. 
	 * Higher values pull towards a sharper crease. 0 is flat (ie no profile, a linear chamfer). If negative, bevel profile will be an inverted-arc 
	 */
	double RoundWeight = 1.0;

	/** Options for MaterialID assignment on the new triangles generated for the bevel */
	enum class EMaterialIDMode
	{
		ConstantMaterialID,
		InferMaterialID,
		InferMaterialID_ConstantIfAmbiguous
	};
	/** Which MaterialID assignment mode to use */
	EMaterialIDMode MaterialIDMode = EMaterialIDMode::ConstantMaterialID;
	/** Constant MaterialID used for various MaterialIDMode settings */
	int32 SetConstantMaterialID = 0;

	/** Set this member to support progress/cancel in the computations below */
	FProgressCancel* Progress = nullptr;


	//
	// Outputs
	//

	/** list of all new triangles created by the operation */
	TArray<int32> NewTriangles;

	/** status of the operation, warnings/errors may be returned here */
	FGeometryResult ResultInfo = FGeometryResult(EGeometryResultType::InProgress);

public:
	/**
	 * Initialize the bevel with all edges of the given GroupTopology
	 */
	UE_API void InitializeFromGroupTopology(const FDynamicMesh3& Mesh, const FGroupTopology& Topology);

	/**
	 * Initialize the bevel with the specified edges of a GroupTopology
	 */
	UE_API void InitializeFromGroupTopologyEdges(const FDynamicMesh3& Mesh, const FGroupTopology& Topology, const TArray<int32>& GroupEdges);

	/**
	 * Initialize the bevel with the specified mesh triangle edges
	 * @param IsCornerVertex Function to determine whether a vertex should be treated as a corner when beveling. Only affects vertices with exactly two adjacent TriangleEdges; otherwise corners will be automatically detected.
	 */
	UE_API void InitializeFromTriangleEdges(const FDynamicMesh3& Mesh, TConstArrayView<int32> TriangleEdges, TFunctionRef<bool(int32)> IsCornerVertex);

	/**
	 * Initialize the bevel with the specified mesh triangle edges
	 */
	void InitializeFromTriangleEdges(const FDynamicMesh3& Mesh, TConstArrayView<int32> TriangleEdges)
	{
		InitializeFromTriangleEdges(Mesh, TriangleEdges, [](int32) {return false;});
	}

	/**
	* Initialize the bevel with the specified faces of a GroupTopology
	* @return false if any selection-bowtie vertices were found, in this case we cannot compute the bevel
	*/
	UE_API bool InitializeFromGroupTopologyFaces(const FDynamicMesh3& Mesh, const FGroupTopology& Topology, const TArray<int32>& GroupFaces);

	/**
	* Initialize the bevel with border loops of the selected triangles. 
	* @return false if any selection-bowtie vertices were found, in this case we cannot compute the bevel
	*/
	UE_API bool InitializeFromTriangleSet(const FDynamicMesh3& Mesh, const TArray<int32>& Triangles);

	/**
	 * Apply the bevel operation to the mesh, and optionally track changes
	 */
	UE_API bool Apply(FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker);




public:

	//
	// Current Bevel computation strategy is basically to fully precompute all the necessary info for the entire bevel,
	// then "unlink" all the edges and vertices, and then stitch it all back together. 
	// Various incremental strategies were attempted, however certain cases like a two "bevel vertices" connected by a 
	// single mesh edge greatly complicate any attempt to decompose the problem into sub-parts. 
	// 
	// The data structures below are used to track this topological information during the operation.
	// Note that some parts of these data structures may become invalid/incorrect as the operation proceeds...
	//

	// POSSIBLE IMPROVEMENTS:
	// * compute wedges for Loop/Edge in setup? would avoid having to deal w/ possibly weird 
	//   configurations introduced by unlink of corners...

	// FBevelLoop is the accumulated data for a closed loop of mesh-edges, with no T-junctions w/ other bevel-edges.
	// This is the easiest case as each mesh-edge of the loop expands out into a quad, with no complex vertex-polygons/etc
	struct FBevelLoop
	{
		// initial topological information that defines what happens in unlink/displace/mesh steps
		TArray<int32> MeshVertices;		// sequential list of mesh vertex IDs along edge loop
		TArray<int32> MeshEdges;		// sequential list of mesh edge IDs along edge loop
		TArray<FIndex2i> MeshEdgeTris;	// the one or two triangles associated w/ each MeshEdges element in the input mesh
		TArray<FVector3d> InitialPositions;		// initial vertex positions

		// new mesh topology computed during unlink step
		TArray<int32> NewMeshVertices;		// list of new vertices on "other" side of unlinked edge, 1-1 with MeshVertices
		TArray<int32> NewMeshEdges;			// list of new edges on "other" side of unlinked edge, 1-1 with MeshEdges

		// buffers for new vertex positions computed during displace step
		TArray<FVector3d> NewPositions0;	// new positions for MeshVertices list
		TArray<FVector3d> NewPositions1;	// new positions for NewMeshVertices list

		// new geometry computed during mesh step
		TArray<int32> NewGroupIDs;
		TArray<FIndex2i> StripQuads;		// triangle-ID-pairs for each new quad added along edge, 1-1 with MeshEdges
		FQuadGridPatch StripQuadPatch;			// only initialized in multi-segment bevel
		TArray<FVector3d> NormalsA, NormalsB;	// normals at NewPositions0 and NewPositions1, before internal meshing is added (ie the tangent-boundary condition)
	};

	// FBevelEdge is the accumulated data for an open span of mesh-edges, which possibly meets up with other bevel-edges
	// at the vertices on either end of the span. Each mesh-edge of the bevel-edge will become a quad
	struct FBevelEdge
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FBevelEdge() = default;
		FBevelEdge(const FBevelEdge&) = default;
		FBevelEdge(FBevelEdge&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// initial topological information that defines what happens in unlink/displace/mesh steps
		int32 EdgeIndex;				// index of this BevelEdge in Edges array
		TArray<int32> MeshVertices;		// sequential list of mesh vertex IDs along edge
		TArray<int32> MeshEdges;		// sequential list of mesh edge IDs along edge
		TArray<FIndex2i> MeshEdgeTris;	// the one or two triangles associated w/ each MeshEdges element in the input mesh
		UE_DEPRECATED(5.5, "Mapping back to source topology is not used.")
		int32 GroupEdgeID;				// ID of this edge in external topology (eg FGroupTopology)
		UE_DEPRECATED(5.5, "Mapping back to source topology is not used.")
		FIndex2i GroupIDs;				// topological IDs of groups on either side of topological edge
		bool bEndpointBoundaryFlag[2];	// flag defining whether vertex at start/end of MeshVertices was a boundary vertex
		TArray<FVector3d> InitialPositions;		// initial vertex positions
		FIndex2i BevelVertices;			// indices of Bevel Vertices at either end of Bevel Edge

		// new mesh topology computed during unlink step
		TArray<int32> NewMeshVertices;		// list of new vertices on "other" side of unlinked edge, 1-1 with MeshVertices
		TArray<int32> NewMeshEdges;			// list of new edges on "other" side of unlinked edge, 1-1 with MeshEdges

		// buffers for new vertex positions computed during displace step
		TArray<FVector3d> NewPositions0;	// new positions for MeshVertices list
		TArray<FVector3d> NewPositions1;	// new positions for NewMeshVertices list

		// new geometry computed during mesh step
		int32 NewGroupID;
		TArray<FIndex2i> StripQuads;		// triangle-ID-pairs for each new quad added along edge, 1-1 with MeshEdges
		FQuadGridPatch StripQuadPatch;			// only initialized in multi-segment bevel
		TArray<FVector3d> NormalsA, NormalsB;	// normals at NewPositions0 and NewPositions1, before internal meshing is added (ie the tangent-boundary condition)
	};


	// A FOneRingWedge represents a sequential set of connected triangles around a vertex, ie a subset of an ordered triangle one-ring.
	// Used to represent the desired bevel topology in FBevelVertex
	struct FOneRingWedge
	{
		TArray<int32> Triangles;				// list of sequential triangles in this wedge
		FIndex2i BorderEdges;					// "first" and "last" Edges of sequential triangles in Triangles list (connected to central Vertex)
		FIndex2i BorderEdgeTriEdgeIndices;		// index 0/1/2 of BorderEdges[j] in start/ed Triangles

		int32 WedgeVertex;						// central vertex of this wedge (updated by unlink functions)

		FVector3d NewPosition;					// new calculated position for vertex of this wedge
		bool bHaveNewPosition = false;			// flag indicating if NewPosition is valid
	};

	// a FBevelVertex can have various types, depending on the topology of the bevel edge graph and input mesh
	enum class EBevelVertexType
	{
		// A JunctionVertex is a vertex at which 2 or more FBevelEdges meet (ie is an endpoint of 2 or more of those vertex-spans)
		// If N>=3 or more edges meet at a JunctionVertex, it will become a polygon with N vertices, one for each "wedge"
		JunctionVertex,
		// A TerminatorVertex is a vertex at which a single FBevelEdge terminates, ie the N=1 case. This requires different handling
		// because we essentially want to turn that vertex into an edge, which means inserting a triangle into the adjacent one-ring
		TerminatorVertex,
		// a BoundaryVertex is a junction/terminator on the mesh boundary
		BoundaryVertex,
		// An Unknown vertex is one at which we don't know what to do, or some error occurred while processing as a Junction/Terminator.
		Unknown
	};

	struct FBevelVertex_InteriorVertex
	{
		int32 VertexID = -1;
		TArray<FVector3d> BorderFrameWeight;
	};

	// A FBevelVertex repesents/stores the accumulated data at a "bevel vertex", which is the mesh vertex at the end of a FBevelEdge.
	// A FBevelVertex may be expanded out into a polygon or just an edge, depending on its Type
	struct FBevelVertex
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FBevelVertex() = default;
		FBevelVertex(const FBevelVertex&) = default;
		FBevelVertex(FBevelVertex&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		int32 VertexID;												// Initial Mesh Vertex ID for the Bevel Vertex
		UE_DEPRECATED(5.5, "Mapping back to source topology is not used.")
		int32 CornerID;												// Initial Group Topology Corner ID for the Bevel Vertex (if exists)
		EBevelVertexType VertexType = EBevelVertexType::Unknown;	// Type of the Bevel Vertex

		TArray<int32> IncomingBevelMeshEdges;						// Set of (unsorted) Mesh Edges that are destined to be Beveled, coming into the vertex
		UE_DEPRECATED(5.5, "Mapping back to source topology is not used.")
		TArray<int32> IncomingBevelTopoEdges;						// Set of (unsorted) Group Topology Edges that are to be Beveled, coming into the vertex
		TArray<int32> IncomingBevelEdgeIndices;						// Set of (unsorted) indices of FBevelEdge, coming into the vertex

		TArray<int32> SortedTriangles;			// ordered triangle one-ring around VertexID
		TArray<FOneRingWedge> Wedges;			// ordered decomposition of one-ring into "wedges" between incoming bevel edges (no correspondence w/ IncomingBevelMeshEdges list)

		int32 NewGroupID;						// new polygroup allocated for the beveled polygon generated by this vertex (if NumEdges > 2)
		TArray<int32> NewTriangles;				// new triangles that make up the beveled polygon for this vertex (if NumEdges > 2)

		FIndex2i TerminatorInfo;				// for TerminatorVertex type, store [EdgeID, FarVertexID] in one-ring, used to unlink/fill (see usage)
		int32 ConnectedBevelVertex = -1;		// If set to another FBevelVertex index, then the TerminatorInfo.EdgeID directly connects to that vertex and special handling is needed

		// these arrays are used for multi-segment bevels, and are initialized in different ways depending on the vertex valence - see cpp comments
		TArray<FBevelVertex_InteriorVertex> InteriorVertices;
		TArray<int32> InteriorBorderLoop;
	};



protected:
	TMap<int32, int32> VertexIDToIndexMap;		// map of mesh-vertex-IDs to indices into Vertices list
	TArray<FBevelVertex> Vertices;				// list of FBevelVertex data structures for mesh vertices that need beveling

	TArray<FBevelEdge> Edges;					// list of FBevelEdge data structures for mesh edge-spans that need beveling
	TArray<FBevelLoop> Loops;					// list of FBevelLoop data structures for mesh edge-loops that need beveling

	TMap<int32, int32> MeshEdgePairs;			// Many edges of the input mesh will be split into edge pairs, which are then stitched together with quads.
												// This map stores the authoritative correspondences between these edge pairs. Both pairs, ie (a,b) and (b,a) are stored.



protected:

	UE_API FBevelVertex* GetBevelVertexFromVertexID(int32 VertexID, int32* IndexOut = nullptr);

	// Setup phase: register Edges (spans) and (isolated) Loops that need to be beveled and precompute/store any mesh topology that must be tracked across the operation
	// Required BevelVertex's are added by AddBevelGroupEdge()
	// Once edges are configured, BuildVertexSets() is called to precompute the vertex topological information

	UE_API void AddBevelGroupEdge(const FDynamicMesh3& Mesh, const FGroupTopology& Topology, int32 GroupEdgeID);
	UE_API void AddBevelEdgeLoop(const FDynamicMesh3& Mesh, const FEdgeLoop& Loop);
	UE_API void BuildVertexSets(const FDynamicMesh3& Mesh);
	UE_API void BuildJunctionVertex(FBevelVertex& Vertex, const FDynamicMesh3& Mesh);
	UE_API void BuildTerminatorVertex(FBevelVertex& Vertex, const FDynamicMesh3& Mesh);

	// Unlink phase - disconnect triangles along bevel edges/loops, and at vertices. 
	// Vertices may expand out into multiple "wedges" depending on incoming bevel-edge topology.

	UE_API void UnlinkEdges( FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker);
	UE_API void UnlinkBevelEdgeInterior( FDynamicMesh3& Mesh, FBevelEdge& BevelEdge, FDynamicMeshChangeTracker* ChangeTracker);

	UE_API void UnlinkLoops( FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker);
	UE_API void UnlinkBevelLoop( FDynamicMesh3& Mesh, FBevelLoop& BevelLoop, FDynamicMeshChangeTracker* ChangeTracker);

	UE_API void UnlinkVertices( FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker);
	UE_API void UnlinkJunctionVertex(FDynamicMesh3& Mesh, FBevelVertex& BevelVertex, FDynamicMeshChangeTracker* ChangeTracker);
	UE_API void UnlinkTerminatorVertex(FDynamicMesh3& Mesh, FBevelVertex& BevelVertex, FDynamicMeshChangeTracker* ChangeTracker);

	UE_API void FixUpUnlinkedBevelEdges(FDynamicMesh3& Mesh);

	// Displace phase - move unlinked vertices to new positions

	UE_API void DisplaceVertices(FDynamicMesh3& Mesh, double Distance);

	// Meshing phase - append quad-strips between unlinked edge spans/loops, polygons at junction vertices where required,
	// and triangles at terminator vertices

	/* Meshing functions for chamfer bevel, ie no subdivisions */
	UE_API void CreateBevelMeshing(FDynamicMesh3& Mesh);
	UE_API void AppendJunctionVertexPolygon(FDynamicMesh3& Mesh, FBevelVertex& Vertex);
	UE_API void AppendTerminatorVertexTriangle(FDynamicMesh3& Mesh, FBevelVertex& Vertex);
	UE_API void AppendTerminatorVertexPairQuad(FDynamicMesh3& Mesh, FBevelVertex& Vertex0, FBevelVertex& Vertex1);
	UE_API void AppendEdgeQuads(FDynamicMesh3& Mesh, FBevelEdge& Edge);
	UE_API void AppendLoopQuads(FDynamicMesh3& Mesh, FBevelLoop& Loop);

	/** Meshing functions for multi-segment bevel with optional round profile */
	UE_API void CreateBevelMeshing_Multi(FDynamicMesh3& Mesh);
	UE_API void AppendEdgeQuads_Multi(FDynamicMesh3& Mesh, FBevelEdge& Edge);
	UE_API void AppendLoopQuads_Multi(FDynamicMesh3& Mesh, FBevelLoop& Loop);
	UE_API void AppendJunctionVertexPolygon_Multi(FDynamicMesh3& Mesh, FBevelVertex& Vertex);
	UE_API void AppendTerminatorVertexTriangles_Multi(FDynamicMesh3& Mesh, FBevelVertex& Vertex);
	UE_API void AppendTerminatorVertexPairQuad_Multi(FDynamicMesh3& Mesh, FBevelVertex& Vertex0, FBevelVertex& Vertex1);
	UE_API void ApplyProfileShape_Round(FDynamicMesh3& Mesh);
	UE_API FInterpCurveVector MakeArcSplineCurve(const FVector3d& PosA, FVector3d& NormalA, const FVector3d& PosB, FVector3d& NormalB) const;


	// Normals phase - calculate normals for new geometry

	UE_API void ComputeNormals(FDynamicMesh3& Mesh);
	UE_API void ComputeUVs(FDynamicMesh3& Mesh);
	UE_API void ComputeMaterialIDs(FDynamicMesh3& Mesh);

private:
	// Detect and fix any bowtie vertices in the bevel operation set
	// Called by "Apply" (because the setup methods all operate on a const Mesh) before the actual bevel operation
	UE_API void FixBowties(FDynamicMesh3& Mesh, FDynamicMeshChangeTracker* ChangeTracker);

	UE_API void InitVertexSet(const FDynamicMesh3& Mesh, FBevelVertex& Vertex);
	UE_API void FinalizeTerminatorVertex(const FDynamicMesh3& Mesh, FBevelVertex& Vertex);
};



} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
