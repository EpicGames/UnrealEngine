// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"
#include "IndexTypes.h"
#include "Clustering/FaceNormalClustering.h"
#include "MeshAdapter.h"

class FProgressCancel;

namespace UE::Geometry
{

class FDynamicMesh3;

namespace MeshClusterSimplify
{

struct FSimplifyOptions
{
	// Vertices within this distance are allowed to be replaced with a single vertex
	double TargetEdgeLength = 1;

	// Optionally locally scales the simplifier's TargetEdgeLength on a per-vertex basis
	// If provided, the target edge length at a vertex VID will be TargetEdgeLength * OptionalTargetEdgeLengthScale(VID)
	TFunction<double(int32)> OptionalTargetEdgeLengthScale = nullptr;

	// If > 0, number of passes to attempt to make sure the simplification does not fully delete islands of the mesh.
	// In areas that would be deleted (because the region is smaller than the TargetEdgeLength),
	// the target edge length will instead be locally reduced to half the local max in-cluster distance,
	// and clustering will be re-run.
	// This does not guarantee all clusters will be preserved: Use a higher number of passes to reducing the 
	// likelihood of losing mesh components/connectivity, for additional cost & more vertices in the result.
	int32 MaxPreserveCollapsedClusterPasses = 3;

	// If > 0, boundary vertices w/ incident boundary edge angle greater than this (in degrees) will be kept in the output
	double FixBoundaryAngleTolerance = 45;

	// Constraint levels control what simplifications are allowed
	// Ordered from most-constrained to least-constrained
	enum class EConstraintLevel : uint8
	{
		// Fixed vertices/edges will generally be preserved in the output, as they will each be given their own cluster
		Fixed,
		// Constrained vertices/edges may be simplified, but the edge flow should be preserved
		// A vertex at an intersection of more than two constrained edges will be automatically preserved as 'Fixed'
		Constrained,
		// No constraints / ok to simplify as much as possible
		Free,
		// The count of total constraint levels, used for iterating over the levels. (Not itself a valid constraint level)
		MAX
	};

	struct FPreserveFeatures
	{
		// Mesh boundaries
		EConstraintLevel Boundary = EConstraintLevel::Constrained;
		// Seam types
		EConstraintLevel UVSeam = EConstraintLevel::Constrained;
		EConstraintLevel NormalSeam = EConstraintLevel::Constrained;
		EConstraintLevel TangentSeam = EConstraintLevel::Free;
		EConstraintLevel ColorSeam = EConstraintLevel::Constrained;
		// Material ID boundaries
		EConstraintLevel Material = EConstraintLevel::Constrained;
		// PolyGroup ID boundaries
		EConstraintLevel PolyGroup = EConstraintLevel::Constrained;

		// Helper to set all seam types to the same constraint level
		void SetSeamConstraints(EConstraintLevel Level)
		{
			UVSeam = Level;
			NormalSeam = Level;
			TangentSeam = Level;
			ColorSeam = Level;
		}
	};

	// Manage which feature edge types we try to retain in the simplified result
	FPreserveFeatures PreserveEdges{};

	// Whether to attempt to transfer attributes to the result mesh
	bool bTransferAttributes = true;

	// Whether to attempt to transfer triangle groups (PolyGroups) to the result mesh
	bool bTransferGroups = true;


};

/**
 * Makes a simplified copy of the input mesh
 * 
 * This cluster simplify method first clusters vertices locally by distance (calculated along mesh edges), and creates new triangles
 * from the connectivity of the clusters. i.e., it is a triangulation of the dual of the graph voronoi diagram over mesh edges.
 * 
 * To preserve feature edges:
 * (1) constrained / feature-edge vertices are prioritized as cluster 'seeds,' and 
 * (2) clusters are grown along feature edges first, then free edges after -- and growth over 'free' edges cannot claim 'constrained' vertices.
 * This locks in clusters along 'constrained' feature edges.
 * 
 * Note that mesh features can be lost if the clusters are large enough that the graph becomes degenerate
 *  -- e.g., if a mesh island has so few clusters that the graph connectivity does not contain triangles.
 * 
 * 
 * @param InMesh The mesh to simplify
 * @param OutSimplifiedMesh This mesh will store the simplified result mesh
 * @param SimplifyOptions Options controlling simplification
 * @return true on success
 */
bool DYNAMICMESH_API Simplify(const FDynamicMesh3& InMesh, FDynamicMesh3& OutSimplifiedMesh, const FSimplifyOptions& SimplifyOptions);

/**
 * Adapter for producing an output simplified triangle mesh in any target triangle mesh structure
 */
struct FResultMeshAdapter
{
	// Add a vertex to the result mesh, and return its ID
	TFunction<int32(FVector3d)> AppendVertex;

	// Add a triangle to the result mesh, and return its ID
	// Note this method should follow the same error conventions as FDynamicMesh3, so
	//  if it refuses to add a non-manifold triangle, it should return FDynamicMesh3::NonManifoldID
	TFunction<int32(FIndex3i)> AppendTriangle;

	// Clear the current mesh from the results
	TFunction<void()> Clear;

	// Accessor for vertices
	TFunction<FVector3d(int32)> GetVertex;
	// Accessor for triangles
	TFunction<FIndex3i(int32)> GetTriangle;

	// Initialize the adapter from a mesh type that has the same basic interface
	// Note that for attributes to transfer, the optional attribute transfer methods must be implemented separately
	template<typename MeshType>
	void Init(MeshType* Mesh)
	{
		AppendVertex = [Mesh](FVector3d V) -> int32
			{
				return Mesh->AppendVertex(V);
			};
		AppendTriangle = [Mesh](FIndex3i T) -> int32
			{
				return Mesh->AppendTriangle(T);
			};
		Clear = [Mesh]() -> void
			{
				Mesh->Clear();
			};
		GetVertex = [Mesh](int32 VID) -> FVector3d
			{
				return Mesh->GetVertex(VID);
			};
		GetTriangle = [Mesh](int32 TID) -> FIndex3i
			{
				return Mesh->GetTriangle(TID);
			};
	}

	// Attribute transfer methods; if not provided, attributes will not be transferred
	TFunction<void(TConstArrayView<int32> ResultToSourceVertexID)> TransferPerVertexAttributes;
	TFunction<void(TConstArrayView<int32> ResultToSourceTriangleID)> TransferPerTriangleAttributes;
	// TODO: also provide a generic transfer method for attributes that may have seams (such as UVs)
};

// Simplify an input mesh provided by the general triangle mesh adapter, placing the result in an output mesh via a similar adapter interface
// Note: The output mesh must not be aliased with input mesh data
bool DYNAMICMESH_API Simplify(const FTriangleMeshAdapterd& InMesh, FResultMeshAdapter& OutSimplifiedMesh, const FSimplifyOptions& SimplifyOptions);

} // end namespace UE::Geometry
} // end namespace UE
