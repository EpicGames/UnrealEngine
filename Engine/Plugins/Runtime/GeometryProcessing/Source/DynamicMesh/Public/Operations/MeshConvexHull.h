// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "IntVectorTypes.h"
#include "GeometryTypes.h"
#include "DynamicMesh/DynamicMesh3.h"

#define UE_API DYNAMICMESH_API

class FProgressCancel;

namespace UE
{
namespace Geometry
{

/**
 * Calculate Convex Hull of a Mesh
 */
class FMeshConvexHull
{
public:
	/** Input Mesh */
	const FDynamicMesh3* Mesh;

	/** If set, hull will be computed on subset of vertices */
	TArray<int32> VertexSet;

	/** If true, output convex hull is simplified down based on the SimplifyOptions */
	bool bPostSimplify = false;

	struct FSimplifyOptions
	{
		/** Target triangle count of the simplified output Convex Hull; hull will be simplified to at most this triangle count if possible */
		int32 MaxTargetFaceCount = 0;
		/** If greater than zero, hull will be futher simplified below target face count where doing so remains within this error tolerance */
		double GeometricToleranceAfterTarget = 0.;
	};

	/** IF bPostSimplify, options controlling the simplification */
	FSimplifyOptions SimplifyOptions;

	UE_DEPRECATED(5.8, "Use SimplifyOptions.MaxTargetFaceCount instead")
	int32 MaxTargetFaceCount = 0;

	/** Minimum extent along the shortest dimension; if greater than zero, the hull may be expanded if it would be too thin */
	double MinDimension = 0;

	/** Output convex hull */
	FDynamicMesh3 ConvexHull;

	/** 
	 *  Choose a more or less evenly-spaced subset of mesh vertices. Conceptually, this function creates a uniform 
	 *  grid with given cell size. (Cell size is given roughly as a percentage of the total mesh bounding box.) Each 
	 *  grid cell can hold up to one vertex. Return the set of representative vertices, maximum one per cell.
	 * 
	 *  @param Mesh							Surface mesh whose vertices we want to sample
	 *  @param CellSizeAsPercentOfBounds	Grid cell size expressed as a percentage of the mesh bounding box size
	 *  @param OutSamples					Indices of chosen vertices
	 */
	static UE_API void GridSample(const FDynamicMesh3& Mesh, 
						   int GridResolutionMaxAxis,
						   TArray<int32>& OutSamples);

	/** Used for testing/debugging */
	static UE_API FVector3i DebugGetCellIndex(const FDynamicMesh3& Mesh,
									   int GridResolutionMaxAxis,
									   int VertexIndex);


public:
	FMeshConvexHull(const FDynamicMesh3* MeshIn)
	{
		Mesh = MeshIn;
	}

	/**
	 * Calculate output ConvexHull mesh for vertices of input Mesh
	 * @return true on success
	 */
	UE_API bool Compute(FProgressCancel* Progress = nullptr);

	/**
	 * Simplify the output convex hull. Assumes ConvexHull is already computed.
	 * @return true on success
	 */
	static bool SimplifyHull(FDynamicMesh3& HullMesh, int32 MaxTargetFaceCount, FProgressCancel* Progress = nullptr)
	{
		FSimplifyOptions Options;
		Options.MaxTargetFaceCount = MaxTargetFaceCount;
		return SimplifyHull(HullMesh, Options, Progress);
	}

	/**
	 * Simplify the output convex hull. Assumes ConvexHull is already computed.
	 * @return true on success
	 */
	static UE_API bool SimplifyHull(FDynamicMesh3& HullMesh, FSimplifyOptions SimplifyOptions, FProgressCancel* Progress = nullptr);

protected:
	UE_API bool Compute_FullMesh(FProgressCancel* Progress);
	UE_API bool Compute_VertexSubset(FProgressCancel* Progress);
private:
	UE_API bool Compute_Helper(FProgressCancel* Progress, int32 MaxVertexIndex, TFunctionRef<FVector3d(int32)> GetVertex, TFunctionRef<bool(int32)> IsVertex, bool bTestMinDimension = true);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
