// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IndexTypes.h"
#include "VectorTypes.h"
#include "MathUtil.h" // for max real
#include "TriangleTypes.h"

#define UE_API DYNAMICMESH_API



namespace UE {
namespace Geometry {

class FDynamicMesh3;


/**
* Utils to define vectors relative to the local surface of a mesh 
* and given such a vector, functions to trace across the face of a single triangle within a mesh
* to find where the vector intersects a triangle edge.
*/
namespace GeodesicSingleTriangleUtils
{

	/**
	* Normalized vector defined relative to specified Mesh edge
	* and only valid in the two faces adjacent to the edge.
	*/
	struct FMeshSurfaceDirection;

	/**
	* Direction in mesh tangent space, defined relative to a vertex and a specified Mesh edge
	*/
	struct FMeshTangentDirection;
	/**
	* Classification for a trace result (i.e. for a points recorded along the path).
	*/
	enum class ETraceClassification
	{
		Start,                  // corresponds to initial location when starting a trace 
		Continuing,             // non-terminated result computed when crossing a single triangle
		DistanceTerminated,     // result terminated by reaching prescribed max distance
		BoundaryTerminated,     // result terminated by reaching mesh boundary
		Failed                  // failed trace, generally a sign that you started a trace with a ray origin outside the triangle.
	};

	/**
	* Data returned from a trace across a triangle face.
	* provides distance traveled, and a new Tangent Vector and accompanying point on the mesh surface.
	*/
	struct FTraceResult;

	/**
	* Triangle in R2, constructed by rotating and aligning a Triangle in R3.
	* Traces carried out across these triangles.
	*/
	class FTangentTri2;


	static bool IsTerminated(const FTraceResult& Result);
	// ----------------------------------------------------------------------------------------
	//  functions for tracing across a single triangle on a mesh
	//-----------------------------------------------------------------------------------------


	/**
	* Trace FTangentTri2 using a local coordinate system defined by aligning the first edge of the triangle with the positive x-direction
	* and the opposing vertex with positive y-coordinate.
	* @param TangentTri2    - Triangle to trace, expressed in 2d
	* @param RayOrigin - ray origin
	* @param Direction - direction of trace, normalized
	* @param MaxDist   - Max distance the path is allowed to travel for this triangle trace.
	*
	* @return Trace Result.
	*/
	FTraceResult TraceTangentTriangle(const FTangentTri2& TangentTri2, const FVector2d& RayOrigin, const FVector2d& RayDir, double MaxDistance);


	/**
	* Trace From an Edge crossing into the triangle TriID
	* @param TriID     - FDynamicMesh3 TriangleID that indicated that triangle we trace across
	* @param BaryPoint - Barycentric coordinates of ray origin.
	* @param Direction - Surface direction of trace
	* @param MaxDist   - Max distance the path is allowed to travel for this triangle trace.
	*
	* @return Trace Result.
	*
	* NB: it is assumed the edge EID is part of the triangle TriID.  ( otherwise the result will be unexpected)
	* NB: it is assumed the BaryPoint is a valid point in the triangle ( otherwise the results will be unexpected)
	*/
	FTraceResult TraceTriangleFromBaryPoint(const FDynamicMesh3& Mesh, const int32 TriID, const FVector3d& BaryPoint, const FMeshSurfaceDirection& Direction, double MaxDistance = TMathUtilConstants<double>::MaxReal);
	FTraceResult TraceTriangleFromBaryPoint(const FDynamicMesh3& Mesh, const int32 TriID, const FVector3d& BaryPoint, const FMeshSurfaceDirection& Direction, FTangentTri2& ScratchTangentTri2, double MaxDistance = TMathUtilConstants<double>::MaxReal);

	/**
	* Trace from  a Vertex Crossing.
	* @param TangentDirection   - Direction in the local tangent space ( also defines the frame by center VID and zero angle EID)
	* @param MaxDistance        - Maximal distance this trace can travel.
	*
	* @return Trace Result.
	*/
	FTraceResult TraceFromVertex(const FDynamicMesh3& Mesh, const FMeshTangentDirection& Direction, double MaxDistance = TMathUtilConstants<double>::MaxReal);
	FTraceResult TraceFromVertex(const FDynamicMesh3& Mesh, const FMeshTangentDirection& Direction, FTangentTri2& ScratchTangentTri2, double MaxDistance = TMathUtilConstants<double>::MaxReal);

	/**
	* Trace from  a Vertex Crossing.
	* @param VID         - Vertex ID (per the FDynamicMesh3) where the trace starts
	* @param Direction   - Surface direction of trace.
	* @param MaxDistance - Maximal distance this trace can travel.
	*
	* @return Trace Result.
	*
	* NB: the direction vector is defined on the surface, not the tangent plane.   This means, it must lie in a triangle adjacent to Direction.EdgeID
	*/
	FTraceResult TraceFromVertex(const FDynamicMesh3& Mesh, int32 VID, const FMeshSurfaceDirection& Direction, double MaxDistance = TMathUtilConstants<double>::MaxReal);
	FTraceResult TraceFromVertex(const FDynamicMesh3& Mesh, int32 VID, const FMeshSurfaceDirection& Direction, FTangentTri2& ScratchTangentTri2, double MaxDistance = TMathUtilConstants<double>::MaxReal);

	/**
	* Trace From an Edge crossing into the triangle TriID
	* @param TriID     - FDynamicMesh3 TriangleID that indicated that triangle we trace across
	* @param EdgeAlpha - [0,1] parameter that determines the location on the Direction edge where the trace originates (measured from Edge.A to Edge.B as ordered in mesh)
	* @param Direction - Surface direction of trace
	*
	* @return Trace Result.
	*
	* NB: it is assumed that the edge EID is part of the triangle TriID.  (otherwise the result will be unexpected)
	*/
	FTraceResult TraceTriangleFromEdge(const FDynamicMesh3& Mesh, const int32 TriID, double EdgeAlpha, const FMeshSurfaceDirection& Direction, double MaxDistance = TMathUtilConstants<double>::MaxReal);
	FTraceResult TraceTriangleFromEdge(const FDynamicMesh3& Mesh, const int32 TriID, double EdgeAlpha, const FMeshSurfaceDirection& Direction, FTangentTri2& ScratchTangentTri2, double MaxDistance = TMathUtilConstants<double>::MaxReal);


	/**
	* Trace, starting with the result of an earlier trace.
	* @param LastTrace   - Struct that hold previous trace.  Note StartTrace.TriID will be the triangle last visited. Will trace across the other edge adjacent tri
	* @param MaxDistance - Maximal distance this trace can travel.
	*
	* @return Trace Result.
	*/
	FTraceResult TraceNextTriangle(const FDynamicMesh3& Mesh, const  FTraceResult& LastTrace, double MaxDistance = TMathUtilConstants<double>::MaxReal);
	FTraceResult TraceNextTriangle(const FDynamicMesh3& Mesh, const  FTraceResult& LastTrace, FTangentTri2& ScratchTangentTri2, double MaxDistance = TMathUtilConstants<double>::MaxReal);

}; // end namespace GeodesicSingleTriangleUtils

class GeodesicSingleTriangleUtils::FTangentTri2
{
public:
	FTangentTri2() = default;

	// The PrimaryEdge will be aligned with the x-axis in 2d ( and must be one of the edges of the triangle indicated by TriID)
	FTangentTri2(const FDynamicMesh3& Mesh, int32 TriID, int32 PrimaryEdgeID);

	// Reset just invalidates Tri3ID  
	void Reset();

	// EdgeIndex = 0, 1, 2
	double DistanceToEdge(int EdgeIndex, const FVector2d& Point) const;

	// Closest Point on the edge, given in terms of 0,1 parameter measured along the edge (EdgeIndex = 0, 1, 2)
	double ProjectOnEdge(int EdgeIndex, const FVector2d& Point) const;

	// Express the vector in terms of a basis that is aligned with the EdgeIndex edge of the triangle
	FVector2d ChangeBasis(const FVector2d& Vec, int32 EdgeIndex) const;

	//Convert vertex data ordered for this FTangentTri2 into vertex data ordered for the source 3d triangle
	template <typename Vector3Type>
	Vector3Type ReorderVertexDataForSrcTri3d(const Vector3Type& DataOrderedForTri2d) const;

	// Barycentric coords for this point in the tri2d
	FVector3d GetBarycentricCoords(const FVector2d& PoinInTri2) const;

	// Convert a point on this tri, to Barycentric coords in the un-rotated source triangle.
	FVector3d GetBarycentricCoordsInSrcTri(const FVector2d& PoinInTri2) const;

public:

	int32 Tri3ID = IndexConstants::InvalidID;  // TriID in the DynamicMesh3 that defined the triangle in R3

	FIndex3i PermutedTriVIDs;      // Map verts to original triangle
	FIndex3i PermutedTriEIDs;      // Map edges to original triangle
	FIndex3i PermutedIndices;      // Maps R2 triangle index order to original R3 triangle index order

	FIndex3i EdgeOrientationSign;  // Per-edge information: +1 if Mesh.GetEdgeV() orientation agrees with ccw traversal of this triangle. -1 if reversed
	FTriangle2d Tri2d;             // The triangle in R2

	FVector2d Tri2dEdges[3];         // cached R2 triangle edge vectors
	FVector3d Tri2dLengths;          // cached R2 triangle edge lengths
};

/**
* Class that, given a starting location and direction, traces a geodesic path along a mesh.
* As the path enters a triangle, it is unfolded a two-dimensional triangle and the path is transformed into 
* the local frame of the triangle.
* 
* At each point where the path crosses into a new triangle, this class records the point and direction 
* in terms of barycentric coordinates, triangle references and local path direction relative 
* to a local frame (in the form of one of the edges of the triangle).
*/
class FMeshGeodesicSurfaceTracer
{
public:

	using FMeshSurfaceDirection = GeodesicSingleTriangleUtils::FMeshSurfaceDirection;
	using FMeshTangentDirection = GeodesicSingleTriangleUtils::FMeshTangentDirection;
	using ETraceClassification  = GeodesicSingleTriangleUtils::ETraceClassification;
	using FTraceResult          = GeodesicSingleTriangleUtils::FTraceResult;
	using FTangentTri2          = GeodesicSingleTriangleUtils::FTangentTri2;


	FMeshGeodesicSurfaceTracer() = default; // will need to associate a mesh to actually use..

	FMeshGeodesicSurfaceTracer(const FDynamicMesh3& MeshIn)
		: Mesh(&MeshIn)
	{}

	void Reset(const FDynamicMesh3& MeshIn)
	{
		SurfaceTrace.Reset();
		ScratchTri2.Reset();
		Mesh = &MeshIn;
	}

	/**
	* Populate internal SurfaceTrace array of FTraceResults while following a "straight" path on the surface of a mesh.
	* The starting location and direction are given in terms of a local parameterization of the initial triangle.
	*
	* @param TriID     - FDynamicMesh3 TriangleID that indicated that triangle where the trace originates
	* @param BaryPoint - Barycentric coordinates of ray origin relative to the specified triangle. See NB below.
	* @param RayDir3   - Initial Direction given in R3.  This will be projected on the face of the specified Triangle (TriID)
	* @param MaxDist   - Max distance the path is allowed to travel for this triangle trace.
	*
	* @return total distance of the trace, this may be less than the MaxDist (e.g. in the event the trace encounters a mesh edge)
	*
	* NB: the BaryPoint must be a valid point within or on the boundary of the triangle.
	*/
	UE_API double TraceMeshFromBaryPoint(const int32 TriID, const FVector3d& BaryPoint, const FVector3d& RayDir3, double MaxDistance = TMathUtilConstants<double>::MaxReal);

	/**
	* Populate internal SurfaceTrace array of FTraceResults while following a "straight" path on the surface of a mesh.
	* The starting location and direction are given in terms of a local parameterization of the initial triangle.
	*
	* @param TriID     - FDynamicMesh3 TriangleID that indicated that triangle where the trace originates
	* @param BaryPoint - Barycentric coordinates of ray origin relative to the specified triangle. See NB below.
	* @param Direction - Surface direction of trace
	* @param MaxDist   - Max distance the path is allowed to travel for this triangle trace.
	*
	* @return total distance of the trace, this may be less than the MaxDist (e.g. in the event the trace encounters a mesh edge)
	*
	* NB: the BaryPoint must be a valid point within or on the boundary of the triangle.
	* NB: it is assumed the edge Direction.EdgeID is part of the triangle TriID.  ( otherwise the result will be unexpected)
	*/
	UE_API double TraceMeshFromBaryPoint(const int32 TriID, const FVector3d& BaryPoint, const FMeshSurfaceDirection& Direction, double MaxDistance = TMathUtilConstants<double>::MaxReal);

	/**
	* Populate internal SurfaceTrace array of FTraceResults while following a "straight" path on the surface of a mesh.
	* The starting location and direction are given in terms of a local parameterization of the initial triangle.
	*
	* @param Direction - Tangent plane direction of trace, defines the origin (VID) and the zero radian direction (EID.A to EID.B) along with a polar angle
	* @param MaxDist   - Max distance the path is allowed to travel for this triangle trace.
	*
	* @return total distance of the trace, this may be less than the MaxDist (e.g. in the event the trace encounters a mesh edge)
	*
	* NB: it is assumed the edge Direction.EdgeID is adjacent to VID.  ( otherwise the result will be unexpected)
	*/
	UE_API double TraceMeshFromVertex( const FMeshTangentDirection& Direction, double MaxDistance);

	/** Access to the last tri traced in 2d form.*/
	const FTangentTri2& GetLastTri()
	{
		return ScratchTri2;
	}

	/** Access to the trace results*/
	TArray<FTraceResult>& GetTraceResults()
	{
		return SurfaceTrace;
	}
	/** Access to the trace results*/
	const TArray<FTraceResult>& GetTraceResults() const
	{
		return SurfaceTrace;
	}

	
protected:

	const FDynamicMesh3* Mesh = nullptr;       // Pointer to the Mesh to be traced.
	TArray<FTraceResult> SurfaceTrace;         // Vector of points along the computed trace
	FTangentTri2 ScratchTri2;                  // 2d representation of the last triangle traversed.  Used as scratch during computations

};



struct GeodesicSingleTriangleUtils::FMeshSurfaceDirection
{
	FMeshSurfaceDirection() :EdgeID(IndexConstants::InvalidID), Dir(0., 0.) {}

	FMeshSurfaceDirection(int EID, const FVector2d& DirIn)
		: EdgeID(EID)
		, Dir(DirIn)
	{}

	FMeshSurfaceDirection(int EID, const double PolarAngle)
		: EdgeID(EID)
		, Dir(FVector2d(FMath::Cos(PolarAngle), FMath::Sin(PolarAngle)))
	{}


	int32 EdgeID;           // Mesh Edge, has implicit order (from Edge.A to Edge.B)
	FVector2d  Dir;         // Normalized direction relative to the frame defined by the mesh edge EdgeID aligned with the x-axis
};

struct GeodesicSingleTriangleUtils::FMeshTangentDirection
{
	int32 VID;           // Tangent space centered on VID vertex
	int32 EdgeID;        // Mesh Edge (adjacent to VID), the vector Edge.A to Edge.B defines the zero radians direction (Edge.B to Edge.A is pi radians)
	double PolarAngle;   // Angle measured between [0, 2pi)  
};

struct GeodesicSingleTriangleUtils::FTraceResult
{
	ETraceClassification Classification = ETraceClassification::Start;

	double TraceDist = -1.;                    // Distance traveled in this tri before either meeting and edge or the max distance.
	int32 TriID = IndexConstants::InvalidID;    // Triangle traversed
	FMeshSurfaceDirection SurfaceDirection;    // Tangent vector.

	// -- Information about the surface point --//
	FVector3d Barycentric;                     // Surface point as Barycentric coords.  Can be reconstructed with FDynamicMesh::GetTriBaryPoint(TriID, Barycentric[0], Barycentric[1], Barycentric[2]);
	bool bIsEdgePoint;                         // Duplicate description of the surface point in the case of edge termination
	int32 EdgeID = IndexConstants::InvalidID;  // In the case of Edge termination, SurfaceDirection.EdgeID = EdgeID
	double EdgeAlpha = -1.;                    // When edge point: locates position between the vertices as (1-Alpha) Edge.A + Alpha Edge.B
};

bool GeodesicSingleTriangleUtils::IsTerminated(const FTraceResult& Result)
{
	return (Result.Classification == ETraceClassification::DistanceTerminated
		|| Result.Classification == ETraceClassification::BoundaryTerminated);
}



template <typename Vector3Type>
Vector3Type GeodesicSingleTriangleUtils::FTangentTri2::ReorderVertexDataForSrcTri3d(const Vector3Type& DataOrderdForTri2d) const
{
	Vector3Type Result;
	Result[PermutedIndices[0]] = DataOrderdForTri2d[0];
	Result[PermutedIndices[1]] = DataOrderdForTri2d[1];
	Result[PermutedIndices[2]] = DataOrderdForTri2d[2];

	return Result;
}


/**
* return AngleR in [0,2pi) range.  
*
* The same polar point (r, AngleR) can be described by an infinite number of alternative 
* angles, each offset by some multiple of 2pi,  i.e. (r, AngleR + n * 2Pi).
* This selects the unique angle that falls in the [0,2pi) range.
*/
static double AsZeroToTwoPi(double AngleR)
{
	while (AngleR < 0.)
	{
		AngleR += TMathUtilConstants<double>::TwoPi;
	}

	while (AngleR > TMathUtilConstants<double>::TwoPi)
	{
		AngleR -= TMathUtilConstants<double>::TwoPi;
	}
	return AngleR;
}

}; // end namespace Geometry
}; // end namespace UE

#undef UE_API
