// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"

#include "MeshAdapter.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

// Methods for cutting meshes along isolines
namespace MeshIsoCurve
{
	// Adapter methods to support updating a mesh with a cut.
	// All triangles will either be labelled via LabelTriangle, or replaced w/ labelled triangles via ReplaceTriangle.
	struct FMeshUpdateAdapter
	{
		// Add a vertex interpolating the two input vertices (but not yet connected to any triangles)
		TFunction<int32(int32 VID0, int32 VID1, double T)> AddInterpolatedEdgeVertex;
		// Add a vertex interpolating the three input vertices (but not yet connected to any triangles)
		TFunction<int32(FIndex3i Tri, const FVector3d& BaryCoords)> AddInterpolatedTriangleVertex;
		// Replace a triangle with the given new triangles, with the specified new per-triangle labels
		TFunction<void(int32 ReplaceTriID, TConstArrayView<FIndex3i> NewTris, TConstArrayView<int32> NewTriLabels)> ReplaceTriangle;
		// Set label on a triangle
		TFunction<void(int32 TID, int32 Label)> LabelTriangle;
	};

	// Adapter methods to help define the iso-curves where a triangle mesh changes between multiple discrete labels
	// where the labels are expressed as a weight per vertex, and the highest weight should be chosen
	struct FMultiLabelIsoCurveAdapter
	{
		// This function should return the 'class' index with the highest weight for the given vertex, and also return that weight by reference
		TFunction<int32(int32 VID, float& OutLabelWeight)> LabelVertex;
		// This function should return the weight for a given vertex and class index
		TFunction<float(int32 VID, int32 Label)> GetVertexLabelWeight;

		// Find the cut parameter along a given edge where the label should change
		TFunction<double(FIndex2i EdgeV, FIndex2i EdgeLabels, FVector2f EdgeWeights)> FindEdgeCutParam;
		// For a triangle with different labels on each vertex, find the barycentric coordinates where those labels should meet
		// Note: This method is allowed to return a coordinate outside of the triangle (should be handled gracefully by the cutting method)
		TFunction<FVector3d(FIndex3i TriV, FIndex3i TriLabels, FVector3f TriWeights)> FindTriCutBaryCoords;

		// Set default implementations for FindEdgeCutParam and FindTriCutBaryCoords
		UE_API void SetDefaultFindCutFunctions(const TFunction<float(int32 VID, int32 Label)>& InGetVertexLabelWeight);

		// Return true if an edge with these two vertex labels should be cut.
		// Note: Will not be called if the labels are the same.
		// If not set, edges with different labels will always be cut.
		TFunction<bool(int32 Vert0Label, int32 Vert1Label)> ShouldCutEdge;
	};

	struct FMultiLabelCutSettings
	{
		// Distance tolerance at which a cut vertex is too close to an existing vertex (or, for an on-face insertion, an existing edge) and should not be inserted
		double SnapToExistingTolerance = UE_DOUBLE_KINDA_SMALL_NUMBER;

		// Option to never cut triangles and only label triangles; equivalent to setting the snap tolerance high enough to always snap
		bool bNeverCut = false;
	};

	/**
	 * Algorithm to cut a mesh's triangles based on per-vertex labels, and transfer those labels to the mesh triangles.
	 * 
	 * Note this will not introduce new boundaries along the cuts; the resulting mesh will just have new edges along the boundaries.
	 * 
	 * @param InMesh The mesh to cut. Note only original mesh vertices will be queried via this adapter -- never added vertices and triangles.
	 * @param IsoCurveAdapter An adapter defining how vertices should be labelled, which edges should be cut, and the desired cut positions
	 * @param UpdateAdapter An adapter that update the mesh (or a new target mesh) with newly cut/labelled triangles
	 * @param Settings Additional settings controlling how the cuts are made
	 */
	UE_API void MultiLabelCut(const FTriangleMeshAdapterd& InMesh, const FMultiLabelIsoCurveAdapter& IsoCurveAdapter, const FMeshUpdateAdapter& UpdateAdapter, const FMultiLabelCutSettings& Settings);
}

// Dynamic Mesh IsoCurve Settings
struct FMeshIsoCurveSettings
{
	// Whether to collapse any degenerate edges created by the curve insertion
	bool bCollapseDegenerateEdgesOnCut = true;

	// New edges shorter than this will be considered degenerate, and collapsed if bCollapseDegenerateEdgesOnCut is true
	double DegenerateEdgeTol = FMathd::ZeroTolerance;

	// Distance at which to snap curve vertices to nearby existing vertices
	double SnapToExistingVertexTol = UE_DOUBLE_KINDA_SMALL_NUMBER;

	// Tolerance distance (in function domain) to an existing vertex to be 'on curve'
	float CurveIsoValueSnapTolerance = 0.f;
};


/**
 * Insert edges on a dynamic mesh along the isocurve where some scalar value function over the mesh surface crosses a specified value
 */
class FMeshIsoCurves
{
public:

	// Input options

	FMeshIsoCurveSettings Settings;

	/**
	 * Insert new edges on the given mesh along the curve where a function over the mesh surface crosses a given isovalue
	 * 
	 * @param Mesh The mesh to cut
	 * @param VertexFn Function from vertex ID to values
	 * @param EdgeCutFn Given the vertices of an edge and their values, return the parameter where the edge should be cut. Only called if IsoValue is crossed between ValueA and ValueB.
	 * @param IsoValue Value at which to insert a new curve on the mesh
	 */
	UE_API void Cut(FDynamicMesh3& Mesh, TFunctionRef<float(int32)> VertexFn, TFunctionRef<float(int32 VertA, int32 VertB, float ValueA, float ValueB)> EdgeCutFn, float IsoValue = 0);

	/**
	 * Insert new edges on the given mesh along the curve where a function over the mesh vertices, and linearly interpolated over edges, crosses a given isovalue
	 *
	 * @param Mesh The mesh to cut
	 * @param VertexFn Function from vertex ID to values
	 * @param IsoValue Value at which to insert a new curve on the mesh
	 */
	void Cut(FDynamicMesh3& Mesh, TFunctionRef<float(int32)> VertexFn, float IsoValue = 0)
	{
		Cut(Mesh, VertexFn, [IsoValue](int32, int32, float ValueA, float ValueB)
			{
				// Note this is only called on crossing edges, where ValueA != ValueB, so there is no divide-by-zero risk here
				return (ValueA - IsoValue) / (ValueA - ValueB);
			}, IsoValue
		);
	}

private:
	UE_API void SplitCrossingEdges(FDynamicMesh3& Mesh, const TArray<float>& VertexValues, TSet<int32>& OnCutEdges,
		TFunctionRef<float(int32 VIDMin, int32 VIDMax, float ValMin, float ValMax)> EdgeCutFn,
		float IsoValue);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
