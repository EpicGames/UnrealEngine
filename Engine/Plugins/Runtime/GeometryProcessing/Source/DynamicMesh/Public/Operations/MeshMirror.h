// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Util/ProgressCancel.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

enum class EMeshMirrorNormalMode
{
	/** Normals are split and mirrored across the mirror plane. */
	MirrorNormals = 0,
	/** Normals are averaged with their mirrored normal across the mirror plane. */
	AverageMirrorNormals = 1
};

class FMeshMirror
{

public:
	FMeshMirror(FDynamicMesh3* Mesh, FVector3d Origin, FVector3d Normal) 
		: Mesh(Mesh)
		, PlaneOrigin(Origin)
		, PlaneNormal(Normal)
	{
	}
	virtual ~FMeshMirror() = default;

	FDynamicMesh3* Mesh;
	FVector3d PlaneOrigin;
	FVector3d PlaneNormal;

	/** Tolerance distance for considering a vertex to be "on the plane". */
	double PlaneTolerance = FMathf::ZeroTolerance * 10.0;

	/** Whether, when using MirrorAndAppend, vertices on the mirror plane should be welded. */
	bool bWeldAlongPlane = true;

	/** The normal compute method for welded vertices along the mirror plane. */
	EMeshMirrorNormalMode WeldNormalMode = EMeshMirrorNormalMode::MirrorNormals;

	/**
	 * Whether, when welding, the creation of new bowtie vertices should be allowed (if a point lies 
	 * in the mirror plane without an edge in the plane).
	 */
	bool bAllowBowtieVertexCreation = false;

	/**
	 * Alters the existing mesh to be mirrored across the mirror plane.
	 *
	 * @param Progress Object used to cancel the operation early. This leaves the mesh in an undefined state.
	 */
	UE_API void Mirror(FProgressCancel *Progress = nullptr);

	/**
	 * Appends a mirrored copy of the mesh to the mesh.
	 *
	 * @param Progress Object used to cancel the operation early. This leaves the mesh in an undefined state.
	 */
	UE_API void MirrorAndAppend(FProgressCancel* Progress = nullptr);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
