// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshPlaneCut

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "MeshBoundaryLoops.h"
#include "Curve/GeneralPolygon2.h"

#include "Operations/LocalPlanarSimplify.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
template<typename RealType> class TDynamicMeshScalarTriangleAttribute;

// Hacky base class to avoid 8 bytes of padding after the vtable
class FMeshPlaneCutFixLayout
{
public:
	virtual ~FMeshPlaneCutFixLayout() = default;
};

/**
 * Cut the Mesh with the Plane. The *positive* side, ie (p-o).n > 0, is removed.
 * If possible, returns boundary loop(s) along cut
 * (this will fail if cut intersected with holes in mesh).
 * Also FillHoles() for a topological fill. Or use CutLoops and fill yourself.
 * 
 * Algorithm is:
 *    1) find all edge crossings
 *	  2) optionally discard any triangles with all vertex distances < epsilon.
 *    3) Do edge splits at crossings
 *    4 option a) (optionally) delete all vertices on positive side
 *	  4 option b) (OR optionally) disconnect all triangles w/ vertices on positive side (if keeping both sides)
 *	  4 option c) do nothing (if keeping both sides and not disconnecting them)
 *    5) (optionally) collapse any degenerate boundary edges 
 *	  6) (optionally) change an attribute tag for all triangles on positive side
 *    7) find loops through valid boundary edges (ie connected to splits, or on-plane edges) (if second half was kept, do this separately for each separate mesh ID label)
 */
class FMeshPlaneCut : public FMeshPlaneCutFixLayout
{
public:

	//
	// Inputs
	//
	FDynamicMesh3 *Mesh;
	FVector3d PlaneOrigin, PlaneNormal;
	
	/**
	 * If set, only edges that pass this filter will be split
	 */
	TUniqueFunction<bool(int32)> EdgeFilterFunc = nullptr;

	/** Control whether we attempt to auto-simplify the small planar triangles that the plane cut operation tends to generate */
	bool bSimplifyAlongNewEdges = false;

	bool bCollapseDegenerateEdgesOnCut = true;

	/** UVs on any hole fill surfaces are scaled by this amount */
	float UVScaleFactor = 1.0f;

	double DegenerateEdgeTol = FMathd::ZeroTolerance;

	/** Tolerance distance for considering a vertex to be 'on plane' */
	double PlaneTolerance = FMathf::ZeroTolerance * 10.0;

	/** Settings to apply if bSimplifyAlongNewEdges == true */
	FLocalPlanarSimplify SimplifySettings;


	// TODO support optionally restricting plane cut to a mesh selection
	//MeshFaceSelection CutFaceSet;

	//
	// Outputs
	//
	struct FOpenBoundary
	{
		int Label; // optional ID, used to transfer label to new hole-fill triangles
		float NormalSign = 1; // -1 for the open boundary on the other side of the cut (for the CutWithoutDelete path)
		TArray<FEdgeLoop> CutLoops;
		TArray<FEdgeSpan> CutSpans;
		bool CutLoopsFailed = false;		// set to true if we could not compute cut loops/spans
		bool FoundOpenSpans = false;     // set to true if we found open spans in cut
	};
	// note: loops and spans within a single FOpenBoundary could be part of the same hole-fill triangulation
	//	separate open boundary structs will be considered separately and will not share hole fill triangles
	TArray<FOpenBoundary> OpenBoundaries;

	// Triangle IDs of hole fill triangles.  Outer array is 1:1 with the OpenBoundaries array
	TArray<TArray<int>> HoleFillTriangles;

	struct FCutResultRegion
	{
		int32 GroupID;
		TArray<int32> Triangles;
	};
	/** List of output cut regions (eg that have separate GroupIDs). Currently only calculated by SplitEdgesOnly() path */
	TArray<FCutResultRegion> ResultRegions;


	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** List of output cut triangles representing the seed triangles along the cut. Currently only calculated by SplitEdgesOnly() path */
	UE_DEPRECATED(5.3, "To preserve a triangle selection when using SplitEdgesOnly(), instead pass the selection as a pointer to that function.")
	TArray<int32> ResultSeedTriangles;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:

	/**
	 *  Cut mesh with plane. Assumption is that plane normal is Z value.
	 */
	FMeshPlaneCut(FDynamicMesh3* Mesh, FVector3d Origin, FVector3d Normal) : Mesh(Mesh), PlaneOrigin(Origin), PlaneNormal(Normal)
	{
	}

	// Note: deprecation warnings disabled on destructor due to deprecated member variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FMeshPlaneCut() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		// @todo validate inputs
		return EOperationValidationResult::Ok;
	}

	/**
	 * Compute the plane cut by splitting mesh edges that cross the cut plane, and then deleting any triangles
	 * on the positive side of the cutting plane.
	 * @return true if operation succeeds
	 */
	UE_API virtual bool Cut();

	/**
	 * Compute the plane cut by splitting mesh edges that cross the cut plane, but not deleting triangles on positive side.
	 * @param bSplitVerticesAtPlane if true, vertices on cutting plane are split into multiple vertices
	 * @param OffsetSeparatedPortion positive side of cut mesh is offset by this distance along plane normal
	 * @param TriLabels optional per-triangle integer labels
	 * @param NewLabelStartID starting new label value
	 * @param bAddBoundariesFirstHalf add open boundaries on "first" half to OpenBoundaries list (default true)
	 * @param bAddBoundariesSecondHalf add open boundaries on "second" half to OpenBoundaries list (default true)
	 * @return true if operation succeeds
	 */
	UE_API virtual bool CutWithoutDelete(bool bSplitVerticesAtPlane, float OffsetSeparatedPortion = 0.0f,
		TDynamicMeshScalarTriangleAttribute<int>* TriLabels = nullptr, int NewLabelStartID = 0,
		bool bAddBoundariesFirstHalf = true, bool bAddBoundariesSecondHalf = true);

	/**
	 * Compute the plane cut by splitting mesh edges that cross the cut plane, and then optionally update groups
	 * @param bAssignNewGroups if true, update group IDs such that each group-connected-component on either side of the cut plane is assigned a new unique group ID
	 * @param OptionalTriangleSelection if non-null, a set of triangle IDs that must be updated along with any triangle splits or deletions in the edge split process
	 * @return true if operation succeeds
	 */
	virtual bool SplitEdgesOnly(bool bAssignNewGroups, TSet<int32>* OptionalTriangleSelection)
	{
		return SplitEdgesOnlyHelper(bAssignNewGroups, OptionalTriangleSelection, false);
	}

	UE_DEPRECATED(5.3, "Instead use the two-parameter version of SplitEdgesOnly, which has direction selection tracking and does not populate the deprecated ResultSeedTriangles")
	virtual bool SplitEdgesOnly(bool bAssignNewGroups)
	{
		return SplitEdgesOnlyHelper(bAssignNewGroups, nullptr, true);
	}

	/**
	 *  Fill cut loops with FSimpleHoleFiller
	 */
	UE_API virtual bool SimpleHoleFill(int ConstantGroupID = -1);

	UE_API virtual bool MinimalHoleFill(int ConstantGroupID = -1);

	/**
	 *  Fill cut loops with FPlanarHoleFiller, using a caller-provided triangulation function
	 */
	UE_API virtual bool HoleFill(TFunction<TArray<FIndex3i>(const FGeneralPolygon2d&)> PlanarTriangulationFunc, bool bFillSpans, int ConstantGroupID = -1, int MaterialID = -1);

	
	UE_API virtual void TransferTriangleLabelsToHoleFillTriangles(TDynamicMeshScalarTriangleAttribute<int>* TriLabels);

protected:

	UE_DEPRECATED(5.3, "Use the single-set CollapseDegenerateEdges instead")
	UE_API void CollapseDegenerateEdges(const TSet<int>& OnCutEdges, const TSet<int>& ZeroEdges);
	
	/// Collapse degenerate edges
	/// @param Edges									Edges to consider for collapse; will updated by removing edges at they are collapsed
	/// @param bRemoveAllDegenerateFromInputSet			Whether we should also check whether the neighbor edges removed by collapse were also in the set
	/// 												(typically this is not needed, and has additional cost)
	/// @param TriangleSelection						Optional set tracking an active selection. Any triangles removed by collapse will also be removed from the set.
	UE_API void CollapseDegenerateEdges(TSet<int>& Edges, bool bRemoveAllDegenerateFromInputSet, TSet<int>* TriangleSelection = nullptr);
	
	UE_API void SplitCrossingEdges(TArray<double>& Signs, TSet<int>& ZeroEdges, TSet<int>& OnCutEdges, bool bDeleteTrisOnPlane = true);
	UE_API void SplitCrossingEdges(TArray<double>& Signs, TSet<int>& ZeroEdges, TSet<int>& OnCutEdges, TSet<int>& OnSplitEdges, bool bDeleteTrisOnPlane = true);
	UE_API void SplitCrossingEdges(bool bDeleteTrisOnPlane, TArray<double>& Signs, TSet<int>& AlreadyOnPlaneEdges, TSet<int32>& CutPlaneEdges, 
		TSet<int>* SplitEdges = nullptr, TSet<int>* OnPlaneVertices = nullptr, TSet<int>* TriangleSelection = nullptr);
	
	UE_API bool ExtractBoundaryLoops(const TSet<int>& OnCutEdges, const TSet<int>& ZeroEdges, FMeshPlaneCut::FOpenBoundary& Boundary);

	// set of vertices lying on plane after calling SplitCrossingEdges
	UE_DEPRECATED(5.3, "If needed, explicitly request the vertices on the cut plane the the OnPlaneVertices argument of SplitCrossingEdges")
	TSet<int32> OnCutVertices;

private:

	// Helper to compute signed distances from the cutting plane for all vertices of the mesh.  Value at Invalid Vertex IDs will be set to InvalidDist.
	UE_API void ComputeVertexSignedDistances(TArray<double>& Signs, double InvalidDist);

	UE_API bool SplitEdgesOnlyHelper(bool bAssignNewGroups, TSet<int32>* OptionalTriangleSelection, bool bAddDeprecatedResultSeedTriangles);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
