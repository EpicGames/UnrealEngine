// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "Polygon2.h"
#include "DynamicMesh/DynamicMesh3.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * Calculate a Convex Hull for a Mesh by first Projecting all vertices to a plane, computing a
 * 2D convex polygon that contains them, and then sweeping that 2D hull to create an extruded 3D volume.
 */
class FMeshProjectionHull
{
public:
	/** Input Mesh */
	const FDynamicMesh3* Mesh;

	/** Input 3D Frame/Plane */
	FFrame3d ProjectionFrame;

	/** If true, 2D convex hull is simplified using MinEdgeLength and DeviationTolerance */
	bool bSimplifyPolygon = false;
	/** Minimum Edge Length of the simplified 2D Convex Hull */
	double MinEdgeLength = 0.01;
	/** Deviation Tolerance of the simplified 2D Convex Hull */
	double DeviationTolerance = 0.1;

	/** Minimum thickness of extrusion. If extrusion length is smaller than this amount, box is expanded symmetrically */
	double MinThickness = 0.0;


	/** Calculated convex hull polygon */
	FPolygon2d ConvexHull2D;

	/** Simplified convex hull polygon. Not initialized if bSimplifyPolygon == false */
	FPolygon2d SimplifiedHull2D;

	/** Output swept-polygon convex hull */
	FDynamicMesh3 ConvexHull3D;

	enum class EKeep3DHullSide : uint8
	{
		// Both sides of the projection hull will be flat
		None,
		// The front of the projection hull will follow the 3D convex hull, and the back will be flat
		Front,
		// The back of the projection hull will follow the 3D convex hull, and the front will be flat
		Back
		// Note: If both the front and back follow the 3D hull, that is just a regular convex hull; see MeshConvexHull.h
	};

	/** Whether to conform to the 3D convex hull surface on the front or back side of the sweep, or to use a flat surface on both sides of the swept hull */
	EKeep3DHullSide Keep3DHullSide = EKeep3DHullSide::None;

public:
	FMeshProjectionHull(const FDynamicMesh3* MeshIn)
	{
		Mesh = MeshIn;
	}

	/**
	 * Calculate output 2D Convex Polygon and Swept-Polygon 3D Mesh for vertices of input Mesh
	 * @return true on success
	 */
	UE_API bool Compute();

private:

	// Helper to compute the 3D hull when Keep3DHullSide is not None; called by Compute after the 2D projection hull is computed
	UE_API bool ComputeWith3DHullSide(FFrame3d CenterFrame, double ExtrudeLength);

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
