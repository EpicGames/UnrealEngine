// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp RegionOperator

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "Util/SparseIndexCollectionTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicSubmesh3.h"
#include "FaceGroupUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "Algo/RemoveIf.h"

namespace UE
{
namespace Geometry
{

/**
 * This class automatically extracts a submesh from a mesh, 
 *  and can re-insert it after you have edited it,
 *  as long as you have not messed up the boundary
 * 
 * [TODO] Should this share any code w/ RegionRemesher?
 * [TODO] ReinsertSubToBaseMapT is not returned by the FDynamicMeshEditor.ReinsertSubmesh, instead we are
   trying to guess it here, by making some assumptions about what happens. It works for now, but
   it would better if FDynamicMeshEditor returned this information.
 */
class FMeshRegionOperator
{
public:
	FDynamicMesh3* BaseMesh;
	FDynamicSubmesh3 Region;

	// this is only valid after BackPropagate() call!! maps submeshverts to base mesh
	FOptionallySparseIndexMap ReinsertSubToBaseMapV;

	// Note computation of this is kind of a hack right now; see TODO in class comment above
	FOptionallySparseIndexMap ReinsertSubToBaseMapT;

	// handle a tricky problem...see comments for EDuplicateTriBehavior enum
	FDynamicMeshEditor::EDuplicateTriBehavior ReinsertDuplicateTriBehavior = FDynamicMeshEditor::EDuplicateTriBehavior::EnsureContinue;

	// IDs of triangles in the base mesh that correspond to the submesh at time of last Compute or BackPropagate
	TArray<int> CurrentBaseTris;

	FMeshRegionOperator(FDynamicMesh3* Base, const TArrayView<const int>& RegionTris, TFunctionRef<void(FDynamicSubmesh3&)> ConfigSubmeshFunc) : BaseMesh(Base)
	{
		check(Base);
		ConfigSubmeshFunc(Region);
		Region.Compute(Base, RegionTris);

		CurrentBaseTris.Append(RegionTris.GetData(), RegionTris.Num());
	}

	FMeshRegionOperator(FDynamicMesh3* Base, const TArrayView<const int>& RegionTris) : BaseMesh(Base)
	{
		check(Base);
		Region.Compute(Base, RegionTris);

		CurrentBaseTris.Append(RegionTris.GetData(), RegionTris.Num());
	}

	/**
	 * @return the array of sub-region triangles. This is either the input regionTris or the submesh triangles after they are re-inserted.
	 */
	const TArray<int>& CurrentBaseTriangles() const
	{
		return CurrentBaseTris;
	}

	/**
	 * @return base mesh interior vertices of region (does not include region boundary vertices)
	 */
	TSet<int> CurrentBaseInteriorVertices() const
	{
		TSet<int> verts;
		const TSet<int>& borderv = Region.GetBaseBorderVertices();
		for (int tid : CurrentBaseTris)
		{
			FIndex3i tv = BaseMesh->GetTriangle(tid);
			if (borderv.Contains(tv.A) == false)
			{
				verts.Add(tv.A);
			}
			if (borderv.Contains(tv.B) == false)
			{
				verts.Add(tv.B);
			}
			if (borderv.Contains(tv.C) == false)
			{
				verts.Add(tv.C);
			}
		}
		return verts;
	}

	/**
	 * After remeshing we may create an internal edge between two boundary vertices [a,b].
	 * Those vertices will be merged with vertices c and d in the base mesh. If the edge
	 * [c,d] already exists in the base mesh, then after the merge we would have at least
	 * 3 triangles at this edge. Dang.
	 *
	 * A common example is a 'fin' triangle that would duplicate a
	 * 'fin' on the border of the base mesh after removing the submesh, but this situation can
	 * arise anywhere (eg think about one-triangle-wide strips).
	 *
	 * This is very hard to remove, but we can at least avoid creating non-manifold edges (which
	 * with the current DMesh3 will be prevented, hence leaving a hole) by splitting the 
	 * internal edge in the submesh (which presumably we were remeshing anyway, so changes are ok).
	 */
	void RepairPossibleNonManifoldEdges()
	{
		// [TODO] do we need to repeat this more than once? I don't think so...

		// repair submesh
		int NE = Region.GetSubmesh().MaxEdgeID();
		TArray<int> split_edges;
		for (int eid = 0; eid < NE; ++eid)
		{
			if (Region.GetSubmesh().IsEdge(eid) == false)
			{
				continue;
			}
			if (Region.GetSubmesh().IsBoundaryEdge(eid))
			{
				continue;
			}
			FIndex2i edgev = Region.GetSubmesh().GetEdgeV(eid);
			if (Region.GetSubmesh().IsBoundaryVertex(edgev.A) && Region.GetSubmesh().IsBoundaryVertex(edgev.B))
			{
				// ok, we have an internal edge where both verts are on the boundary
				// now check if it is an edge in the base mesh
				int base_a = Region.MapVertexToBaseMesh(edgev.A);
				int base_b = Region.MapVertexToBaseMesh(edgev.B);
				if (base_a != FDynamicMesh3::InvalidID && base_b != FDynamicMesh3::InvalidID)
				{
					// both vertices in base mesh...right?
					ensure(BaseMesh->IsVertex(base_a) && BaseMesh->IsVertex(base_b));
					int base_eid = Region.GetBaseMesh()->FindEdge(base_a, base_b);
					if (base_eid != FDynamicMesh3::InvalidID)
					{
						split_edges.Add(eid);
					}
				}
			}
		}

		// split any problem edges we found and repeat this loop
		for (int i = 0; i < split_edges.Num(); ++i)
		{
			DynamicMeshInfo::FEdgeSplitInfo split_info;
			Region.GetSubmesh().SplitEdge(split_edges[i], split_info);
		}
	}


	/**
	 * @param gid New group ID to set for entire submesh
	 */
	void SetSubmeshGroupID(int GID)
	{
		FaceGroupUtil::SetGroupID(Region.GetSubmesh(), GID);
	}

	/**
	 * Remove the original submesh region and merge in the remeshed version.
	 * You can call this multiple times as the base-triangle-set is updated.
	 *
	 * @param bAllowSubmeshRepairs if true, we allow the submesh to be modified to prevent creation of non-manifold edges.
	 *							   You can disable this, however then some of the submesh triangles may be discarded.
	 *
	 * @return false if there were errors in insertion, ie if some triangles failed to insert.
	 *         Does not revert changes that were successful.
	 */
	bool BackPropropagate(bool bAllowSubmeshRepairs = true)
	{
		if (bAllowSubmeshRepairs)
		{
			RepairPossibleNonManifoldEdges();
		}

		// remove existing submesh triangles
		FDynamicMeshEditor Editor(BaseMesh);
		Editor.RemoveTriangles(CurrentBaseTris, true);

		// insert new submesh
		TArray<int> new_tris; 
		new_tris.Reserve(Region.GetSubmesh().TriangleCount());
		bool bOK = Editor.ReinsertSubmesh(Region, ReinsertSubToBaseMapV, &new_tris, ReinsertDuplicateTriBehavior);

		// reconstruct this...hacky? TODO: have ReinsertSubmesh directly build this
		int NT = Region.GetSubmesh().MaxTriangleID();
		ReinsertSubToBaseMapT.Initialize(NT, Region.GetSubmesh().TriangleCount());
		int nti = 0;
		bool bHasInvalid = false;
		for (int ti = 0; ti < NT; ++ti)
		{
			if (Region.GetSubmesh().IsTriangle(ti) == false)
			{
				continue;
			}
			int new_tri = new_tris[nti++];
			// should only happen if not allowing submesh repairs and ReinsertDuplicateTriBehavior is EnsureContinue
			if (!bAllowSubmeshRepairs && !ensure(new_tri != -2))
			{
				bHasInvalid = true;
				continue;
			}
			ReinsertSubToBaseMapT.Set(ti, new_tri);
		}
		// remove invalid triangles due to insertion failures (must be kept before this point for correspondence)
		if (bHasInvalid)
		{
			new_tris.SetNum(Algo::RemoveIf(new_tris, [](int id) { return id == -2; }));
		}

		// assert that new triangles are all valid (goes wrong sometimes??)
		checkSlow(IndexUtil::ArrayCheck(new_tris, [this](int TID) {return BaseMesh->IsTriangle(TID); }));

		CurrentBaseTris = new_tris; // TODO: instead of creating a separate new_tris, operate directly on CurrentBaseTris?
		return bOK;
	}


	/**
	 * Transfer vertex positions in submesh back to base mesh; for use when the vertices in submesh still directly correspond back
	 * @return true on success
	 */
	bool BackPropropagateVertices(bool bRecomputeBoundaryNormals = false)
	{
		bool bNormals = (Region.GetSubmesh().HasVertexNormals() && Region.GetBaseMesh()->HasVertexNormals());
		for (int subvid : Region.GetSubmesh().VertexIndicesItr()) {
			int basevid = Region.MapVertexToBaseMesh(subvid);
			FVector3d v = Region.GetSubmesh().GetVertex(subvid);
			BaseMesh->SetVertex(basevid, v);
			if (bNormals)
			{
				BaseMesh->SetVertexNormal(basevid, (FVector3f)Region.GetSubmesh().GetVertexNormal(subvid));
			}
		}

		if (bRecomputeBoundaryNormals)
		{
			for (int basevid : Region.GetBaseBorderVertices())
			{
				FVector3d n = FMeshNormals::ComputeVertexNormal(*BaseMesh, basevid);
				BaseMesh->SetVertexNormal(basevid, (FVector3f)n);
			}
		}

		return true;
	}
};


} // end namespace UE::Geometry
} // end namespace UE
