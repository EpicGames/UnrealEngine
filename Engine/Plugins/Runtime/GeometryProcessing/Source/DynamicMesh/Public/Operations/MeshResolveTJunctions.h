// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * FMeshResolveTJunctions splits edges to create matching vertices at T-Junctions in the mesh.
 * T-Junctions are edge configurations where a border vertex lies on an opposing border edge. 
 * This frequently occurs in (eg) CAD meshes where the tessellator had bugs or simply did not 
 * bother to ensure that tessellations match on adjacent spline patches. 
 * 
 * The simplest case would be, one side has two collinear edges [A,B] and [B,C],
 * and the other side has edge [F,G] such that F/A and G/C are coincident. 
 * The implementation works by adding a matching vertex to [F,G], via an edge split
 * at the projected location of B, to create a new vertex X that could be welded with B.
 * 
 *                    F  A                         F  A
 * So basically it    |  |                         |  |
 * turns this         |  B         into this one   X  B
 * configuration      |  |                         |  |
 *                    G  C                         G  C
 *
 * If everything is within-tolerance, then after resolving all the T-Junctions, 
 * a FMergeCoincidentMeshEdges would successfully weld the new set of border 
 * edges back together (note: caller must do this, FMeshResolveTJunctions only splits, it does not weld!)
 * 
 * Caller can provide a subset of edges via BoundaryEdges, otherwise all boundary edges
 * in Mesh will be used.
 * 
 * Current implementation is O(N*M) in the number of boundary edges (N) and boundary vertices (M).
 * Could be improved with a spatial data structure. 
 */
class FMeshResolveTJunctions
{
public:
	/** default tolerance is float ZeroTolerance */
	static UE_API const double DEFAULT_TOLERANCE;  // = FMathf::ZeroTolerance;

	/** The mesh that we are modifying */
	FDynamicMesh3* Mesh;

	/** Subset of mesh boundary edges (otherwise all boundary edges are processed) */
	TSet<int32> BoundaryEdges;

	/** Distance threshold used for various checks (eg is vertex on edge, end endpoint tolerance, etc) */
	double DistanceTolerance = DEFAULT_TOLERANCE;

	/** Number of edges that were split to resolve T-junctions */
	int32 NumSplitEdges = 0;


public:
	FMeshResolveTJunctions(FDynamicMesh3* MeshIn) : Mesh(MeshIn)
	{
	}

	/**
	 * Run the resolve operation and modify .Mesh
	 * @return true if the algorithm succeeds
	 */
	UE_API bool Apply();

};

/**
 * Similar to FMeshResolveTJunctions, but does not add any vertices to the mesh.
 * Supports running multiple snapping iterations, because snapped-to edges may move in subsequent snaps.
 */
class FMeshSnapOpenBoundaries
{
public:
	/** default tolerance is float ZeroTolerance */
	DYNAMICMESH_API static const double DEFAULT_TOLERANCE;  // = FMathf::ZeroTolerance;

	/** The mesh that we are modifying */
	FDynamicMesh3* Mesh;

	/** Subset of mesh boundary edges (otherwise all boundary edges are processed) */
	TSet<int32> BoundaryEdges;

	/** Distance threshold used for various checks (eg is vertex on edge, end endpoint tolerance, etc) */
	double DistanceTolerance = DEFAULT_TOLERANCE;

	/** Scalar multiple of DistanceTolerance at which we snap a vertex directly to another vertex, rather than an edge */
	double VertexSnapToleranceFactor = 1.0;

	/** Number of vertex snaps performed (cumulative over iterations) */
	int32 NumVertexSnaps = 0;

	/** Maximum number of snapping iterations to perform */
	int32 MaxIterations = 1;

	/** Whether vertices can be snapped to edges; otherwise, vertices are only snapped to other vertices */
	bool bSnapToEdges = true;

	/** Whether to avoid snapping in cases where doing so would locally flip a triangle */
	bool bPreventFlips = true;


public:
	FMeshSnapOpenBoundaries(FDynamicMesh3* MeshIn) : Mesh(MeshIn)
	{
	}

	/**
	 * Run the resolve operation and modify .Mesh
	 * @return true if the algorithm succeeds
	 */
	DYNAMICMESH_API bool Apply();

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
