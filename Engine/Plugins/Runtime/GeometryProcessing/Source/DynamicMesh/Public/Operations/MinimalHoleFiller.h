// Copyright Epic Games, Inc. All Rights Reserved.
// Port of geometry3Sharp MinimalHoleFiller

#pragma once

#include "HoleFiller.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "MeshBoundaryLoops.h"
#include "SimpleHoleFiller.h"
#include "Containers/Set.h"
#include "MeshRegionOperator.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "QueueRemesher.h"
#include "MeshConstraintsUtil.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{


/// Construct a "minimal" fill surface for the hole. This surface is often quasi-developable, reconstructs sharp edges, 
/// etc. There are various options.

class FMinimalHoleFiller : public IHoleFiller
{
public:

	FMinimalHoleFiller(FDynamicMesh3* InMesh, FEdgeLoop InFillLoop) :
		Mesh(InMesh),
		FillLoop(InFillLoop),
		FillMesh(nullptr)
	{}

	UE_API bool Fill(int32 GroupID = -1) override;

	// Settings
	bool bIgnoreBoundaryTriangles = false;
	bool bOptimizeDevelopability = true;
	bool bOptimizeTriangles = true;
	double DevelopabilityTolerance = 0.0001;
	// whether to automatically fill in PrimaryColors on new hole fill triangles
	bool bAutoFillPrimaryColors = true;

private:

	// Inputs
	FDynamicMesh3* Mesh;
	FEdgeLoop FillLoop;

	TUniquePtr<FMeshRegionOperator> RegionOp;
	FDynamicMesh3* FillMesh;
	TSet<int> BoundaryVertices;
	TMap<int, double> ExteriorAngleSums;
	TArray<double> Curvatures;

	UE_API void AddAllEdges(int EdgeID, TSet<int>& EdgeSet);

	UE_API double AspectMetric(int eid);
	static UE_API double GetTriAspect(const FDynamicMesh3& mesh, FIndex3i& tri);

	UE_API void UpdateCurvature(int vid);
	UE_API double CurvatureMetricCached(int a, int b, int c, int d);
	UE_API double CurvatureMetricEval(int a, int b, int c, int d);
	UE_API double ComputeGaussCurvature(int vid);

	// Steps in Fill:
	UE_API void CollapseToMinimal();
	UE_API void RemoveRemainingInteriorVerts();
	UE_API void FlipToFlatter();
	UE_API void FlipToMinimizeCurvature();
	UE_API void FlipToImproveAspectRatios();
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
