// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "MeshRegionBoundaryLoops.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
class FMeshNormals;
class FDynamicMeshChangeTracker;

// Hacky base class to avoid 8 bytes of padding after the vtable
class FOffsetMeshRegionFixLayout
{
public:
	virtual ~FOffsetMeshRegionFixLayout() = default;
};

/**
 * FOffsetMeshRegion implements local extrusion/offset of a mesh region. 
 * The selected triangles are separated and then stitched back together, creating
 * an new strip of triangles around their border (s). The offset region is
 * then transformed using the OffsetPositionFunc.
 *
 * Complex input regions are handled, eg it can be multiple disconnected components, donut-shaped, etc
 * 
 * Each quad of the border loop is assigned it's own normal and UVs (ie each is a separate UV-island)
 */
class FOffsetMeshRegion : public FOffsetMeshRegionFixLayout
{
public:

	/**
	 * Method used to determine the per-vertex offset direction vector
	 */
	enum class EVertexExtrusionVectorType
	{
		/** No geometric offset vector, client must define offset via custom OffsetPositionFunc */
		Zero,
		/** Offset vector is per-vertex normal, either provided by FDynamicMesh3 Per-Vertex Normals if defined, or computed by averaging one-ring face normals */
		VertexNormal,
		/** Angle weighted average of the triangles in the selection that contain this vertex(unit length) */
		SelectionTriNormalsAngleWeightedAverage,
		/** 
		 *  Like SelectionTriNormalsAngleWeightedAverage, but with the vertex length adjusted to try to keep
		 *  the selection triangles parallel to their original location. 
		 */
		SelectionTriNormalsAngleWeightedAdjusted,
	};

	//
	// Inputs
	//

	/** The mesh that we are modifying */
	FDynamicMesh3* Mesh;

	/** The triangle region we are modifying */
	TArray<int32> Triangles;

	/** OffsetPositionFunc function is called to generate the offset vertex position. */
	TFunction<FVector3d(const FVector3d& Position, const FVector3d& VertexVector, int Vid)> OffsetPositionFunc = 
		[this](const FVector3d& Position, const FVector3d& VertexVector, int Vid)
	{
		return Position + this->DefaultOffsetDistance * (FVector3d)VertexVector;
	};

	/** Used in the default OffsetPositionFunc. */
	double DefaultOffsetDistance = 1.0;

	/** Determines the type of vector passed in per-vertex to OffsetPositionFunc */
	EVertexExtrusionVectorType ExtrusionVectorType = EVertexExtrusionVectorType::Zero;

	/** 
	 * When stitching the extruded portion to the source loop, this determines the grouping of the side quads by 
	 * passing in successive Eids in the original loop. For instance, always returning true will result in one 
	 * group for the stitched portion, and always returning true will result in a separate group for every edge
	 * quad. By default, the same group is given to adjacent edges that separate the same two groups, or to border
	 * edges that are colinear.
	 */
	TFunction<bool(int32 Eid1, int32 Eid2)> LoopEdgesShouldHaveSameGroup =
		[this](int32 Eid1, int32 Eid2) { return EdgesSeparateSameGroupsAndAreColinearAtBorder(Mesh, Eid1, Eid2, true); };

	/** quads on the stitch loop are planar-projected and scaled by this amount */
	float UVScaleFactor = 1.0f;

	/** If a sub-region of Triangles is a full connected component, offset into a solid instead of leaving a shell*/
	bool bOffsetFullComponentsAsSolids = true;

	/** 
	 * When bOffsetFullComponentsAsSolids is true and the extrusion applied by OffsetPositionFunc is negative,
	 * then fully-connected components need to be turned inside-out. Thus, in this circumstance we need to
	 * be notified whether the OffsetPositionFunc is applying a positive or negative offset.
	 */
	bool bIsPositiveOffset = true;

	/** When using SelectionTriNormalsAngleWeightedAdjusted to keep triangles parallel, clamp the offset scale to at most this factor */
	double MaxScaleForAdjustingTriNormalsOffset = 4.0;

	/** 
	 * Number of subdivisions along the length of the "tubes" used to stitch the offset regions.
	 * This parameter is only supported in EVersion::Version1 and later
	 */
	int32 NumSubdivisions = 0;

	/**
	 * If true, each extruded area gets a single new group, instead of remapping the input groups
	 * This parameter is only supported in EVersion::Version1 and later
	 */
	bool bSingleGroupPerArea = true;

	/**
	 * If true, each subdivision level gets a new group
	 * This parameter is only supported in EVersion::Version1 and later
	 */
	bool bGroupPerSubdivision = true;

	/** 
	 * if true, new UV island is assigned for each group in stitch region. 
	 * This parameter is only supported in EVersion::Version1 and later
	 * This parameter is only supported in EVersion::Version1 and later
	 */
	bool bUVIslandPerGroup = true;

	/** 
	* If true, Material IDs around the border of the extrude area are propagated "down" the extrusion tube 
	* This parameter is only supported in EVersion::Version1 and later
	*/
	bool bInferMaterialID = true;

	/**
	 * Split the extrude "tube" into separate groups based on the opening angle between quads.
	 * This split is done relative to the above group options, so even if the tube would 
	 * only have a single group, but the extrude area is a square, it will be split into
	 * groups based on the corners.
	 * This parameter is only supported in EVersion::Version1 and later
	 */
	double CreaseAngleThresholdDeg = 180.0;


	/** 
	 * Constant Material ID to set along the extrusion tubes 
	 * This parameter is only supported in EVersion::Version1 and later
	 */
	int SetMaterialID = 0;

	/**
	 * Support for different versions of the OffsetMeshRegion geometric operation.
	 * We default to the latest version.
	 */
	enum class EVersion
	{
		// always use most recent version
		Current = 0,
		// initial implementation, various limitations and issues, maintained for back-compat
		Legacy = 1,
		// support NumSubdivisions, UVs and Normals computed by polygroup
		Version1 = 2,
	};

	/** Version of Offset operation being used. Defaults to Current, ie most recent version */
	EVersion UseVersion = EVersion::Current;
	

	//
	// Outputs
	//

	/**
	 * Offset information for a single connected component
	 */
	struct FOffsetInfo
	{
		/**
		 * Set of triangles for this region. These will be the same Tids that were
		 * originally in the region. 
		 */
		TArray<int32> OffsetTids;
		/** 
		 * Groups on offset faces. These IDs will be the same as the original groups 
		 * in the region. 
		 */
		TArray<int32> OffsetGroups;

		/** Initial loops on the mesh */
		TArray<FEdgeLoop> BaseLoops;
		/** 
		 * Offset loops on the mesh. If the region had bowtie vertices (in terms of selection, not
		 * even necessarily in terms of original mesh topology), then the number of loops here may
		 * no longer match number of loops in BaseLoops, StitchTriangles, etc (for example, imagine
		 * a circular hole tangent inside a circular region; when the tangent bowtie is split, this
		 * becomes one C-shaped loop instead of two nested ones)
		 */
		TArray<FEdgeLoop> OffsetLoops;

		/** Lists of triangle-strip "tubes" that connect each loop-pair */
		TArray<TArray<int>> StitchTriangles;
		/** List of group ids / polygon ids on each triangle-strip "tube" */
		TArray<TArray<int>> StitchPolygonIDs;

		/** If true, full region was thickened into a solid */
		bool bIsSolid = false;
	};

	/**
	 * List of offset regions/components
	 */
	TArray<FOffsetInfo> OffsetRegions;

	/**
	 * List of all triangles created/modified by this operation
	 */
	TArray<int32> AllModifiedAndNewTriangles;

public:
	UE_API FOffsetMeshRegion(FDynamicMesh3* mesh);

	virtual ~FOffsetMeshRegion() = default;


	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		// @todo calculate MeshBoundaryLoops and make sure it is valid

		// is there any reason we couldn't do this??

		return EOperationValidationResult::Ok;
	}


	/**
	 * Apply the Offset operation to the input mesh.
	 * @return true if the algorithm succeeds
	 */
	UE_API virtual bool Apply();

	// The default function used for LoopEdgesShouldHaveSameGroup. 
	static UE_API bool EdgesSeparateSameGroupsAndAreColinearAtBorder(FDynamicMesh3* Mesh,
		int32 Eid1, int32 Eid2, bool bCheckColinearityAtBorder);
protected:

	UE_API virtual bool ApplyOffset(FOffsetInfo& Region);

	UE_API virtual bool ApplyOffset_Legacy(FOffsetInfo& Region);
	UE_API virtual bool ApplyOffset_Version1(FOffsetInfo& Region);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
