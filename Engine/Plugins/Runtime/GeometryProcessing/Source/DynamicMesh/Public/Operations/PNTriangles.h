// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"

#define UE_API DYNAMICMESH_API

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * FPNTriangles implements curved PN (Point-Normal) Triangles (Curved PN Triangles, Vlachos, 2001).
 * Each PN triangle replaces one original flat triangle by a curved shape that is retriangulated into 
 * a number of small subtriangles. The geometry of a PN triangle is defined as one cubic Bezier patch using control 
 * points. The patch matches the point and normal information at the vertices of the original flat triangle.
 */

class FPNTriangles
{
public:

	//
	// Input & Output
	//

	/** The mesh that we are modifying. If the operator fails or is canceled by the user, the mesh will not be changed. */
	FDynamicMesh3* Mesh = nullptr;

	//
	// Input
	//

	/** Set this to be able to cancel running operation. */
	FProgressCancel* Progress = nullptr;
	
	/** How many times we are recursively subdividing triangles (loop style subdivision). */
	int32 TessellationLevel = 1;

	/**
	 * If true and the mesh has the normal overlay then recalculate it. 
	 * If true and the normal overlay doesn't exists, compute per-vertex normals (if they are enabled in the mesh).
	 * If false, only perform the displacement.
	 */
	bool bRecalculateNormals = false;

public:
	FPNTriangles(FDynamicMesh3* Mesh) : Mesh(Mesh)
	{
	}

	virtual ~FPNTriangles() = default; 

	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot.
	 */
	virtual EOperationValidationResult Validate()
	{
		if (TessellationLevel < 0 || Mesh == nullptr) 
		{
			return EOperationValidationResult::Failed_UnknownReason;    
		}

		return EOperationValidationResult::Ok;
	}

	/**
	 * Generate PN Triangles geometry and optionally compute quadratically varying normals.
	 * @return true if the algorithm succeeds, false if it failed or was canceled by the user.
	 */
	UE_API virtual bool Compute();
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
