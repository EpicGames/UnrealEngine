// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * If both triangles on an edge are coplanar, we can arbitrarily flip the interior edge to
 * improve triangle quality. Similarly if one triangle on an edge is degenerate, we can flip
 * the edge without affecting the shape to try to remove it. This code does one or more passes of
 * such an optimization.
 */
class FPlanarFlipsOptimization
{
public:

	explicit FPlanarFlipsOptimization(FDynamicMesh3* MeshIn, int32 NumPassesIn = 1, double PlanarDotThreshIn = 0.99)
		: Mesh(MeshIn)
		, NumPasses(NumPassesIn)
		, PlanarDotThresh(PlanarDotThreshIn)
	{
	}

	FDynamicMesh3* Mesh;
	int32 NumPasses;
	double PlanarDotThresh;
	bool bRespectGroupBoundaries = true;

	/**
	 * Apply the operation to the mesh.
	 */
	UE_API void Apply();

protected:
	UE_API void ApplySinglePass();
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
