// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp PlanarHoleFiller

#pragma once

#include "HoleFiller.h"
#include "CoreMinimal.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "IndexTypes.h"
#include "GeometryTypes.h"
#include "Curve/GeneralPolygon2.h"
#include "MeshBoundaryLoops.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * Fill a set of boundary loops with planar surfaces.  User must provide the triangulation function.
 */
class FPlanarHoleFiller : public IHoleFiller
{
public:
	//
	// Inputs
	//
	FDynamicMesh3* Mesh;
	const TArray<TArray<int>> *VertexLoops;

	TFunction<TArray<FIndex3i>(const FGeneralPolygon2d&)> PlanarTriangulationFunc;

	FVector3d PlaneOrigin;
	FVector3d PlaneNormal;

	// whether to automatically fill in PrimaryColors on new hole fill triangles
	bool bAutoFillPrimaryColors = true;

public:
	/**
	 *  Construct planar hole filler (projects the hole edges to a plane and fills the hole w/ a general 2D polygon triangulation)
	 */
	FPlanarHoleFiller(FDynamicMesh3* Mesh, const TArray<TArray<int>> *VertexLoops, TFunction<TArray<FIndex3i>(const FGeneralPolygon2d&)> PlanarTriangulationFunc, FVector3d PlaneOrigin, FVector3d PlaneNormal) 
		: Mesh(Mesh), VertexLoops(VertexLoops), PlanarTriangulationFunc(PlanarTriangulationFunc), PlaneOrigin(PlaneOrigin), PlaneNormal(PlaneNormal)
	{
	}
	virtual ~FPlanarHoleFiller() = default;
	
	
	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		// TODO: validation

		return EOperationValidationResult::Ok;
	}

	UE_API bool Fill(int GroupID = -1) override;
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
