// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "IntVectorTypes.h"
#include "GeometryTypes.h"
#include "LineTypes.h"

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * For each edge in EdgeList, compute a line segment offset from the original edge by
 * distance InsetDistance. Assumes that each Edge is a boundary edge and so Triangle0
 * of that edge defines "inside" direction.
 */
DYNAMICMESH_API void ComputeInsetLineSegmentsFromEdges(
	const FDynamicMesh3& Mesh, 
	const TArray<int32>& EdgeList,
	double InsetDistance,
	TArray<FLine3d>& InsetLinesOut );

/**
 * For each vertex in VertexIDs, compute inset position based on list of precomputed
 * inset lines and store in VertexPositionsOut. Assumption is that each vertex position 
 * was at the intersection point of the pre-inset lines, ie sequential edge-line-pairs
 * determine each sequential vertex position
 * @param bIsLoop if true, vertex list is treated as a loop, otherwise as an open span
 */
DYNAMICMESH_API void SolveInsetVertexPositionsFromInsetLines(
	const FDynamicMesh3& Mesh,
	const TArray<FLine3d>& InsetEdgeLines,
	const TArray<int32>& VertexIDs,
	TArray<FVector3d>& VertexPositionsOut,
	bool bIsLoop);

/**
 * Solve for a new inset position by finding the intersection point
 * a pair of inset-lines (or handle parallel-lines case). Assumption
 * is that Position is the location of the initial vertex on the
 * pre-inset edge-line-pair, ie at their intersection point.
 */
DYNAMICMESH_API FVector3d SolveInsetVertexPositionFromLinePair(
	const FVector3d& Position,
	const FLine3d& InsetEdgeLine1,
	const FLine3d& InsetEdgeLine2);


/**
 * Compute a new set of GroupIDs along an edge loop under the assumption that 
 * the edge loop is about to be split and new geometry introduced. The decision
 * is based on a provided EdgesShouldHaveSameGroupFunc function which is called
 * with sequential pairs of edges. For example if this function returns true when 
 * the GroupIDs are the same on both edges, then this function will create a new
 * GroupID for each sequential span of non-matching GroupIDs (ie often the "expected" behavior).
 * 
 * @param LoopEdgeIDs the EdgeIDs of the loop
 * @param NewLoopEdgeGroupIDsOut returned with new GroupID for each input EdgeID, same length as LoopEdgeIDs
 * @param NewGroupIDsOut list of newly-allocated GroupIDs
 * @param EdgesShouldHaveSameGroupFunc predicate that determines where new GroupIDs will be allocated
 */
DYNAMICMESH_API void ComputeNewGroupIDsAlongEdgeLoop(
	FDynamicMesh3& Mesh,
	const TArray<int32>& LoopEdgeIDs,
	TArray<int32>& NewLoopEdgeGroupIDsOut,
	TArray<int32>& NewGroupIDsOut,
	TFunctionRef<bool(int32 Eid1, int32 Eid2)> EdgesShouldHaveSameGroupFunc);


} // end namespace UE::Geometry
} // end namespace UE