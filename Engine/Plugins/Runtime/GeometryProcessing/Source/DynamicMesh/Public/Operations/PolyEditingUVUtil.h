// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "IntVectorTypes.h"
#include "GeometryTypes.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

class FProgressCancel;

namespace UE
{
namespace Geometry
{

/**
 * Create a new single UV island for the provided TriangleSet in the given Mesh/UVOverlay,
 * by computing an ExpMap parameterization based on an automatically-detected midpoint of the triangle patch.
 * The UV scale is determined automatically based on the UV/3D area ratio of triangles adjacent
 * to the TriangleSet (using a ratio of 1 if the triangles have no neighbours).
 * 
 * @param TriangleSet a list of triangles, must be a single connected component
 */
DYNAMICMESH_API void ComputeArbitraryTrianglePatchUVs(
	FDynamicMesh3& Mesh, 
	FDynamicMeshUVOverlay& UVOverlay,
	const TArray<int32>& TriangleSet );


/**
 * Compute the total UVLength/MeshLength ratio along the given VertexPath.
 * Other 3D lengths can then be scaled to comparable UV-space lengths by multiplying by this value.
 * Edges must exist between the vertices along the path, and loops are not considered.
 * Each UV edge for a 3D edge is considered, ie if a 3D edge corresponds to 2 UV edges, both are counted,
 * and if the 3D edge has no UV edges, it is skipped.
 * @param VertexPath list of vertices which are connected by edges
 * @param PathLengthOut optional output param for sum of 3D edge lengths along path (arc length)
 * @param UVPathLengthOut optional output param for sum of UV edge lengths along path
 * @return UVPathLength / PathLength, or 1.0 if no valid UV edges along the path existed
 */
DYNAMICMESH_API double ComputeAverageUVScaleRatioAlongVertexPath(
	const FDynamicMesh3& Mesh,
	const FDynamicMeshUVOverlay& UVOverlay,
	const TArray<int32>& VertexPath,
	double* PathLengthOut = nullptr,
	double* UVPathLengthOut = nullptr );


} // end namespace UE::Geometry
} // end namespace UE