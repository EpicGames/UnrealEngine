// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "FrameTypes.h"

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;


/**
 * Try to compute a good "Frame" for the selected Triangles of the Mesh.
 * Only the largest triangle-connected-component of Triangles is considered.
 * The Frame Z is aligned with the area-weighted average normal of the connected triangles.
 * Initially the Frame is placed at the area-weighted centroid of the connected triangles.
 * Rays are cast along Z in both directions, if either hits, the Frame is translated to
 * the nearer hit.
 * 
 * @param bIsDefinitelySingleComponent if this is known, passing as true will speed up the computation
 */
DYNAMICMESH_API FFrame3d ComputeFaceSelectionFrame(
	const FDynamicMesh3& Mesh, 
	const TArray<int32>& Triangles,
	bool bIsDefinitelySingleComponent = false);



} // end namespace UE::Geometry
} // end namespace UE