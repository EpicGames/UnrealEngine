// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "GeometryTypes.h"
#include "IntBoxTypes.h"

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;


/**
 * Compute inclusive range [0...N] of MaterialID values currently in use on Mesh (ie range always includes 0)
 * @return true if MaterialIDs attribute exists (if false, Range is returned as [0,0])
 */
DYNAMICMESH_API bool ComputeMaterialIDRange(
	const FDynamicMesh3& Mesh, 
	FInterval1i& MaterialIDRange );


/**
 * Compute a MaterialID for each edge along the VertexPath based on the MaterialID of adjacent triangles. 
 * Currently, for each Edge, EdgeTriangle.A is used.
 * @param VertexPath list of sequential vertices that are connected by mesh edges
 * @param bIsLoop is the path a loop? 
 * @param EdgeMaterialIDsOut returned list of per-edge Material ID. Length is VertexPath.Num-1 for open paths and VertexPath.Num for loops
 * @param FallbackMaterialID default MaterialID to use if no Material attribute exists or no edge is found
 * @return true if MaterialIDs attribute exists
 */
DYNAMICMESH_API bool ComputeMaterialIDsForVertexPath(
	const FDynamicMesh3& Mesh, 
	const TArray<int32>& VertexPath,
	bool bIsLoop,
	TArray<int32>& EdgeMaterialIDsOut,
	int32 FallbackMaterialID = 0);


} // end namespace UE::Geometry
} // end namespace UE