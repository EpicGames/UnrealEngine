// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshBoolean

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"

#include "Curve/GeneralPolygon2.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshBoundaryLoops.h"
#include "GroupTopology.h"

#include "Util/ProgressCancel.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

// Hacky base class to avoid 8 bytes of padding after the vtable
class FPolygroupRemeshFixLayout
{
public:
	virtual ~FPolygroupRemeshFixLayout() = default;
};

/**
 * PolygroupRemesh -- remesh only considering polygroup features (topological corners and bends on polygroup edges)
 * This can help clean up low poly meshes that have extra vertices along straight edges e.g. after mesh Boolean operations
 */
class FPolygroupRemesh : public FPolygroupRemeshFixLayout
{
public:

	//
	// Inputs
	//
	
	/** The mesh to remesh -- will be updated in place */
	FDynamicMesh3* Mesh;
	
	/** Topology of polygroups in the mesh, before the remesh has been computed */
	const FGroupTopology* Topology;

	/** Planar triangulation function (e.g. ConstrainedDelaunayTriangulate<double> in GeometryAlgorithms) */
	TFunction<TArray<FIndex3i>(const FGeneralPolygon2d&)> PlanarTriangulationFunc;

	/** Units are degrees.  The angle change in boundary edge directions at a group-boundary vertex beyond which we need to keep the vertex in our remesh */
	double SimplificationAngleTolerance = .1;
	
	/** Set this to be able to cancel running operation */
	FProgressCancel* Progress = nullptr;


public:

	FPolygroupRemesh(FDynamicMesh3* Mesh, const FGroupTopology* Topology, TFunction<TArray<FIndex3i>(const FGeneralPolygon2d&)> PlanarTriangulationFunc)
		: Mesh{ Mesh }, Topology(Topology), PlanarTriangulationFunc(PlanarTriangulationFunc)
	{
		check(Mesh != nullptr && Topology != nullptr);
	}

	virtual ~FPolygroupRemesh() = default;
	
	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	EOperationValidationResult Validate()
	{
		// @todo validate inputs
		return EOperationValidationResult::Ok;
	}

	/**
	 * Compute the plane cut by splitting mesh edges that cross the cut plane, and then deleting any triangles
	 * on the positive side of the cutting plane.
	 * @return true if operation succeeds
	 */
	UE_API bool Compute();

protected:
	/** If this returns true, abort computation.  */
	virtual bool Cancelled()
	{
		return (Progress == nullptr) ? false : Progress->Cancelled();
	}

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
