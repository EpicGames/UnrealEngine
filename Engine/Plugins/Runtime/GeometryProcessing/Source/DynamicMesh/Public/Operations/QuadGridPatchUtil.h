// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Selections/QuadGridPatch.h"


namespace UE
{
namespace Geometry
{
	class FDynamicMesh3;

/**
 * Create a new Normals "island" (ie surrounded by hard edges) for the QuadPatch of the Mesh,
 */
DYNAMICMESH_API void ComputeNormalsForQuadPatch(
	FDynamicMesh3& Mesh,
	const FQuadGridPatch& QuadPatch );

/**
 * Create a new UV Island for the QuadPatch triangles of the Mesh, 
 * and then assign UVs based on the grid layout.
 */
DYNAMICMESH_API bool ComputeUVIslandForQuadPatch(
	FDynamicMesh3& Mesh,
	const FQuadGridPatch& QuadPatch,
	double UVScaleFactor = 1.0,
	int UVOverlayIndex = 0);



} // end namespace UE::Geometry
} // end namespace 