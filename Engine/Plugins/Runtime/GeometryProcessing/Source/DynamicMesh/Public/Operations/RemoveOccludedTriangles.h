// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp RemoveOccludedTriangles

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Spatial/MeshAABBTree3.h"
#include "Spatial/FastWinding.h"

#include "Math/RandomStream.h"

#include "MeshAdapter.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"

#include "Async/ParallelFor.h"
#include "Misc/ScopeLock.h"
#include "HAL/ThreadSafeBool.h"

#include "Util/ProgressCancel.h"

namespace UE
{
namespace Geometry
{

enum class EOcclusionTriangleSampling
{
	Centroids = 0,
	Vertices = 1,
	VerticesAndCentroids = 2
};

enum class EOcclusionCalculationMode
{
	FastWindingNumber = 0,
	SimpleOcclusionTest = 1
};


} // end namespace UE::Geometry
} // end namespace UE


namespace UE
{
	namespace MeshAutoRepair
	{
		using namespace UE::Geometry;

		/**
		 * Remove any triangles that are internal to the input mesh
		 * @param Mesh Input mesh to analyze and remove triangles from
		 * @param bTestPerComponent Remove if inside *any* connected component of the input mesh, instead of testing the whole mesh at once.
		 *							Note in FastWindingNumber mode, this can remove internal pockets that otherwise would be missed
		 * @param SamplingMethod Whether to sample centroids, vertices or both
		 * @param RandomSamplesPerTri Number of additional random samples to check before deciding if a triangle is occluded
		 * @param WindingNumberThreshold Threshold to decide whether a triangle is inside or outside (only used if OcclusionMode is WindingNumber)
		 */
		bool DYNAMICMESH_API RemoveInternalTriangles(FDynamicMesh3& Mesh, bool bTestPerComponent = false,
			EOcclusionTriangleSampling SamplingMethod = EOcclusionTriangleSampling::Centroids, 
			EOcclusionCalculationMode OcclusionMode = EOcclusionCalculationMode::FastWindingNumber,
			int RandomSamplesPerTri = 0, double WindingNumberThreshold = .5, bool bTestOccludedByAny = false);
	}
}

namespace UE
{
namespace Geometry
{

/**
 * Remove "occluded" triangles, i.e. triangles on the "inside" of the mesh(es).
 * This is a fuzzy definition, current implementation has a couple of options
 * including a winding number-based version and an ambient-occlusion-ish version,
 * where if face is occluded for all test rays, then we classify it as inside and remove it.
 *
 * Note this class always removes triangles from an FDynamicMesh3, but can use any mesh type
 * to define the occluding geometry (as long as the mesh type implements the TTriangleMeshAdapter fns)
 */
template<typename OccluderTriangleMeshType>
class TRemoveOccludedTriangles
{
public:

	FDynamicMesh3* Mesh;

	TRemoveOccludedTriangles(FDynamicMesh3* Mesh) : Mesh(Mesh)
	{
	}
	virtual ~TRemoveOccludedTriangles() = default;

	/**
	 * Select the occluded triangles, considering the given occluder AABB trees (which may represent more geometry than a single mesh)
	 * See simpler invocations below for the single instance case or the case where you'd like the spatial data structures built for you
	 * Selection will be stored in the RemovedT array, but no triangles will be removed.
	 *
	 * @param MeshLocalToOccluderSpaces Transforms to take instances of the local mesh into the space of the occluders
	 * @param Spatials AABB trees for all occluders
	 * @param FastWindingTrees Precomputed fast winding trees for occluders
	 * @param SpatialTransforms Transforms AABB/winding tree to shared occluder space (if empty, Identity is used)
	 * @param bTestOccludedByAny If true, a triangle is occluded if it is fully occluded by *any* occluder.
	 *							 Otherwise, we test if it's occluded by the combination of all occluders.
	 * @return true on success
	 */
	virtual bool Select(const TArrayView<const FTransformSRT3d> MeshLocalToOccluderSpaces,
		const TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials, const TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees,
		const TArrayView<const FTransformSRT3d> SpatialTransforms = TArrayView<const FTransformSRT3d>(), bool bTestOccludedByAny = false)
	{
		if (Cancelled())
		{
			return false;
		}

		// ray directions
		TArray<FVector3d> RayDirs; int NR = 0;

		FRandomStream RaysRandomStream(2123123);
		if (InsideMode == EOcclusionCalculationMode::SimpleOcclusionTest)
		{
			RayDirs.Add(FVector3d::UnitX()); RayDirs.Add(-FVector3d::UnitX());
			RayDirs.Add(FVector3d::UnitY()); RayDirs.Add(-FVector3d::UnitY());
			RayDirs.Add(FVector3d::UnitZ()); RayDirs.Add(-FVector3d::UnitZ());

			for (int AddRayIdx = 0; AddRayIdx < AddRandomRays; AddRayIdx++)
			{
				RayDirs.Add(FVector3d(RaysRandomStream.VRand()));
			}
			NR = RayDirs.Num();
		}

		// triangle samples get their own random stream to make behavior slightly more predictable (e.g. moving the ray samples up shouldn't change all the triangle sample locations)
		FRandomStream TrisRandomStream(124233);
		TArray<FVector3d> TriangleBaryCoordSamples;
		for (int AddSampleIdx = 0; AddSampleIdx < AddTriangleSamples; AddSampleIdx++)
		{
			FVector3d BaryCoords(TrisRandomStream.FRand() * .999 + .001, TrisRandomStream.FRand() * .999 + .001, TrisRandomStream.FRand() * .999 + .001);
			BaryCoords /= (BaryCoords.X + BaryCoords.Y + BaryCoords.Z);
			TriangleBaryCoordSamples.Add(BaryCoords);
		}
		if (TriangleSamplingMethod == EOcclusionTriangleSampling::Centroids || TriangleSamplingMethod == EOcclusionTriangleSampling::VerticesAndCentroids)
		{
			TriangleBaryCoordSamples.Add(FVector3d(1 / 3.0, 1 / 3.0, 1 / 3.0));
		}

		// Helper struct to track both the transform and its inverse
		struct FTransformWithInv
		{
			FTransformSRT3d Transform;
			FQuaterniond InvRot;
			FVector3d InvScale;

			FTransformWithInv() : 
				Transform(FTransformSRT3d::Identity()), InvRot(FQuaterniond::Identity()), InvScale(FVector3d::One())
			{}
			FTransformWithInv(const FTransformSRT3d& Transform) : Transform(Transform)
			{
				ComputeInv();
			}

			void ComputeInv()
			{
				InvRot = Transform.GetRotation().Inverse();
				InvScale = Transform.GetScale();
				const double ScaleTolerance = FMathd::ZeroTolerance * 100;
				for (int i = 0; i < 3; i++)
				{
					// for near-zero scale, pretend scale was a not-quite-as-small number instead; TODO handle these cases more correctly
					if (FMath::Abs(InvScale[i]) < ScaleTolerance)
					{
						InvScale[i] = FMathd::SignNonZero(InvScale[i]) / ScaleTolerance;
					}
					else
					{
						InvScale[i] = 1.0 / InvScale[i];
					}
				}
			}

			inline FVector3d InverseTransformPosition(const FVector3d& P) const
			{
				return InvScale * (InvRot * (P - Transform.GetTranslation()));
			}

			inline FVector3d InverseTransformVector(const FVector3d& V) const
			{
				return InvScale * (InvRot * V);
			}

			// checks if the transform is an exact match
			inline bool IsSameTransform(const FTransformSRT3d& Other) const
			{
				return Other.GetTranslation() == Transform.GetTranslation()
					&& Other.GetScale() == Transform.GetScale()
					&& Other.GetRotation().EpsilonEqual(Transform.GetRotation(), 0);
			}

			// apply the inverse transform unless the point was originally transformed by this same transform; then just return the original point
			// use to avoid floating point error shoving a sample into the surface that it came from
			inline FVector3d InverseTransformUnlessMatch(const FVector3d& Pt, const FVector3d& OriginalPt, const FTransformSRT3d& OriginalXF) const
			{
				if (IsSameTransform(OriginalXF))
				{
					return OriginalPt;
				}
				else
				{
					return InverseTransformPosition(Pt);
				}
			}
		};

		typedef TFunction<bool(const TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials, const TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees,
			const FVector3d& Pt, const FVector3d& OriginalPt, const FTransformSRT3d& OriginalXF, const TArray<FTransformWithInv>& SpatialTransforms)> FIsOccludedFn;

		// test if point is occluded by the combination of all spatials, according to winding number
		FIsOccludedFn IsOccludedFWNTotal =
			[this](const TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials, const TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees,
			const FVector3d& Pt, const FVector3d& OriginalPt, const FTransformSRT3d& OriginalXF, const TArray<FTransformWithInv>& SpatialTransforms) -> bool
		{
			double WindingSum = 0;
			for (int Idx = 0, NumPts = FastWindingTrees.Num(); Idx < NumPts; ++Idx)
			{
				FVector3d XFPt = SpatialTransforms[Idx].InverseTransformUnlessMatch(Pt, OriginalPt, OriginalXF);
				WindingSum += FastWindingTrees[Idx]->FastWindingNumber(XFPt);
			}
			return WindingSum > WindingIsoValue;
		};
		// test if point is occluded by the any of the spatials, according to winding number
		FIsOccludedFn IsOccludedFWNAny =
			[this](const TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials, const TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees,
				const FVector3d& Pt, const FVector3d& OriginalPt, const FTransformSRT3d& OriginalXF, const TArray<FTransformWithInv>& SpatialTransforms) -> bool
		{
			for (int Idx = 0, NumPts = FastWindingTrees.Num(); Idx < NumPts; ++Idx)
			{
				FVector3d XFPt = SpatialTransforms[Idx].InverseTransformUnlessMatch(Pt, OriginalPt, OriginalXF);
				if (FastWindingTrees[Idx]->FastWindingNumber(XFPt) > WindingIsoValue)
				{
					return true;
				}
			}
			return false;
		};
		// test if point is occluded by the combination of all spatials, according to raycasts
		FIsOccludedFn IsOccludedSimpleTotal = [this, &RayDirs, &NR]
		(const TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials, const TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees,
			const FVector3d& Pt, const FVector3d& OriginalPt, const FTransformSRT3d& OriginalXF, const TArray<FTransformWithInv>& SpatialTransforms) -> bool
		{
			FRay3d Ray;
			
			for (int RayIdx = 0; RayIdx < NR; ++RayIdx)
			{
				bool bAnyHit = false;
				for (int SpatialIdx = 0, NumPts = Spatials.Num(); SpatialIdx < NumPts; ++SpatialIdx)
				{
					FVector3d XFPt = SpatialTransforms[SpatialIdx].InverseTransformUnlessMatch(Pt, OriginalPt, OriginalXF);
					Ray.Direction = SpatialTransforms[SpatialIdx].InverseTransformVector(RayDirs[RayIdx]);
					Ray.Origin = XFPt;
					bool bFoundHit = Spatials[SpatialIdx]->TestAnyHitTriangle(Ray);
					if (bFoundHit)
					{
						bAnyHit = true;
						break;
					}
				}
				if (!bAnyHit) // found a ray direction that hits none of the spatials
				{
					return false;
				}
			}
			return true;
		};
		// test if point is occluded by the any of the spatials, according to raycasts
		FIsOccludedFn IsOccludedSimpleAny = [this, &RayDirs, &NR]
		(const TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials, const TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees,
			const FVector3d& Pt, const FVector3d& OriginalPt, const FTransformSRT3d& OriginalXF, const TArray<FTransformWithInv>& SpatialTransforms) -> bool
		{
			FRay3d Ray;

			for (int SpatialIdx = 0, NumPts = Spatials.Num(); SpatialIdx < NumPts; ++SpatialIdx)
			{
				FVector3d XFPt = SpatialTransforms[SpatialIdx].InverseTransformUnlessMatch(Pt, OriginalPt, OriginalXF);
				Ray.Origin = XFPt;
				bool bIsOccluded = true;
				for (int RayIdx = 0; RayIdx < NR; ++RayIdx)
				{
					Ray.Direction = SpatialTransforms[SpatialIdx].InverseTransformVector(RayDirs[RayIdx]);
					bool bFoundHit = Spatials[SpatialIdx]->TestAnyHitTriangle(Ray);
					if (bFoundHit == false)
					{
						bIsOccluded = false;
						break;
					}
				}
				if (bIsOccluded) // found a spatial that every ray direction hit
				{
					return true;
				}
			}
			return false;
		};

		FIsOccludedFn IsOccludedF = InsideMode == EOcclusionCalculationMode::FastWindingNumber ?
			(bTestOccludedByAny ? IsOccludedFWNAny : IsOccludedFWNTotal)
				:
			(bTestOccludedByAny ? IsOccludedSimpleAny : IsOccludedSimpleTotal);

		bool bForceSingleThread = false;

		bool bHasSpatialTransforms = Spatials.Num() == SpatialTransforms.Num();

		TArray<FTransformWithInv> SpatialTransformsWithInv;
		if (bHasSpatialTransforms)
		{
			for (int32 Idx = 0; Idx < Spatials.Num(); Idx++)
			{
				SpatialTransformsWithInv.Emplace(SpatialTransforms[Idx]);
			}
		}
		else
		{
			// fill with identity transforms
			for (int32 Idx = 0; Idx < Spatials.Num(); Idx++)
			{
				SpatialTransformsWithInv.Emplace();
			}
		}

		TArray<bool> VertexOccluded;
		if ( TriangleSamplingMethod == EOcclusionTriangleSampling::Vertices || TriangleSamplingMethod == EOcclusionTriangleSampling::VerticesAndCentroids )
		{
			VertexOccluded.Init(false, Mesh->MaxVertexID());

			// do not trust source mesh normals; safer to recompute
			FMeshNormals Normals(Mesh);
			Normals.ComputeVertexNormals();

			ParallelFor(Mesh->MaxVertexID(), 
			[this, &Normals, &VertexOccluded, &IsOccludedF, &MeshLocalToOccluderSpaces,
				&Spatials, &FastWindingTrees, &SpatialTransformsWithInv](int32 VID)
			{
				if (!Mesh->IsVertex(VID) || VertexOccluded[VID])
				{
					return;
				}
				FVector3d SamplePos = Mesh->GetVertex(VID);
				FVector3d Normal = Normals[VID];
				SamplePos += Normal * NormalOffset;
				bool bAllOccluded = true;
				for (int32 TransformIdx = 0, TransformNum = MeshLocalToOccluderSpaces.Num(); TransformIdx < TransformNum; TransformIdx++)
				{
					const FTransformSRT3d& OriginalTransform = MeshLocalToOccluderSpaces[TransformIdx];
					FVector3d XFPos = OriginalTransform.TransformPosition(SamplePos);

					bAllOccluded = bAllOccluded && IsOccludedF(Spatials, FastWindingTrees, 
						XFPos, SamplePos, OriginalTransform, SpatialTransformsWithInv);
				}
				checkSlow(VertexOccluded[VID] == false); // should have skipped the vertex if we already knew it was occluded (e.g. from another occluder)
				VertexOccluded[VID] = bAllOccluded;
			}, bForceSingleThread);
		}
		if (Cancelled())
		{
			return false;
		}

		TArray<FThreadSafeBool> TriOccluded;
		TriOccluded.Init(false, Mesh->MaxTriangleID());

		ParallelFor(Mesh->MaxTriangleID(), 
		[this, &VertexOccluded, &IsOccludedF, &MeshLocalToOccluderSpaces,
			&TriangleBaryCoordSamples, &TriOccluded,
			&Spatials, &FastWindingTrees, &SpatialTransformsWithInv](int32 TID)
		{
			if (!Mesh->IsTriangle(TID))
			{
				return;
			}

			bool bInside = true;
			if (TriangleSamplingMethod == EOcclusionTriangleSampling::Vertices || TriangleSamplingMethod == EOcclusionTriangleSampling::VerticesAndCentroids)
			{
				FIndex3i Tri = Mesh->GetTriangle(TID);
				bInside = VertexOccluded[Tri.A] && VertexOccluded[Tri.B] && VertexOccluded[Tri.C];
			}
			if (bInside && TriangleBaryCoordSamples.Num() > 0)
			{
				FVector3d Normal = Mesh->GetTriNormal(TID);
				FVector3d V0, V1, V2;
				Mesh->GetTriVertices(TID, V0, V1, V2);
				for (int32 SampleIdx = 0, NumSamples = TriangleBaryCoordSamples.Num(); bInside && SampleIdx < NumSamples; SampleIdx++)
				{
					FVector3d BaryCoords = TriangleBaryCoordSamples[SampleIdx];
					FVector3d SamplePos = V0 * BaryCoords.X + V1 * BaryCoords.Y + V2 * BaryCoords.Z + Normal * NormalOffset;
					for (int32 TransformIdx = 0, TransformNum = MeshLocalToOccluderSpaces.Num(); TransformIdx < TransformNum; TransformIdx++)
					{
						const FTransformSRT3d& OriginalTransform = MeshLocalToOccluderSpaces[TransformIdx];
						FVector3d XFPos = OriginalTransform.TransformPosition(SamplePos);

						bInside = bInside && IsOccludedF(Spatials, FastWindingTrees,
							XFPos, SamplePos, OriginalTransform, SpatialTransformsWithInv);
					}
				}
			}
			if (bInside)
			{
				TriOccluded[TID] = true;
			}
		}, bForceSingleThread);


		if (Cancelled())
		{
			return false;
		}

		RemovedT.Reset();
		for (int TID = 0; TID < Mesh->MaxTriangleID(); TID++)
		{
			if (TriOccluded[TID])
			{
				RemovedT.Add(TID);
			}
		}

		return true;
	}

	/**
	 * Remove triangles that were selected (the triangle IDs in the RemoveT array)
	 */
	virtual bool RemoveSelected()
	{
		if (RemovedT.Num() > 0)
		{
			FDynamicMeshEditor Editor(Mesh);
			bool bOK = Editor.RemoveTriangles(RemovedT, true);
			if (!bOK)
			{
				bRemoveFailed = true;
				return false;
			}
			// TODO: do we want to consider if we have made the mesh non-manifold or do any cleanup?
		} 

		return true;
	}
	
	/**
	 * Remove the occluded triangles, considering the given occluder AABB trees (which may represent more geometry than a single mesh)
	 * See simpler invocations below for the single instance case or the case where you'd like the spatial data structures built for you
	 *
	 * @param MeshLocalToOccluderSpaces Transforms to take instances of the local mesh into the space of the occluders
	 * @param Spatials AABB trees for all occluders
	 * @param FastWindingTrees Precomputed fast winding trees for occluders
	 * @param SpatialTransforms Transforms AABB/winding tree to shared occluder space (if empty, Identity is used)
	 * @param bTestOccludedByAny If true, a triangle is occluded if it is fully occluded by *any* occluder.
	 *							 Otherwise, we test if it's occluded by the combination of all occluders.
	 * @return true on success
	 */
	virtual bool Apply(const TArrayView<const FTransformSRT3d> MeshLocalToOccluderSpaces, 
		const TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials, const TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees,
		const TArrayView<const FTransformSRT3d> SpatialTransforms = TArrayView<const FTransformSRT3d>(), bool bTestOccludedByAny = false)
	{
		if (!Select(MeshLocalToOccluderSpaces, Spatials, FastWindingTrees, SpatialTransforms, bTestOccludedByAny))
		{
			return false;
		}

		return RemoveSelected();
	}


	/**
	 * Select the occluded triangles, considering the given occluder AABB tree (which may represent more geometry than a single mesh)
	 * See simpler invocations below for the single instance case or the case where you'd like the spatial data structures built for you
	 *
	 * @param MeshLocalToOccluderSpaces Transforms to take instances of the local mesh into the space of the occluders
	 * @param Spatials AABB trees for all occluders
	 * @param FastWindingTrees Precomputed fast winding trees for occluders
	 * @return true on success
	 */
	virtual bool Select(const TArrayView<const FTransformSRT3d> MeshLocalToOccluderSpaces,
		TMeshAABBTree3<OccluderTriangleMeshType>* Spatial, TFastWindingTree<OccluderTriangleMeshType>* FastWindingTree)
	{
		TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials(&Spatial, 1);
		TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees(&FastWindingTree, 1);
		return Select(MeshLocalToOccluderSpaces, Spatials, FastWindingTrees, TArrayView<const FTransformSRT3d>());
	}


	/**
	 * Remove the occluded triangles, considering the given occluder AABB tree (which may represent more geometry than a single mesh)
	 * See simpler invocations below for the single instance case or the case where you'd like the spatial data structures built for you
	 *
	 * @param MeshLocalToOccluderSpaces Transforms to take instances of the local mesh into the space of the occluders
	 * @param Spatials AABB trees for all occluders
	 * @param FastWindingTrees Precomputed fast winding trees for occluders
	 * @return true on success
	 */
	virtual bool Apply(const TArrayView<const FTransformSRT3d> MeshLocalToOccluderSpaces,
		TMeshAABBTree3<OccluderTriangleMeshType>* Spatial, TFastWindingTree<OccluderTriangleMeshType>* FastWindingTree)
	{
		TArrayView<TMeshAABBTree3<OccluderTriangleMeshType>*> Spatials(&Spatial, 1);
		TArrayView<TFastWindingTree<OccluderTriangleMeshType>*> FastWindingTrees(&FastWindingTree, 1);
		return Apply(MeshLocalToOccluderSpaces, Spatials, FastWindingTrees, TArrayView<const FTransformSRT3d>());
	}

	/**
	 * Select the occluded triangles -- single instance case
	 *
	 * @param LocalToWorld Transform to take the local mesh into the space of the occluder geometry
	 * @param Occluder AABB tree of occluding geometry
	 * @return true on success
	 */
	virtual bool Select(const FTransformSRT3d& MeshLocalToOccluderSpace, TMeshAABBTree3<OccluderTriangleMeshType>* Spatial, TFastWindingTree<OccluderTriangleMeshType>* FastWindingTree)
	{
		TArrayView<const FTransformSRT3d> MeshLocalToOccluderSpaces(&MeshLocalToOccluderSpace, 1); // array view of the single transform
		return Select(MeshLocalToOccluderSpaces, Spatial, FastWindingTree);
	}


	/**
	 * Remove the occluded triangles -- single instance case
	 *
	 * @param LocalToWorld Transform to take the local mesh into the space of the occluder geometry
	 * @param Occluder AABB tree of occluding geometry
	 * @return true on success
	 */
	virtual bool Apply(const FTransformSRT3d& MeshLocalToOccluderSpace, TMeshAABBTree3<OccluderTriangleMeshType>* Spatial, TFastWindingTree<OccluderTriangleMeshType>* FastWindingTree)
	{
		TArrayView<const FTransformSRT3d> MeshLocalToOccluderSpaces(&MeshLocalToOccluderSpace, 1); // array view of the single transform
		return Apply(MeshLocalToOccluderSpaces, Spatial, FastWindingTree);
	}


	/**
	 * Select the occluded triangles -- single instance case w/out precomputed winding tree
	 *
	 * @param LocalToWorld Transform to take the local mesh into the space of the occluder geometry
	 * @param Occluder AABB tree of occluding geometry
	 * @return true on success
	 */
	virtual bool Select(const FTransformSRT3d& MeshLocalToOccluderSpace, TMeshAABBTree3<OccluderTriangleMeshType>* Occluder)
	{
		TFastWindingTree<OccluderTriangleMeshType> FastWindingTree(Occluder, InsideMode == EOcclusionCalculationMode::FastWindingNumber);
		return Select(MeshLocalToOccluderSpace, Occluder, &FastWindingTree);
	}


	/**
	 * Remove the occluded triangles -- single instance case w/out precomputed winding tree
	 *
	 * @param LocalToWorld Transform to take the local mesh into the space of the occluder geometry
	 * @param Occluder AABB tree of occluding geometry
	 * @return true on success
	 */
	virtual bool Apply(const FTransformSRT3d& MeshLocalToOccluderSpace, TMeshAABBTree3<OccluderTriangleMeshType>* Occluder)
	{
		TFastWindingTree<OccluderTriangleMeshType> FastWindingTree(Occluder, InsideMode == EOcclusionCalculationMode::FastWindingNumber);
		return Apply(MeshLocalToOccluderSpace, Occluder, &FastWindingTree);
	}

	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		// TODO: validate input
		return EOperationValidationResult::Ok;
	}

	//
	// Input settings
	//

	// how/where to sample triangles when testing for occlusion

	EOcclusionTriangleSampling TriangleSamplingMethod = EOcclusionTriangleSampling::Vertices;

	// we nudge points out by this amount to try to counteract numerical issues
	double NormalOffset = FMathd::ZeroTolerance;

	/** use this as winding isovalue for WindingNumber mode */
	double WindingIsoValue = 0.5;

	EOcclusionCalculationMode InsideMode = EOcclusionCalculationMode::FastWindingNumber;

	/** Number of additional ray directions to add to raycast-based occlusion checks, beyond the default +/- major axis directions */
	int AddRandomRays = 0;

	/** Number of additional samples to add per triangle */
	int AddTriangleSamples = 0;

	/**
	 * Set this to be able to cancel running operation
	 */
	FProgressCancel* Progress = nullptr;


	//
	// Outputs
	//

	/** indices of removed triangles. will be empty if nothing removed */
	TArray<int> RemovedT;

	/** true if it wanted to remove triangles but the actual remove operation failed */
	bool bRemoveFailed = false;


protected:
	/**
	 * if this returns true, abort computation. 
	 */
	virtual bool Cancelled()
	{
		return (Progress == nullptr) ? false : Progress->Cancelled();
	}

};


} // end namespace UE::Geometry
} // end namespace UE
