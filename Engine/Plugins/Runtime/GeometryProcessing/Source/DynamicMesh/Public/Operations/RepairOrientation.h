// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp MeshRepairOrientation

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * Invert triangles as needed to a consistent ~"outward" orientation
 */
class FMeshRepairOrientation
{
public:
	//
	// Inputs
	//
	FDynamicMesh3* Mesh;

	FMeshRepairOrientation(FDynamicMesh3* Mesh) : Mesh(Mesh)
	{
	}
	virtual ~FMeshRepairOrientation() = default;
	
	
	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		// TODO: validate input

		return EOperationValidationResult::Ok;
	}
	
	// locally orient connected components
	UE_API void OrientComponents();

	// globally orient -- to be called after OrientComponents if a more globally consistent result is desired
	UE_API void SolveGlobalOrientation(FDynamicMeshAABBTree3* Tree);

protected:

	struct Component
	{
		TArray<int> Triangles;
		double OutFacing;
		double InFacing;
	};
	TArray<Component> Components;

	UE_API void ComputeStatistics(FDynamicMeshAABBTree3* Tree);
	UE_API void ComputeComponentStatistics(FDynamicMeshAABBTree3* Tree, Component& c);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
