// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"
#include "IndexTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Math/MathFwd.h"

#define UE_API DYNAMICMESH_API

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * An abstract class for writing tessellation patterns. You can subclass it and implement its virtual methods to specify 
 * a custom tessellation behavior. Original mesh edges and triangles are referred to as edge patches and triangle patches. 
 * The edge patch handles all the vertices we are adding along the edge. The triangle patch handles all the vertices we 
 * are adding inside of the patch (excluding the edges) and the new triangles we are splitting the patch into.
 * 
 * Patterns describe how individual patches are tessellated. Although you can access mesh data during the tessellation, 
 * the patches do not generate actual vertex coordinates or other attribute data. Instead, new vertices are generated in 
 * terms of linear or barycentric coefficients with respect to the edge or triangle vertices. The coefficient data can 
 * later be used to interpolate the actual mesh data (vertex positions, colors, normals, etc). 
 * 
 * Patterns do not need to worry about generating the vertex IDs as those will be provided for each patch during the
 * tessellation and need to simply be referred to when generating triangles.
 * 
 * The job of the tessellation pattern is:
 * 
 *      - Provide the number of new vertices introduced for an edge or a triangle patch.
 *      
 *      - Provide the number of new triangles introduced for a triangle patch.
 *        
 *      - Given a single edge patch with vertices [v1,v2], provide coordinates of the new vertices 
 *        [a_i, a_(i+1), ...] inserted along the edge via a linear coeffcient [t_i, t_(i+1), ...] such that 
 *        a_i = v1 + t_i*(v2-v1).
 *        
 *      - Given a triangle patch with vertices [v1,v2,v3], provide coordinates of the new vertices [b_i, b_(i+1), ...]  
 *        inserted inside the triangle patch via barycentric coordinates [(u_i, v_i, w_i), (u_(i+1), v_(i+1), w_(i+1)), ...] 
 *        such that b_i = u_i*v1 + v_i*v2 + w_i*v3.
 *        
 *      - Using the new vertices (along edges and inside the triangle), provide connectivity for a triangle patch by 
 *        stitching those vertices together and creating triangles.
 */
class FTessellationPattern
{
public:
    
    constexpr static int InvalidIndex = -1;

protected:

    const FDynamicMesh3* Mesh;

public:

    /**
     * Represents an abstract edge as a parameterized line segment with two endpoints [v1,v2]. Contains information  
     * about the linear interpolation parameter of the new vertices added along the edge and their vertex IDs.
     * 
     *      O----x----x----x----O           x  -  New vertices added. We only store their lerp coefficients (t1,t2,t3)  
     *    v1     a1   a2   a3    v2               with respect to the end points, s.t. a_i = vi + t_i*(v2-v1).
     */
    struct EdgePatch 
    {
        int EdgeID = IndexConstants::InvalidID; 
        TArrayView<int> VIDs;                   // Vertex IDs of the new vertices added.
        TArrayView<double> LinearCoord;         // Lerp cooeficients of the new vertices added.
        bool bIsReversed = false;               // If true, we need to traverse BaryCoord and VIDs in reverse order.    
                                                // This happens when the directed edge of the triangle is reversed 
                                                // with the respect to the edge direction stored in the FDynamicMesh3.
    };

    /**
     * Represents an abstract triangle patch with corners [u,v,w], s.t. u = (1,0,0), v = (0,1,0), w = (0,0,1). 
     * Contains information about the barycentric coordinates and IDs of the new vertices and the new triangles 
     * generated during the tessellation. 
     */
    struct TrianglePatch 
    {   
        // ID of the triangle in the original mesh this patch represents
        int TriangleID = IndexConstants::InvalidID;

        // Vertex IDs of the 3 corners of the patch. These are the triangle vertex IDs in the original mesh.
        FIndex3i UVWCorners = FIndex3i::Invalid();

        // Edge patches
        EdgePatch UVEdge;
        EdgePatch VWEdge;
        EdgePatch UWEdge;
        
        // Vertex IDs of the inner vertices added inside the triangle.
        TArrayView<int> VIDs;

        // Barycentric coordinates of the inner vertices.
        TArrayView<FVector3d> BaryCoord;
        
        // Triangles the patch is split into
        TArrayView<FIndex3i> Triangles;
    };
        
    FTessellationPattern(const FDynamicMesh3* InMesh) 
    : 
    Mesh(InMesh)
    {
    }

    virtual ~FTessellationPattern() = default; 

	/**
	* @return EOperationValidationResult::Ok if we can perform tessellation on the given mesh with this pattern
	*/
	virtual EOperationValidationResult Validate() const = 0;

    /** 
     * @return the number of new vertices added along the edge patch. If InEdgeID is not valid then return 
     * FTessellationPattern::InvalidIndex. 
     */
    virtual int GetNumberOfNewVerticesForEdgePatch(const int InEdgeID) const = 0;

    /** 
     * @return the number of new vertices added inside the triangle patch (i.e. exclude vertices added 
     * along edges). If InTriangleID is not valid then return FTessellationPattern::InvalidIndex.
     */
    virtual int GetNumberOfNewVerticesForTrianglePatch(const int InTriangleID) const = 0;

    /** 
     * @return the number of triangles that will be generated after tessellating the patch. If InTriangleID is not 
     * valid then return FTessellationPattern::InvalidIndex.
     */
    virtual int GetNumberOfPatchTriangles(const int InTriangleID) const = 0;

    /** 
     * Tessellate the edge patch by generating linear interpolation coefficients for each new vertex and storing the 
     * result in the EdgePatch.LinearCoord.
     * 
     * @param EdgePatch Edge patch has EdgePatch.EdgeID and EdgePatch.VIDs fields set. The order of IDs is along the 
     * edge from one corner to another. The edge direction is same as the direction stored by the FDynamicMesh3 class.
     */
    virtual void TessellateEdgePatch(EdgePatch& EdgePatch) const = 0;

    /**
     * Tessellate the triangle patch by generating new vertices inside the triangle. 
     * Compute the barycentric coordinates of the new vertices and store them in the TriPatch.BaryCoord. 
     * Using the vertices generated in the edge patches and inside the triangle patch, generate new triangles and 
     * store the result in the TriPatch.Triangles. 
     * 
     * @param TriPatch Has all the fields set except the values in the TriPatch.BaryCoord and TriPatch.Triangles which 
     *                 should be set by this method. Both arrays are already sized correctly to be able to hold all the 
     *                 data needed as specified by the GetNumberOfNewVerticesForTrianglePatch() and GetNumberOfPatchTriangles().
     */
    virtual void TessellateTriPatch(TrianglePatch& TriPatch) const = 0;




   //
   // Common helper methods used during tessellation of various patterns. These might be helpful when writing custom 
   // tessellation patterns.
   //
protected:

    /**
     * Takes a triangle defined by three vertices and a TessellationLevel value. For each corner of the outer triangle, 
     * an inner triangle corner is produced at the intersection of two lines extended perpendicular to the corner's two 
     * adjacent edges running through the vertex of the subdivided outer edge nearest to that corner.
     *                            
     *              U          
     *              O             
     *             / \            O - triangle corner
     *            /   \           x - closest vertex to the corner inserted along the edge during tessellation
     *           /     \          + - inner vertex resulting from intersection of two lines extened perpendicular to the edges
     *          x       x
     *         /  .   .  \
     *        /     +     \
     *       /   innerU    \
     * 
     * @note Vertices should be passed in CW order.
     */
    UE_API void ComputeInnerConcentricTriangle(const FVector3d& U,
                                        const FVector3d& V,
                                        const FVector3d& W,
                                        const int EdgeTessellationLevel, 
                                        FVector3d& InnerU,
                                        FVector3d& InnerV,
                                        FVector3d& InnerW) const;
};




/**
 * Given an input mesh and a tessellation pattern, this operator generates a new tessellated mesh where the
 * triangles affected are subtriangulated according to the rules specified by the pattern. Per-vertex normals, UVs, 
 * colors, and extended per-vertex attributes are linearly interpolated to the new vertices. Per-triangle group 
 * identifiers/materials and extended triangle attributes for the new triangles are inherited from the corresponding 
 * input mesh triangles they replaced.
 * 
 * @note Currently does not support interpolation of the GenericAttributes besides Skin Weights.
 */
class FSelectiveTessellate
{
public:


	//
	// Inputs
	//
	
	/** 
     * Set this to be able to cancel running operation. 
     * @note currently not implemented 
     */
	FProgressCancel* Progress = nullptr;

	/** Should tessellation be multi-threaded. */
	bool bUseParallel = true;

    /** Tessellation pattern determines how we tessellate edges and triangles. */
    FTessellationPattern* Pattern = nullptr;

	//
	// Input/Output
	//

	/** The tessellated mesh. */
	FDynamicMesh3* ResultMesh = nullptr;

    //
    // Output
    //
    
    /** 
     * Stores any additional tessellation information that was requested by the caller.
     * The caller should set the pointers to request the information to be computed. 
     * 
     * @note The caller is responsible for managing the memmory the pointers are pointing to.
     *       The operator simply populates the data structures if they are not null.
     */
    struct FTessellationInformation
    {
        /** 
         * If not null, construction of tessellated mesh will append a list of all the vertices that belong to 
         * triangles affected by the tessellation.
         */
        TArray<int32>* SelectedVertices = nullptr;

		TMap<int32, TArray<int32>>* TriangleIDToNewTriangleIDs = nullptr;
    };

    FTessellationInformation TessInfo;  
    	
protected:
	
	/** 
	 * Points to either the input mesh to be used to generate the new tessellated mesh or will point to a copy of the 
     * mesh we are tessellating inplace.
	 */
	const FDynamicMesh3* Mesh = nullptr;

	/** 
	 * If true, the mesh to tessellate is contained in the ResultMesh and will be overwritten with the tessellation 
     * result. 
	 */
	bool bInPlace = false;


public:

	/** 
	 * Tessellate the mesh inplace. If the tessellation fails or the user cancelled the operation, OutMesh will not 
	 * be changed.
     * //TODO: Add an option to turn off making the backup copy of the input mesh in case the computation fails.
	 */
	FSelectiveTessellate(FDynamicMesh3* OutMesh) 
	:
	ResultMesh(OutMesh), bInPlace(true)
	{
	}

	/** 
	 * Tessellate the mesh and write the result into another mesh. This will overwrite any data stored in the OutMesh. 
	 */
	FSelectiveTessellate(const FDynamicMesh3* Mesh, FDynamicMesh3* OutMesh) 
	:
	ResultMesh(OutMesh), Mesh(Mesh), bInPlace(false)
	{
	}
	
	virtual ~FSelectiveTessellate() = default; 

	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot.
	 */
	virtual EOperationValidationResult Validate()
	{	
		const bool bIsInvalid = bInPlace == false && (Mesh == nullptr || ResultMesh == nullptr || Pattern == nullptr);
		const bool bIsInvalidInPlace = bInPlace == true && (ResultMesh == nullptr || Pattern == nullptr);
		
		if (bIsInvalid || bIsInvalidInPlace) 
		{
			return EOperationValidationResult::Failed_UnknownReason;
		}

		if (Pattern)
		{
			return Pattern->Validate();
		}
		else
		{
			return EOperationValidationResult::Failed_UnknownReason;
		}
	}

    inline void SetPattern(FTessellationPattern* InPattern)
    {
        Pattern = InPattern;
    }
    
	/**
	 * Generate tessellated geometry.
	 * @return true if the algorithm succeeds, false if it failed or was cancelled by the user.
	 */
	UE_API virtual bool Compute();




    //
    // Helper methods for setting up built-in tessellation patterns. Currently glsl, uniform and inner uniform 
    // patterns are provided. You can create custom patterns by subclassing FTessellationPattern.
    //
public:


    /**
     * Pattern where the inner area is tessellated using the style of the OpenGL tessellation shader. The inner area
     * of each tessellated triangle consists of multiple "rings". Areas between the rings are tessellated with the new 
     * triangles.
     * 
     * You can specify per-edge and per-triangle tessellation levels. 
     * Per-edge tessellation level determines how many new vertices are inserted along the new edge.
     * Per-triangle tessellation level determines the number of new vertices along each of the edges of the first 
     * inner ring. Each subsequent ring will have 2 fewer vertices.
     * 
     * Triangles with a zero inner tessellation level but with a non-zero edge tessellation level (common for those triangles 
     * adjacent to the triangles marked for tessellation) will have a single vertex inserted in the middle and connected 
     * to all edge vertices (triangle fan).
     * 
     *            
     *                 O
     *                / \
     *               O. .O
     *              /  O  \
     *             O. / \ .O
     *            /  O. .O  \
     *           /  /  O  \  \
     *          /  /  / \  \  \
     *         O. /  /   \  \ .O
     *        /  O. /     \ .O  \
     *       /  /  O-------O  \  \
     *      O. /   .       .   \ .O
     *     /  O----O-------O----O  \
     *    /   .    .       .    .   \
     *   O----O----O-------O----O----O
     *  
     * @param InMesh Mesh we are tessellating.
     * @param InEdgeTessLevels Array of InMesh->MaxEdgeID() size, where InEdgeTessLevels[EdgeID] is the number of new 
     *                         vertices inserted along an EdgeID edge.
     * @param InInnerTessLevels Array of InMesh->MaxTriangleID() size, where InInnerTessLevels[TriangleID] is the 
     *                          number of new vertices along each of the edges of the first inner ring.
     */
    static UE_API TUniquePtr<FTessellationPattern> CreateConcentricRingsTessellationPattern(const FDynamicMesh3* InMesh,
                                                                                     const TArray<int>& InEdgeTessLevels, 
                                                                                     const TArray<int>& InInnerTessLevels);
    
    /** 
     * Tessellate the whole mesh such that all the edge and inner tessellation levels are equal to InTessellationLevel. 
     */
    static UE_API TUniquePtr<FTessellationPattern> CreateConcentricRingsTessellationPattern(const FDynamicMesh3* InMesh,
                                                                                     const int InTessellationLevel);

    /** 
     * Tessellate only selected triangles such that the edge and inner tessellation levels of those triangles are set to
     * InTessellationLevel.
     */
    static UE_API TUniquePtr<FTessellationPattern> CreateConcentricRingsTessellationPattern(const FDynamicMesh3* InMesh,
                                                                                     const int InTessellationLevel,
                                                                                     const TArray<int>& InTriangleList);

    /** 
     * InEdgeFunc and InTriFunc functions will be evaluated on all edges and triangles respectively to determine their 
     * desired tessellation levels. This can be helpful when the tessellation levels depend on some condition. For 
     * example, how close/far the edge/triangle is from some point in space.
     * 
     * @note not yet implemented
     */
    static UE_API TUniquePtr<FTessellationPattern> CreateConcentricRingsTessellationPattern(const TFunctionRef<int(const int EdgeID)> InEdgeFunc, 
                                                                                     const TFunctionRef<int(const int TriangleID)> InTriFunc);

    
    /** 
     * Tessellate only triangles that belong to a given triangle group id. Triangle groups are defined by the 
     * FDynamicMesh3::GetTriangleGroup().
     */
    static UE_API TUniquePtr<FTessellationPattern> CreateConcentricRingsPatternFromTriangleGroup(const FDynamicMesh3* InMesh,
                                                                                          const int InTessellationLevel,
                                                                                          const int InPolygroupID);


    /** 
     * Tessellate only triangles that belong to a given polygroup id within the polygroup layer attribute 
     * (FDynamicMeshPolygroupAttribute) specified by its name. 
     * 
     * @return nullptr if InMesh has no attributes or if no polygroup layer attribute with the given name or id exists.
     */
    static UE_API TUniquePtr<FTessellationPattern> CreateConcentricRingsPatternFromPolyGroup(const FDynamicMesh3* InMesh,
                                                                                      const int InTessellationLevel,
                                                                                      const FString& InLayerName,
                                                                                      const int InPolygroupID);


    /** 
     * Tessellate only triangles that belong to a given material (FDynamicMeshMaterialAttribute). 
     * 
     * @return nullptr if InMesh has no attributes, no material attribute or if no material attribute with the given id exists.
     */
	static UE_API TUniquePtr<FTessellationPattern> CreateConcentricRingsPatternFromMaterial(const FDynamicMesh3* InMesh,
																				 	 const int InTessellationLevel,
																					 const int MaterialID);


	/**
	* Tessellate only triangles that are selected AND which belong to a given material (FDynamicMeshMaterialAttribute).
	*
	*  @return nullptr if InMesh has no attributes, no material attribute, if no material attribute with the given id exists, or if selection is invalid
	*/
	static UE_API TUniquePtr<FTessellationPattern> CreateConcentricRingsPatternFromSelectionAndMaterial(const FDynamicMesh3* InMesh,
																					  const int InTessellationLevel,
																					  const int MaterialID,
																					  const TArray<int>& SelectedTriangles);


    /**
     * Pattern where the inner area is tessellated using the uniform loop style subdivision. Each edge can have 
     * different tessellation levels and edge vertices are connected to the inner uniformly tessellated sub-triangle.
     * 
     * Triangles with a zero inner tessellation level but with a non-zero edge tessellation level (common for those triangles 
     * adjacent to the triangles marked for tessellation) will have a single vertex inserted in the middle and connected 
     * to all edge vertices (triangle fan).
     *  
     *              o
     *             / \
     *            /   \
     *           o  o  o
     *          /  / \  \
     *         /  /   \  \
     *        /  o-----o  \
     *       o  / \   / \  o
     *      /  /   \ /   \  \
     *     /  o-----o-----o  \
     *    /                   \
     *   o----------o----------o
     *
     * @note not yet implemented
     */         

    static UE_API TUniquePtr<FTessellationPattern> CreateInnerUnifromTessellationPattern(const FDynamicMesh3* InMesh,
                                                                                  const TArray<int>& InEdgeTessLevels, 
                                                                                  const TArray<int>& InInnerTessLevels);


    //TODO: add more variation of different ways to provide tessellation levels similar to the ConcentricRings pattern.


    /**
     * Pattern where the whole triangle is tessellated using the loop style subdivision. Defined by a single 
     * tessellation level, so you can not specify per-triangle or per-edge levels separately.
     * 
     * Triangles which are not marked for tessellation but which are adjacent to the triangles that are being tessellated 
     * will have a single vertex inserted in the middle and connected to all edge vertices (triangle fan).
     *   
     *            O
     *           / \
     *          /   \
     *         O-----O
     *        / \   / \
     *       /   \ /   \
     *      O-----O-----O
     *     / \   / \   / \
     *    /   \ /   \ /   \
     *   O-----O-----O-----O

     * 
     * @note not yet implemented
     */
    
    static UE_API TUniquePtr<FTessellationPattern> CreateUniformTessellationPattern(const FDynamicMesh3* InMesh,
                                                                             const int InTessellationLevel,
                                                                             const TArray<int>& InTriangleList);

	static UE_API TUniquePtr<FTessellationPattern> CreateRedGreenTessellationPattern(const FDynamicMesh3* InMesh, const TArray<int>& InTriangleTessLevels);

    //TODO: add more variation of different ways to provide tessellation levels similar to the ConcentricRings pattern.

protected:

	/** @return true if we need to abort the computation. */
	UE_API virtual bool Cancelled();
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
