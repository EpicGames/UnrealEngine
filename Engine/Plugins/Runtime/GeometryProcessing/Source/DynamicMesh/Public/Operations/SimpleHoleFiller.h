// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp SimpleHoleFiller

#pragma once

#include "HoleFiller.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "GeometryTypes.h"
#include "MeshRegionBoundaryLoops.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * Fill an EdgeLoop hole with triangles.
 * Supports two fill modes, either a fan connected to a new central vertex, or a triangulation of the boundary polygon
 */
class FSimpleHoleFiller : public IHoleFiller
{
public:
	enum class EFillType
	{
		TriangleFan,
		PolygonEarClipping
	};

	//
	// Inputs
	//
	FDynamicMesh3 *Mesh;
	FEdgeLoop Loop;
	EFillType FillType = EFillType::TriangleFan;

	// whether to automatically fill in PrimaryColors on new hole fill triangles
	bool bAutoFillPrimaryColors = true;

	//
	// Outputs
	//
	int32 NewVertex = IndexConstants::InvalidID;

public:
	/**
	 *  Construct simple hole filler (just adds a central vertex and a triangle fan)
	 */
	FSimpleHoleFiller(FDynamicMesh3* Mesh, FEdgeLoop Loop, EFillType InFillType = EFillType::TriangleFan) : 
		Mesh(Mesh), 
		Loop(Loop), 
		FillType(InFillType)
	{
	}

	virtual ~FSimpleHoleFiller() = default;
	
	
	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot
	 */
	virtual EOperationValidationResult Validate()
	{
		if (!Loop.IsBoundaryLoop(Mesh))
		{
			return EOperationValidationResult::Failed_UnknownReason;
		}
		// TODO: For EFillType::PolygonEarClipping, we actually have more constraints; specifically, there
		// must not be any edges between non-adjacent loop vertices that we choose to connect (this can
		// happen in a few weird topology cases, for instance if a hole is attached to a single triangle
		// flap), because we would end up trying to add two more triangles to an existing edge.
		// To do a proper check we'd need to know which verts we're planning to connect, which would require
		// going through the whole retriangulation. That does not seem worthwhile...

		return EOperationValidationResult::Ok;
	}

	UE_API bool Fill(int32 GroupID = -1) override;	

	/**
	 * Updates the normals and UV's of NewTriangles. UV's are taken from VidUVMaps,
	 * which is an array of maps (1:1 with UV layers) that map vid's of vertices on the
	 * boundary to their UV elements and values. If an entry for NewVertex is not provided,
	 * it is set to be the average of the boundary UV's. For other UV's, an entry must
	 * exist, though the UV element ID can be InvalidID if an element does not exist for
	 * that UV value. The function will update the element ID in the map to point to the
	 * new element once it inserts it.

	 * Normals are shared among NewTriangles but not with the surrounding portions of the mesh.
	 *
	 * @param VidsToUVsMap A map from vertex ID's of the boundary vertices to UV element ID's
	 *  and/or their values. When the element ID is invalid, a new element is generated using
	 *  the value, and the map is updated accordingly.
	 *
	 * @returns false if there is an error, usually if VidsToUVsMap did not have an entry
	 *  for a needed vertex ID.
	 */
	UE_API bool UpdateAttributes(TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>>& VidUVMaps);


protected:
	UE_API bool Fill_Fan(int32 NewGroupID);
	UE_API bool Fill_EarClip(int32 NewGroupID);

private:
	// Fill colors on triangles added by the fill, by copying boundary colors
	UE_API void FillColors();
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
