// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"
#include "UObject/NameTypes.h"
#include "TransformTypes.h"

#define UE_API DYNAMICMESH_API


namespace UE::Geometry
{

template<typename ParentType> class TDynamicVertexSkinWeightsAttribute;
class FDynamicMesh3;
using FDynamicMeshVertexSkinWeightsAttribute = TDynamicVertexSkinWeightsAttribute<FDynamicMesh3>;


enum class ESkinBindingType : uint8
{
	// Computes the binding strength by computing the Euclidean distance to the closest set of bones,
	// where the strength of binding is proportional to the inverse distance. May cause bones to affects
	// parts of geometry that, although close in space, may be topologically distant.
	DirectDistance = 0,

	// Computes the binding by computing the geodesic distance from each set of bones. This is slower than the
	// direct distance.
	GeodesicVoxel = 1,
};


namespace SkinBinding
{

	struct FBonePoseInfo
	{
		FTransform LocalTransform;
		FName Name;
		int32 ParentIndex;
	};

	struct FBindSettings
	{
		float Stiffness = 0.2f;
		int32 MaxInfluences = 5;
		int32 VoxelResolution = 256;
		ESkinBindingType BindType = ESkinBindingType::DirectDistance;
	};

	// Create skin weights from the input bone hierarchy, and set them on the dynamic mesh
	UE_API void CreateSkinWeights(FDynamicMesh3& InMesh, TConstArrayView<FBonePoseInfo> TransformHierarchy, FName ProfileName, const FBindSettings& Settings);

	// Create skin weights from the input bone hierarchy, with vertices constrained to only allow weights on specific subsets of bones.
	// @param VIDtoGroupIndex Mapping from vertex ID to BoneGroups index, or INDEX_NONE if the vertex has no assigned group (will use unconstrained weights in this case)
	UE_API void CreateConstrainedSkinWeights(FDynamicMesh3& InMesh, TConstArrayView<FBonePoseInfo> TransformHierarchy, FName ProfileName, const FBindSettings& Settings,
		TConstArrayView<TArray<int32, TInlineAllocator<16>>> BoneGroups, TConstArrayView<int32> VIDtoGroupIndex);

};


} // namespace UE::Geometry

#undef UE_API
