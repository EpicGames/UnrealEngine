// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"
#include "UObject/NameTypes.h"
#include "Containers/Map.h"
#include "BoneIndices.h"
#include "BoneWeights.h" //  UE::AnimationCore::MaxInlineBoneWeightCount;

#define UE_API DYNAMICMESH_API

// Forward declarations
class FProgressCancel;
namespace UE::Geometry 
{ 
	class FDynamicMesh3; 
	template<typename ParentType> class TDynamicVertexSkinWeightsAttribute;
	using FDynamicMeshVertexSkinWeightsAttribute = TDynamicVertexSkinWeightsAttribute<FDynamicMesh3>;
}


namespace UE
{
namespace Geometry
{

/** Method used to compute edge weights during bone weight smoothing. */
enum class ESmoothBoneWeightsEdgeWeightMethod : uint8
{
	/** Each neighbor contributes equally (existing behavior). */
	Uniform = 0,
	/** Neighbors weighted by cotangent of opposite angles on shared edge. */
	CotanWeights = 1
};

/** 
 * Interface for getting the bone weights data. Should be subclassed and implemented if the bone weights data
 * is not stored using the FDynamicMeshVertexSkinWeightsAttribute.
 */
template <typename BoneIndexType, typename BoneWeightType>
class TBoneWeightsDataSource
{
public:
	virtual ~TBoneWeightsDataSource() = default;

	/** Number of bones at a vertex. */
	virtual int32 GetBoneNum(const int32 InVertexIDs) = 0;

	/** Bone ID of the bone stored at an index Index, where Index is between 0 and GetBoneNum(InVertexIDs). */
	virtual BoneIndexType GetBoneIndex(const int32 InVertexIDs, const int32 Index) = 0;

	/** Bone weight of the bone stored at an index Index, where Index is between 0 and GetBoneNum(InVertexIDs). */
	virtual BoneWeightType GetBoneWeight(const int32 InVertexIDs, const int32 Index) = 0;

	/** Bone weight of the bone with Bone ID at a vertex. Must return 0 if bone has no influence. */
	virtual BoneWeightType GetWeightOfBoneOnVertex(const int32 InVertexIDs, const BoneIndexType BoneIndex) = 0;
};




/**
 * Collection of algorithms for smoothing bone weights. Bone weight data is fetched using the TBoneWeightsDataSource
 * which is implemented by the caller to match their custom data storage of bone weights.
 *
 *
 * Example usage:
 *
 * // Mesh whose bone weights we are smoothing
 * FDynamicMesh SourceMesh = ... ;
 * 
 * // The caller should create a subclass of TBoneWeightsDataSource that implements its interface
 * TUniquePtr<TBoneWeightsDataSource> DataSource = ... ;
 * 
 * // Setup the smoothing operator
 * TSmoothBoneWeights SmoothBoneWeights(&SourceMesh, DataSource.Get());
 *
 * // Array of vertex IDs to apply smoothing to
 * TArray<int32> VerticesToSmooth = ... ;
 * 
 * // Run the operator
 * const float SmoothStrength = 0.2;
 * if (SmoothBoneWeights.Validate() == EOperationValidationResult::Ok)
 * {
 * 		for (const int32 VertexID : VerticesToSmooth)
 * 		{
 * 			TMap<BoneIndexType, BoneWeightType> FinalWeights;
 *      	SmoothBoneWeights.SmoothWeightsAtVertex(VertexID, SmoothStrength, FinalWeights);
 * 
 * 			// Use FinalWeights to modify your weights
 * 		}
 * }
 */
template <typename BoneIndexType, typename BoneWeightType>
class TSmoothBoneWeights
{
public:

	//
	// Optional Inputs
	//
	
    /** Set this to be able to cancel the running operation. */
	FProgressCancel* Progress = nullptr;

	/** Only consider the weights above the threshold. */
	BoneWeightType MinimumWeightThreshold = (BoneWeightType) 0.0f;

	/** The edge weighting method used during smoothing. */
	ESmoothBoneWeightsEdgeWeightMethod EdgeWeightMethod = ESmoothBoneWeightsEdgeWeightMethod::Uniform;

protected:
		
	const FDynamicMesh3* SourceMesh;
	
	/** Data source supplied by the caller. */
	TBoneWeightsDataSource<BoneIndexType, BoneWeightType>* DataSource = nullptr;

	/** Pre-computed cotangent edge weights indexed by EdgeID. Populated lazily when CotanWeights is selected. */
	TArray<double> CotangentEdgeWeights;

	/** Pre-compute edge weights if needed based on EdgeWeightMethod. */
	void ComputeEdgeWeightsIfNeeded();

public:

	/**
	 * @param InSourceMesh The mesh whose vertices contain bone weights data.
	 * @param InDataSource The bone weights data.
	 */
	TSmoothBoneWeights(const FDynamicMesh3* InSourceMesh, TBoneWeightsDataSource<BoneIndexType, BoneWeightType>* InDataSource = nullptr);

	virtual ~TSmoothBoneWeights() = default;

	/** @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot. */
	virtual EOperationValidationResult Validate();

	/**
     * Compute smoothed weights at a vertex and return the result.
	 * 
	 * @param VertexID		  The vertex whose weights we are smoothing.
     * @param VertexFalloff   The lerp value between the old and the new weight.
     * @param OutFinalWeights The smoothed weights.
	 * 
     * @return true if the algorithm succeeds, false if it failed or was canceled by the user.
	 */
	virtual bool SmoothWeightsAtVertex(const int32 VertexID,
									   const BoneWeightType VertexFalloff, 
									   TMap<BoneIndexType, BoneWeightType>& OutFinalWeights);
	
	/**
	 * Compute smoothed weights at a vertex and return the result.
	 * 
	 * @param VertexID		  The vertex whose weights we are smoothing.
	 * @param VertexFalloff   The lerp value between the old and the new weight.
	 * @param LockedBones	  The bones to exclude from the smooth op
	 * @param OutFinalWeights The smoothed weights.
	 * 
	 * @return true if the algorithm succeeds, false if it failed or was canceled by the user.
	 */
	virtual bool SmoothWeightsAtVertex(const int32 VertexID,
									   const BoneWeightType VertexFalloff,
									   const TSet<BoneIndexType>& LockedBones,
									   TMap<BoneIndexType, BoneWeightType>& OutFinalWeights);
protected:
	
    /** @return if true, abort the computation. */
	virtual bool Cancelled();
};




/**
 * Subclass of TSmoothBoneWeights for dealing with smoothing of bone weights stored in the 
 * FDynamicMeshVertexSkinWeightsAttribute. 
 *
 *
 * Example usage:
 *
 * // Mesh whose bone weights we are smoothing
 * FDynamicMesh SourceMesh = ... ;
 * 
 * // Setup the smoothing operator
 * FSmoothDynamicMeshVertexSkinWeights SmoothBoneWeights(&SourceMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
 *
 * // Array of vertex IDs to apply smoothing to
 * TArray<int32> VerticesToSmooth = ... ;
 * 
 * // Run the operator
 * const float SmoothStrength = 0.2;
 * if (SmoothBoneWeights.Validate() == EOperationValidationResult::Ok)
 * {
 * 		for (const int32 VertexID : VerticesToSmooth)
 * 		{
 *      	SmoothBoneWeights.SmoothWeightsAtVertex(VertexID, SmoothStrength);
 * 		}
 * }
 */
class FSmoothDynamicMeshVertexSkinWeights : public TSmoothBoneWeights<FBoneIndexType, float>
{
public:

	using TSmoothBoneWeights<FBoneIndexType, float>::SmoothWeightsAtVertex; 
	
	
	/** 
	 * During the smoothing, only use this number of influences per vertex. Prune the excess with the smallest influences 
	 * and re-normalize. 
	 */
	int32 MaxNumInfluences = UE::AnimationCore::MaxInlineBoneWeightCount;

	/**
	 * @param InSourceMesh The mesh whose vertices contain bone weights data.
	 * @param InProfile    The name of the skin weights profile containing the bone weights data.
	 */
	UE_API FSmoothDynamicMeshVertexSkinWeights(const FDynamicMesh3* InSourceMesh, const FName InProfile);

	/**
	 * @param InSourceMesh The mesh whose vertices contain bone weights data.
	 * @param InAttribute  The skin weight attribute containing the bone data.
	 */
	UE_API FSmoothDynamicMeshVertexSkinWeights(const FDynamicMesh3* InSourceMesh, FDynamicMeshVertexSkinWeightsAttribute* InAttribute);

	virtual ~FSmoothDynamicMeshVertexSkinWeights() = default;
	
	/** @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot. */
	UE_API virtual EOperationValidationResult Validate() override;
	
	/**
	 * Compute smoothed weights at a vertex and write the result into FDynamicMeshVertexSkinWeightsAttribute passed 
	 * upon construction.
	 *
	 * @param VertexID		  The vertex whose weights we are smoothing.
	 * @param VertexFalloff   The lerp value between the old and the new weight.
	 *
	 * @return true if the algorithm succeeds, false if it failed or was canceled by the user.
	 */
	UE_API bool SmoothWeightsAtVertex(const int32 VertexID, const float VertexFalloff);

	/**
	 * Compute smoothed weights at vertices and neighbors within distance and write the result into 
	 * FDynamicMeshVertexSkinWeightsAttribute passed upon construction.
	 *
	 * @param Vertices		  		Array of vertex IDs we are smoothing weights for.
	 * @param Strength 				The lerp value between the old and the new weight.
	 * @param FloodFillUpToDistance	For every vertex in Vertices, find vertices that are within FloodFillUpToDistance 
	 * 								geodesic distance away and smooth them as well.
	 * @param NumIterations			Number of times the smoothing will be run
	 *
	 * @return true if the algorithm succeeds, false if it failed or was canceled by the user.
	 */
	UE_API bool SmoothWeightsAtVerticesWithinDistance(const TArray<int32>& Vertices, const float Strength, const double FloodFillUpToDistance, const int32 NumIterations);

protected:
	TUniquePtr<TBoneWeightsDataSource<FBoneIndexType, float>> SkinWeightsAttributeDataSource = nullptr;
	FDynamicMeshVertexSkinWeightsAttribute* Attribute = nullptr;

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
