// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/Array.h"

#define UE_API DYNAMICMESH_API

// Forward declarations
class FProgressCancel;
namespace UE::Geometry 
{ 
	class FDynamicMesh3; 
    template<typename RealType, int ElementSize>
    class TDynamicMeshOverlay; 
}

namespace UE::MeshDeformation
{
    class CotanTriangleData;
}

namespace UE
{
namespace Geometry
{

/** 
 * Smooth scalar data of any dimension that is stored via TDynamicMeshOverlay. The operator doesn't know anything about 
 * constraints that are associated with the actual overlay type. For example, if the overlay contains unit length 
 * normals, the resulting "smoothed" normals might not have unit length.
 * 
 * The smoothing happens in stages:
 * 
 *  - First, we average the overlay element values at each vertex to obtain the per-vertex values.
 * 
 *  - For each vertex, we compute the weighted average of its connected neighbors using the EdgeWeightMethod weights.
 *    The smoothed value equals the (1-Strength)*PerVertexValue + (Strength)*NeighborAverage. Strength is expected to be 
 *    in the zero to one range, otherwise the operation can turn from smoothing to more of an unsharp filter. 
 *    We repeat this NumIterations times.
 * 
 *  - Finally, the smoothed per-vertex values are written back to the overlay by either overwriting the content of 
 *    all per-vertex elements (SplitVertexModel == ESplitVertexModel::SetAllToSmoothed) or blending on per-element 
 *    basis using the BlendWithSmoothedStrength blend value. The calling code could use FSplitAttributeWelder to weld 
 *    split-attributes that happen to have identical values after this smoothing (happens when the 
 *    SplitVertexModel == ESplitVertexModel::SetAllToSmoothed).
 * 
 * 
 * Example Usage:
 * 
 *  // Smoothing the color
 *  FDynamicMesh3 Mesh = ...
 *  FDynamicMeshColorOverlay* Colors = Mesh.Attributes()->PrimaryColors();
 * 
 *  FSmoothDynamicMeshAttributes BlurOp(&Mesh);
 *  
 *  // Setup the parameters
 *  BlurOp.bUseParallel = true;
 *  BlurOp.NumIterations = 10;
 *  BlurOp.Strength = 0.2;
 *  BlurOp.EdgeWeightMethod = EEdgeWeights::CotanWeights;
 * 
 *  // Run the smoothing
 *  BlurOp.SmoothOverlay(Colors);
 */
class FSmoothDynamicMeshAttributes
{
public:

	//
	// Optional Inputs
	//
	
    /** Set this to be able to cancel the running operation. */
	FProgressCancel* Progress = nullptr;

    /** Enable/disable multi-threading. */
    bool bUseParallel = true;

    /** Subset of points to smooth. */
    TArray<int32> Selection;

    enum class EEdgeWeights : uint8
    {
        /** Smooth the attributes where each neighbor is weighted equally. */
        Uniform = 0,
        /** Smooth the attributes where each neighbor is weighted proportionally to the shared edge length. */
        EdgeLength = 1,
		/** Smooth the attributes where each neighbor is weighted proportionally to the cotangent weight of the shared edge. */
		CotanWeights = 2
    };
    EEdgeWeights EdgeWeightMethod = EEdgeWeights::CotanWeights;

    /** How to deal with the split vertices. */
    enum class ESplitVertexModel : uint8
    {
        /** Each element will be assigned the same value. */
        SetAllToSmoothed, 
        /** Each element will be blended with he smoothed value. Set BlendWithSmoothedStrength to control the amount of blend. */
        BlendWithSmoothed 
    };
    ESplitVertexModel SplitVertexModel = ESplitVertexModel::SetAllToSmoothed;

    /** The number of smoothing iterations. */
    int32 NumIterations = 0;

    /** The strength of each smoothing iteration. */
    double Strength = 0.0; 

    /** 
     * Control the amount of blending between the final smoothed value and the original element value. Only used when 
     * the SplitVertexModel==ESplitVertexModel::BlendWithSmoothed 
     */
    double BlendWithSmoothedStrength = 0.0;

protected:
		
	const FDynamicMesh3* Mesh;

    TArray<double> CotangentEdgeWeights;

    TArray<double> EdgeLengthWeights;

    // If Selection array is not empty, will contain all vertices in Selection plus one-ring neighbors for each vertex
    TArray<int32> ExpandedSelection;
	
    // Maps mesh Vertex ID to ExpandedSelection index
    TMap<int32, int32> VIDToExpandedSelectionIdx;

    double EdgeLengthThreshold = 1.0;

public:

	UE_API FSmoothDynamicMeshAttributes(const FDynamicMesh3& InMesh);

	virtual ~FSmoothDynamicMeshAttributes() = default;

	/** 
     * @param InOverlay Overlay to smooth
     * @param DimensionsToSmooth ElementSize array, where DimensionsToSmooth[Idx] is true if we want smooth the 
     *                           dimension Idx. If empty, we smooth all dimensions.
     * @return true if the algorithm succeeds, false if it failed or was canceled by the user. 
     */
    template<typename RealType, int ElementSize>
	bool SmoothOverlay(TDynamicMeshOverlay<RealType, ElementSize>* InOverlay, const TArray<bool>& DimensionsToSmooth = TArray<bool>());

protected:
	
    /** @return if true, abort the computation. */
	UE_API virtual bool Cancelled();

    /** @return the index of the vertex into ExpandedSelection array if Selection is not empty, otherwise returns VertexID. */
    inline int32 MeshVIDToSelectionIdx(int32 VertexID) const
    {
        return !VIDToExpandedSelectionIdx.IsEmpty() && ensure(VIDToExpandedSelectionIdx.Contains(VertexID)) ? VIDToExpandedSelectionIdx[VertexID] : VertexID;
    }
};

}
}

#undef UE_API
