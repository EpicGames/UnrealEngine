// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp SmoothHoleFiller

#pragma once

#include "HoleFiller.h"
#include "GeometryTypes.h"
#include "VectorTypes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "ProjectionTargets.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
class FSubRegionRemesher;
class FMeshFaceSelection;
class FMeshBoundaryLoops;
class FEdgeLoop;

/**
 * Fill parameters
 */
struct FSmoothFillOptions
{
	/**
	* If this is true, we don't modify any triangles outside hole (often results in lower-quality fill)
	*/
	bool bConstrainToHoleInterior = true;

	/**
	 * Controls the trade off between smoothness in the fill region vs faithfulness to the original fill surface.
	 * This value is inversely proportional to the vertex constraint weight in Laplacian smoothing, for
	 * vertices in the fill interior far away from the boundary.
	 */
	double InteriorSmoothness = 0.2;

	/**
	 * If we are not constraining remeshing to the fill interior, how many one-rings outside of the fill should we
	* include in remeshing.
	*/
	int RemeshingExteriorRegionWidth = 2;

	/**
	 * Number of one-rings to include when smoothing the fill region. Use this to control smoothness across the boundary.
	 */
	int SmoothingExteriorRegionWidth = 2;

	/**
	 * Smoothing constraint falloff region from border into the interior
	 */
	int SmoothingInteriorRegionWidth = 2;

	/**
	 * Controls the target edge length during remeshing. The target edge length is set to the average length of
	 * the input loop edges divided by this value.
	 */
	double FillDensityScalar = 1.0;

	/**
	 * Whether to use projection in the post-smooth remeshing
	 */
	bool bProjectDuringRemesh = false;
};

/**
 * This fills a hole in a mesh by doing a trivial fill, then doing a remesh, then a laplacian smooth, then a second 
 * remesh.
 */
class FSmoothHoleFiller : public IHoleFiller
{
public:
	UE_API FSmoothHoleFiller(FDynamicMesh3& Mesh, const FEdgeLoop& FillLoop);

	FSmoothFillOptions FillOptions;

	UE_API bool Fill(int32 GroupID) override;

protected:

	// Mesh to operate on
	FDynamicMesh3& Mesh;

	// Loop to fill
	const FEdgeLoop& FillLoop;

	// initialized to the average edge length of FillLoop / FillDensityScalar
	double RemeshingTargetEdgeLength;

	UE_API void SmoothAndRemeshPreserveRegion(FMeshFaceSelection& tris, bool bFinal);

	UE_API void SmoothAndRemesh(FMeshFaceSelection& tris);

	UE_API void ConstrainSubregionSeams(FSubRegionRemesher& Remesher, bool bConstrainROIBoundary);

	UE_API void DefaultConfigureRemesher(FSubRegionRemesher& Remesher, bool bConstrainROIBoundary);

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
