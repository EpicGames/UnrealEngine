// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"
#include "UObject/NameTypes.h"
#include "TransformTypes.h"
#include "BoneWeights.h" // UE::AnimationCore::MaxInlineBoneWeightCount

#define UE_API DYNAMICMESH_API

// Forward declarations
class FProgressCancel;


namespace UE::Geometry
{
	class FDynamicMesh3;
	template<typename MeshType> class TMeshAABBTree3;
	typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3;
	class FMeshNormals;
}


namespace UE
{
namespace Geometry
{
/**
 * Transfer bone weights from one mesh (source) to another (target). Uses the dynamic mesh bone attributes to reindex
 * the bone indices of the transferred weights from the source to the target skeletons. If both meshes have identical
 * bone name attributes, then reindexing is skipped.
 *
 * During the reindexing, if a weighted bone in the source skeleton is not present in the target skeleton, then the
 * weight is not transferred (skipped), and an error is printed to the console. For best results, the target skeleton
 * should be a superset of all the bones that are indexed by the transferred weights.
 *
 *
 * Example usage:
 *
 * FDynamicMesh SourceMesh = ...; // Mesh we transferring weights from. Must have bone attributes.
 * FDynamicMesh TargetMesh = ...; // Mesh we are transferring weights to.
 *
 * FTransferBoneWeights TransferBoneWeights(&SourceMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
 *
 * // Optionally, transform the target mesh. This is useful when you want to align the two meshes in world space.
 * TransferBoneWeights.TargetToWorld = ...;
 *
 * // When transferring weights from a dynamic mesh with bone attributes to a dynamic mesh without bone attributes,
 * // first copy over the bone attributes from the source to the target.
 * if (!TargetMesh.HasAttributes() || !TargetMesh.Attributes()->HasBones())
 * {
 *     TargetMesh.EnableAttributes();
 *     TargetMesh.Attributes()->CopyBoneAttributes(*SourceMesh.Attributes());
 * }
 *
 * // Set the transfer method.
 * TransferBoneWeights.TransferMethod = ETransferBoneWeightsMethod::...;
 *
 * // if ETransferBoneWeightsMethod::ClosestPointOnSurface is used and you simply want to copy weights over from the
 * // closest points then set the radius and normal threshold to -1 (default).
 * TransferBoneWeights.SearchRadius = -1
 * TransferBoneWeights.NormalThreshold = -1
 *
 * // if ETransferBoneWeightsMethod::InpaintWeights is used then additionally set the radius and normal parameters
 * TransferBoneWeights.SearchRadius = ...    // Good estimate is to use a small value (0.05) of the bounding box radius
 * TransferBoneWeights.NormalThreshold = ... // 30 degrees (0.52 rad) works well in practice
 *
 * if (TransferBoneWeights.Validate() == EOperationValidationResult::Ok)
 * {
 *      TransferBoneWeights.TransferWeightsToMesh(TargetMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
 * }
 *
 * // Alternatively if you don't want to use FDynamicMesh3 to represent your target mesh you can transfer weights to
 * // to each point separately by calling
 * if (TransferBoneWeights.Validate() == EOperationValidationResult::Ok)
 * {
 *      for each Point:
 *          FBoneWeights Weights;
 *          TransferBoneWeights.TransferWeightsToPoint(Weights, Point);
 * }
 *
 * // After the transfer you can check which target mesh vertices had the weight transferred directly from the source mesh
 * // via the FTransferBoneWeights:MatchedVertices array
 *
 */



class FTransferBoneWeights
{
public:

	enum class ETransferBoneWeightsMethod : uint8
	{
        // For every vertex on the target mesh, find the closest point on the surface of the source mesh. If that point 
        // is within the SearchRadius, and their normals differ by less than the NormalThreshold, then we directly copy  
        // the weights from the source point to the target mesh vertex.
		ClosestPointOnSurface = 0,

        // Same as the ClosestPointOnSurface but for all the vertices we didn't copy the weights directly, automatically 
		// compute the smooth weights.
		InpaintWeights = 1
	};

	//
	// Optional Inputs
	//
	
    /** Set this to be able to cancel the running operation. */
	FProgressCancel* Progress = nullptr;

	/** Enable/disable multi-threading. */
	bool bUseParallel = true;
	
	/** The transfer method to compute the bone weights. */
	ETransferBoneWeightsMethod TransferMethod = ETransferBoneWeightsMethod::ClosestPointOnSurface;

	/** Transform applied to the input target mesh or target point before transfer. */
	FTransformSRT3d TargetToWorld = FTransformSRT3d::Identity(); 

	/** 
     * Completely ignore the source and target mesh bone attributes when transferring weights from one dynamic mesh to another.
     * This skips re-indexing and simply copies skin weights over. Use with caution.
	 */
	bool bIgnoreBoneAttributes = false;

	//
	// Optional Inputs for ETransferBoneWeightsMethod::InpaintWeights method
	//

	/**  Radius for searching the closest point. If negative, all points are considered. */
	double SearchRadius = -1;

	/** 
	 * Maximum angle (in radians) difference between target and source point normals to be considred a match. 
	 * If negative, normals are ignored.
	 */
	double NormalThreshold = -1;
	
	/** 
	 * If true, when the closest point doesn't pass the normal threshold test, will try again with a flipped normal. 
	 * This helps with layered meshes where the "inner" and "outer" layers are close to each other but whose normals 
	 * are pointing in the opposite directions. 
	 */
	bool LayeredMeshSupport = false;

	/** The number of optional post-processing smoothing iterations applied to the vertices without the match. */
	int32 NumSmoothingIterations = 0; 

	/** The strength of each post-processing smoothing iteration. */
	float SmoothingStrength = 0.0f;

	/** If true, will use the intrinsic Delaunay mesh to construct sparse Cotangent Laplacian matrix. */
	bool bUseIntrinsicLaplacian = false;

	/** 
	 * During the transfer, only use this number of influences per vertex. Prune the excess with the smallest influences 
	 * and re-normalize. 
	 */
	int32 MaxNumInfluences = UE::AnimationCore::MaxInlineBoneWeightCount;

	/** 
	 * Optional mask where if ForceInpaint[VertexID] != 0 we want to force the weights for the vertex to be computed  
	 * automatically.
	 * 
	 * @note Only used when TransferMethod == ETransferBoneWeightsMethod::InpaintWeights.
	 * 		 The size must be equal to the InTargetMesh.MaxVertexID(), otherwise the mask is ignored.
	 */
	TArray<float> ForceInpaint;

	/** 
	 * Alternatively, if the mask is stored as a target mesh weight map attribute, specify its name. Will be ignored if 
	 * the ForceInpaint array is not empty and valid.
	 */
	FName ForceInpaintWeightMapName = NAME_None;

	/** 
	 * Optional subset of target mesh vertices to transfer weights to.
	 * If left empty, skin weights will be transferred to all target mesh vertices.
	 */
	TArray<int32> TargetVerticesSubset;

	//
	// Outputs
	//

	/** MatchedVertices[VertexID] is set to true for a target mesh vertex ID with a match found, false otherwise. */
	TArray<bool> MatchedVertices;

protected:
		
	/** Source mesh we are transfering weights from. */
	const FDynamicMesh3* SourceMesh;
	
	/** The name of the source mesh skinning profile name. */
	FName SourceProfileName;

	/** 
	 * The caller can optionally specify the source mesh BVH in case this operator is run on multiple target meshes 
	 * while the source mesh remains the same. Otherwise BVH tree will be computed.
	 */
	const FDynamicMeshAABBTree3* SourceBVH = nullptr;

	/** If the caller doesn't pass BVH for the source mesh then we compute one. */
	TUniquePtr<FDynamicMeshAABBTree3> InternalSourceBVH;

	/** If the source mesh doesn't have per-vertex normals then compute them */
	TUniquePtr<FMeshNormals> InternalSourceMeshNormals;

public:
	
	/**
	 * @param InSourceMesh The mesh we are transferring weights from 
	 * @param InSourceProfileName The profile name of the skin weight attribute we are transferring weights from.
	 * @param SourceBVH Optional source mesh BVH. If not provided, one will be computed internally. 
	 * 
	 * @note Assumes that the InSourceMesh has bone attributes, use bIgnoreBoneAttributes flag to ignore the bone 
	 * 		 attributes and skip re-indexing.
	 */
	UE_API FTransferBoneWeights(const FDynamicMesh3* InSourceMesh, 
					     const FName& InSourceProfileName,
					     const FDynamicMeshAABBTree3* SourceBVH = nullptr); 
	
	UE_API virtual ~FTransferBoneWeights();

	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot.
	 */
	UE_API virtual EOperationValidationResult Validate();

	/**
     * Transfer the bone weights from the source mesh to the given target mesh and store the result in the skin weight  
     * attribute with the given profile name.
	 * 
	 * @param InOutTargetMesh	  Target mesh we are transfering weights into
     * @param InTargetProfileName Skin weight profile name we are writing into. If the profile with that name exists,  
     *							  then the data will be overwritten, otherwise a new attribute will be created.
     * 
     * @return true if the algorithm succeeds, false if it failed or was canceled by the user.
	 * 
	 * @note Assumes that the InOutTargetMesh has bone attributes, use bIgnoreBoneAttributes flag to ignore the bone 
	 * 		 attributes and skip re-indexing.
	 */
	UE_API virtual bool TransferWeightsToMesh(FDynamicMesh3& InOutTargetMesh, const FName& InTargetProfileName);


	/**
	 * Compute the bone weights for a given point using the ETransferBoneWeightsMethod::ClosestPointOnSurface algorithm.
	 *
	 * @param OutWeights 		Bone weight computed for the input transformed point.
	 * @param InPoint			Point for which we are computing the bone weight.
	 * @param TargetBoneToIndex Optional map from the bone names to the bone indices of the target skeleton. 
	 * 							If null, the bone indices of the skinning weights will not be re-indexed after the transfer.
	 * @param InNormal			Normal at the input point. Should be set if NormalThreshold >= 0.
	 * 
	 * @return true if the algorithm succeeds, false if it failed to find the matching point or was canceled by the user.
	 */
	UE_API virtual bool TransferWeightsToPoint(UE::AnimationCore::FBoneWeights& OutWeights, 
										const FVector3d& InPoint, 
										const TMap<FName, uint16>* TargetBoneToIndex = nullptr,
										const FVector3f& InNormal = FVector3f::Zero());
	
	/**
	 * Compute the bone weights for a given point using the ETransferBoneWeightsMethod::ClosestPointOnSurface algorithm.
	 *
	 * @param OutBones			Array of bone indices. Array size is equal to the number of bone influences.
	 * @param OutWeights		Array of bone weights. Array size is equal to the number of bone influences.
	 * @param InPoint			Point for which we are computing the bone weight.
	 * @param TargetBoneToIndex Optional map from the bone names to the bone indices of the target skeleton. 
	 * 							If null, the bone indices of the skinning weights will not be re-indexed after the transfer.
	 * @param InNormal			Normal at the input point. Should be set if NormalThreshold >= 0.
	 * 
	 * @return true if the algorithm succeeds, false if it failed to find the matching point or was canceled by the user.
	 * 
	 * @note Add specialization in the source file for template types you want to use. 
	 * 		 Currently supporting int for BoneIndexType and float for BoneFloatWeightType, PosVectorType, NormalVectorType
	 */
	template<typename BoneIndexType, typename BoneFloatWeightType, typename PosVectorType,  typename NormalVectorType = float>
	bool TransferWeightsToPoint(TArray<BoneIndexType>& OutBones,
								TArray<BoneFloatWeightType>& OutWeights,
								const UE::Math::TVector<PosVectorType>& InPoint,
								const TMap<FName, uint16>* TargetBoneToIndex = nullptr,
								const UE::Math::TVector<NormalVectorType>& InNormal = UE::Math::TVector<NormalVectorType>::Zero());
protected:
	
    /** @return if true, abort the computation. */
	UE_API virtual bool Cancelled();
	
	/** 
	 * Find the closest point on the surface of the source mesh and return the ID of the triangle containing it and its 
	 * barycentric coordinates.
	 * 
	 * @return true if point is found, false otherwise
	 */
	UE_API bool FindClosestPointOnSourceSurface(const FVector3d& InPoint, const FTransformSRT3d& InToWorld, int32& OutTriID, FVector3d& OutBary);

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
