// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"
#include "UObject/NameTypes.h"
#include "TransformTypes.h"

#define UE_API DYNAMICMESH_API

// Forward declarations
class FProgressCancel;

namespace UE::Geometry
{
	class FDynamicMesh3;
	template<typename MeshType> class TMeshAABBTree3;
	typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3;
	class FMeshNormals;
}


namespace UE
{
namespace Geometry
{

/**
 * Transfer vertex colors from one mesh (source) to another (target).
 */

class FTransferVertexColorAttribute
{
public:

	enum class ETransferMethod : uint8
	{
        // For every vertex on the target mesh, find the closest point on the surface of the source mesh. If that point 
        // is within the SearchRadius, and their normals differ by less than the NormalThreshold, then we directly copy  
        // the weights from the source point to the target mesh vertex.
		ClosestPointOnSurface = 0,

        // Same as the ClosestPointOnSurface but for all the vertices we didn't copy the weights directly, automatically 
		// compute the smooth weights.
		Inpaint = 1
	};

	//
	// Optional Inputs
	//
	
    /** Set this to be able to cancel the running operation. */
	FProgressCancel* Progress = nullptr;

	/** Enable/disable multi-threading. */
	bool bUseParallel = true;
	
	/** The transfer method to compute the bone weights. */
	ETransferMethod TransferMethod = ETransferMethod::ClosestPointOnSurface;

	/** Transform applied to the input target mesh or target point before transfer. */
	FTransformSRT3d TargetToWorld = FTransformSRT3d::Identity();

	//
	// Optional Inputs for ETransferMethod::Inpaint method
	//

	/**  Radius for searching the closest point. If negative, all points are considered. */
	double SearchRadius = -1;

	/** 
	 * Maximum angle (in radians) difference between target and source point normals to be considered a match. 
	 * If negative, normals are ignored.
	 */
	double NormalThreshold = -1;
	
	/** 
	 * If true, when the closest point doesn't pass the normal threshold test, will try again with a flipped normal. 
	 * This helps with layered meshes where the "inner" and "outer" layers are close to each other but whose normals 
	 * are pointing in the opposite directions. 
	 */
	bool LayeredMeshSupport = false;

	/** The number of optional post-processing smoothing iterations applied to the vertices without the match. */
	int32 NumSmoothingIterations = 0; 

	/** The strength of each post-processing smoothing iteration. */
	float SmoothingStrength = 0.0f;

	/** If true, will use the intrinsic Delaunay mesh to construct sparse Cotangent Laplacian matrix. */
	bool bUseIntrinsicLaplacian = false;

	/** 
	 * Optional mask where if ForceInpaint[VertexID] != 0 we want to force the colors for the vertex to be computed  
	 * automatically.
	 * 
	 * @note Only used when TransferMethod == ETransferMethod::Inpaint.
	 * 		 The size must be equal to the InTargetMesh.MaxVertexID(), otherwise the mask is ignored.
	 */
	TArray<float> ForceInpaint;

	/** 
	 * Optional subset of target mesh vertices to transfer weights to.
	 * If left empty, skin weights will be transferred to all target mesh vertices.
	 */
	TArray<int32> TargetVerticesSubset;

	//
	// Outputs
	//

	/** MatchedVertices[VertexID] is set to true for a target mesh vertex ID with a match found, false otherwise. */
	TArray<bool> MatchedVertices;

	/** Creates vertex instances per triangle to be able to have per-face vertex colors. */
	bool bHardEdges = false;
	
	/** Ratio used to blend a vertex between its position and the center of the face (0 = vertex position, 1 = face centroid) */
	float BiasRatio = UE_KINDA_SMALL_NUMBER;

protected:
		
	/** Source mesh we are transferring colors from. */
	const FDynamicMesh3* SourceMesh = nullptr;
	
	/** 
	 * The caller can optionally specify the source mesh BVH in case this operator is run on multiple target meshes 
	 * while the source mesh remains the same. Otherwise BVH tree will be computed.
	 */
	const FDynamicMeshAABBTree3* SourceBVH = nullptr;

	/** If the caller doesn't pass BVH for the source mesh then we compute one. */
	TUniquePtr<FDynamicMeshAABBTree3> InternalSourceBVH;

	/** If the source mesh doesn't have per-vertex normals then compute them */
	TUniquePtr<FMeshNormals> InternalSourceMeshNormals;

public:
	
	/**
	 * @param InSourceMesh The mesh we are transferring colors from 
	 * @param SourceBVH Optional source mesh BVH. If not provided, one will be computed internally. 
	 * 
	 * @note Assumes that the InSourceMesh has a primary colors attribute.
	 */
	UE_API FTransferVertexColorAttribute(const FDynamicMesh3* InSourceMesh, const FDynamicMeshAABBTree3* SourceBVH = nullptr); 
	
	UE_API virtual ~FTransferVertexColorAttribute();

	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot.
	 */
	UE_API virtual EOperationValidationResult Validate();

	/**
     * Transfer the colors from the source mesh to the given target mesh and store the result in the primary colors attribute.
	 * 
	 * @param InOutTargetMesh	  Target mesh we are transferring colors into
     * 
     * @return true if the algorithm succeeds, false if it failed or was canceled by the user.
	 */
	UE_API virtual bool TransferColorsToMesh(FDynamicMesh3& InOutTargetMesh);


	/**
	 * Compute the color for a given point using the ETransferMethod::ClosestPointOnSurface algorithm.
	 *
	 * @param OutColor 		Color computed for the input transformed point.
	 * @param InPoint		Point for which we are computing the color.
	 * @param InNormal		Normal at the input point. Should be set if NormalThreshold >= 0.
	 * 
	 * @return true if the algorithm succeeds, false if it failed to find the matching point or was canceled by the user.
	 */
	UE_API bool TransferColorToPoint(FVector4f& OutColor, const FVector3d& InPoint, const FVector3f& InNormal = FVector3f::Zero()) const;
	
protected:
	
    /** @return if true, abort the computation. */
	UE_API virtual bool Cancelled();
	
	/** 
	 * Find the closest point on the surface of the source mesh and return the ID of the triangle containing it and its 
	 * barycentric coordinates.
	 * 
	 * @return true if point is found, false otherwise
	 */
	UE_API bool FindClosestPointOnSourceSurface(const FVector3d& InPoint, const FTransformSRT3d& InToWorld, int32& OutTriID, FVector3d& OutBary) const;

	/**
     * Transfer the colors from the source mesh to the given target mesh using the closest point algorithm.
     * @return the number of matching vertices / elements
	 */
	UE_API int32 TransferUsingClosestPoint(FDynamicMesh3& InOutTargetMesh, const TUniquePtr<FMeshNormals>& InTargetMeshNormals);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
