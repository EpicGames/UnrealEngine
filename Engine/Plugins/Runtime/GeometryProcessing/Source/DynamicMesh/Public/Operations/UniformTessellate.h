// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryTypes.h"
#include "IndexTypes.h"

#define UE_API DYNAMICMESH_API

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * Given an input mesh and a tessellation level, this operator generates a new tessellated mesh where every triangle in 
 * the input mesh is uniformly subtriangulated into (TessellationNum + 1)^2 triangles. Per-vertex normals, uvs, 
 * colors and extended per-vertex attributes are linearly interpolated to the new vertices. Per-triangle group 
 * identifiers/materials and extended triangle attributes for the new triangles are inherited from the corresponding 
 * input mesh triangles they replaced. 
 * 
 * Note: Currently does not support interpolation of the GenericAttributes besides Skin Weights.
 *   
 *            o                                 o
 *           / \     TessellationNum = 2       / \
 *          /   \                             /   \
 *         /     \                           x-----x
 *        /       \    FUniformTessellate   / \   / \
 *       /         \    ------------->     /   \ /   \
 *      /           \                     x-----*-----x
 *     /             \                   / \   / \   / \
 *    /               \                 /   \ /   \ /   \
 *   o-----------------o               o-----x-----x-----o
 */

class FUniformTessellate
{
public:
	//
	// Inputs
	//
	
	/** Set this to be able to cancel running operation. */
	FProgressCancel* Progress = nullptr;

	/** Determines the number of triangles we generate. */
	int TessellationNum = 0;

	/** Should multi-threading be enabled. */
	bool bUseParallel = true;

	/** If true, populate VertexMap, VertexEdgeMap, VertexTriangleMap. */
	bool bComputeMappings = false;

	//
	// Input/Output
	//

	/** The tessellated mesh. */
	FDynamicMesh3* ResultMesh = nullptr;
	
	//
	// Output
	//

	/** 
	 * Map the vertex ID in the input mesh to the corresponding vertex ID in the output mesh. 
	 * If the vertex ID is Invalid in the input mesh, then it is mapped to InvalidID.
	 */
	TArray<int32> VertexMap;
	
	/**
	 * Map the ID of the vertex inserted along the input mesh edge to the input mesh triangle IDs (t1,t2) that share 
	 * that edge. If its a boundary edge then t2 is InvalidID.
	 */
	TMap<int32, FIndex2i> VertexEdgeMap;

	/** Map the ID of the vertex inserted inside the input mesh triangle to the triangle's ID. */
	TMap<int32, int32> VertexTriangleMap;

protected:
	
	/** 
	 * Points to either the input mesh to be used to generate the new tessellated mesh or will point to a copy of 
	 * the mesh we are tessellating inplace.
	 */
	const FDynamicMesh3* Mesh = nullptr;

	/** 
	 * If true, the mesh to tessellate is contained in the ResultMesh and will be overwritten with the 
	 * tessellation result. 
	 */
	bool bInPlace = false;

public:

	/** 
	 * Tessellate the mesh inplace. If the tessellation fails or the user cancelled the operation, OutMesh will not 
	 * be changed.
	 */
	FUniformTessellate(FDynamicMesh3* OutMesh) 
	:
	ResultMesh(OutMesh), bInPlace(true)
	{
	}

	/** 
	 * Tessellate the mesh and write the result into another mesh. This will overwrite any data stored in the OutMesh. 
	 */
	FUniformTessellate(const FDynamicMesh3* Mesh, FDynamicMesh3* OutMesh) 
	: 
	ResultMesh(OutMesh), Mesh(Mesh), bInPlace(false)
	{
	}
	
	virtual ~FUniformTessellate() = default; 

	/** @return The number of vertices after the tessellation. */
	static UE_API int32 ExpectedNumVertices(const FDynamicMesh3& Mesh, const int32 TessellationNum);
	
	/** @return The number of triangles after the tessellation. */
	static UE_API int32 ExpectedNumTriangles(const FDynamicMesh3& Mesh, const int32 TessellationNum);

	/**
	 * @return EOperationValidationResult::Ok if we can apply operation, or error code if we cannot.
	 */
	virtual EOperationValidationResult Validate()
	{	
		const bool bIsInvalid = bInPlace == false && (Mesh == nullptr || ResultMesh == nullptr);
		const bool bIsInvalidInPlace = bInPlace && ResultMesh == nullptr;
		
		if (TessellationNum < 0 || bIsInvalid || bIsInvalidInPlace) 
		{
			return EOperationValidationResult::Failed_UnknownReason;
		}

		return EOperationValidationResult::Ok;
	}

	/**
	 * Generate tessellated geometry.
	 * 
	 * @return true if the algorithm succeeds, false if it failed or was cancelled by the user.
	 */
	UE_API virtual bool Compute();

protected:

	/** If this returns true, abort computation. */
	UE_API virtual bool Cancelled();
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
