// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdgeSpan.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * Weld a pair of group edges. The two input spans must be boundary spans. Their
 *  position in the world is irrelevant, as the welding will always be done to
 *  preserve appropriate triangle winding, i.e. so that the newly welded triangles
 *  are not flipped relative to their neighbor.
 * 
 * User can optionally allow triangle deletion which handles cases
 *  where the group edges are connected by an edge at the end points.
 */

class FWeldEdgeSequence
{
public:
	enum class EWeldResult
	{
		Ok								= 0,

		Failed_EdgesNotBoundaryEdges	= 10,		// Occurs when any edge in either input span isn't a boundary edge.

		Failed_CannotSplitEdge			= 21,		// Occurs when SplitEdge() fails, haven't encountered this and I'm not sure what would cause it
		Failed_TriangleDeletionDisabled = 22,		// Occurs when bAllowIntermediateTriangleDeletion is false and edge spans are connected by an edge
		Failed_CannotDeleteTriangle		= 23,		// Occurs when bAllowIntermediateTriangleDeletion is true, edge spans are connected, but edge deletion fails

		Failed_Other					= 100,		// Catch all for general failure
	};

public:
	/**
	* Inputs/Outputs
	*/
	FDynamicMesh3* Mesh;
	FEdgeSpan EdgeSpanToDiscard;	// This data is junk once Weld() is called
	FEdgeSpan EdgeSpanToKeep;		// This is the updated edge span which can be used once Weld() is called

	// Whether triangle deletion is allowed in order to merge edges which are connected by a different edge
	bool bAllowIntermediateTriangleDeletion = false;

	// When true, failed calls to MergeEdges() will be handled by moving the edges without merging such
	// that the final result appears to be welded but has invisible seam(s) instead of just failing.
	bool bAllowFailedMerge = false;

	// When vertices are welded, each kept vertex will be placed at Lerp(KeepPos, RemovePos, InterpolationT)
	double InterpolationT = 0;

	// This is populated with pairs of eids which were not able to be merged.
	// Only valid when bAllowFailedMerge is true
	TArray<TPair<int, int>> UnmergedEdgePairsOut;

public:
	FWeldEdgeSequence(FDynamicMesh3* Mesh, FEdgeSpan SpanDiscard, FEdgeSpan SpanKeep)
		: Mesh(Mesh)
		, EdgeSpanToDiscard(SpanDiscard)
		, EdgeSpanToKeep(SpanKeep)
	{
	}
	virtual ~FWeldEdgeSequence() = default;


	/**
	* Alters the existing mesh by welding two edge sequences, preserving sequence A.
	* Conditions the mesh by splitting edges and optionally deleting triangles.
	*
	* @return EWeldResult::OK on success
	*/
	UE_API EWeldResult Weld();

	/**
	 * Helper that splits the edges in the shorter span until the spans have the same number
	 *  of edges. Weld() will automatically do this, but this is public in case the user wants
	 *  to equalize spans that get concatenated together before all being welded at once.
	 */
	static UE_API EWeldResult SplitEdgesToEqualizeSpanLengths(FDynamicMesh3& Mesh, FEdgeSpan& Span1, FEdgeSpan& Span2);
protected:
	/**
	* Verifies validity of input edges by ensuring they are
	* correctly-oriented boundary edges
	*
	* @return EWeldResult::OK on success
	*/
	UE_API EWeldResult CheckInput();

	/**
	* Splits largest edges of the span with fewest vertices so that
	* both input spans have an equal number of vertices and edges after 
	* 
	* @return EWeldResult::OK on success
	*/
	UE_API EWeldResult SplitSmallerSpan();


	UE_DEPRECATED(5.5, "Side triangles are handled appropriately in WeldEdgeSequence")
	UE_API EWeldResult CheckForAndCollapseSideTriangles();

	/**
	* Welds edge sequence together
	*
	* @return EWeldResult::OK on success
	*/
	UE_API EWeldResult WeldEdgeSequence();
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
