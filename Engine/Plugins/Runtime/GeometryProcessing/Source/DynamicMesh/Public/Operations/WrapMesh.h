// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryTypes.h"
#include "IndexTypes.h"
#include "Templates/PimplPtr.h"

#define UE_API DYNAMICMESH_API

// Forward declarations
namespace UE::Geometry
{
	class FDynamicMesh3;
}

namespace UE
{
namespace Geometry
{

struct FWrapMeshCorrespondence
{
	int32 SourceVertexIndex = IndexConstants::InvalidID;
	int32 TargetVertexIndex = IndexConstants::InvalidID;
};

/**
 * Wrap a source mesh's topology to a target mesh's shape. Users can supply vertex correspondences for landmarks to aid the mesh wrap process.
 */
class FWrapMesh final
{
public:

	UE_API explicit FWrapMesh(const FDynamicMesh3* InSourceMesh);

	UE_API ~FWrapMesh();

	enum struct ELaplacianType : uint8
	{
		CotangentNoArea,
		CotangentAreaWeighted,
		AffineInvariant,
	};

	// Optional Inputs

	/** Mesh Wrap is calculated with an inner and outer loop. This is the maximum number of outer loops. Each outer loop increases the Projection Stiffness by ProjectionStiffnessMultiplier.*/
	int32 MaxNumOuterIterations = 10;
	/** Mesh Wrap is calculated with an inner and outer loop. This is the number of inner loops run before increasing the Projection Stiffness in the outer loop.*/
	int32 NumInnerIterations = 10;
	/** Mesh Wrap will terminate early if the Projection tolerance is within this threshold.*/
	double ProjectionTolerance = 1e-4;

	/** Weld source mesh vertices with matching positions during mesh wrap. Final WrappedMesh topology will match original Source Mesh, not the welded one. Without welding, dynamic meshes with duplicate vertices may split at the seams. */
	bool bWeldSourceMesh = true;
	/** Threshold for welding positions */
	double SourceMeshWeldingThreshold = UE_THRESH_POINTS_ARE_SAME;
	/** Weight of mesh wrap to retain Source Topology mesh features. Metric is defined by Laplacian Type.*/
	double LaplacianStiffness = 1.;
	/** Initial weight of mesh wrap to match projected Target Shape. Each outer loop will multiply this stiffness by ProjectionStiffnessMultiplier.*/
	double InitialProjectionStiffness = 0.1;
	/** Each outer loop will multiply InitialProjectionStiffness by this to improve Target Shape match.*/
	double ProjectionStiffnessMuliplier = 10.;
	/** Weight of mesh wrap to match Landmark correspondences.*/
	double CorrespondenceStiffness = 0.1;
	/** Laplacian type.*/
	ELaplacianType LaplacianType = ELaplacianType::AffineInvariant;

	UE_API void SetMesh(const FDynamicMesh3* InSourceMesh);

	UE_API void WrapToTargetShape(const FDynamicMesh3& TargetShape, const TArray<FWrapMeshCorrespondence>& SourceTargetVertexCorrespondences, FDynamicMesh3& WrappedMesh) const;

private:
	void Initialize();

	const FDynamicMesh3* SourceMesh;

	struct FImpl; // Used to keep Eigen stuff out of the header
	TPimplPtr<FImpl> Impl;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
