// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{


struct FUVEditResult
{
	TArray<int32> NewUVElements;

};


/**
 * FDynamicMeshUVEditor implements various UV overlay editing operations.
 */
class FDynamicMeshUVEditor
{
private:
	/** The mesh we will be editing */
	FDynamicMesh3* Mesh = nullptr;
	/** The UV Overlay we will be editing */
	FDynamicMeshUVOverlay* UVOverlay = nullptr;
	/** The index of the UV Overlay we will be editing */
	int32 UVOverlayIndex = -1;

public:
	/**
	 * Construct UV Editor for a UV Overlay of the given Mesh.
	 * @param UVLayerIndex index of target UV layer
	 * @param bCreateIfMissing if true, target UV layers up to UVLayerIndex will be created if it is not there. Otherwise UVOverlay will be nullptr and class is incomplete.
	 */
	UE_API explicit FDynamicMeshUVEditor(FDynamicMesh3* MeshIn, int32 UVLayerIndex, bool bCreateIfMissing);

	/**
	 * Construct UV Editor for a UV Overlay of the given Mesh.
	 */
	UE_API explicit FDynamicMeshUVEditor(FDynamicMesh3* MeshIn, FDynamicMeshUVOverlay* UVOverlayIn);


	FDynamicMesh3* GetMesh() { return Mesh; }
	const FDynamicMesh3* GetMesh() const { return Mesh; }

	FDynamicMeshUVOverlay* GetOverlay() { return UVOverlay; }
	const FDynamicMeshUVOverlay* GetOverlay() const { return UVOverlay; }

	/**
	 * Create specified UVLayer if it does not exist
	 */
	UE_API void CreateUVLayer(int32 UVLayerIndex);

	/**
	 * Append new UV layer to the end of the array, returning the newly added index, or -1 if at max layers already
	 */
	UE_API int32 AddUVLayer();

	/**
	 * Switch to new layer
	 */
	UE_API void SwitchActiveLayer(int32 UVLayerIndex);

	/**
	 * Remove active layer, setting active layer to highest, preceeding layer. Will not remove the final layer from the mesh. Returns the resulting active layer index.
	 */
	UE_API int32 RemoveUVLayer();

	/**
	 * Clear UVs for all triangles on active layer
	 */
	UE_API void ResetUVs();

	/**
	* Clear UVs for given triangles on active layer
	*/
	UE_API void ResetUVs(const TArray<int32>& Triangles);

	/**
	 * Copy UVs from another overlay
	 */
	UE_API bool CopyUVLayer(FDynamicMeshUVOverlay* FromUVOverlay);


	/**
	 * Initialize UV topology to be one UV per vertex. UVs are initialized to (0,0)
	 * @param VertexToUVOut returned new overlay UV ID for each mesh Vertex ID (array size is mesh MaxVertexID)
	 * @param bIsIdentityMapOut returned as true if UV ID == Vertex ID for every vertex  (even if there are gaps)
	 */
	UE_API void SetToPerVertexUVs(TArray<int32>& VertexToUVOut, bool& bIsIdentityMapOut, FUVEditResult* Result = nullptr);


	/**
	 * Create new UV island for each Triangle, by planar projection onto plane of Triangle. No transforms/etc are applied.
	 */
	UE_API void SetPerTriangleUVs(const TArray<int32>& Triangles, double ScaleFactor = 1.0, FUVEditResult* Result = nullptr);

	/**
	 * Create new UV island for given Triangles, and set UVs by planar projection to ProjectionFrame. No transforms/etc are applied.
	 */
	UE_API void SetPerTriangleUVs(double ScaleFactor = 1.0, FUVEditResult* Result = nullptr);

	/**
	 * Apply arbitrary transform to UV elements
	 */
	UE_API void TransformUVElements(const TArray<int32>& ElementIDs, TFunctionRef<FVector2f(const FVector2f&)> TransformFunc);


	/**
	 * Create new UV island for given Triangles, and set UVs by planar projection to ProjectionFrame. No transforms/etc are applied.
	 */
	UE_API void SetTriangleUVsFromProjection(const TArray<int32>& Triangles, const FFrame3d& ProjectionFrame, FUVEditResult* Result = nullptr);

	/**
	 * Create new UV island for given Triangles, and set UVs by planar projection to ProjectionFrame. 
	 * PointTransform is applied to points before projectiong onto ProjectionFrame X/Y axes
	 * Projected U/V coordinates are divided by Dimensions.X/Y
	 */
	UE_API void SetTriangleUVsFromPlanarProjection(const TArray<int32>& Triangles, TFunctionRef<FVector3d(const FVector3d&)> PointTransform, const FFrame3d& ProjectionFrame, const FVector2d& Dimensions, FUVEditResult* Result = nullptr);


	/**
	 * FExpMapOptions provides additional control over ExpMap UV generation below
	 */
	struct FExpMapOptions
	{
		/** Number of rounds of explicit uniform normal smoothing to apply to mesh normals */
		int32 NormalSmoothingRounds;
		/** Alpha for smoothing, valid range is 0-1 */
		double NormalSmoothingAlpha;

		FExpMapOptions()
		{
			NormalSmoothingRounds = 0;
			NormalSmoothingAlpha = 0.25f;
		}
	};

	/**
	 * Create new UV island for given Triangles, and set UVs for that island using Discrete Exponential Map.
	 * ExpMap center-point is calculated by finding maximum (Dijkstra-approximated) geodesic distance from border of island.
	 * @warning computes a single ExpMap, so input triangle set must be connected, however this is not verified internally
	 */
	UE_API bool SetTriangleUVsFromExpMap(const TArray<int32>& Triangles, const FExpMapOptions& Options = FExpMapOptions(), FUVEditResult* Result = nullptr);


	/**
	 * Create new UV island for given Triangles, and set UVs for that island using Discrete Exponential Map.
	 * ExpMap center-point is calculated by finding maximum (Dijkstra-approximated) geodesic distance from border of island.
	 * @warning computes a single ExpMap, so input triangle set must be connected, however this is not verified internally
	 */
	UE_API bool SetTriangleUVsFromExpMap(
		const TArray<int32>& Triangles, 
		TFunctionRef<FVector3d(const FVector3d&)> PointTransform,
		const FFrame3d& ProjectionFrame,
		const FVector2d& Dimensions,
		int32 NormalSmoothingRounds = 0,
		double NormalSmoothingAlpha = 0.5,
		double FrameNormalBlendWeight = 0,
		FUVEditResult* Result = nullptr);


	/**
	 * Create new UV island for given Triangles, and set UVs for that island using Discrete Natural Conformal Map (equivalent to Least-Squares Conformal Map)
	 * @warning computes a single parameterization, so input triangle set must be connected, however this is not verified internally
	 */
	UE_API bool SetTriangleUVsFromFreeBoundaryConformal(const TArray<int32>& Triangles, FUVEditResult* Result = nullptr);

	/**
	 * Create new UV island for given Triangles, and set UVs for that island using Discrete Natural Conformal Map (equivalent to Least-Squares Conformal Map)
	 * @param Triangles list of triangles
	 * @param bUseExistingUVTopology if true, re-solve for existing UV set, rather than constructing per-vertex UVs from triangle set. Allows for solving w/ partial seams, interior cuts, etc. 
	 * @warning computes a single parameterization, so input triangle set must be connected, however this is not verified internally
	 */
	UE_API bool SetTriangleUVsFromFreeBoundaryConformal(const TArray<int32>& Triangles, bool bUseExistingUVTopology, FUVEditResult* Result = nullptr);

	/**
	 * Create new UV island for given Triangles, and set UVs for that island using Discrete Natural Conformal Map (equivalent to Least-Squares Conformal Map).
	 * Using this overload implies that we're re-solving for existing UV set rather than constructing per-vertex UVs from triangle set,
	 * since pinned elements rely on existing values.
	 * @param Triangles list of triangles
	 * @param PinnedElementIDs Elements for which to use unmodified UV values.
	 * @warning computes a single parameterization, so input triangle set must be connected, however this is not verified internally
	 */
	UE_API bool SetTriangleUVsFromFreeBoundaryConformal(const TArray<int32>& Triangles, const TSet<int32>& PinnedElementIDs, FUVEditResult* Result = nullptr);

	/**
	 * Create new UV island for given Triangles, and set UVs for that island using Spectral Conformal Map.
	 * @param Triangles list of triangles
	 * @param bUseExistingUVTopology if true, re-solve for existing UV set, rather than constructing per-vertex UVs from triangle set. Allows for solving w/ partial seams, interior cuts, etc. 
	 * @param bPreserveIrregularity if true, reduces distortion for meshes with triangles of vastly different sizes.
	 * @warning computes a single parameterization, so input triangle set must be connected, however this is not verified internally
	 */
	UE_API bool SetTriangleUVsFromFreeBoundarySpectralConformal(const TArray<int32>& Triangles, bool bUseExistingUVTopology, bool bPreserveIrregularity, FUVEditResult* Result = nullptr);

	/**
	 * Initializes any uninitialized triangles in the set to per-vertex (0,0) UVs. Doesn't clear or modify any existing 
	 *  UVs, and only operates on the given set.
	 */
	UE_API void MakeSureUVsAreSet(const TSet<int32>& Triangles, FUVEditResult* Result = nullptr, TSet<int32>* ChangedTrianglesOut = nullptr);

	/** 
	 * Merge existing UV topology with a set of edges, removing seams at edges if they exist within the UV topology.
	 * 
	 * @param EidsToRemoveAsSeams list of edges to remove seams from
	 * @return true on success
	 */
	UE_API bool RemoveSeamsAtEdges(const TSet<int32>& EidsToRemoveAsSeams);

	/**
	 * Cut existing UV topology with a set of edges. This allows for creating partial seams/darts, interior cuts, etc.
	 * 
	 * Avoids creating bowties. In cases where an edge is not next to any present or future seams/borders, some
	 * adjacent edge will be picked to be made into a seam as well, since it's impossible to make the original
	 * into a seam otherwise.

	 * @param EidsToMakeIntoSeams list of edges to turn into seams
	 * @param Result if non-null, list of new UV elements created along the path will be stored here (not ordered)
	 * @return true on success
	 */
	UE_API bool CreateSeamsAtEdges(const TSet<int32>& EidsToMakeIntoSeams, FUVEditResult* Result = nullptr);

	/**
	 * Takes the currently selected triangles and makes a separate UV island out of them, i.e. any interior seams
	 *  are removed, and seams are added around the boundary of the selection. If the selection is not connected
	 *  in the mesh, islands will be created for each connected component of selected triangles. If some of the 
	 *  selected triangles have unset UVs, they will be initialized to per-vertex zero UVs for the purposes of
	 *  creating an island.
	 * 
	 * @param TidsToMakeIntoIsland Triangles in the mesh
	 * @param Result if non-null, list of new UV elements created for new seams, or to initialize UVs.
	 * @param ChangedTrianglesOut Optional output of triangles that had their UVs changed. Currently this is a conservative
	 *   superset, so some might not have actually changed. Not cleared before use.
	 * @return true on success
	 */
	UE_API bool MakeIsland(const TSet<int32>& TidsToMakeIntoIsland, FUVEditResult* Result = nullptr, TSet<int32>* ChangedTrianglesOut = nullptr);

	/**
	 * Set UVs by box projection. Triangles will be grouped to "best" box face
	 * PointTransform is applied to points before projectiong onto ProjectionFrame X/Y axes
	 * Projected U/V coordinates are divided by Dimensions.X/Y
	 * @param MinIslandTriCount Any UV island with fewer triangles than this count will be merged into a neighbouring island
	 */
	UE_API void SetTriangleUVsFromBoxProjection(const TArray<int32>& Triangles, TFunctionRef<FVector3d(const FVector3d&)> PointTransform, const FFrame3d& BoxFrame, const FVector3d& BoxDimensions, int32 MinIslandTriCount = 2, FUVEditResult* Result = nullptr);


	/**
	 * Set UVs by cylinder projection. Triangles will be grouped to cylinder endcaps or side based on CylinderSplitAngle.
	 * PointTransform is applied to points before projectiong onto ProjectionFrame X/Y axes
	 * Projected U/V coordinates are divided by Dimensions.X/Y
	 */
	UE_API void SetTriangleUVsFromCylinderProjection(const TArray<int32>& Triangles, TFunctionRef<FVector3d(const FVector3d&)> PointTransform, const FFrame3d& CylFrame, const FVector3d& CylDimensions, float CylinderSplitAngle, FUVEditResult* Result = nullptr);


	struct FTransferFromMeshViaProjectionSettings
	{
		// Range to search for source triangles
		float MinDistance = -FMathf::MaxReal, MaxDistance = FMathf::MaxReal;

		// For triangles without matching source triangles, whether reset the UVs -- if false, they will be left as-is
		bool bResetUVsForUnmatched = false;

		// If a filter is provided, will only transfer UVs from triangles where the filter returns 'true'
		TFunction<bool(int32)> SourceMeshTriFilter = nullptr;
	};

	/**
	 * Transfer UVs from a source mesh by projecting along the specified direction
	 * @param Triangles Triangles to update with new UVs
	 * @param TransformTargetToSourceSpacePosition Transform to take target mesh vertices to the space of the source mesh
	 * @param ProjectionDirection Direction to project, in local space of TargetMesh
	 * @param ProjectionOffset Distance to offset from TargetMesh vertices before projecting
	 * @param TransformProjectionToSourceSpaceVector Transform to take the ProjectionDirection to the source mesh space
	 * @param SourceMeshSpatial AABB Tree of source mesh
	 * @param SourceMeshUVChannel UV Channel to read from on source mesh
	 * @param Settings Projection settings
	 * @param Result If non-null, contains new UV elements IDs created on target mesh
	 */
	UE_API void TransferTriangleUVsFromMeshViaDirectionProjection(TArrayView<const int32> Triangles, TFunctionRef<FVector3d(const FVector3d&)> TransformTargetToSourceSpacePosition, 
		const FVector3d& ProjectionDirection, double ProjectionOffset, TFunctionRef<FVector3d(const FVector3d&)> TransformProjectionToSourceSpaceVector,
		const FDynamicMeshAABBTree3& SourceMeshSpatial, int32 SourceMeshUVChannel,
		const FTransferFromMeshViaProjectionSettings& Settings, FUVEditResult* Result = nullptr);


	/**
	* Compute the UV-space and 3D area of the given Triangles, and then scale the UV area to be equvalent
	* to the 3D area. Scaling is around the center of the UV bounding box.
	* @param bRecenterAtOrigin if true, UVs are translated after scaling such that the UV bounding box center is at (0,0)
	* @param ScaleFactor Additional scale factor to multiply into resulting UVs
	*/
	UE_API bool ScaleUVAreaTo3DArea(const TArray<int32>& Triangles, bool bRecenterAtOrigin, float ScaleFactor = 1.0f);


	/**
	* Scale UVs of the given triangles to fit within the provided provided bounding box.
	* @param BoundingBox the 2D bounding box to constraint the requested UVs within.
	* @param bPreserveAspectRatio if true, when scaling UVs, maintain original aspect ratio instead of fitting to box bounds
	* @param bRecenterAtBoundingBox if true, UVs are translated such that their centroid is at the bounding box center
	*/
	UE_API bool ScaleUVAreaToBoundingBox(const TArray<int32>& Triangles, const FAxisAlignedBox2f& BoundingBox, bool bPreserveAspectRatio, bool bRecenterAtBoundingBox);


	/**
	* Compute an oriented UV-space bounding box for the given Triangles, and then rotate the UVs such that the
	* box is aligned with the X axis. 
	* The UV-space box is found by computing the 2D Convex Hull and then finding the hull edge that results in the minimal box.
	*/
	UE_API bool AutoOrientUVArea(const TArray<int32>& Triangles);


	/**
	 * Pack UVs into unit rectangle
	 */
	UE_API bool QuickPack(int32 TargetTextureResolution = 512, float GutterSize = 1.0f);

	/**
    * Pack specific UVs triangles into the specific unit rectangle at the specified index
	* @param UDIMCoordsIn Which unit rectangle tile should the UVs be packed in, indicated by the coordinate of it's lower left hand corner
	* @param Triangles Which UV triangles to pack, all triangles if nullptr
    */
	UE_API bool UDIMPack(int32 TargetTextureResolution = 512, float GutterSize = 1.0f, const FVector2i& UDIMCoordsIn = FVector2i(0,0), const TArray<int32>* Triangles = nullptr);


	//
	// Utility functions that could maybe go elsewhere but are mainly useful in parameterization context
	//

	/**
	 * Compute geodesic center of given Mesh
	 * @return true if a central vertex was found (but if false, will still give a frame for *some* vertex as long as there is at least one vertex in the mesh)
	 */
	static UE_API bool EstimateGeodesicCenterFrameVertex(const FDynamicMesh3& Mesh, FFrame3d& FrameOut, int32 &VertexIDOut, bool bAlignToUnitAxes = true);

	/**
	 * Compute geodesic center of given Mesh triangles (assumes they are connected)
	 * @return true if a central vertex was found (but if false, will still give a frame for *some* vertex as long as there is at least one triangle)
	 */
	static UE_API bool EstimateGeodesicCenterFrameVertex(const FDynamicMesh3& Mesh, const TArray<int32>& Triangles, FFrame3d& FrameOut, int32& VertexIDOut, bool bAlignToUnitAxes = true);

	/**
	* Compute the occupied 2D area for the UVs for the specified triangles and channel.
	* @param BoundingBox if not null, return the overall bounding box of the UVs for the specified triangles along with the exact area occupied.
	*/
	static UE_API double DetermineAreaFromUVs(const FDynamicMeshUVOverlay& UVOverlay, const TArray<int32>& Triangles, FAxisAlignedBox2f* BoundingBox = nullptr);

	/**
	 * Transform UVs of the specified triangles
	 */
	static UE_API void TransformTriangleSelectionUVs(FDynamicMeshUVOverlay& UVOverlay, TConstArrayView<int32> Triangles, TFunctionRef<FVector2f(const FVector2f&)> TransformFunc);

private: 

	struct FSetUVsFromConformalOptions
	{
		bool bUseExistingUVTopology = false;
		bool bUseSpectral = false; 
		bool bPreserveIrregularity = false;
		// currently only used if bUseSpectral is false
		const TSet<int32>* PinnedElementIDs = nullptr;
	};
	/**
	 * Helper method to solve either the Least Squares Conformal Map or the Spectral Conformal Map problems.
	 */
	UE_API bool SetTriangleUVsFromConformal(const TArray<int32>& Triangles, const FSetUVsFromConformalOptions& Options, FUVEditResult* Result);

	// Helper to split bowties on element IDs
	UE_API void SplitBowtiesOnUVElements(TArray<int32>& UVElementIDs, bool bAddNewElementsToInputArray = true);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
