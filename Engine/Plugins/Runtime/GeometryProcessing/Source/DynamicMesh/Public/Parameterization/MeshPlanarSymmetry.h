// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Quaternion.h"
#include "Spatial/PointHashGrid3.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * FMeshPlanarSymmetry detects pairwise symmetry relationships between vertices in a mesh, given a symmetry frame.
 * Once those relationships are known, symmetry can be re-enforced after mesh edits.
 * 
 * Vertices within a tolerance band of the symmetry plane do not have a mirror and are considered "on plane"
 * and will be snapped to the symmetry plane when enforcing symmetry after edits.
 * 
 * The Symmetry Plane is specified by a Frame3d where the XY axes define the plane and the Z axis is the plane normal.
 * So references to "Positive" side below are relative to that Z axis/plane
 */
class FMeshPlanarSymmetry
{
public:

	// If true, symmetry-finding can return false if it detects a symmetry where all points are on the symmetry plane, rather than attempt to fit a symmetry plane in that case.
	bool bCanIgnoreDegenerateSymmetries = true;

	/**
	 * Given a Mesh, an AABBTree, and a Symmetry Plane/Frame, detect any pairs of vertices with
	 * planar/mirror-symmetry relationships, as well as "on plane" vertices
	 * @return false if any non-on-plane vertex fails to find a match. 
	 */
	UE_API bool Initialize(FDynamicMesh3* Mesh, FDynamicMeshAABBTree3* Spatial, FFrame3d SymmetryFrameIn);
	/**
	 * Given a Mesh and its bounding box, and a Symmetry Plane/Frame, detect any pairs of vertices with
	 * planar/mirror-symmetry relationships, as well as "on plane" vertices
	 * @return false if any non-on-plane vertex fails to find a match.
	 */
	UE_API bool Initialize(FDynamicMesh3* Mesh, const FAxisAlignedBox3d& Bounds, FFrame3d SymmetryFrameIn);

	/**
	 * Given a Mesh and its bounding box, and a Symmetry Plane/Frame, detect any pairs of vertices with
	 * planar/mirror-symmetry relationships, as well as "on plane" vertices
	 * @param bForceComputeVertexPairs Computes vertex pairs even if the mesh doesn't pass quick full symmetry test
	 * @return false if any non-on-plane vertex fails to find a match.
	 */
	UE_API bool Initialize(FDynamicMesh3* Mesh, const FAxisAlignedBox3d& Bounds, FFrame3d SymmetryFrameIn, bool bForceComputeVertexPairs);
	
	/**
	 * Given a Mesh and its bounding box, find a Symmetry Plane/Frame and detect any pairs of vertices with
	 * planar/mirror-symmetry relationships, as well as "on plane" vertices
	 * @param SymmetryFrameOut Returns the discovered symmetry frame by reference, if one was found.
	 * @param PreferredNormals Optionally try to find a symmetry frame aligned to any normals passed in to this array. Tries the normals in order, so the first normal that fits (if any) will be used.
	 * @return false if any non-on-plane vertex fails to find a match.
	 */
	UE_API bool FindPlaneAndInitialize(FDynamicMesh3* Mesh, const FAxisAlignedBox3d& Bounds, FFrame3d& SymmetryFrameOut, TArrayView<const FVector3d> PreferredNormals = TArrayView<const FVector3d>());

	/**
	 * @return the input Point mirrored across the Symmetry plane
	 */
	UE_API FVector3d GetMirroredPosition(const FVector3d& Position) const;

	/**
	 * @return the input Vector/Axis mirrored across the Symmetry plane
	 */
	UE_API FVector3d GetMirroredAxis(const FVector3d& Axis) const;

	/**
	 * @return the input Quaternion mirrored across the Symmetry plane
	 */
	UE_API FQuaterniond GetMirroredOrientation(const FQuaterniond& Orientation) const;

	/**
	 * @return the input Frame mirrored to the "positive" side (unchanged if it is already on the positive side)
	 */
	UE_API FFrame3d GetPositiveSideFrame(FFrame3d FromFrame) const;


	/**
	 * Update all the symmetry vertex positions based on their Positive-side pair vertex.
	 * Also snap all on-plane vertices to the symmetry plane
	 */
	UE_API void FullSymmetryUpdate();


	/** @return true if the TestMesh vertices match the current tracked vertex symmetry correspondence (i.e., the TestMesh vertex IDs match the known correspondence, and have the expected mirror symmetry) */
	UE_API bool ValidateSymmetry(const FDynamicMesh3& TestMesh) const;

	/** @return true if the TestMesh vertices in VertexROI match the current tracked vertex symmetry correspondence (i.e., the vertex IDs match the known correspondence, and have the expected mirror symmetry) */
	UE_API bool ValidateSymmetry(const FDynamicMesh3& TestMesh, TConstArrayView<int32> VertexROI) const;

	//
	// The set of functions below are intended to be used together in situations like 3D sculpting
	// where we want to apply symmetry constraints within a "brush region of interest (ROI)"
	// See UMeshVertexSculptTool for example usage
	//


	/**
	 * Computes list of vertices that are mirror-constrained to vertices in VertexROI.
	 * If bForceSameSizeWithGaps is true, MirrorVertexROIOut will be the same size as VertexROI, and -1 is stored 
	 * for vertices in VertexROI that are source vertices, or on the symmetry plane, or excluded by ShouldExcludeVertex().
	 * Otherwise those vertices are skipped.
	 * 
	 * @param ShouldExcludeVertex Optional filter to exclude vertices: takes vertex ID, returns true if the vertex should not be added to the ROI, false otherwise
	 */
	UE_API void GetMirrorVertexROI(const TArray<int>& VertexROI, TArray<int>& MirrorVertexROIOut, bool bForceSameSizeWithGaps, TFunctionRef<bool(int32 VID)> ShouldExcludeVertex = [](int32) {return false;}) const;

	/**
	 * For any vertices in VertexIndices that are on-plane, take the position in VertexPositionsInOut, apply the
	 * on-plane constraint, and return the constrained position in VertexPositionsInOut  (ie VertexPositionsInOut is updated to new positions)
	 */
	UE_API void ApplySymmetryPlaneConstraints(const TArray<int>& VertexIndices, TArray<FVector3d>& VertexPositionsInOut) const;

	/**
	 * Given the pairing (SourceVertexROI, SourceVertexPositions), compute the symmetry-constrained vertex positions for
	 * MirrorVertexROI and store in MirrorVertexPositionsOut. This function assumes that the MirrorVertexROI was computed
	 * by calling GetMirrorVertexROI(SourceVertexROI, MirrorVertexROI, bForceSameSizeWithGaps=true), ie all the arrays
	 * must be the same length
	 */
	UE_API void ComputeSymmetryConstrainedPositions(
		const TArray<int>& SourceVertexROI,
		const TArray<int>& MirrorVertexROI,
		const TArray<FVector3d>& SourceVertexPositions,
		TArray<FVector3d>& MirrorVertexPositionsOut) const;


protected:
	FDynamicMesh3* TargetMesh = nullptr;

	FFrame3d SymmetryFrame;
	FVector3d CachedSymmetryAxis;

	struct FSymmetryVertex
	{
		// initial distance to symmetry plane
		double PlaneSignedDistance = 0.0;
		// true if this is a positive-side vertex
		bool bIsSourceVertex = true;
		// true if this is an on-plane vertex (todo: possibly combine with bIsSourceVertex as an enum)
		bool bOnPlane = false;
		// VertexID of paired vertex
		int32 PairedVertex = -1;
	};

	// list of vertices, size == TargetMesh.MaxVertexID()
	TArray<FSymmetryVertex> Vertices;

	UE_API void UpdateSourceVertex(int32 VertexID);
	UE_API void UpdatePlaneVertex(int32 VertexID);

private:

	// Helper for computing + assigning symmetry matches
	bool AssignMatches(const FDynamicMesh3* Mesh, const TPointHashGrid3d<int32>& VertexHash, const TArray<FVector3d>& InvariantFeatures, FFrame3d SymmetryFrameIn);
	void ComputeMeshInfo(const FDynamicMesh3* Mesh, const FAxisAlignedBox3d& Bounds, TArray<FVector3d>& InvariantFeaturesOut, FVector3d& MeshCentroidOut);
	bool Validate(const FDynamicMesh3* Mesh);

	//
	// Tolerances used for matching / hashing in symmetry-finding
	//
	
	// Point must be within this distance from the symmetry plane to be considered "on" the plane.
	// Note: Vertices can still be matched below this tolerance; this is just the distance at which a lack of match is considered a lack of symmetry
	// Note: Each Tolerance Factor will be multiplied by "ErrorScale", so that it can appropriately scale up for larger inputs.  SetErrorScale() must be called before using these tolerances.
	constexpr static double OnPlaneToleranceFactor = (double)FMathf::ZeroTolerance * .5;
	constexpr static double MatchVertexToleranceFactor = OnPlaneToleranceFactor * 2;
	// Note performance of vertex hashing is much better when VertexHashCellSize is large enough that most hash lookups only need to look at a single cell
	constexpr static double VertexHashCellSizeFactor = MatchVertexToleranceFactor * 10;
	// Note for features we specify a direct tolerance instead of a factor, as features are already in their own re-scaled space
	constexpr static double MatchFeaturesTolerance = UE_DOUBLE_KINDA_SMALL_NUMBER;
	// Internal error scale; for large meshes, we need larger tolerances because floating point values will be less accurate
	double ErrorScale = 1;

	void SetErrorScale(const FAxisAlignedBox3d& Bounds);

};



} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
