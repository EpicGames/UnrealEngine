// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Selections/MeshConnectedComponents.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{



/**
 * FMeshRegionGraph represents neighbourhood relationships between mesh patches. 
 * This is similar in some ways to a FGroupTopology, however only patch-adjacency
 * connections are tracked. Connectivity between patches can be queried, and
 * patches can be merged without rebuilding the graph.
 * 
 * Each mesh Patch/Region is assigned an integer ID/RegionIndex.
 * After removing or merging regions, some RegionIndex values may be invalid, use IsRegion() to check
 * 
 * An integer ID can be associated with each Region using the ExternalIDFunc passed to the various Build() functions.
 */
class FMeshRegionGraph
{
public:

	/**
	 * Build a region graph for a Mesh using the given ConnectedComponents
	 * @param ExternalIDFunc This allows the client to provide an integer ID that will be associated with each Region - called with the region index in Components
	 */
	UE_API void BuildFromComponents(const FDynamicMesh3& Mesh, 
							 const FMeshConnectedComponents& Components,
							 TFunctionRef<int32(int32)> ExternalIDFunc,
							 TFunctionRef<bool(int32, int32)> TrisConnectedPredicate = [](int, int) { return true; } );

	/**
	 * Build a region graph for a Mesh using the given triangle sets
	 * @param ExternalIDFunc This allows the client to provide an integer ID that will be associated with each Region - called with the region index in TriangleSets
	 */
	UE_API void BuildFromTriangleSets(const FDynamicMesh3& Mesh,
							 const TArray<TArray<int32>>& TriangleSets,
							 TFunctionRef<int32(int32)> ExternalIDFunc,
		                     TFunctionRef<bool(int32, int32)> TrisConnectedPredicate = [](int, int) { return true; });

	/** 
	 * @return Max valid region index. Some region indices may be invalid. 
	 */
	int32 MaxRegionIndex() const { return Regions.Num(); }

	/** 
	 * @return true if the RegionIndex is valid 
	 */
	int32 IsRegion(int32 RegionIdx) const { return RegionIdx >= 0 && RegionIdx < Regions.Num() && Regions[RegionIdx].bValid; }
	
	/** 
	 * @return the ExternalID for a Region, defined at Build time 
	 */
	int32 GetExternalID(int32 RegionIdx) const { return IsRegion(RegionIdx) ? Regions[RegionIdx].ExternalID : -1; }

	/** 
	 * @return the triangle count for a Region 
	 */
	int32 GetRegionTriCount(int32 RegionIdx) const { return IsRegion(RegionIdx) ? Regions[RegionIdx].Triangles.Num() : 0; }

	/** 
	 * @return list of triangles in a Region 
	 */
	const TArray<int32>& GetRegionTris(int32 RegionIdx) const { return IsRegion(RegionIdx) ? Regions[RegionIdx].Triangles : EmptyTriSet; }

	/** 
	 * @return list of Neigbour Region Indices for a Region 
	 */
	UE_API TArray<int32> GetNeighbours(int32 RegionIdx) const;

	/**
	 * @return true if the two regions are adjacent, ie they share an edge on at least one triangle
	 */
	UE_API bool AreRegionsConnected(int32 RegionAIndex, int32 RegionBIndex) const;

	/** @return true if the two triangles are in the same region */
	bool AreTrianglesConnected(int32 TriangleA, int32 TriangleB) const
	{
		return TriangleToRegionMap[TriangleA] == TriangleToRegionMap[TriangleB];
	}

	/**
	 * Incrementally merge regions with a triangle count below the given SmallThreshold.
	 * A Small region is merged to it's most-similar neighbour, as defined by RegionSimilarityFunc.
	 * The merge order is sorted by largest Similarity, ie most-similar regions are merged first.
	 */
	UE_API bool MergeSmallRegions(int32 SmallThreshold,
						   TFunctionRef<float(int32 SmallRgnIdx, int32 NbrRgnIdx)> RegionSimilarityFunc );

	/**
	 * Merge one existing Region into another
	 * @param RemoveRgnIdx this region will be removed
	 * @param MergeToRgnIdx this region will be added to
	 */
	UE_API bool MergeRegion(int32 RemoveRgnIdx, int32 MergeToRgnIdx);

	/**
	 * Optimize region borders by swapping triangles. The swap criteria is, if 2/3 of the
	 * neighbours of a given Triangle in region A are both in region B, swap it to region B.
	 * This swapping process is repeated in rounds, MaxRounds times.
	 * @return true if any swaps were applied
	 */
	UE_API bool OptimizeBorders(int32 MaxRounds = 25);


	/**
	 * @return list of triangles in a Region
	 * @warning This method leaves the FMeshRegionGraph in an invalid state, and should only be used as a means to avoid memory copies while extracting the triangle sets from a FMeshRegionGraph that will then be discarded
	 */
	TArray<int32>&& MoveRegionTris(int32 RegionIdx) { return MoveTemp(Regions[RegionIdx].Triangles); }



public:

	struct FNeighbour
	{
		int32 RegionIndex;
		int32 Count;
	};

	struct FRegion
	{
		bool bValid = true;

		int32 ExternalID;

		TArray<int32> Triangles;
		TArray<FNeighbour> Neighbours;

		bool bIsOnMeshBoundary = false;
		bool bIsOnROIBoundary = false;
	};

	TArray<FRegion> Regions;
	TArray<int32> TriangleToRegionMap;
	TArray<FIndex3i> TriangleNbrTris;

	TArray<int32> EmptyTriSet;

protected:
	UE_API void BuildNeigbours(int32 RegionIdx);
	
	UE_API bool MergeSmallRegionsPass(int32 SmallThreshold,
						   TFunctionRef<float(int32 SmallRgnIdx, int32 NbrRgnIdx)> RegionSimilarityFunc);

private:
	// MergeRegion without validation, for internal use where we know the inputs are valid neighbors
	bool MergeRegionUnsafe(int32 RemoveRgnIdx, int32 MergeToRgnIdx);

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
