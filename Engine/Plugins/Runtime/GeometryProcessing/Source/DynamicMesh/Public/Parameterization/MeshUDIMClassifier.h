// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
	namespace Geometry
	{

		struct FUVOverlayView;
		class FMeshConnectedComponents;

		/**
		* FDynamicMeshUDIMClassifier is a utility class for identifying active UDIMs from a FDynamicMesh's UV overlay
		 */
		class FDynamicMeshUDIMClassifier
		{
		public:			
			UE_API explicit FDynamicMeshUDIMClassifier(const FDynamicMeshUVOverlay* UVOverlay, TOptional<TArray<int32>> Selection = TOptional<TArray<int32>>());

			UE_API TArray<FVector2i> ActiveTiles() const;
			UE_API TArray<int32> TidsForTile(FVector2i TileIndexIn) const;

			static UE_API FVector2i ClassifyTrianglesToUDIM(const FDynamicMeshUVOverlay* UVOverlay, TArray<int32> Tids);
			static UE_API FVector2i ClassifyBoundingBoxToUDIM(const FDynamicMeshUVOverlay* UVOverlay, const FAxisAlignedBox2d& BoundingBox);
			static UE_API FVector2i ClassifyPointToUDIM(const FVector2f& UVPoint);

		protected:

			UE_API void ClassifyUDIMs();

			/** The UV Overlay to analyze for UDIMs */
			const FDynamicMeshUVOverlay* UVOverlay = nullptr;
			
			TOptional<TArray<int32>> Selection;
			
			TMap<FVector2i, TArray<int32> > UDIMs;
		};
	}
}

#undef UE_API
