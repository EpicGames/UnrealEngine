// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

struct FUVOverlayView;
class FMeshConnectedComponents;

/**
 * FDynamicMeshUVPacker implements various strategies for packing UV islands in a 
 * UV Overlay. The island topology and UV unwraps must already be created, this
 * class simply scales/rotates/translates the islands to fit.
 */
class FDynamicMeshUVPacker
{
public:
	/** The UV Overlay we will be repacking */
	FDynamicMeshUVOverlay* UVOverlay = nullptr;

	/** The explicit triangle ids to repack, repack all triangles if null */
	TUniquePtr<TArray<int32>> TidsToRepack;

	/** Resolution of the target texture. This is used to convert pixel gutter/border thickness to UV space */
	int32 TextureResolution = 512;

	/** Thickness of gutter/border in pixel dimensions. Not supported by all packing methods  */
	float GutterSize = 1.0;

	/** If true, original island scale is maintained during any packing process. */
	bool bPreserveScale = false;

	/** If true, original island rotation is maintained during any packing process. Automatically prevents bAllowFlips from applying, if set. */
	bool bPreserveRotation = false;

	/** If true, islands can be flipped in addition to rotate/translate/scale */
	bool bAllowFlips = false;

	/** Attempt to rescale islands to match texel-to-world-space ratio across islands, based on ratio of World- and UV-space edge lengths */
	bool bScaleIslandsByWorldSpaceTexelRatio = false;

	UE_API explicit FDynamicMeshUVPacker(FDynamicMeshUVOverlay* UVOverlay);
	UE_API explicit FDynamicMeshUVPacker(FDynamicMeshUVOverlay* UVOverlay, TUniquePtr<TArray<int32>>&& TidsToRepackIn);


	/**
	 * Standard UnrealEngine UV layout, similar to that used for Lightmap UVs. 
	 * All UV islands are packed into standard positive-unit-square.
	 * Only supports single-pixel border size.
	 */
	UE_API bool StandardPack();

	/**
	 * Uniformly scale all UV islands so that the largest fits in positive-unit-square,
	 * and translate each islands separately so that it's bbox-min is at the origin.
	 * So the islands are "stacked" and all fit in the unit box.
	 */
	UE_API bool StackPack();


protected:

	UE_API FMeshConnectedComponents CollectUVIslandsToPack(const FUVOverlayView& MeshView);

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
