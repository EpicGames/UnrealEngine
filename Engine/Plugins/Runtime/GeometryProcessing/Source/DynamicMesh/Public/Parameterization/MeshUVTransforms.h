// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"



/**
 * Utility functions for applying transformations to meshes
 */
namespace UE
{
	namespace MeshUVTransforms
	{
		using namespace UE::Geometry;

		enum class EIslandPositionType
		{
			CurrentPosition,
			MinBoxCornerToOrigin,
			CenterToOrigin
		};

		/**
		 * Recenter given UV element IDs using given IslandPositionType strategy (based on centroid, bounding-box, etc)
		 * and then apply uniform scaling 
		 */
		DYNAMICMESH_API void RecenterScale(FDynamicMeshUVOverlay* UVOverlay, const TArray<int32>& UVElementIDs,
			EIslandPositionType NewPosition, double UVScale);
		
		/**
		 * Fit given UV element IDs inside the specified Box. 
		 * @param bUniformScale if false, UVs will be non-uniformly scaled to maximally fill the box
		 */
		DYNAMICMESH_API void FitToBox(FDynamicMeshUVOverlay* UVOverlay, const TArray<int32>& UVElementIDs, const FAxisAlignedBox2d& Box, bool bUniformScale = true);

		/**
		 * Fit all UV element IDs of the UV Overlay inside the specified Box.
		 * @param bUniformScale if false, UVs will be non-uniformly scaled to maximally fill the box
		 */
		DYNAMICMESH_API void FitToBox(FDynamicMeshUVOverlay* UVOverlay, const FAxisAlignedBox2d& Box, bool bUniformScale = true);


		/**
		 * For all UV seam elements, apply a small displacement so that they have unique positions. 
		 * This allows unwelding/re-welding the UVs without breaking the seams (which is horrible but necessary in some cases)
		 */
		DYNAMICMESH_API void MakeSeamsDisjoint(FDynamicMeshUVOverlay* UVOverlay);
	}
}