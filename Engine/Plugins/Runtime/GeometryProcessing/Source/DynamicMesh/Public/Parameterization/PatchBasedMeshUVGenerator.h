// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Selections/MeshConnectedComponents.h"
#include "Polygroups/PolygroupSet.h"
#include "Util/ProgressCancel.h"

namespace UE
{
namespace Geometry
{


/**
 * FPatchBasedMeshUVGenerator is an automatic UV generator for a triangle mesh that, generally, works by
 * first decomposing the mesh into small patches for which a "known good" parameterization can be computed,
 * and then incrementally merging those patches into larger ones to create final UV islands.
 * 
 * The various steps of the generator can be called independently, eg if one already has a patch decomposition,
 * then the initial steps can be skipped
 * 
 */
class FPatchBasedMeshUVGenerator
{
public:

	//
	// If this is set, then the output patches will respect the borders of the
	// input polygroups, eg as if they were disconnected components.
	// Must be a PolygroupSet for the TargetMesh passed to functions below
	//
	FPolygroupSet* GroupConstraint = nullptr;


	//
	// Parameters for patch generation (ComputeInitialMeshPatches)
	//
	int32 TargetPatchCount = 100;
	bool bNormalWeightedPatches = true;
	double PatchNormalWeight = 1.0;
	int32 MinPatchSize = 2;

	//
	// Parameters for island merging (ComputeIslandsByRegionMerging)
	//
	double MergingThreshold = 1.5;
	double CompactnessThreshold = 9999999.0;		// effectively disabled as it usually is not a good idea
	double MaxNormalDeviationDeg = 45.0;

	//
	// ExpMap parameters (ComputeUVsFromTriangleSets)
	//
	int32 NormalSmoothingRounds = 0;
	double NormalSmoothingAlpha = 0.25;

	//
	// Packing parameters (used in AutoComputeUVs)
	//
	bool bAutoAlignPatches = true;
	bool bAutoPack = true;
	int32 PackingTextureResolution = 512;
	float PackingGutterWidth = 1.0f;


	/**
	 * Top-level driver function, automatically computes UVs on the given TargetMesh and stores in the
	 * given TargetUVOverlay, using the steps below, based on the parameters above
	 * @return result information, which may include warnings/etc
	 */
	DYNAMICMESH_API FGeometryResult AutoComputeUVs(
		FDynamicMesh3& TargetMesh,
		FDynamicMeshUVOverlay& TargetUVOverlay,
		FProgressCancel* Progress = nullptr);

	/**
	 * Compute a decomposition of TargetMesh into many small patches (driven by TargetPatchCount).
	 * @param InitialComponentsOut returned decomposition into triangle sets
	 * @return true on success
	 */
	DYNAMICMESH_API bool ComputeInitialMeshPatches(
		FDynamicMesh3& TargetMesh,
		FMeshConnectedComponents& InitialComponentsOut,
		FProgressCancel* Progress = nullptr);

	/**
	 * Incrementally combine existing mesh patches into larger patches that meet the various criteria defined
	 * by the parameters above.
	 * Note: This overload always outputs all islands (i.e., equivalent to calling the other overload with bOnlyOutputMergedRegions=false)
	 * 
	 * @param InitialComponents the initial mesh patches, ie triangle lists
	 * @param IslandsOut the set of output mesh patches / triangle lists
	 * @return false if catastrophic failure ocurred (ie IslandsOut is invalid) or if Progress was cancelled
	 */
	DYNAMICMESH_API bool ComputeIslandsByRegionMerging(
		FDynamicMesh3& TargetMesh,
		FDynamicMeshUVOverlay& TargetUVOverlay,
		const FMeshConnectedComponents& InitialComponents, 
		TArray<TArray<int32>>& IslandsOut,
		FProgressCancel* Progress = nullptr);

	/**
	 * Incrementally combine existing mesh patches into larger patches that meet the various criteria defined
	 * by the parameters above
	 * 
	 * @param InitialComponents the initial mesh patches, ie triangle lists
	 * @param bOnlyOutputMergedRegions whether to restrict IslandsOut to just the regions where components were merged, skipping unchanged islands
	 * @param IslandsOut the set of output mesh patches / triangle lists
	 * @return false if catastrophic failure ocurred (ie IslandsOut is invalid) or if Progress was cancelled
	 */
	DYNAMICMESH_API bool ComputeIslandsByRegionMerging(
		FDynamicMesh3& TargetMesh,
		FDynamicMeshUVOverlay& TargetUVOverlay,
		const FMeshConnectedComponents& InitialComponents,
		bool bOnlyOutputMergedRegions,
		TArray<TArray<int32>>& IslandsOut,
		FProgressCancel* Progress = nullptr);

	/**
	 * Calculate UVs for the input mesh patches of TargetMesh and store in TargetUVOVerlay
	 * @param TriangleSets the set of desired UV islands, defined as triangle lists
	 * @param bValidUVsComputedOut output list of booleans the same length as TriangleSets, set to false if UV computation failed for that island
	 * @return number of failed UV islands, 0 if all were successful, and -1 if Progress was cancelled
	 */
	DYNAMICMESH_API int32 ComputeUVsFromTriangleSets(
		FDynamicMesh3& TargetMesh,
		FDynamicMeshUVOverlay& TargetUVOverlay,
		const TArray<TArray<int32>>& TriangleSets,
		TArray<bool>& bValidUVsComputedOut,
		FProgressCancel* Progress = nullptr );

};


} // end namespace UE::Geometry
} // end namespace UE