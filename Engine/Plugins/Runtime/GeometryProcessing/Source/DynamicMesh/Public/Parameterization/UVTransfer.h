// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Spatial/PointHashGrid3.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h" // FDynamicMeshUVOverlay
#include "Templates/UniquePtr.h"

class FProgressCancel;

namespace UE::Geometry
{
class FDynamicMesh3;

/**
 * Transfers UVs from a low-resolution mesh to a high-resolution mesh. The source mesh is expected
 *  to be a simplified version of the destination mesh, simplified using existing vertices only. Thus,
 *  correspondences are found by position.
 */
class FDynamicMeshUVTransfer
{
public:
	FDynamicMeshUVTransfer(const FDynamicMesh3* SourceMeshIn, FDynamicMesh3* DestinationMeshIn,
		int32 UVLayerIndexIn)
		: FDynamicMeshUVTransfer(SourceMeshIn, DestinationMeshIn, UVLayerIndexIn, UVLayerIndexIn)
	{}

	DYNAMICMESH_API FDynamicMeshUVTransfer(const FDynamicMesh3* SourceMeshIn, FDynamicMesh3* DestinationMeshIn,
		int32 SourceUVLayerIndex, int32 DestUVLayerIndex);

	const FDynamicMesh3* SourceMesh = nullptr;
	FDynamicMesh3* DestinationMesh = nullptr;
	int UVLayerIndex = 0; // source UV layer index
	int DestUVLayerIndex = 0;

	/** How far to search for a matching vertex on the destination mesh. */
	double VertexSearchDistance = KINDA_SMALL_NUMBER;
	/** Cell size used in hash grid when finding correspndence (only affects performance) */
	double VertexSearchCellSize = VertexSearchDistance * 3.0;

	/** 
	 * Tuning parameter to make found destination paths follow source edges closer. The higher
	 *  the value, the more the path similarity metric gets weighed relative to simple euclidean
	 *  length of the path, which helps select a zigzagging path that follows the edge closer over
	 *  an "up and over" path that might have slightly shorter length (particularly relevant on
	 *  against-the-grain diagonals in a gridded region, where path lengths are manhattan distance-like).
	 */
	double PathSimilarityWeight = 200.0;

	/**
	 * If true, existing seams in the destination are removed before adding new ones.
	 */
	bool bClearExistingSeamsInDestination = true;

	DYNAMICMESH_API bool TransferSeams(FProgressCancel* Progress);
	DYNAMICMESH_API bool TransferSeamsAndUVs(FProgressCancel* Progress);
private:
	// TODO: Expose and test selection-constrained transfers when support for selecting elements on
	//  multiple meshes is available. Selection should be settable on either mesh (it is not necessary
	//  to set both).
	TSet<int32>* SourceSelectionTids = nullptr;
	TSet<int32>* DestinationSelectionTids = nullptr;

	void InitializeHashGrid();
	// Should only be called after InitializeHashGrid
	int32 GetCorrespondingDestVid(int32 SourceVid);

	void ResetDestinationUVTopology(FProgressCancel* Progress);
	bool PerformSeamTransfer(FProgressCancel* Progress);
	// Expects PerformSeamTransfer to have been called so that seams are transferred, and
	//  SourceEidToDestinationEndpointEidsVids and SourceBoundaryElements are initialized
	bool PerformElementsTransfer(FProgressCancel* Progress);

	const FDynamicMeshUVOverlay* SourceOverlay = nullptr;
	FDynamicMeshUVOverlay* DestOverlay = nullptr;

	// Used for getting a correspondence between low res mesh and high res mesh
	TUniquePtr<TPointHashGrid3<int32, double>> HashGrid;
	TMap<int32, int32> SourceVidToDestinationVid;

	TMap<int32, TPair<FIndex2i, FIndex2i>> SourceEidToDestinationEndpointEidsVids;
	TSet<int32> SourceBoundaryElements;

	// This gets multiplied by (SourceEdgeLength + PathSimilarityWeight * PathSimilarityMetric) to limit the
	//  max search distance when looking for corresponding paths. This is speculative, to avoid looking for inexistent
	//  paths on very dense meshes on incorrect inputs.
	double PathLengthToleranceMultiplier = 4.0;
	// Minimal max distance to use in looking for corresponding paths, in case the source edge is near-degenerate.
	double MinimalPathSearchDistance = 20.0;
};

}