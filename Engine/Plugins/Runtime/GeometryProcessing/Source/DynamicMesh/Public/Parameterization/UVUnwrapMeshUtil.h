// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h" //FDynamicMeshUVOverlay
#include "VectorTypes.h"

#include "GeometryBase.h"

PREDECLARE_GEOMETRY(class FDynamicMesh3);

namespace UE {
namespace Geometry {
namespace UVUnwrapMeshUtil {

	/** 
	 * Given a UV overlay, generate a mesh out of the overlay using the given UV to 3d position function.
	 * The resulting mesh's vertex IDs will match the overlay element IDs, and the triangle IDs will match
	 * the overlay triangle IDs. The mesh will have a UV overlay of its own that will match the source overlay
	 * except for the parent verts.
	 */
	DYNAMICMESH_API void GenerateUVUnwrapMesh(const FDynamicMeshUVOverlay& UVOverlay, FDynamicMesh3& UnwrapMeshOut,
		TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition);

	/**
	 * Given a mesh that corresponds to an overlay with an exact mapping of triangle IDs and element IDs to
	 * vertex IDs (i.e. one generated by GenerateUVUnwrapMesh), update the vertex locations and triangle
	 * connectivity of the mesh.
	 * 
	 * @param ChangedElementIDs If not null, only update vertex locations corresponding to the given elements.
	 * @param ChangedConnectivityTids If not null, only update triangle connectivity corresponding to the given triangle IDs.
	 */
	DYNAMICMESH_API void UpdateUVUnwrapMesh(const FDynamicMeshUVOverlay& UVOverlayIn,
		FDynamicMesh3& UnwrapMeshOut, TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition,
		const TArray<int32>* ChangedElementIDs = nullptr, const TArray<int32>* ChangedConnectivityTids = nullptr);

	/**
	 * Given a copy of a UV Unwrap mesh whose positions and connectivity have been updated, update the
	 * destination unwrap mesh. Note that SourceUnwrapMesh is expected to have its overlay already updated.
	 *
	 * @param ChangedVids If not null, only update elements corresponding to the given vertex IDs.
	 * @param ChangedConnectivityTids If not null, only update triangle connectivity corresponding to the given triangle IDs.
	 */
	DYNAMICMESH_API void UpdateUVUnwrapMesh(const FDynamicMesh3& SourceUnwrapMesh, FDynamicMesh3& DestUnwrapMesh,
		const TArray<int32>* ChangedVids = nullptr, const TArray<int32>* ChangedConnectivityTids = nullptr);

	/**
	 * Given a mesh that corresponds to an overlay with an exact mapping of triangle IDs and element IDs to
	 * vertex IDs (i.e. one generated by GenerateUVUnwrapMesh), update the element values and connectivity
	 * of the overlay.
	 * 
	 * @param ChangedVids If not null, only update elements corresponding to the given vertex IDs.
	 * @param ChangedConnectivityTids If not null, only update triangle connectivity corresponding to the given triangle IDs.
	 */
	DYNAMICMESH_API void UpdateUVOverlayFromUnwrapMesh(
		const FDynamicMesh3& UnwrapMeshIn, FDynamicMeshUVOverlay& UVOverlayOut,
		TFunctionRef<FVector2f(const FVector3d&)> VertPositionToUV,
		const TArray<int32>* ChangedVids = nullptr, const TArray<int32>* ChangedConnectivityTids = nullptr);

	/**
	 * Given two overlays that have the same triangle IDs, copy all or part of one overlay into another.
	 *
	 * @param bSameParentVerts If true, underlying meshes have same connectivity/id's, meaning that overlays
	 *   have same parent verts, and we can use a simple copy when updating everything. Can be set true when 
	 *   copying between a canonical/preview version of the same mesh, but must be false when copying between
	 *   an unwrap mesh and an applied mesh.
	 * @param ChangedVids If not null, only update elements corresponding to the given vertex IDs.
	 * @param ChangedConnectivityTids If not null, only update triangle connectivity corresponding to the given triangle IDs.
	 */
	DYNAMICMESH_API void UpdateOverlayFromOverlay(const FDynamicMeshUVOverlay& OverlayIn, FDynamicMeshUVOverlay& OverlayOut,
		bool bParentVertsIdentical, const TArray<int32>* ChangedElements = nullptr,
		const TArray<int32>* ChangedConnectivityTids = nullptr);

	/**
	 * Checks that the vertices in the unwrap correspond to elements in the overlay in both ID and value, and that
	 * set triangles in the overlay correspond to triangles in the unwrap.
	 * 
	 * The fail mode is a template parameter so that we can do compile time branching. The function will return on
	 * the first mismatch it finds.
	 * 
	 * @param Tolerance Used in comparing the expected unwrap position to the actual unwrap position for an element/vertex
	 */
	template <EValidityCheckFailMode FailMode = EValidityCheckFailMode::ReturnOnly>
	DYNAMICMESH_API bool DoesUnwrapMatchOverlay(const FDynamicMeshUVOverlay& OverlayIn, const FDynamicMesh3& UnwrapMeshIn,
		TFunctionRef<FVector3d(const FVector2f&)> UVToVertPosition, double Tolerance = 0);

}}}//end UE::Geometry::UVEditorToolUtil