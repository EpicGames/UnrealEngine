// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PolygroupSet.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
	
namespace Geometry
{
	
/**
 * ITriangleGroupSetAdapter allows tools that operate on triangle groups to also operate on arbitrary per-triangle values
 * by mapping unique values to integers.
 */
	
class UE_INTERNAL ITriangleGroupSetAdapter
{
public:
	/** Default Destructor */
	UE_API virtual ~ITriangleGroupSetAdapter() = 0;
	/** @return Mesh this GroupSetAdapter references. */
	const FDynamicMesh3* GetMesh() const;
	/** @return Group ID for a TriangleID */
	virtual int32 GetGroup(const int32 TriangleID) const = 0;
	/** Set the Group ID for a TriangleID */
	virtual void SetGroup(const int32 TriangleID, const int32 GroupID, FDynamicMesh3& WritableMesh) = 0;
	/**  */
	UE_API int32 GetMaxGroupID() const;
	
protected:
	/** Initialize a GroupSetAdapter for the given Mesh. */
	UE_API ITriangleGroupSetAdapter(const FDynamicMesh3* InMesh);
	/** The mesh this GroupSetAdapter references. */
	const FDynamicMesh3* Mesh = nullptr;
	/** Max Group ID. */
	int32 MaxGroupID = 0;
};

/**
 * FPolygroupLayerAdapter provides an interface to Get/Set group IDs per triangle using a FDynamicMeshPolygroupAttribute as the actual data storage.  
 */
class UE_INTERNAL FPolygroupLayerAdapter: public ITriangleGroupSetAdapter
{
public:
	/** Initialize a FPolygroupLayerAdapter for given Mesh and specific Polygroup attribute layer. */
	UE_API FPolygroupLayerAdapter(const FDynamicMesh3* InMesh, const FDynamicMeshPolygroupAttribute* Attribute);
	/** @return Group ID for a TriangleID. */
	virtual int32 GetGroup(const int32 TriangleID) const override;
	/** Set the Group ID for a TriangleID. */
	virtual void SetGroup(const int32 TriangleID, const int32 NewGroupID, FDynamicMesh3& WritableMesh) override;

protected:
	/** the PolygroupAttribute this GroupSetAdapter references. */
	FPolygroupSet PolygroupSet;
	/** @return index of current PolygroupAttribute into Mesh AttributeSet, or -1 if this information does not exist. */
	int32 GetLayerIndex() const;
	/** Default layer index. */
	int32 GroupLayerIndex = -1;
};

/**
 * FTriangleReadOnlyAttributeAdapter provides a read-only typed interface to Get group IDs per triangle (as typed values instead on integers)
 * using a TDynamicMeshSingleTriangleAttribute as the actual data storage and ensuring a group ID - AttribValueType value mapping.
 */
template<typename AttribValueType>
class UE_INTERNAL FTriangleReadOnlyAttributeAdapter: public ITriangleGroupSetAdapter
{
public:
	using TriangleAttributeType = TDynamicMeshSingleTriangleAttribute<AttribValueType>;
	using ValueType = AttribValueType;
	
	/** Initialize a FTriangleReadOnlyAttributeAdapter for given Mesh and specific triangle attribute. */
	FTriangleReadOnlyAttributeAdapter(const FDynamicMesh3* InMesh, const TriangleAttributeType* InAttribute)
		: ITriangleGroupSetAdapter(InMesh)
		, TriangleAttribute(InAttribute)
	{
		check(CheckValidity());
		
		AttribValueType DefaultValue = const_cast<TriangleAttributeType*>(InAttribute)->GetDefaultAttributeValue();
		ValueToID.Emplace(DefaultValue, MaxGroupID);
		IDToValue.Emplace(MaxGroupID, DefaultValue);
		MaxGroupID++;
		
		for (int32 TriangleID : Mesh->TriangleIndicesItr())
		{
			const ValueType Value = TriangleAttribute->GetValue(TriangleID);
			if (!ValueToID.Contains(Value))
			{
				ValueToID.Emplace(Value, MaxGroupID);
				IDToValue.Emplace(MaxGroupID, Value);
				MaxGroupID++;
			}
		}
	}
	
	/** @return Group ID for a TriangleID or INDEX_NONE if there's no value mapped to that Group ID. */
	virtual int32 GetGroup(const int32 TriangleID) const override 
	{
		const ValueType Value = TriangleAttribute->GetValue(TriangleID);
		const int32* GroupID = ValueToID.Find(Value);
		return GroupID ? *GroupID : INDEX_NONE;
	}
	
	/** Do not update any group id. */
	virtual void SetGroup(const int32 TriangleID, const int32 GroupID, FDynamicMesh3& WritableMesh) override {};
	
protected:

	/** @return true if this adapter is valid for use. */
	bool CheckValidity() const
	{
		return Mesh && TriangleAttribute &&
			TriangleAttribute->GetParentMesh() == Mesh &&
			TriangleAttribute->CheckValidity(true, EValidityCheckFailMode::ReturnOnly);
	}
	
	/** the per-triangle attribute this adapter references. */
	const TriangleAttributeType* TriangleAttribute = nullptr;
	/** Value-Group ID mapping. */
	TMap<ValueType, int32> ValueToID;
	/** Group ID-Value mapping. */
	TMap<int32, ValueType> IDToValue;
};
	
/**
 * FTriangleAttributeAdapter provides a typed interface to Get/Set group IDs per triangle (as typed values instead on integers)
 * using a TDynamicMeshSingleTriangleAttribute as the actual data storage and ensuring a group ID - AttribValueType value mapping.
 */
template<typename AttribValueType>
class UE_INTERNAL FTriangleAttributeAdapter: public ITriangleGroupSetAdapter
{
public:
	using TriangleAttributeType = TDynamicMeshSingleTriangleAttribute<AttribValueType>;
	using ValueType = AttribValueType;
	
	/** Initialize a FTriangleAttributeAdapter for given Mesh and specific triangle attribute. */
	FTriangleAttributeAdapter(const FDynamicMesh3* InMesh, TriangleAttributeType* InAttribute)
		: ITriangleGroupSetAdapter(InMesh)
		, TriangleAttribute(InAttribute)
	{
		check(CheckValidity());
	
		AttribValueType DefaultValue = InAttribute->GetDefaultAttributeValue();
		ValueToID.Emplace(DefaultValue, MaxGroupID);
		IDToValue.Emplace(MaxGroupID, DefaultValue);
		MaxGroupID++;
		
		for (int32 TriangleID : Mesh->TriangleIndicesItr())
		{
			const ValueType Value = GetValue(TriangleID);
			if (!ValueToID.Contains(Value))
			{
				ValueToID.Emplace(Value, MaxGroupID);
				IDToValue.Emplace(MaxGroupID, Value);
				MaxGroupID++;
			}
		}
	}
	
	FTriangleAttributeAdapter(const FTriangleAttributeAdapter* InCopy)
		: ITriangleGroupSetAdapter(InCopy->Mesh)
		, TriangleAttribute(InCopy->TriangleAttribute)
		, ValueToID(InCopy->ValueToID)
		, IDToValue(InCopy->IDToValue)
	{
		MaxGroupID = InCopy->MaxGroupID;
	}

	/** Set the Group ID for a TriangleID and updates the value-id mappings. */
	void SetGroup(const int32 TriangleID, const int32 GroupID)
	{
		const ValueType* Value = IDToValue.Find(GroupID);
		if (Value)
		{
			SetValue(TriangleID, *Value);
		}
		else
		{
			ValueType NewValue = GetUniqueValue();
			ensure(!ValueToID.Contains(NewValue));
			ValueToID.Emplace(NewValue, GroupID);
			IDToValue.Emplace(GroupID, NewValue);
			SetValue(TriangleID, NewValue);
			MaxGroupID = FMath::Max(MaxGroupID+1, GroupID+1);
		}
	}
	
	/** Set the Group ID for a TriangleID and updates the value-id mappings. InMesh can be used to perform additional tests. */
	virtual void SetGroup(const int32 TriangleID, const int32 GroupID, FDynamicMesh3& InMesh) override
	{
		SetGroup(TriangleID, GroupID);
	}
	
	/** @return Group ID for a TriangleID or INDEX_NONE if there's no value mapped to that Group ID. */
	virtual int32 GetGroup(const int32 TriangleID) const override 
	{
		const int32* GroupID = ValueToID.Find(GetValue(TriangleID));
		if (ensure(GroupID))
		{
			return *GroupID;
		}
		return INDEX_NONE;
	}
	
	/** @return Group ID from a Value. If the value is not yet registered, a new group ID is added and the mapping created. */
	int32 GetGroupFromValue(const ValueType Value)
	{
		if (const int32* GroupID = ValueToID.Find(Value))
		{
			return *GroupID;
		}
		
		const int32 NewGroupID = MaxGroupID;
		ValueToID.Emplace(Value, MaxGroupID);
		IDToValue.Emplace(MaxGroupID, Value);
		MaxGroupID++;
		
		return NewGroupID;
	}
	
	/** @return Value from a Group ID. If the group ID is not yet registered, a new value is added and the mapping is created. */
	ValueType GetValueFromGroup(const int32 GroupID)
	{
		if (const ValueType* Value = IDToValue.Find(GroupID))
		{
			return *Value;
		}
		
		ValueType NewValue = GetUniqueValue();
		ensure(!ValueToID.Contains(NewValue));
		ValueToID.Emplace(NewValue, GroupID);
		IDToValue.Emplace(GroupID, NewValue);
		MaxGroupID = FMath::Max(MaxGroupID+1, GroupID+1);
		return NewValue;
	}
	
	/** @return Value for a TriangleID. */
	ValueType GetValue(const int32 TriangleID) const
	{
		return TriangleAttribute->GetValue(TriangleID);
	}
	
	/** Set the Value for a TriangleID and register mapping if needed. */
	void SetValue(const int32 TriangleID, ValueType NewValue)
	{
		if (!ValueToID.Contains(NewValue))
		{
			ensure(!IDToValue.Contains(MaxGroupID));
			ValueToID.Emplace(NewValue, MaxGroupID);
			IDToValue.Emplace(MaxGroupID, NewValue);
			MaxGroupID++;
		}
		return TriangleAttribute->SetValue(TriangleID, NewValue);
	}
	
protected:
	
	/** @return New unique value. */
	virtual ValueType GetUniqueValue() const
	{
		ensure(false);
		return TriangleAttribute->GetDefaultAttributeValue();
	}

	/** @return true if this adapter is valid for use. */
	bool CheckValidity() const
	{
		return Mesh && TriangleAttribute &&
			TriangleAttribute->GetParentMesh() == Mesh &&
			TriangleAttribute->CheckValidity(true, EValidityCheckFailMode::ReturnOnly);
	}
	
	/** the per-triangle attribute this adapter references. */
	TriangleAttributeType* TriangleAttribute = nullptr;
	/** Value-Group ID mapping. */
	TMap<ValueType, int32> ValueToID;
	/** Group ID-Value mapping. */
	TMap<int32, ValueType> IDToValue;
};
	
/**
 * FTriangleLabelAdapter provides a FTriangleAttributeAdapter interface specialized for FName. The actual attribute that this adapter references
 * must be contained in FDynamicMeshAttributeSet::TriangleLabelAttributes. (this is actually checked in the constructor).
 */
class UE_INTERNAL FTriangleLabelAdapter: public FTriangleAttributeAdapter<FName>
{
public:
	UE_API FTriangleLabelAdapter(const FDynamicMesh3* InMesh, TriangleAttributeType* InAttribute, const FName InBaseLabel = "label");
	UE_API FTriangleLabelAdapter(const FTriangleLabelAdapter* InCopy);
	UE_API FName GetLayerName() const;
protected:
	/**  */
	FName LayerName = NAME_None;
	/** @return New unique FName. */
	UE_API virtual FName GetUniqueValue() const override;
	/** Base name used for unique FName generation. */
	FName BaseLabel = "label";
};

/**
 * FTriangleReadOnlyLabelAdapter provides a read-only FTriangleAttributeAdapter interface specialized for FName.
 */
class UE_INTERNAL FTriangleReadOnlyLabelAdapter: public FTriangleReadOnlyAttributeAdapter<FName>
{
public:
	UE_API FTriangleReadOnlyLabelAdapter(const FDynamicMesh3* InMesh, const TriangleAttributeType* InAttribute);
};
	
}	// end namespace Geometry
}	// end namespace UE

#undef UE_API
