// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

/**
 * FPolygroupLayer represents a polygroup set on a FDynamicMesh3, which supports a "default"
 * group set stored on the mesh, and then N extended group layers stored in the mesh AttributeSet.
 * This struct can represent either.
 */
struct FPolygroupLayer
{
	/** If true, layer is the default FDynamicMesh3 triangle groups layer */
	bool bIsDefaultLayer = true;
	/** If bIsDefaultLayer is false, this is the index of the AttributeSet Polygroup Layer */
	int32 LayerIndex = -1;

	/** Construct a FPolygroupLayer for the default layer */
	static FPolygroupLayer Default() { return FPolygroupLayer{ true, -1 }; }
	/** Construct a FPolygroupLayer for an extended layer */
	static FPolygroupLayer Layer(int32 Index) { return FPolygroupLayer{ false, Index }; }

	bool operator==(const FPolygroupLayer& OtherLayer) const 
	{
		if (bIsDefaultLayer || OtherLayer.bIsDefaultLayer)
		{
			return bIsDefaultLayer && OtherLayer.bIsDefaultLayer;
		}
		else
		{
			return LayerIndex == OtherLayer.LayerIndex;
		}
	}

	/** @return true if the specified layer (default or extended) exist and is initialized on the given Mesh */
	UE_API bool CheckExists(const FDynamicMesh3* Mesh) const;

	/** Enable the specified layer on the given Mesh, if it's not already there */
	UE_API void EnableOnMesh(FDynamicMesh3& Mesh) const;
};


/**
 * Polygroup sets can be stored in multiple places. The default location is in the per-triangle group integer stored
 * directly on a FDynamicMesh3. Additional layers may be stored in the FDynamicMeshAttributeSet. Future iterations
 * could store packed polygroups in other places, store them in separate arrays, and so on.
 * FPolygroupSet can be used to abstract these different cases, by providing a standard Polygroup Get/Set API.
 * 
 * To support unique Polygroup ID allocation, FPolygroupSet calculates the maximum GroupID on creation, and 
 * updates this maximum across SetGroup() calls. AllocateNewGroupID() can be used to provide new unused GroupIDs.
 * For consistency with FDynamicMesh3, MaxGroupID is set such that all GroupIDs are less than MaxGroupID
 *
 */
struct FPolygroupSet
{
	const FDynamicMesh3* Mesh = nullptr;
	const FDynamicMeshPolygroupAttribute* PolygroupAttrib = nullptr;
	int32 GroupLayerIndex = -1;
	int32 MaxGroupID = 0; // Note: all group IDs are less than MaxGroupID

	/** Initialize a PolygroupSet for the given Mesh, and standard triangle group layer */
	UE_API explicit FPolygroupSet(const FDynamicMesh3* MeshIn);

	/** Initialize a PolygroupSet for the given Mesh, and standard triangle group layer */
	UE_API explicit FPolygroupSet(const FDynamicMesh3* MeshIn, FPolygroupLayer GroupLayer);

	/** Initialize a PolygroupSet for given Mesh and specific Polygroup attribute layer */
	UE_API explicit FPolygroupSet(const FDynamicMesh3* MeshIn, const FDynamicMeshPolygroupAttribute* PolygroupAttribIn);

	/** Initialize a PolygroupSet for given Mesh and specific Polygroup attribute layer, found by index. If not valid, fall back to standard triangle group layer. */
	UE_API explicit FPolygroupSet(const FDynamicMesh3* MeshIn, int32 PolygroupLayerIndex);

	/** Initialize a PolygroupSet for given Mesh and specific Polygroup attribute layer, found by name. If not valid, fall back to standard triangle group layer. */
	UE_API explicit FPolygroupSet(const FDynamicMesh3* MeshIn, FName AttribName);

	/** Initialize a PolygroupSet by copying an existing PolygroupSet */
	UE_API explicit FPolygroupSet(const FPolygroupSet* CopyIn);


	/** @return Mesh this PolygroupSet references  */
	const FDynamicMesh3* GetMesh() { return Mesh; }

	/** @return PolygroupAttribute this PolygroupSet references, or null if no PolygroupAttribute is in use */
	const FDynamicMeshPolygroupAttribute* GetPolygroup() { return PolygroupAttrib; }

	/** @return index of current PolygroupAttribute into Mesh AttributeSet, or -1 if this information does not exist */
	int32 GetPolygroupIndex() const { return GroupLayerIndex; }


	/**
	 * @return PolygroupID for a TriangleID
	 */
	int32 GetGroup(int32 TriangleID) const
	{
		return (PolygroupAttrib) ? PolygroupAttrib->GetValue(TriangleID) : Mesh->GetTriangleGroup(TriangleID);
	}

	/**
	 * @return PolygroupID for a TriangleID
	 */
	int32 GetTriangleGroup(int32 TriangleID) const
	{
		return (PolygroupAttrib) ? PolygroupAttrib->GetValue(TriangleID) : Mesh->GetTriangleGroup(TriangleID);
	}

	/**
	 * Set the PolygroupID for a TriangleID
	 */
	void SetGroup(int32 TriangleID, int32 NewGroupID, FDynamicMesh3& WritableMesh)
	{
		checkSlow(&WritableMesh == this->Mesh);		// require the same mesh
		if (WritableMesh.IsTriangle(TriangleID))
		{
			if (PolygroupAttrib)
			{
				FDynamicMeshPolygroupAttribute* WritableGroupAttrib = WritableMesh.Attributes()->GetPolygroupLayer(GroupLayerIndex);
				checkSlow(WritableGroupAttrib == PolygroupAttrib)
				WritableGroupAttrib->SetValue(TriangleID, NewGroupID);
			}
			else
			{
				WritableMesh.SetTriangleGroup(TriangleID, NewGroupID);
			}
		}
		MaxGroupID = FMath::Max(MaxGroupID, NewGroupID + 1);
	}

	/**
	 * Calculate the current maximum PolygroupID used in the active set and store in MaxGroupID member
	 */
	UE_API void RecalculateMaxGroupID();

	/**
	 * Allocate a new unused PolygroupID by incrementing the MaxGroupID member
	 */
	int32 AllocateNewGroupID()
	{
		return MaxGroupID++;
	}
};






}	// end namespace Geometry
}	// end namespace UE

#undef UE_API
