// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"


namespace UE
{
namespace Geometry
{

/**
 * Find the first FDynamicMeshPolygroupAttribute with the given FName in the AttributeSet of a Mesh.
 * @return nullptr if no Polygroup layer is found
 */
DYNAMICMESH_API FDynamicMeshPolygroupAttribute* FindPolygroupLayerByName(FDynamicMesh3& Mesh, FName Name);

/**
 * Find the first FDynamicMeshPolygroupAttribute with the given FName in the AttributeSet of a Mesh.
 * @return nullptr if no Polygroup layer is found
 */
DYNAMICMESH_API const FDynamicMeshPolygroupAttribute* FindPolygroupLayerByName(const FDynamicMesh3& Mesh, FName Name);

/**
 * Find the FDynamicMeshTriangleLabelAttribute with the given FName in the AttributeSet of a Mesh.
 * @return nullptr if no triangle label layer is found.
 */
DYNAMICMESH_API FDynamicMeshTriangleLabelAttribute* FindTriangleLabelLayerByName(FDynamicMesh3& Mesh, const FName Name);

/**
 * Find the FDynamicMeshTriangleLabelAttribute with the given FName in the AttributeSet of a Mesh.
 * @return nullptr if no triangle label layer is found.
 */
DYNAMICMESH_API const FDynamicMeshTriangleLabelAttribute* FindTriangleLabelLayerByName(const FDynamicMesh3& Mesh, FName Name);
	
/**
 * @return index of the first Layer with the given FName in Mesh AttributeSet, or -1 if not found
 */
DYNAMICMESH_API int32 FindPolygroupLayerIndexByName(const FDynamicMesh3& Mesh, FName Name);

/**
 * @return index of Layer in Mesh AttributeSet, or -1 if not found
 */
DYNAMICMESH_API int32 FindPolygroupLayerIndex(const FDynamicMesh3& Mesh, const FDynamicMeshPolygroupAttribute* Layer);

/**
 * @return Compute the upper bound of the group IDs for the given layer (one greater than the highest group ID)
 */
DYNAMICMESH_API int32 ComputeGroupIDBound(const FDynamicMesh3& Mesh, const FDynamicMeshPolygroupAttribute* Layer);

/**
* @return unique group layer name derived from BaseName by appending a numeric suffix in the form _0, _1, etc
*/
DYNAMICMESH_API FString MakeUniqueGroupLayerName(const FDynamicMesh3& Mesh, FString BaseName);

}	// end namespace Geometry
}	// end namespace UE