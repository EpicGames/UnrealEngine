// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Polygroups/PolygroupSet.h"

#define UE_API DYNAMICMESH_API

using UE::Geometry::FDynamicMesh3;


namespace UE
{
namespace Geometry
{

/**
 * FPolygroupsGenerator generates (face/tri/poly)groups for an input mesh based on the geometry and attributes of the input mesh.
 * The various FindPolygroupsFromX() are the driving functions, each performs the full computation. 
 */
class FPolygroupsGenerator
{
public:

	FPolygroupsGenerator() {}
	UE_API FPolygroupsGenerator(FDynamicMesh3* MeshIn);

	// 
	// Input parameters
	//

	// source mesh
	FDynamicMesh3* Mesh = nullptr;

	// if true, then groups will be post-processed to optimize them, based on parmeters below
	bool bApplyPostProcessing = true;

	// if > 1, groups with a triangle count smaller than this will be merged with a neighbouring group
	int32 MinGroupSize = 0;

	// if true, after groups are computed they will be copied to the output mesh
	bool bCopyToMesh = true;

	// start at this GroupID when assigning new groups
	int32 InitialGroupID = 0;

	//
	// Outputs
	//

	// lists of triangle IDs, each list defines a polygroup/polygon
	TArray<TArray<int>> FoundPolygroups;

	// list of edge IDs of mesh edges that are on polygroup borders
	TArray<int> PolygroupEdges;


	/**
	 * Find Polygroups by randomly picking initial seed triangles and then flood-filling outwards,
	 * stopping when the opening angle at an edge is larger than the angle defined by the OneMinusCosAngleTolerance.
	 */
	UE_API bool FindPolygroupsFromFaceNormals(
		double OneMinusCosAngleTolerance = 0.0001,
		bool bRespectUVSeams = false,
		bool bRespectNormalSeams = false,
		bool bUseAveragePolygroupNormals = false);

	/**
	 * Find Polygroups based on UV Islands, ie each UV Island becomes a Polygroup
	 */
	UE_API bool FindPolygroupsFromUVIslands(int32 UVLayer = 0);

	/**
	 * Find PolyGroups based on Material IDs -- i.e., connected groups of triangles with the same Material ID become a PolyGroup
	 */
	UE_API bool FindPolygroupsFromMaterialIDs();

	/**
	* Find Polygroups based on Seams in UV Overlay
	*/
	UE_API bool FindPolygroupsFromHardNormalSeams();

	/**
	 * Find PolyGroups based on mesh connectivity and optionally a custom indicator function
	 * @param AreTrisConnected Optional custom function: Takes neighboring triangle IDs and returns true if they should be in the same PolyGroup
	 * @return true if at least one polygroup was found
	 */
	UE_API bool FindPolygroupsFromConnectedTris(TFunction<bool(int32 TriA, int32 TriB)> AreTrisConnected = nullptr);

	/**
	* Find Polygroups by trying to invert triangluation of a polygon mesh
	*/
	UE_API bool FindSourceMeshPolygonPolygroups(
		bool bRespectUVSeams = true,
		bool bRespectNormalSeams = false,
		double QuadAdjacencyWeight = 1.0,
		double QuadMetricClamp = 1.0,
		int MaxSearchRounds = 1
	);

	/**
	 * Weight potions for algorithms below
	 */
	enum class EWeightingType
	{
		None,
		NormalDeviation
	};

	/**
	 * Incrementally compute approximate-geodesic-based furthest-point sampling of the mesh until NumPoints
	 * samples have been found, then compute local geodesic patches (eg approximate surface voronoi diagaram).
	 * Optionally weight geodesic-distance computation, which will produce different patch shapes.
	 * If StartingGroups are provided, then new groups are constrained to subdivide existing groups, ie
	 * none of the new groups will cross the boundaries of the StartingGroups
	 */
	UE_API bool FindPolygroupsFromFurthestPointSampling(
		int32 NumPoints, 
		EWeightingType WeightingType, 
		FVector3d WeightingCoeffs = FVector3d::One(),
		FPolygroupSet* StartingGroups = nullptr);

	/**
	*
	* Similar to the technique used in FindPolygroupsFromFurthestPointSampling, except this algorithm
	* attempts to select sample points in connected mesh regions based on the rough area of each region,
	* granting more initial points to larger regions and fewer points to smaller regions, minium of one.
	* This approach handles cases better when there is a large varience in region sizes and a large number
	* of small regions compared to large regions. In this case, the alternative routine can over focus on
	* small regions, granting the larger regions too few points. This leads to a relative over sampling of
	* small regions and creates large polygroups out of the larger regions instead of more equitably breaking
	* up larger regions into multiple groups. 
	* 
	* In this algorithm, NumPoints is less a strict target of final patches, but a rough guideline to direct
	* the area sampler on how much total area each sample should occupy. E.g. a value of 100 would indicate
	* we ideally want 100 equal sized regions of the mesh. However, since each connected region must have at
	* least one sample, the final count may not be exactly NumPoints. Instead, each region, and it's area, is
	* considered independently according to this desired "sample density". A "perfect" mesh (with one connected
	* region and a high tesselation of equal sized triangles) should generate equal sized patches of NumPoints
	* in count. A poorer quality mesh will result in a best effort take at this, while ensuring all triangles
	* are part of a group.
	* 
	*/
	UE_API bool FindPolygroupsFromAreaDensityPointSampling(
		int32 NumPoints,
		EWeightingType WeightingType,
		FVector3d WeightingCoeffs = FVector3d::One(),
		FPolygroupSet* StartingGroups = nullptr);

	/**
	* Copy the computed Polygroups to the input Mesh. Will be automatically called by the 
	* FindPolygroupsFromX() functions above if bCopyToMesh=true
	*/
	UE_API void CopyPolygroupsToMesh();

	/**
	* Copy the computed Polygroups to the given PolygroupSet and Mesh
	*/
	UE_API void CopyPolygroupsToPolygroupSet(FPolygroupSet& Polygroups, FDynamicMesh3& TargetMesh);

	/**
	 * Initialize the PolygroupEdges output member by finding all the mesh edges that are on polygroup borders.
	 * Requires that polygroups have been computed and copied to the mesh
	 */
	UE_API bool FindPolygroupEdges();


protected:
	UE_API void PostProcessPolygroups(bool bApplyMerging, TFunctionRef<bool(int32, int32)> TrisConnectedPredicate = [](int, int) { return true; });
	UE_API void OptimizePolygroups(TFunctionRef<bool(int32, int32)> TrisConnectedPredicate = [](int, int) { return true; });

	UE_API void GetSeamConstraintEdges(bool bUVSeams, bool bNormalSeams, TSet<int32>& ConstraintEdgesOut) const;
};


}	// end namespace Geometry
}	// end namespace UE

#undef UE_API
