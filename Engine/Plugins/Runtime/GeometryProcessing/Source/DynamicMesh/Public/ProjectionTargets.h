// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Port of geometry3cpp ProjectionTargets

#include "DynamicMesh/DynamicMeshAABBTree3.h"


namespace UE
{
namespace Geometry
{


/**
 * FMeshProjectionTarget provides an IProjectionTarget interface to a FDynamicMesh + FDynamicMeshAABBTree3
 * Use to project points to mesh surface.
 */
class FMeshProjectionTarget : public IOrientedProjectionTarget
{
public:
	/** The mesh to project onto */
	const FDynamicMesh3* Mesh = nullptr;
	/** An AABBTree for Mesh */
	FDynamicMeshAABBTree3* Spatial = nullptr;

	virtual ~FMeshProjectionTarget() = default;
	FMeshProjectionTarget() = default;

	FMeshProjectionTarget(const FDynamicMesh3* MeshIn, FDynamicMeshAABBTree3* SpatialIn)
	{
		Mesh = MeshIn;
		Spatial = SpatialIn;
	}


	/**
	 * @return Projection of Point onto this target
	 */
	DYNAMICMESH_API virtual FVector3d Project(const FVector3d& Point, int Identifier = -1) override;

	/**
	 * @return Projection of Point onto this target, and set ProjectNormalOut to the triangle normal at the returned point (*not* interpolated vertex normal)
	 */
	DYNAMICMESH_API virtual FVector3d Project(const FVector3d& Point, FVector3d& ProjectNormalOut, int Identifier = -1) override;


};


class FWorldSpaceProjectionTarget : public FMeshProjectionTarget
{
public:

	virtual ~FWorldSpaceProjectionTarget() = default;
	FWorldSpaceProjectionTarget() = default;

	FWorldSpaceProjectionTarget(const FDynamicMesh3* MeshIn, 
								FDynamicMeshAABBTree3* SpatialIn,
								const FTransformSRT3d& TargetMeshLocalToWorldIn,
								const FTransformSRT3d& ToolMeshLocalToWorldIn) :
		FMeshProjectionTarget(MeshIn, SpatialIn),
		TargetMeshLocalToWorld(TargetMeshLocalToWorldIn),
		ToolMeshLocalToWorld(ToolMeshLocalToWorldIn)
	{}

	/**
	 * @return Projection of Point onto this target
	*/
	virtual FVector3d Project(const FVector3d& Point, int Identifier = -1) override
	{
		// Put query point in Target space
		FVector3d TransformedPoint = Point;
		TransformedPoint = ToolMeshLocalToWorld.TransformPosition(TransformedPoint);
		TransformedPoint = TargetMeshLocalToWorld.InverseTransformPosition(TransformedPoint);

		FVector3d ProjectedPosition = FMeshProjectionTarget::Project(TransformedPoint, Identifier);

		// ProjectedPosition is in Target space, put it in Tool space
		ProjectedPosition = TargetMeshLocalToWorld.TransformPosition(ProjectedPosition);
		ProjectedPosition = ToolMeshLocalToWorld.InverseTransformPosition(ProjectedPosition);

		return ProjectedPosition;
	}

	/**
	 * @return Projection of Point onto this target, and set ProjectNormalOut to the triangle normal at the returned point (*not* interpolated vertex normal)
	 */
	virtual FVector3d Project(const FVector3d& Point, FVector3d& ProjectNormalOut, int Identifier = -1) override
	{
		// Put query point in Target space
		FVector3d TransformedPoint = Point;
		TransformedPoint = ToolMeshLocalToWorld.TransformPosition(TransformedPoint);
		TransformedPoint = TargetMeshLocalToWorld.InverseTransformPosition(TransformedPoint);

		FVector3d ProjectedPosition = FMeshProjectionTarget::Project(TransformedPoint, ProjectNormalOut, Identifier);

		// ProjectedPosition and ProjectedNormal are in Target space, put them in Tool space
		ProjectedPosition = TargetMeshLocalToWorld.TransformPosition(ProjectedPosition);
		ProjectedPosition = ToolMeshLocalToWorld.InverseTransformPosition(ProjectedPosition);

		ProjectNormalOut = TargetMeshLocalToWorld.TransformNormal(ProjectNormalOut);
		ProjectNormalOut = ToolMeshLocalToWorld.InverseTransformNormal(ProjectNormalOut);

		return ProjectedPosition;
	}

protected:
	FTransformSRT3d TargetMeshLocalToWorld;
	FTransformSRT3d ToolMeshLocalToWorld;

};



} // end namespace UE::Geometry
} // end namespace UE
