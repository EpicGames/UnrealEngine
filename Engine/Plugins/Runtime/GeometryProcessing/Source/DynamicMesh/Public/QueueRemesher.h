// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Remesher.h"


namespace UE
{
namespace Geometry
{


/**
 *
 * Extension to Remesher that is smarter about which edges/vertices to touch:
 *  - A buffer tracks edges that were affected on last pass, and hence might need to be updated
 *  - FastSplitIteration() just does splits, to reach target edge length as quickly as possible
 *  - RemeshIteration() applies remesh pass for modified edges
 *  - TrackedFullSmoothPass_Buffer() smooths all vertices but only adds to queue if edge changes enough
 *  - TrackedFullProjectionPass() projects all vertices but only adds to queue if edge changes enough
 * 
 */
class FQueueRemesher : public FRemesher
{

public:

	FQueueRemesher(FDynamicMesh3* MeshIn) : FRemesher(MeshIn) {}

	/// Max number of passes just doing edge splits
	int MaxFastSplitIterations = 20;

	/// Max number of passes doing full remeshing operations
	int MaxRemeshIterations = 20;

	/// If fraction of active edges falls below this threshold, consider result converged and terminate before MaxRemeshIterations
	double MinActiveEdgeFraction = 0.01;

	/// Converge on remeshed result as quickly as possible
	DYNAMICMESH_API void FastestRemesh();

	/// "Outer loop" for all remeshing operations
	void BasicRemeshPass() override
	{
		FastestRemesh();
	}

protected:

	// Note: FRemesher has this: virtual void FullProjectionPass();
	DYNAMICMESH_API virtual void TrackedFullProjectionPass(bool bParallel);

	// Set of edges that have been modified during a given iteration.
	// Before the first iteration, this Optional is unset, signaling that the first iteration should process all edges
	// in the mesh.  Subsequent iterations will only operate on this set of edges.
	TOptional<TSet<int>> ModifiedEdges;

	// Persistent buffer of edges to be processed. This buffer is filled then iterated over.
	TSet<int> EdgeBuffer;

	// The function called after an edge split occurs. This function will enqueue edges in ModifiedEdges.
	TFunction<void(int, int, int, int)> PostEdgeSplitFunction;

	// Call the function above to enqueue edges in the ModifiedEdges set
	void OnEdgeSplit(int EdgeID, int VertexA, int VertexB, const FDynamicMesh3::FEdgeSplitInfo& SplitInfo) final
	{
		if (PostEdgeSplitFunction)
		{
			PostEdgeSplitFunction(EdgeID, VertexA, VertexB, SplitInfo.NewVertex);
		}
	}

	// We occasionally want to override the user-specified choices of edge operations to achieve some sub-step of a
	// full remesh algorithm. This struct encapsulates the Remesher settings so they can be saved, overwritten, and
	// restored.
	struct SettingState
	{
		bool bEnableFlips;
		bool bEnableCollapses;
		bool bEnableSplits;
		bool bEnableSmoothing;
		double MinEdgeLength;
		double MaxEdgeLength;
		double SmoothSpeedT;
		ESmoothTypes SmoothType;
		ETargetProjectionMode ProjectionMode;
	};

	SettingState SavedState;

	void PushState()
	{
		SavedState.bEnableFlips = bEnableFlips;
		SavedState.bEnableCollapses = bEnableCollapses;
		SavedState.bEnableSplits = bEnableSplits;
		SavedState.bEnableSmoothing = bEnableSmoothing;
		SavedState.MinEdgeLength = MinEdgeLength;
		SavedState.MaxEdgeLength = MaxEdgeLength;
		SavedState.SmoothSpeedT = SmoothSpeedT;
		SavedState.SmoothType = SmoothType;
		SavedState.ProjectionMode = ProjectionMode;
	}

	void PopState()
	{
		bEnableFlips = SavedState.bEnableFlips;
		bEnableCollapses = SavedState.bEnableCollapses;
		bEnableSplits = SavedState.bEnableSplits;
		bEnableSmoothing = SavedState.bEnableSmoothing;
		MinEdgeLength = SavedState.MinEdgeLength;
		MaxEdgeLength = SavedState.MaxEdgeLength;
		SmoothSpeedT = SavedState.SmoothSpeedT;
		SmoothType = SavedState.SmoothType;
		ProjectionMode = SavedState.ProjectionMode;
	}


	void QueueOneRing(int vid)
	{
		check(ModifiedEdges);
		if (Mesh->IsVertex(vid))
		{
			for (int eid : Mesh->VtxEdgesItr(vid))
			{
				ModifiedEdges->Add(eid);
			}
		}
	}

	void QueueEdge(int eid)
	{
		check(ModifiedEdges);
		ModifiedEdges->Add(eid);
	}

	void ResetQueue() { ModifiedEdges.Reset(); }

	
	/// This pass only does edge splits and enqueues modified vertex neighborhoods.
	/// @return Number of split edges.
	DYNAMICMESH_API int FastSplitIteration();

	/// This pass does all edge operations plug smoothing and reprojection.
	/// EarlyTerminationCheck will be called after remesh operations, before smoothing/reprojection. Caller can return true to abort further computation.
	DYNAMICMESH_API void RemeshIteration(
		TFunctionRef<bool(void)> EarlyTerminationCheck
	);

	// Whether an edge should be added to ModifiedEdges (used in async tasks)
	TArray<bool> EdgeShouldBeQueuedBuffer;	

	DYNAMICMESH_API virtual void InitializeVertexBufferForPass();

	// Move all vertices in parallel. Update the timestamps. Add edges whose endpoints change to ModifiedEdges.
	// The NewVertexPosition function accepts a vertex index and returns a new vertex position, as well as setting the 
	// bool out-param to indicate that the vertex has moved.
	DYNAMICMESH_API void TrackedMoveVerticesParallel(TFunction<FVector3d(int, bool&)> NewVertexPosition);

	// Note: superclass has this: virtual void FullSmoothPass_Buffer(bool bParallel);
	DYNAMICMESH_API void TrackedFullSmoothPass_Buffer(bool bParallel);


};



} // end namespace UE::Geometry
} // end namespace UE