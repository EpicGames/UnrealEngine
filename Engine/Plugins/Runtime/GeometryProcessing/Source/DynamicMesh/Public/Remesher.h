// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3cpp Remesher

#pragma once

#include "MeshRefinerBase.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{

/**
 * FRemesher implements edge flip/split/collapse/smooth Remeshing.
 * The basic process is very similar to the one described in:
 *   A Remeshing Approach to Multiresolution Modeling. Botsch & Kobbelt, SGP 2004.
 *   https://graphics.uni-bielefeld.de/publications/sgp04.pdf
 *   
 * In the iterative usage, each call to BasicRemeshPass() makes a pass over the mesh
 * and attempts to update the topology and shape to satisfy the target criteria.
 * This function iterates over the mesh edges and calls ProcessEdge(), which tries to
 * Collapse, then Split, then Flip the edge. After the topology pass, we optionally
 * do a full-mesh Smoothing pass, and a full-mesh Reprojection pass, to project
 * onto a target surface.
 * 
 * A highly flexible constraint system can be used to control what is allowed to
 * happen to individual edges and vertices, as well as Project vertices onto arbitrary
 * proejction targets. This is done via FMeshConstraints which is set in the MeshRefinerBase base class.
 * 
 * Many aspects of the algorithm can be controlled by the public variables in the block below.
 * In addition many of the core internal functions can be overriden to customize behavior.
 * Various callbacks functions are called when topology changes occur which allows 
 * you to for instance track changed regions of the mesh.
 * 
 * @todo parallel smoothing and projection
 * @todo port fast queue-based convergent remeshing
 * 
 */
class FRemesher : public FMeshRefinerBase
{
public:
	//
	// Configuration variables and flags
	//

	/** Controls whether edge-flips are allowed during remeshing pass */
	bool bEnableFlips = true;
	/** Controls whether edge collapses are allowed during remeshing pass */
	bool bEnableCollapses = true;
	/** Controls whether edge splits are allowed during remeshing pass */
	bool bEnableSplits = true;
	/** Controls whether vertex smoothing is applied during remeshing pass */
	bool bEnableSmoothing = true;

	/** Controls whether we try to prevent normal flips inside operations and smoothing.
	 * This is moderately expensive and often is not needed. Disabled by default.
	 */
	bool bPreventNormalFlips = false;

	/** Controls whether we disallow creation of triangles with small areas inside edge operations.
	 * This is moderately expensive and in some cases can result in lower-quality meshes. Disabled by default.
	 */
	bool bPreventTinyTriangles = false;

	/** Available Edge Flip metrics */
	enum class EFlipMetric
	{
		OptimalValence = 0,		/** Flip towards Valence=6 */
		MinEdgeLength = 1		/** Flip towards minimum edge lengths */
	};

	/** Type of flip metric that will be applied */
	EFlipMetric FlipMetric = EFlipMetric::OptimalValence;

	/** For MinEdgeLength metric, only flip if NewLength < (MinLengthFlipThresh * CurLength) */
	double MinLengthFlipThresh = 0.9;

	/** Minimum target edge length. Edges shorter than this will be collapsed if possible. */
	double MinEdgeLength = 1.0;
	/** Maximum target edge length. Edges longer than this will be split if possible. */
	double MaxEdgeLength = 3.0;

	/** Smoothing speed, in range [0,1] */
	double SmoothSpeedT = 0.1;

	/** Built-in Smoothing types */
	enum class ESmoothTypes 
	{
		Uniform = 0,	/** Uniform weights, produces regular mesh and fastest convergence */
		Cotan = 1,		/** Cotangent weights prevent tangential flow and hence preserve triangle shape / texture coordinates, but can become unstable... */
		MeanValue = 2   /** Mean Value weights also have reduced tangential flow but are never negative and hence more stable */
	};

	/** Type of smoothing that will be applied unless overridden by CustomSmoothF */
	ESmoothTypes SmoothType = ESmoothTypes::Uniform;

	/** Override default smoothing function */
	TFunction<FVector3d(const FDynamicMesh3&, int, double)> CustomSmoothF;

	/** Override constant SmoothSpeedT with function */
	TFunction<double(const FDynamicMesh3&, int)> CustomSmoothSpeedF;

	/** Scale the computed edge length to allow for adaptive remeshing */
	TFunction<double(const FDynamicMesh3& Mesh, int VertexA, int VertexB)> CustomEdgeLengthScaleF;


	/** enable parallel projection. Only applied in AfterRefinement mode */
	bool bEnableParallelProjection = true;

	/**
	 * Enable parallel smoothing. This will produce slightly different results
	 * across runs because we smooth in-place and hence there will be order side-effects.
	 */
	bool bEnableParallelSmooth = true;

	/**
	 * If smoothing is done in-place, we don't need an extra buffer, but also there will some randomness introduced in results. Probably worse.
	 */
	bool bEnableSmoothInPlace = false;



public:
	//
	// Public API
	//

	FRemesher(FDynamicMesh3* MeshIn) : FMeshRefinerBase(MeshIn)
	{
	}

	virtual ~FRemesher() = default;

	/** Set min/max edge-lengths to sane values for given target edge length */
	UE_API void SetTargetEdgeLength(double fLength);


	/**
	 * We can speed things up if we precompute some invariants. 
	 * You need to re-run this if you are changing the mesh externally
	 * between remesh passes, otherwise you will get weird results.
	 * But you will probably still come out ahead, computation-time-wise
	 */
	UE_API virtual void Precompute();


	/**
	 * Linear edge-refinement pass, followed by smoothing and projection
	 * - Edges are processed in prime-modulo-order to break symmetry
	 * - smoothing is done in parallel if EnableParallelSmooth = true
	 * - Projection pass if ProjectionMode == AfterRefinement
	 * - number of modified edges returned in ModifiedEdgesLastPass
	 */
	UE_API virtual void BasicRemeshPass();



public:
	//
	// Callbacks and in-progress information
	//


	/** Callback for subclasses to override to implement custom behavior */
	virtual void OnEdgeSplit(int EdgeID, int VertexA, int VertexB, const FDynamicMesh3::FEdgeSplitInfo& SplitInfo)
	{
		// this is for subclasses...
	}

	/** Callback for subclasses to override to implement custom behavior */
	virtual void OnEdgeCollapse(int EdgeID, int VertexA, int VertexB, const FDynamicMesh3::FEdgeCollapseInfo& CollapseInfo)
	{
		// this is for subclasses...
	}

	/** Callback for subclasses to override to implement custom behavior */
	virtual void OnEdgeFlip(int EdgeID, const FDynamicMesh3::FEdgeFlipInfo& FlipInfo)
	{
		// this is for subclasses...
	}


	/**
	 * Number of edges that were modified in previous Remesh pass.
	 * If this number gets small relative to edge count, you have probably converged (ish)
	 */
	int ModifiedEdgesLastPass = 0;


protected:

	FRemesher()        // for subclasses that extend our behavior
	{
	}

	/** we can avoid some checks/etc if we know that the mesh has no boundary edges. Set by Precompute() */
	bool bMeshIsClosed = false;

	// this just lets us write more concise code
	bool EnableInlineProjection() const { return ProjectionMode == ETargetProjectionMode::Inline; }


	// StartEdges() and GetNextEdge() control the iteration over edges that will be refined.
	// Default here is to iterate over entire mesh->
	// Subclasses can override these two functions to restrict the affected edges (eg EdgeLoopFRemesher)


	// We are using a modulo-index loop to break symmetry/pathological conditions. 
	// For example in a highly tessellated minimal cylinder, if the top/bottom loops have
	// sequential edge IDs, and all edges are < min edge length, then we can easily end
	// up successively collapsing each tiny edge, and eroding away the entire mesh!
	// By using modulo-index loop we jump around and hence this is unlikely to happen.
	uint64 MaxEdgeID = 0;
	virtual int StartEdges()
	{
		MaxEdgeID = static_cast<uint64>(Mesh->MaxEdgeID());
		return 0;
	}

	virtual int GetNextEdge(int CurEdgeID, bool& bDone)
	{
		constexpr uint64 ModuloPrime = 4294967311ull;     // choose prime > max uint32, to always be co-prime with MaxEdgeID
		int new_eid = static_cast<int>( (static_cast<uint64>(CurEdgeID) + ModuloPrime) % MaxEdgeID );
		bDone = (new_eid == 0);
		return new_eid;
	}





	enum class EProcessResult 
	{
		Ok_Collapsed,
		Ok_Flipped,
		Ok_Split,
		Ignored_EdgeIsFine,
		Ignored_EdgeIsFullyConstrained,
		Failed_OpNotSuccessful,
		Failed_NotAnEdge
	};

	UE_API virtual EProcessResult ProcessEdge(int edgeID);



	// After we split an edge, we have created a new edge and a new vertex.
	// The edge needs to inherit the constraint on the other pre-existing edge that we kept.
	// In addition, if the edge vertices were both constrained, then we /might/
	// want to also constrain this new vertex, possibly project to constraint target. 
	UE_API virtual void UpdateAfterSplit(int edgeID, int va, int vb, const FDynamicMesh3::FEdgeSplitInfo& splitInfo);

	TDynamicVector<FVector3d> TempPosBuffer;	// this is a temporary buffer used by smoothing and projection
	TArray<bool> TempFlagBuffer;				// list of which indices in TempPosBuffer were modified

	UE_API virtual void InitializeVertexBufferForPass();

	UE_API virtual void ApplyVertexBuffer(bool bParallel);



	// applies a smoothing pass to the mesh, storing intermediate positions in a buffer and then writing them at the end (so, no order effect)
	UE_API virtual void FullSmoothPass_Buffer(bool bParallel);

	// computes smoothed vertex position w/ proper constraints/etc. Does not modify mesh. 
	UE_API virtual FVector3d ComputeSmoothedVertexPos(int VertexID,
		TFunction<FVector3d(const FDynamicMesh3 &, int, double)> SmoothFunc, bool& bModified);

	// FullSmoothPass_Buffer this to calls VertexSmoothFunc for each vertex of the mesh
	UE_API virtual void ApplyToSmoothVertices(const TFunction<void(int)>& VertexSmoothFunc);

	// returns the function we want to use to compute a smoothed vertex position - will be CustomSmoothF if set, otherwise one of cotan/meanvalue/uniform
	UE_API virtual TFunction<FVector3d(const FDynamicMesh3&, int, double)> GetSmoothFunction();


	// Project vertices onto projection target.
	UE_API virtual void FullProjectionPass(bool bParallel);

	// Project a single vertex using the given target
	UE_API virtual void ProjectVertex(int VertexID, IProjectionTarget* UseTarget);

	// used by collapse-edge to get projected position for new vertex
	UE_API virtual FVector3d GetProjectedCollapsePosition(int VertexID, const FVector3d& vNewPos);

	// Apply the given projection function to all vertices in the mesh
	UE_API virtual void ApplyToProjectVertices(const TFunction<void(int)>& VertexProjectFunc);

	// Move all vertices in parallel. Update the timestamps.
	// The NewVertexPosition function accepts a vertex index and returns a new vertex position, as well as setting the 
	// bool out-param to indicate that the vertex has moved.
	UE_API void MoveVerticesParallel(TFunction<FVector3d(int, bool&)> NewVertexPosition);


	/*
	 * testing/debug/profiling stuff
	 */
protected:

	//
	// profiling functions, turn on ENABLE_PROFILING to see output in console
	// 
	int COUNT_SPLITS, COUNT_COLLAPSES, COUNT_FLIPS;
	//Stopwatch AllOpsW, SmoothW, ProjectW, FlipW, SplitW, CollapseW;

	virtual void ProfileBeginPass() 
	{
		if (ENABLE_PROFILING) 
		{
			COUNT_SPLITS = COUNT_COLLAPSES = COUNT_FLIPS = 0;
			//AllOpsW = new Stopwatch();
			//SmoothW = new Stopwatch();
			//ProjectW = new Stopwatch();
			//FlipW = new Stopwatch();
			//SplitW = new Stopwatch();
			//CollapseW = new Stopwatch();
		}
	}

	virtual void ProfileEndPass() 
	{
		if (ENABLE_PROFILING) {
			//System.Console.WriteLine(string.Format(
			//    "RemeshPass: T {0} V {1} splits {2} flips {3} collapses {4}", mesh->TriangleCount, mesh->VertexCount, COUNT_SPLITS, COUNT_FLIPS, COUNT_COLLAPSES
			//    ));
			//System.Console.WriteLine(string.Format(
			//    "           Timing1:  ops {0} smooth {1} project {2}", Util.ToSecMilli(AllOpsW.Elapsed), Util.ToSecMilli(SmoothW.Elapsed), Util.ToSecMilli(ProjectW.Elapsed)
			//    ));
			//System.Console.WriteLine(string.Format(
			//    "           Timing2:  collapse {0} flip {1} split {2}", Util.ToSecMilli(CollapseW.Elapsed), Util.ToSecMilli(FlipW.Elapsed), Util.ToSecMilli(SplitW.Elapsed)
			//    ));
		}
	}

	virtual void ProfileBeginOps() 
	{
		//if ( ENABLE_PROFILING ) AllOpsW.Start();
	}
	virtual void ProfileEndOps() 
	{
		//if ( ENABLE_PROFILING ) AllOpsW.Stop();
	}
	virtual void ProfileBeginSmooth() 
	{
		//if ( ENABLE_PROFILING ) SmoothW.Start();
	}
	virtual void ProfileEndSmooth()
	{
		//if ( ENABLE_PROFILING ) SmoothW.Stop();
	}
	virtual void ProfileBeginProject() 
	{
		//if ( ENABLE_PROFILING ) ProjectW.Start();
	}
	virtual void ProfileEndProject() 
	{
		//if ( ENABLE_PROFILING ) ProjectW.Stop();
	}

	virtual void ProfileBeginCollapse() 
	{
		//if ( ENABLE_PROFILING ) CollapseW.Start();
	}
	virtual void ProfileEndCollapse() 
	{
		//if ( ENABLE_PROFILING ) CollapseW.Stop();
	}
	virtual void ProfileBeginFlip() 
	{
		//if ( ENABLE_PROFILING ) FlipW.Start();
	}
	virtual void ProfileEndFlip()
	{
		//if ( ENABLE_PROFILING ) FlipW.Stop();
	}
	virtual void ProfileBeginSplit() 
	{
		//if ( ENABLE_PROFILING ) SplitW.Start();
	}
	virtual void ProfileEndSplit() 
	{
		//if ( ENABLE_PROFILING ) SplitW.Stop();
	}

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
