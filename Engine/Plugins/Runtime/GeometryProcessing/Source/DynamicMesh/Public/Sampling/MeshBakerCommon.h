// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTangents.h"
#include "Image/ImageTile.h"
#include "Image/ImageBuilder.h"
#include "MeshCurvature.h"
#include "MeshMapEvaluator.h"


namespace UE
{
namespace Geometry
{

class IMeshBakerDetailSampler;	

/**
 * Compute base/detail mesh correspondence by raycast
 * @return a pointer to the hit mesh, otherwise nullptr.
 */
const void* GetDetailMeshTrianglePoint_Raycast(
	const IMeshBakerDetailSampler* DetailSpatial,
	const FVector3d& BasePoint,
	const FVector3d& BaseNormal,
	int32& DetailTriangleOut,
	FVector3d& DetailTriBaryCoords,
	double Thickness,
	bool bFailToNearestPoint
);

	
/**
 * Compute base/detail mesh correspondence by nearest distance
 * @return a pointer to the nearest mesh, otherwise nullptr.
 */
const void* GetDetailMeshTrianglePoint_Nearest(
	const IMeshBakerDetailSampler* DetailSpatial,
	const FVector3d& BasePoint,
	int32& DetailTriangleOut,
	FVector3d& DetailTriBaryCoords);

	
/** Image tile storage for map bakes. */
class FMeshMapTileBuffer
{
public:
	FMeshMapTileBuffer(const FImageTile& TileIn, const int32 PixelSizeIn)
		: Tile(TileIn)
		, PixelSize(PixelSizeIn + 1) // + 1 for accumulated pixel weight.
	{
		Buffer = static_cast<float*>(FMemory::MallocZeroed(sizeof(float) * PixelSize * Tile.Num()));
	}

	~FMeshMapTileBuffer()
	{
		FMemory::Free(Buffer);
	}

	float& GetPixelWeight(const int64 LinearIdx) const
	{
		checkSlow(LinearIdx >= 0 && LinearIdx < Tile.Num());
		return Buffer[LinearIdx * PixelSize];
	}

	float* GetPixel(const int64 LinearIdx) const
	{
		checkSlow(LinearIdx >= 0 && LinearIdx < Tile.Num());
		return &Buffer[LinearIdx * PixelSize + 1];
	}

	const FImageTile& GetTile() const
	{
		return Tile;
	}

private:
	const FImageTile Tile;
	const int32 PixelSize;
	float* Buffer;
};


/**
 * Base bake detail sampler class.
 *
 * This class defines the interface for all sample queries that
 * FMeshBaseBaker supports including the spatial queries for
 * base to detail correspondence sampling. 
 */
class IMeshBakerDetailSampler
{
public:
	/** (Image, UVLayer) pair for detail textures */
	using FBakeDetailTexture = TTuple<const TImageBuilder<FVector4f>*, int>;

	enum class EBakeDetailNormalSpace
	{
		Tangent,
		Object
	};

	/** (Image, NormalSpace, UVLayer) tuple for detail normal textures */
	using FBakeDetailNormalTexture = TTuple<const TImageBuilder<FVector4f>*, int, EBakeDetailNormalSpace>;
	
	virtual ~IMeshBakerDetailSampler() = default;

	/** Iterate over each mesh in the detail set. */
	virtual void ProcessMeshes(TFunctionRef<void(const void*)> ProcessFn) const = 0;

	/** @return the triangle count of a given mesh. */
	virtual int32 GetTriangleCount(const void* Mesh) const = 0;

	/** Set the tangents for a given mesh */
	virtual void SetTangents(const void* Mesh, FMeshTangentsd* Tangents)
	{		
	}

	/** Associate a texture map and UV layer index for a given mesh in the detail set */
	virtual void SetTextureMap(const void* Mesh, const FBakeDetailTexture& Map) = 0;

	/** Associate a normal map and UV layer index for a given mesh in the detail set */
	virtual void SetNormalTextureMap(const void* Mesh, const FBakeDetailNormalTexture& Map) = 0;

	/** Retrieve a texture map and UV layer index from a given mesh in the detail set */
	virtual const FBakeDetailTexture* GetTextureMap(const void* Mesh) const = 0;

	virtual const FBakeDetailNormalTexture* GetNormalTextureMap(const void* Mesh) const = 0;

	/** @return true if identity correspondence is supported */
	virtual bool SupportsIdentityCorrespondence() const = 0;
	
	/** @return true if nearest point to triangle distance correspondence is supported */
	virtual bool SupportsNearestPointCorrespondence() const = 0;

	/** @return true if triangle ray intersection correspondence is supported */
	virtual bool SupportsRaycastCorrespondence() const = 0;

	/** @return true if a user-defined correspondence test is supported */
	virtual bool SupportsCustomCorrespondence() const
	{
		return false;
	}

	virtual void* ComputeCustomCorrespondence(const FMeshUVSampleInfo& SampleInfo, FMeshMapEvaluator::FCorrespondenceSample& ValueOut) const
	{
		return nullptr;
	}

	virtual bool IsValidCorrespondence(const FMeshMapEvaluator::FCorrespondenceSample& Sample) const
	{
		return Sample.DetailMesh && IsTriangle(Sample.DetailMesh, Sample.DetailTriID);
	}

	/**
	 * @param Point query point
	 * @param NearestDistSqrOut [Out] returned nearest squared distance, if triangle is found
	 * @param TriId [Out] ID of triangle nearest to Point within MaxDistance, or InvalidID if not found
	 * @param TriBaryCoords [Out] barycentric coordinates of the nearest point on the nearest tri.
	 * @param Options query options
	 * @return a pointer to the mesh that holds the nearest triangle
	 */
	virtual const void* FindNearestTriangle(
		const FVector3d& Point,
		double& NearestDistSqrOut,
		int& TriId,
		FVector3d& TriBaryCoords,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const
	{
		return nullptr;
	}

	/**
	 * Find nearest triangle from the given ray
	 * @param Ray query ray
	 * @param NearestT [Out] parameter of the nearest hit
	 * @param TriId [Out] ID of triangle intersected by ray within MaxDistance, or InvalidID if not found
	 * @param TriBaryCoords [Out] barycentric coordinates of intersection point on the hit tri.
	 * @param Options query options
	 * @return a pointer to the mesh whose triangle was intersected, nullptr otherwise.
	 */
	virtual const void* FindNearestHitTriangle(
		const FRay3d& Ray,
		double& NearestT,
		int& TriId,
		FVector3d& TriBaryCoords,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const
	{
		return nullptr;
	}

	/**
	 * Return true if any triangles hit by the given ray
	 * @param Ray query ray
	 * @param Options query options
	 * @return true if any triangles hit by the given ray
	 */
	virtual bool TestAnyHitTriangle(
		const FRay3d& Ray,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const
	{
		return false;
	}

	/** @return axis aligned bounding box for the detail mesh(es). */
	virtual FAxisAlignedBox3d GetBounds() const = 0;
	
	/**
	 * @param Mesh pointer to mesh to query
	 * @param TriId the triangle index to test
	 * @return true if the given triangle index is valid.
	 */
	virtual bool IsTriangle(const void* Mesh, const int TriId) const = 0;

	/**
	 * @param Mesh pointer to mesh to query
	 * @param TriId the triangle index to test
	 * @return 3 vertex indices of the given triangle.
	 */
	virtual FIndex3i GetTriangle(const void* Mesh, const int TriId) const = 0;

	/**
	 * @param Mesh pointer to mesh to query
	 * @param TriId the triangle index to test
	 * @return the normal for the given triangle.
	 */
	virtual FVector3d GetTriNormal(const void* Mesh, const int TriId) const = 0;

	/**
	 * @param Mesh pointer to mesh to query 
	 * @param TriId the triangle index to test
	 * @param UVLayer the UVLayer index to query
	 * @param UV0 the output UV for triangle vertex 0
	 * @param UV1 the output UV for triangle vertex 1
	 * @param UV2 the output UV for triangle vertex 2
	 * @return true if a valid UV was returned, false otherwise.
	 */
	virtual bool GetTriUVs(const void* Mesh, int TriId, int UVLayer, FVector2f& UV0, FVector2f& UV1, FVector2f& UV2) const = 0;

	/**
	 * @param Mesh pointer to mesh to query
	 * @param TriId the triangle index to test
	 * @return the material ID for the given triangle.
	 */
	virtual int32 GetMaterialID(const void* Mesh, const int TriId) const = 0;

	/**
	 * @param Mesh pointer to mesh to query
	 * @param TriId the triangle index to test
	 * @return the group ID for the given triangle.
	 */
	virtual int32 GetPolyGroupID(const void* Mesh, const int TriId) const = 0;

	/**
	 * @param Mesh pointer to mesh to query
	 * @return true if this mesh has normals
	 */
	virtual bool HasNormals(const void* Mesh) const = 0;

	/**
	 * @param Mesh pointer to mesh to query
	 * @param UVLayer the UV layer to check for existence
	 * @return true if this mesh has UVs on the given layer
	 */
	virtual bool HasUVs(const void* Mesh, int UVLayer = 0) const = 0;

	/**
	 * @param Mesh pointer to mesh to query
	 * @return true if this mesh has tangents
	 */
	virtual bool HasTangents(const void* Mesh) const = 0;

	/**
	 * @param Mesh pointer to mesh to query
	 * @return true if this mesh has vertex colors on the given layer
	 */
	virtual bool HasColors(const void* Mesh) const = 0;

	/**
	 * Compute interpolated Position value inside triangle using barycentric coordinates
	 * @param Mesh pointer to mesh to query
	 * @param TriId index of triangle
	 * @param BaryCoords 3 barycentric coordinates inside triangle
	 * @return resulting interpolated Position parameter value
	 */
	virtual FVector3d TriBaryInterpolatePoint(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords) const = 0;

	/**
	 * Compute interpolated Normal value inside triangle using barycentric coordinates
	 * @param Mesh pointer to mesh to query
	 * @param TriId index of triangle
	 * @param BaryCoords 3 barycentric coordinates inside triangle
	 * @param NormalOut resulting interpolated Normal parameter value
	 * @return true if a valid normal was returned, false otherwise
	 */
	virtual bool TriBaryInterpolateNormal(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector3f& NormalOut) const = 0;

	/**
	 * Compute interpolated UV value inside triangle using barycentric coordinates
	 * @param Mesh pointer to mesh to query
	 * @param TriId index of triangle
	 * @param BaryCoords 3 barycentric coordinates inside triangle
	 * @param UVLayer UV layer to query
	 * @param UVOut resulting interpolated UV parameter value
	 * @return true if a valid UV was returned, false otherwise
	 */
	virtual bool TriBaryInterpolateUV(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		const int UVLayer,
		FVector2f& UVOut ) const = 0;

	/**
	 * Compute interpolated vertex color value inside triangle using barycentric coordinates
	 * @param Mesh pointer to mesh to query
	 * @param TriId index of triangle
	 * @param BaryCoords 3 barycentric coordinates inside triangle
	 * @param ColorOut resulting interpolated vertex color parameter value
	 * @return true if a valid vertex color was returned, false otherwise
	 */
	virtual bool TriBaryInterpolateColor(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector4f& ColorOut) const = 0;

	/**
	 * Compute interpolated tangent values inside triangle using barycentric coordinates
	 * @param Mesh pointer to mesh to query
	 * @param TriId index of triangle
	 * @param BaryCoords 3 barycentric coordinates inside triangle
	 * @param TangentX resulting interpolated tangentX value
	 * @param TangentY resulting interpolated tangentY value
	 * @return true if valid tangents were returned, false otherwise
	 */
	virtual bool TriBaryInterpolateTangents(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector3d& TangentX,
		FVector3d& TangentY) const = 0;

	// TODO: Rework the curvature interface to only compute the minimal set of work.
	/**
	 * @param Mesh pointer to mesh to query
	 * @param CurvatureCache [Out] the curvature data to populate
	 */
	virtual void GetCurvature(
		const void* Mesh,
		FMeshVertexCurvatureCache& CurvatureCache) const = 0;
};


/**
 * DynamicMesh bake detail sampler for baking 1 detail mesh to 1 target mesh.
 */
class FMeshBakerDynamicMeshSampler : public IMeshBakerDetailSampler
{
public:
	UE_NONCOPYABLE(FMeshBakerDynamicMeshSampler)

	FMeshBakerDynamicMeshSampler(const FDynamicMesh3* Mesh, const FDynamicMeshAABBTree3* Spatial, const FMeshTangentsd* Tangents = nullptr)
		: DetailMesh(Mesh), DetailSpatial(Spatial), DetailTangents(Tangents)
	{
	}

	virtual void ProcessMeshes(TFunctionRef<void(const void*)> ProcessFn) const override
	{
		ProcessFn(DetailMesh);
	}

	virtual int32 GetTriangleCount(const void* Mesh) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh->TriangleCount();
	}

	virtual void SetTangents(const void* Mesh, FMeshTangentsd* Tangents) override
	{
		DetailTangents = Tangents;
	}

	virtual void SetTextureMap(const void* Mesh, const FBakeDetailTexture& Map) override
	{
		DetailTextureMap = Map;
	}

	virtual void SetNormalTextureMap(const void* Mesh, const FBakeDetailNormalTexture& Map) override
	{
		DetailNormalTextureMap = Map;
	}

	virtual const FBakeDetailTexture* GetTextureMap(const void* Mesh) const override
	{
		return &DetailTextureMap;
	}

	virtual const FBakeDetailNormalTexture* GetNormalTextureMap(const void* Mesh) const override
	{
		return &DetailNormalTextureMap;
	}

	virtual bool SupportsIdentityCorrespondence() const override
	{
		return true;
	}

	virtual bool SupportsNearestPointCorrespondence() const override
	{
		return true;
	}

	virtual bool SupportsRaycastCorrespondence() const override
	{
		return true;
	}

	virtual const void* FindNearestTriangle(
		const FVector3d& Point,
		double& NearestDistSqrOut,
		int& TriId,
		FVector3d& TriBaryCoords,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const override
	{
		TriId = DetailSpatial->FindNearestTriangle(Point, NearestDistSqrOut, Options);
		if (DetailMesh->IsTriangle(TriId))
		{
			const FDistPoint3Triangle3d DistQuery = TMeshQueries<FDynamicMesh3>::TriangleDistance(*DetailMesh, TriId, Point);
			TriBaryCoords = DistQuery.TriangleBaryCoords;
		}
		return DetailMesh;
	}

	virtual const void* FindNearestHitTriangle(
		const FRay3d& Ray,
		double& NearestT,
		int& TriId,
		FVector3d& TriBaryCoords,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const override
	{
		const bool bHit = DetailSpatial->FindNearestHitTriangle(Ray, NearestT, TriId, TriBaryCoords, Options);
		return bHit ? DetailMesh : nullptr;
	}

	virtual bool TestAnyHitTriangle(
		const FRay3d& Ray,
		const IMeshSpatial::FQueryOptions& Options = IMeshSpatial::FQueryOptions()) const override
	{
		return DetailSpatial->TestAnyHitTriangle(Ray, Options);
	}

	virtual FAxisAlignedBox3d GetBounds() const override
	{
		return DetailMesh->GetBounds();
	}

	virtual bool IsTriangle(const void* Mesh, const int TriId) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh->IsTriangle(TriId);
	}

	virtual FIndex3i GetTriangle(const void* Mesh, const int TriId) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh->GetTriangle(TriId);
	}

	virtual FVector3d GetTriNormal(const void* Mesh, const int TriId) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh->GetTriNormal(TriId);
	}

	virtual bool GetTriUVs(const void* Mesh, int TriId, int UVLayer, FVector2f& UV0, FVector2f& UV1, FVector2f& UV2) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		if (const FDynamicMeshAttributeSet* Attributes = DynamicMesh->Attributes())
		{
			if (const FDynamicMeshUVOverlay* UVOverlay = Attributes->GetUVLayer(UVLayer))
			{
				if (UVOverlay->IsSetTriangle(TriId))
				{
					UVOverlay->GetTriElements(TriId, UV0, UV1, UV2);
					return true;
				}
			}
		}
		return false;
	}

	virtual int32 GetMaterialID(const void* Mesh, const int TriId) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		const FDynamicMeshMaterialAttribute* MaterialIDOverlay = DynamicMesh->Attributes()->GetMaterialID();
		return MaterialIDOverlay ? MaterialIDOverlay->GetValue(TriId) : IndexConstants::InvalidID;
	}

	virtual int32 GetPolyGroupID(const void* Mesh, const int TriID) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh->GetTriangleGroup(TriID);
	}

	virtual bool HasNormals(const void* Mesh) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh && DynamicMesh->HasAttributes() && DynamicMesh->Attributes()->PrimaryNormals();
	}

	virtual bool HasUVs(const void* Mesh, const int UVLayer = 0) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh && DynamicMesh->HasAttributes() && DynamicMesh->Attributes()->GetUVLayer(UVLayer);
	}

	virtual bool HasTangents(const void* Mesh) const override
	{
		return DetailTangents != nullptr;
	}

	virtual bool HasColors(const void* Mesh) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh && DynamicMesh->HasAttributes() && DynamicMesh->Attributes()->PrimaryColors();
	}

	virtual FVector3d TriBaryInterpolatePoint(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		return DynamicMesh->GetTriBaryPoint(TriId, BaryCoords.X, BaryCoords.Y, BaryCoords.Z);
	}

	virtual bool TriBaryInterpolateNormal(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector3f& NormalOut) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		const FDynamicMeshNormalOverlay* NormalOverlay = DynamicMesh->Attributes()->PrimaryNormals();
		const bool bValid = NormalOverlay && NormalOverlay->IsSetTriangle(TriId);
		if (bValid)
		{
			FVector3d Normal;
			NormalOverlay->GetTriBaryInterpolate(TriId, &BaryCoords.X, &Normal.X);
			NormalOut = FVector3f(Normal);
		}
		return bValid;
	}

	virtual bool TriBaryInterpolateUV(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		const int UVLayer,
		FVector2f& UVOut ) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		const FDynamicMeshUVOverlay* UVOverlay = DynamicMesh->Attributes()->GetUVLayer(UVLayer);
		const bool bValid = UVOverlay && UVOverlay->IsSetTriangle(TriId);
		if (bValid)
		{
			FVector2d UV;
			UVOverlay->GetTriBaryInterpolate(TriId, &BaryCoords.X, &UV.X);
			UVOut = FVector2f(UV);
		}
		return bValid;
	}

	virtual bool TriBaryInterpolateColor(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector4f& ColorOut) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		const FDynamicMeshColorOverlay* ColorOverlay = DynamicMesh->Attributes()->PrimaryColors();
		const bool bValid = ColorOverlay && ColorOverlay->IsSetTriangle(TriId);
		if (bValid)
		{
			FVector4d Color;
			ColorOverlay->GetTriBaryInterpolate(TriId, &BaryCoords.X, &Color.X);
			ColorOut = FVector4f(Color);
		}
		return bValid;
	}

	virtual bool TriBaryInterpolateTangents(
		const void* Mesh,
		const int32 TriId,
		const FVector3d& BaryCoords,
		FVector3d& TangentX,
		FVector3d& TangentY) const override
	{
		// TODO: Can we use the tangent overlay here for a unified interface? (PrimaryTangents/PrimaryBiTangents)
		bool bSuccess = false;
		if (DetailTangents)
		{
			DetailTangents->GetInterpolatedTriangleTangent(TriId, BaryCoords, TangentX, TangentY);
			bSuccess = true;
		}
		return bSuccess;
	}

	virtual void GetCurvature(
		const void* Mesh,
		FMeshVertexCurvatureCache& CurvatureCache) const override
	{
		const FDynamicMesh3* DynamicMesh = static_cast<const FDynamicMesh3*>(Mesh);
		CurvatureCache.BuildAll(*DynamicMesh);
		ensure(CurvatureCache.Num() == DetailMesh->MaxVertexID());
	}
	
protected:
	const FDynamicMesh3* DetailMesh = nullptr;
	const FDynamicMeshAABBTree3* DetailSpatial = nullptr;
	const FMeshTangentsd* DetailTangents = nullptr;
	FBakeDetailTexture DetailTextureMap = FBakeDetailTexture(nullptr, 0);
	FBakeDetailNormalTexture DetailNormalTextureMap = FBakeDetailNormalTexture(nullptr, 0, EBakeDetailNormalSpace::Tangent);
};		
	

} // end namespace UE::Geometry
} // end namespace UE
	