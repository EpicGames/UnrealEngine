// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshTangents.h"
#include "Sampling/MeshBakerCommon.h"


namespace UE
{
namespace Geometry
{


class FMeshBaseBaker
{
public:
	virtual ~FMeshBaseBaker() = default;
	
	/**
	* ECorrespondenceStrategy determines the basic approach that will be used to establish a
	* mapping from points on the BakeTarget Mesh (usually low-poly) to points on the Detail Mesh (eg highpoly).
	* Geometrically this is not a 1-1 mapping so there are various options
	*/
	enum class ECorrespondenceStrategy
	{
		/** Raycast inwards from Point+Thickness*Normal, if that misses, try Outwards from Point, then Inwards from Point */
		RaycastStandard,
		/** Use geometrically nearest point. Thickness is ignored */
		NearestPoint,
		/** Use RaycastStandard but fall back to NearestPoint if none of the rays hit */
		RaycastStandardThenNearest,
		/** Assume that BakeTarget == DetailMesh and so no mapping is necessary */
		Identity,
		/** Use user-defined strategy for finding correspondence */
		Custom
	};

	// Setters
	void SetDetailSampler(IMeshBakerDetailSampler* Sampler)
	{
		DetailSampler = Sampler;
	}
	void SetTargetMesh(const FDynamicMesh3* Mesh)
	{
		TargetMesh = Mesh;
	}
	void SetTargetMeshTangents(const TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe>& Tangents)
	{
		TargetMeshTangents = Tangents;
	}
	void SetTargetMeshUVLayer(const int32 UVLayerIn)
	{
		TargetMeshUVLayer = UVLayerIn;
	}
	void SetProjectionDistance(const double ProjectionDistanceIn)
	{
		ProjectionDistance = ProjectionDistanceIn;
	}
	void SetCorrespondenceStrategy(const ECorrespondenceStrategy Strategy)
	{
		CorrespondenceStrategy = Strategy;
	}

	// TargetMesh Getters
	const FDynamicMesh3* GetTargetMesh() const
	{
		return TargetMesh;
	}
	const FDynamicMeshUVOverlay* GetTargetMeshUVs() const
	{
		check(TargetMesh && TargetMesh->HasAttributes() && TargetMeshUVLayer < TargetMesh->Attributes()->NumUVLayers());
        return TargetMesh->Attributes()->GetUVLayer(TargetMeshUVLayer);
	}
	int32 GetTargetMeshUVLayer() const
	{
		return TargetMeshUVLayer;
	}
	const FDynamicMeshNormalOverlay* GetTargetMeshNormals() const
	{
		check(TargetMesh && TargetMesh->HasAttributes());
		return TargetMesh->Attributes()->PrimaryNormals();
	}
	const FMeshTangentsd* GetTargetMeshTangents() const
	{
		return TargetMeshTangents.Get();
	}

	// DetailMesh Getters
	const IMeshBakerDetailSampler* GetDetailSampler() const
	{
		return DetailSampler;
	}

	// Other Getters
	double GetProjectionDistance() const
	{
		return ProjectionDistance;
	}
	ECorrespondenceStrategy GetCorrespondenceStrategy() const
	{
		return CorrespondenceStrategy;
	}

protected:
	const FDynamicMesh3* TargetMesh = nullptr;
	TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe> TargetMeshTangents;
	int32 TargetMeshUVLayer = 0;
	
	double ProjectionDistance = 3.0;
	ECorrespondenceStrategy CorrespondenceStrategy = ECorrespondenceStrategy::RaycastStandard;

	IMeshBakerDetailSampler* DetailSampler = nullptr;
};

	
} // end namespace UE::Geometry
} // end namespace UE
	