// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshMapEvaluator.h"
#include "DynamicMesh/MeshTangents.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * A mesh evaluator for constant data. This evaluator can be useful as a filler
 * when computing per channel color data.
 */
class FMeshConstantMapEvaluator : public FMeshMapEvaluator
{
public:
	FMeshConstantMapEvaluator() = default;
	UE_API explicit FMeshConstantMapEvaluator(float ValueIn);
	
	// Begin FMeshMapEvaluator interface
	UE_API virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	UE_API virtual const TArray<EComponents>& DataLayout() const override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::Constant; }
	// End FMeshMapEvaluator interface

	static UE_API void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

	static UE_API void EvaluateDefault(float*& Out, void* EvalData);

	static UE_API void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);
	static UE_API void EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData);

public:
	float Value = 0.0f;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
