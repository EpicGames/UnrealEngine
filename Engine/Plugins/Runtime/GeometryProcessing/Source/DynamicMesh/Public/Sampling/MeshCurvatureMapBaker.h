// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshImageBaker.h"
#include "Sampling/MeshImageBakingCache.h"
#include "Image/ImageBuilder.h"

namespace UE
{
namespace Geometry
{

class FMeshVertexCurvatureCache;

class FMeshCurvatureMapBaker : public FMeshImageBaker
{
public:
	virtual ~FMeshCurvatureMapBaker() = default;

	//
	// Options
	//

	enum class ECurvatureType
	{
		Mean = 0,
		Gaussian = 1,
		MaxPrincipal = 2,
		MinPrincipal = 3
	};
	ECurvatureType UseCurvatureType = ECurvatureType::Mean;

	enum class EColorMode
	{
		BlackGrayWhite = 0,
		RedGreenBlue = 1,
		RedBlue = 2
	};
	EColorMode UseColorMode = EColorMode::RedGreenBlue;

	enum class EClampMode
	{
		FullRange = 0,
		Positive = 1,
		Negative = 2
	};
	EClampMode UseClampMode = EClampMode::FullRange;

	double RangeScale = 1.0;
	double MinRangeScale = 0.0;

	// allows override of the max curvature; if false, range is set based on [-(avg+stddev), avg+stddev]
	bool bOverrideCurvatureRange = false;
	double OverrideRangeMax = .1;


	double BlurRadius = 0.0;

	//
	// Required input data, can be provided, will be computed otherwise
	//

	TSharedPtr<FMeshVertexCurvatureCache> Curvatures;


	//
	// Compute functions
	//

	/** Calculate bake result */
	DYNAMICMESH_API virtual void Bake() override;

	/** populate Curvatures member if valid data has not been provided */
	DYNAMICMESH_API void CacheDetailCurvatures(const FDynamicMesh3* DetailMesh);

	//
	// Output
	//

	const TUniquePtr<TImageBuilder<FVector3f>>& GetResult() const { return ResultBuilder; }

	TUniquePtr<TImageBuilder<FVector3f>> TakeResult() { return MoveTemp(ResultBuilder); }


protected:
	TUniquePtr<TImageBuilder<FVector3f>> ResultBuilder;

	void Bake_Single();

	void Bake_Multi();


	void GetColorMapRange(FVector3f& NegativeColor, FVector3f& ZeroColor, FVector3f& PositiveColor);
};


} // end namespace UE::Geometry
} // end namespace UE
