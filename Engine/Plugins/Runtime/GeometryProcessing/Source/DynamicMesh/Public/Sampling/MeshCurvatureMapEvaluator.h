// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshMapEvaluator.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class IMeshBakerDetailSampler;
class FMeshVertexCurvatureCache;

/**
 * A mesh evaluator for mesh curvatures.
 */
class FMeshCurvatureMapEvaluator : public FMeshMapEvaluator
{
public:
	enum class ECurvatureType
	{
		Mean = 0,
		Gaussian = 1,
		MaxPrincipal = 2,
		MinPrincipal = 3
	};
	ECurvatureType UseCurvatureType = ECurvatureType::Mean;

	enum class EColorMode
	{
		BlackGrayWhite = 0,
		RedGreenBlue = 1,
		RedBlue = 2
	};
	EColorMode UseColorMode = EColorMode::RedGreenBlue;

	enum class EClampMode
	{
		FullRange = 0,
		Positive = 1,
		Negative = 2
	};
	EClampMode UseClampMode = EClampMode::FullRange;

	double RangeScale = 1.0;
	double MinRangeScale = 0.0;

	// Allows override of the max curvature; if false, range is set based on [-(avg+stddev), avg+stddev]
	bool bOverrideCurvatureRange = false;
	double OverrideRangeMax = 0.1;

	// Required input data, can be provided, will be computed otherwise
	TSharedPtr<FMeshVertexCurvatureCache> Curvatures;

public:
	// Begin FMeshMapEvaluator interface
	UE_API virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	UE_API virtual const TArray<EComponents>& DataLayout() const override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::Curvature; }
	// End FMeshMapEvaluator interface

	static UE_API void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

	static UE_API void EvaluateDefault(float*& Out, void* EvalData);

	static UE_API void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);

	static UE_API void EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData);

	/** Populate Curvatures member if valid data has not been provided */
	UE_API void CacheDetailCurvatures();

protected:
	// Cached data
	const IMeshBakerDetailSampler* DetailSampler = nullptr;
	double MinPreClamp = -TNumericLimits<double>::Max();
	double MaxPreClamp = TNumericLimits<double>::Max();
	FInterval1d ClampRange;
	FVector3f NegativeColor;
	FVector3f ZeroColor;
	FVector3f PositiveColor;

protected:
	UE_API double GetCurvature(int32 vid) const;
	UE_API void GetColorMapRange(FVector3f& NegativeColorOut, FVector3f& ZeroColorOut, FVector3f& PositiveColorOut) const;

private:
	UE_API double SampleFunction(const FCorrespondenceSample& SampleData) const;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
