// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Sampling/MeshImageBaker.h"
#include "Sampling/MeshImageBakingCache.h"
#include "Image/ImageBuilder.h"
#include "DynamicMesh/MeshTangents.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * Bake Colors based on arbitrary Position/Normal sampling. Useful for (eg) baking
 * from an arbitrary function defined over space
 */
class FMeshGenericWorldPositionColorBaker : public FMeshImageBaker
{
public:
	virtual ~FMeshGenericWorldPositionColorBaker() = default;

	//
	// Required input data
	//

	FVector4f DefaultColor = FVector4f(0, 0, 0, 1);

	TFunction<FVector4f(FVector3d, FVector3d)> ColorSampleFunction 
		= [this](FVector3d Position, FVector3d Normal) { return DefaultColor; };

	//
	// Compute functions
	//

	UE_API virtual void Bake() override;

	//
	// Output
	//

	const TUniquePtr<TImageBuilder<FVector4f>>& GetResult() const { return ResultBuilder; }

	TUniquePtr<TImageBuilder<FVector4f>> TakeResult() { return MoveTemp(ResultBuilder); }

protected:
	TUniquePtr<TImageBuilder<FVector4f>> ResultBuilder;
};




/**
 * Bake Tangent-Space Normals based on arbitrary Position/Normal sampling. Useful for (eg) 
 * baking from an arbitrary function defined over space.
 */
class FMeshGenericWorldPositionNormalBaker : public FMeshImageBaker
{
public:
	virtual ~FMeshGenericWorldPositionNormalBaker() = default;

	//
	// Required input data
	//

	const TMeshTangents<double>* BaseMeshTangents;

	FVector3f DefaultNormal = FVector3f(0, 0, 1);

	TFunction<FVector3f(FVector3d, FVector3d)> NormalSampleFunction
		= [this](FVector3d Position, FVector3d Normal) { return DefaultNormal; };

	//
	// Compute functions
	//

	UE_API virtual void Bake() override;

	//
	// Output
	//

	const TUniquePtr<TImageBuilder<FVector3f>>& GetResult() const { return NormalsBuilder; }

	TUniquePtr<TImageBuilder<FVector3f>> TakeResult() { return MoveTemp(NormalsBuilder); }

protected:
	TUniquePtr<TImageBuilder<FVector3f>> NormalsBuilder;
};






} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
