// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshMapEvaluator.h"
#include "Sampling/MeshBakerCommon.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class IMeshBakerDetailSampler;	

/**
 * A mesh evaluator for mesh height as color data.
 */
class FMeshHeightMapEvaluator : public FMeshMapEvaluator
{
public:
	enum class EHeightRangeMode
	{
		// Absolute units in object space
		Absolute,
		// Ratio of maximum bounding box axis
		RelativeBounds
	};
	EHeightRangeMode RangeMode = EHeightRangeMode::Absolute;
	FInterval1f Range;
	
public:
	FMeshHeightMapEvaluator() = default;
	FMeshHeightMapEvaluator(const FMeshHeightMapEvaluator&) = default;
	FMeshHeightMapEvaluator& operator=(const FMeshHeightMapEvaluator&) = default;

	// Begin FMeshMapEvaluator interface
	UE_API virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	UE_API virtual const TArray<EComponents>& DataLayout() const override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::Height; }
	// End FMeshMapEvaluator interface

	static UE_API void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

	static UE_API void EvaluateDefault(float*& Out, void* EvalData);

	static UE_API void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);
	static UE_API void EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData);

protected:
	// Cached data
	const IMeshBakerDetailSampler* DetailSampler = nullptr;
	FInterval1f CachedRange;
	
private:
	UE_API float SampleFunction(const FCorrespondenceSample& SampleData) const;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
