// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshImageBakingCache.h"

namespace UE
{
namespace Geometry
{

class FMeshImageBaker
{
public:
	virtual ~FMeshImageBaker() = default;

	void SetCache(const FMeshImageBakingCache* CacheIn)
	{
		ImageBakeCache = CacheIn;
	}

	const FMeshImageBakingCache* GetCache() const
	{
		return ImageBakeCache;
	}

	virtual void Bake()
	{

	}


protected:

	const FMeshImageBakingCache* ImageBakeCache;

};

} // end namespace UE::Geometry
} // end namespace UE
