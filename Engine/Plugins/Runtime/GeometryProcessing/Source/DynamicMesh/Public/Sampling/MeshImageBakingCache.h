// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Sampling/MeshSurfaceSampler.h"
#include "Spatial/DenseGrid2.h"
#include "Image/ImageDimensions.h"
#include "Image/ImageOccupancyMap.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FMeshImageBakingCache
{
public:

	/**
	 * ECorrespondenceStrategy determines the basic approach that will be used to establish a 
	 * mapping from points on the BakeTarget Mesh (usually low-poly) to points on the Detail Mesh (eg highpoly).
	 * Geometrically this is not a 1-1 mapping so there are various options
	 */
	enum class ECorrespondenceStrategy
	{
		/** Raycast inwards from Point+Thickness*Normal, if that misses, try Outwards from Point, then Inwards from Point */
		RaycastStandard,
		/** Use geometrically nearest point. Thickness is ignored */
		NearestPoint,
		/** Use RaycastStandard but fall back to NearestPoint if none of the rays hit */
		RaycastStandardThenNearest,
		/** Assume that BakeTarget == DetailMesh and so no mapping is necessary */
		Identity
	};

	UE_API void SetDetailMesh(const FDynamicMesh3* Mesh, const FDynamicMeshAABBTree3* Spatial);
	UE_API void SetBakeTargetMesh(const FDynamicMesh3* Mesh);

	UE_API void SetDimensions(FImageDimensions Dimensions);
	UE_API void SetUVLayer(int32 UVLayer);
	UE_API void SetThickness(double Thickness);
	UE_API void SetCorrespondenceStrategy(ECorrespondenceStrategy Strategy);
	UE_API void SetGutterSize(int32 GutterSize);

	FImageDimensions GetDimensions() const { return Dimensions; }
	int32 GetUVLayer() const { return UVLayer; }
	double GetThickness() const { return Thickness; }
	ECorrespondenceStrategy GetCorrespondenceStrategy() const { return CorrespondenceStrategy; }

	const FDynamicMesh3* GetBakeTargetMesh() const { return TargetMesh; }
	UE_API const FDynamicMeshUVOverlay* GetBakeTargetUVs() const;
	UE_API const FDynamicMeshNormalOverlay* GetBakeTargetNormals() const;

	const FDynamicMesh3* GetDetailMesh() const { return DetailMesh; }
	const FDynamicMeshAABBTree3* GetDetailSpatial() const { return DetailSpatial; }
	UE_API const FDynamicMeshNormalOverlay* GetDetailNormals() const;


	bool IsCacheValid() const { return bOccupancyValid && bSamplesValid; }

	UE_API bool ValidateCache();


	struct FCorrespondenceSample
	{
		FMeshUVSampleInfo BaseSample;
		FVector3d BaseNormal;

		int32 DetailTriID;
		FVector3d DetailBaryCoords;
	};


	UE_API void EvaluateSamples(TFunctionRef<void(const FVector2i&, const FCorrespondenceSample&)> SampleFunction,
		bool bParallel = true) const;

	UE_API const FImageOccupancyMap* GetOccupancyMap() const;


	/**
	 * Iterate over valid samples in the Occupancy map to find "holes", ie pixels
	 * where no valid sample could be computed according to IsValidSampleFunction. 
	 * This can be used after a bake to find locations where the baked map needs to
	 * be repaired as a post-process (eg using image infilling/etc), which can produce 
	 * better results than (eg) using a "worse" sampling strategy to guarantee some
	 * non-garbage result (eg Nearest-point sampling)
	 */
	UE_API void FindSamplingHoles(TFunctionRef<bool(const FVector2i&)> IsValidSampleFunction,
		TArray<FVector2i>& HolePixelsOut,
		bool bParallel = true) const;


	/** if this function returns true, we should abort calculation */
	TFunction<bool(void)> CancelF = []() { return false; };

protected:
	const FDynamicMesh3* DetailMesh = nullptr;
	const FDynamicMeshAABBTree3* DetailSpatial = nullptr;
	const FDynamicMesh3* TargetMesh = nullptr;

	FImageDimensions Dimensions = FImageDimensions(128,128);
	int32 UVLayer = 0;
	double Thickness = 3.0;
	ECorrespondenceStrategy CorrespondenceStrategy = ECorrespondenceStrategy::RaycastStandard;
	int32 GutterSize = 4;

	TDenseGrid2<FCorrespondenceSample> SampleMap;
	bool bSamplesValid = false;
	UE_API void InvalidateSamples();


	TUniquePtr<FImageOccupancyMap> OccupancyMap;
	bool bOccupancyValid = false;
	UE_API void InvalidateOccupancy();

};



} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API