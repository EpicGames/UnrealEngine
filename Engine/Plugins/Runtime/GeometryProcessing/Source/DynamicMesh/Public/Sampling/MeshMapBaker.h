// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

#include "Sampling/MeshBaseBaker.h"
#include "Sampling/MeshMapEvaluator.h"
#include "Sampling/MeshSurfaceSampler.h"
#include "Image/ImageBuilder.h"
#include "Image/ImageDimensions.h"
#include "Image/BoxFilter.h"
#include "Image/BCSplineFilter.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FImageOccupancyMap;
class FImageTile;
class FMeshMapTileBuffer;

class FMeshMapBaker : public FMeshBaseBaker
{
public:

	//
	// Bake
	//

	/** Process all bakers to generate image results for each. */
	UE_API void Bake();

	/** Add a baker to be processed. */
	UE_API int32 AddEvaluator(const TSharedPtr<FMeshMapEvaluator, ESPMode::ThreadSafe>& Eval);

	/** @return the evaluator at the given index. */
	UE_API FMeshMapEvaluator* GetEvaluator(int32 EvalIdx) const;

	/** @return the number of bake evaluators on this baker. */
	UE_API int32 NumEvaluators() const;

	/** Reset the list of bakers. */
	UE_API void Reset();

	/** @return the bake result image for a given baker index. */
	UE_API const TArrayView<TUniquePtr<TImageBuilder<FVector4f>>> GetBakeResults(int32 EvalIdx);

	/** @return true if we should abort calculation */
	TFunction<bool(void)> CancelF = []() { return false; };

	/** Function to call after evaluator data is written to the final image, but before any gutter processing occurs */
	TFunction<void(TArray<TUniquePtr<TImageBuilder<FVector4f>>>&)> PostWriteToImageCallback
		= [](TArray<TUniquePtr<TImageBuilder<FVector4f>>>& PostWriteToImageBakeResults) {};

	/* Function to call for each interior sample */
	TFunction<void(bool, const FMeshMapEvaluator::FCorrespondenceSample&, const FVector2d&, const FVector2i&)> InteriorSampleCallback
		= [](bool bSampleValid, const FMeshMapEvaluator::FCorrespondenceSample& Sample, const FVector2d& UVPosition, const FVector2i& ImageCoords) {};

	/**
	 * @param ImageCoords the output image coordinates to be evaluated
	 * @param UV the target mesh UV coordinates to be evaluated
	 * @param TriID the target mesh triangle ID to be evaluated
	 * @return a weight (clamped by the baker to the range [0,1]) used to combine evaluator sample values with evaluator
	 *  defaults. For evaluators using EAccumulateMode::Overwrite the default is used when weight == 0, and the value is
	 *  used otherwise. For evaluators using EAccumulateMode::Add the sample's value and default are blended using the
	 *  expression: `weight * value + (1 - weight) * default`
	 */
	TFunction<float(const FVector2i&, const FVector2d&, int32)> SampleFilterF = nullptr;

	//
	// Parameters
	//

	enum class EBakeFilterType
	{
		None,
		Box,
		BSpline,
		MitchellNetravali
	};
	
	UE_API void SetDimensions(FImageDimensions DimensionsIn);
	UE_API void SetGutterEnabled(bool bEnabled);
	UE_API void SetGutterSize(int32 GutterSizeIn);
	UE_API void SetSamplesPerPixel(int32 SamplesPerPixelIn);
	UE_API void SetFilter(EBakeFilterType FilterTypeIn);
	UE_API void SetTileSize(int TileSizeIn);

	FImageDimensions GetDimensions() const { return Dimensions; }
	bool GetGutterEnabled() const { return bGutterEnabled; }
	int32 GetGutterSize() const { return GutterSize; }
	int32 GetSamplesPerPixel() const { return SamplesPerPixel; }
	EBakeFilterType GetFilter() const { return FilterType; }
	int32 GetTileSize() const { return TileSize; }

	/**
	 * Computes the connected UV triangles and returns an array containing
	 * the mapping from triangle ID to unique UV chart ID. If the mesh
	 * has no UVs, the UVCharts will be initialized to 0.
	 *
	 * @param Mesh the mesh to compute UV charts.
	 * @param MeshUVCharts the triangle ID to UV Chart ID array.
	 */
	static UE_API void ComputeUVCharts(const FDynamicMesh3& Mesh, TArray<int32>& MeshUVCharts);

	/**
	 * Set an a Triangle ID to UV Chart ID array for TargetMesh.
	 * If this is not set, then the baker will compute it as part of Bake().
	 * Since ComputeUVCharts() is non-trivial, this method is intended
	 * to allow a client to externally cache the result of ComputeUVCharts
	 * to minimize the overhead per bake.
	 * 
	 * @param UVChartsIn the TriID to UVChartID map
	 */
	void SetTargetMeshUVCharts(TArray<int32>* UVChartsIn)
	{
		TargetMeshUVCharts = UVChartsIn;
	}

	/** @return the Triangle ID to UV Chart ID mapping */
	const TArray<int32>* GetTargetMeshUVCharts() const
	{
		return TargetMeshUVCharts;
	}

	//
	// Analytics
	//
	struct FBakeAnalytics
	{
		double TotalBakeDuration = 0.0;
		double WriteToImageDuration = 0.0;
		double WriteToGutterDuration = 0.0;
		std::atomic<int64> NumSamplePixels = 0;
		std::atomic<int64> NumGutterPixels = 0;

		void Reset()
		{
			TotalBakeDuration = 0.0;
			WriteToImageDuration = 0.0;
			WriteToGutterDuration = 0.0;
			NumSamplePixels = 0;
			NumGutterPixels = 0;
		}
	};
	FBakeAnalytics BakeAnalytics;

protected:
	/** Evaluate this sample. */
	UE_API void BakeSample(
		FMeshMapTileBuffer& TileBuffer,
		const FMeshMapEvaluator::FCorrespondenceSample& Sample,
		const FVector2d& QueryUVPosition,
		const FVector2i& ImageCoords,
		const FImageOccupancyMap& OccupancyMap);

	/** Initialize evaluation contexts and precompute data for bake evaluation. */
	UE_API void InitBake();

	/** Initialize bake sample default floats and colors. */
	UE_API void InitBakeDefaults();

	/** Initialize filter */
	UE_API void InitFilter();

protected:
	const bool bParallel = true;

	FDynamicMesh3 FlatMesh;
	FMeshSurfaceUVSampler MeshUVSampler;

	FImageDimensions Dimensions = FImageDimensions(128, 128);

	/** @return evaluator ids with the corresponding mode */
	const TArray<int32>& EvaluatorIdsForMode(FMeshMapEvaluator::EAccumulateMode Mode) const
	{
		return BakeAccumulateLists[static_cast<int32>(Mode)];
	}

	/**
	 * If true, the baker will pad the baked content past the UV borders by GutterSize.
	 * This is useful to minimize artifacts when filtering or mipmapping.
	 */
	bool bGutterEnabled = true;

	/** The pixel distance (in texel diagonal length) to pad baked content past the UV borders. */
	int32 GutterSize = 4;

	/** The number of samples to evaluate per pixel. */
	int32 SamplesPerPixel = 1;

	/** The square dimensions for tiled processing of the output image(s). */
	int32 TileSize = 32;

	/** The amount of padding for tiled processing of the output image(s). */
	int32 TilePadding = 2;

	/** The pixel distance around the sample texel to be considered by the filter. [0, TilePadding] */
	int32 FilterKernelSize = 0;

	/** The texture filter type. */
	EBakeFilterType FilterType = EBakeFilterType::BSpline;

	/** Texture filters */
	static UE_API FBoxFilter BoxFilter;
	static UE_API FBSplineFilter BSplineFilter;
	static UE_API FMitchellNetravaliFilter MitchellNetravaliFilter;

	/** Texture filter function */
	using TextureFilterFn = float(*)(const FVector2d& Dist);
	TextureFilterFn TextureFilterEval = nullptr;
	using IsInFilterRegionFn = bool(*)(const FVector2d& Dist);
	IsInFilterRegionFn IsInFilterRegionEval = nullptr;

	template<EBakeFilterType BakeFilterType>
	static float EvaluateFilter(const FVector2d& Dist);

	template<EBakeFilterType BakeFilterType>
	static bool EvaluateIsInFilterRegion(const FVector2d& Dist);

	/** The total size of the temporary float buffer for BakeSample. */
	int32 BakeSampleBufferSize = 0;

	/** The list of evaluators to process. */
	TArray<TSharedPtr<FMeshMapEvaluator, ESPMode::ThreadSafe>> Bakers;

	/** Evaluation contexts for each mesh evaluator. */
	TArray<FMeshMapEvaluator::FEvaluationContext> BakeContexts;

	/** Lists of Bake indices for each accumulation mode. */
	TArray<TArray<int32>> BakeAccumulateLists;

	/** Array of default values/colors per BakeResult. */
	TArray<float> BakeDefaults;
	TArray<FVector4f> BakeDefaultColors;
	
	/** Offsets per Baker into the BakeResults array.*/
	TArray<int32> BakeOffsets;

	/** Offsets per BakeResult into the BakeSample buffer.*/
	TArray<int32> BakeSampleOffsets;

	/** Array of bake result images. */
	TArray<TUniquePtr<TImageBuilder<FVector4f>>> BakeResults;

	/**
	 * Array of TargetMesh triangle ID to UV chart ID mapping.
	 * Can be optionally provided by the client. If not provided,
	 * will be computed as part of the bake.
	 */
	TArray<int32>* TargetMeshUVCharts = nullptr;

	/**
	 * Local Array of TargetMesh triangle ID to UV chart ID mapping.
	 * This will be populated only if not provided by the client via
	 * TargetMeshUVCharts.
	 */
	TArray<int32> TargetMeshUVChartsLocal;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
