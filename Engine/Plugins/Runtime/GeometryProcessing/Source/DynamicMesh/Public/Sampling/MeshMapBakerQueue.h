// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"

namespace UE
{
namespace Geometry
{

/**
 * Multi-producer/single-consumer queue for the ordered processing of data
 * in FMeshMapBaker.
 * 
 * This queue is designed to mediate the aggregation of data into a
 * shared output buffer. The number of data to be aggregated must
 * be provided at initialization. Producer threads then "post" data
 * to the queue and any single thread can temporarily assume the role
 * of consumer to "process" the data.
 */
class FMeshMapBakerQueue
{
private:
	// Align to cache line size and pad our slot data to avoid false sharing.
	static constexpr SIZE_T Alignment = PLATFORM_CACHE_LINE_SIZE;
	struct alignas(Alignment) FSlotData
	{
		std::atomic<void*> Ptr;
	};
	static_assert(sizeof(FSlotData) == Alignment);

	/** Cached size of the queue */
	SIZE_T NumSlots = 0;

	/** The aligned pointer for our atomic data. */
	FSlotData* Slots = nullptr;

	/** The current index to be processed. */
	SIZE_T Current = 0;

	/** Atomic flag to control ownership of consumer. */
	std::atomic_flag Acquired = ATOMIC_FLAG_INIT;

public:
	explicit FMeshMapBakerQueue(SIZE_T Size)
		: NumSlots(Size)
	{
		Slots = new FSlotData[NumSlots];
		for (SIZE_T Idx = 0; Idx < NumSlots; ++Idx)
		{
			Slots[Idx].Ptr = nullptr;
		}
	}
	
	~FMeshMapBakerQueue()
	{
		for (SIZE_T Idx = 0; Idx < NumSlots; ++Idx)
		{
			ensure(Slots[Idx].Ptr.load(std::memory_order_acquire) == nullptr);
		}
		delete[] Slots;
		ensure(Current == NumSlots);
	}

	/**
	 * Post Data pointer into the queue at a given index.
	 *
	 * @param Idx The slot index to update.
	 * @param Data The data to post to the queue.
	 */
	void Post(SIZE_T Idx, void* Data)
	{
		Slots[Idx].Ptr.store(Data, std::memory_order_release);
	}

	/**
	 * Acquire ownership over processing queue data.
	 *
	 * It is the responsibility of the caller to release the ownership
	 * if it was acquired via ReleaseProcessLock().
	 *
	 * @return true if ownership of queue processing was acquired.
	 */
	bool AcquireProcessLock()
	{
		// Only attempt to acquire process lock if data for current index is available.
		void* Data = Current < NumSlots ? Slots[Current].Ptr.load(std::memory_order_relaxed) : nullptr;
		return Data ? !Acquired.test_and_set(std::memory_order_acquire) : false;
	}

	/**
	 * Retrieve the next item in the queue to process. The current index
	 * is only advanced if the item is valid or <bFlush = true>.
	 *
	 * @param bFlush When true, increments the current index regardless of
	 *               the validity of the data. Useful for flushing/clearing
	 *               the contents of the queue.
	 * @return A void* to the data to be processed, nullptr if the data
	 *         for the next item in the queue is not available yet.
	 */
	template<bool bFlush = false>
	void* Process()
	{
		void* Data = Current < NumSlots ? Slots[Current].Ptr.load(std::memory_order_acquire) : nullptr;
		if (Data)
		{
			Slots[Current++].Ptr.store(nullptr, std::memory_order_release);
		}
		else if constexpr (bFlush)
		{
			Current++;
		}
		return Data;
	}

	/**
	 * Release ownership of queue processing.
	 *
	 * This should only be invoked by the thread that successfully
	 * acquired ownership of queue processing via AcquireProcessLock().
	 */
	void ReleaseProcessLock()
	{
		Acquired.clear(std::memory_order_release);
	}

	/** @return true if all items in the queue have been processed. */
	bool IsDone() const
	{
		return Current >= NumSlots;
	}
};

} // end namespace UE::Geometry
} // end namespace UE

