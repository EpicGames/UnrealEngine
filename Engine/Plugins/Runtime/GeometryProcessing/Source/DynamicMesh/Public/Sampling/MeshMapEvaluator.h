// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshSurfaceSampler.h"

namespace UE
{
namespace Geometry
{

class FMeshBaseBaker;

enum class EMeshMapEvaluatorType
{
	Constant,
	Normal,
	Occlusion,
	Curvature,
	Property,
	ResampleImage,
	MultiResampleImage,
	RenderCapture,
	UVShell,
	Height
};

class FMeshMapEvaluator
{
public:
	/** The number of float components of an evaluator result/target. */
	enum class EComponents
	{
		Float1 = 1,
		Float2 = 2,
		Float3 = 3,
		Float4 = 4
	};

	/** Accumulate mode informs the baker how samples should be accumulated into the buffer. */
	enum class EAccumulateMode
	{
		/** Accumulate sample results (valid & default) into the buffer. */
		Add,
		/** Overwrite the previous sample result with the new value into the buffer. */
		Overwrite,
		/** Last mode. Used to count modes. Ensure this is always last. */
		Last
	};

	/** Detail to target mesh correspondence sample. */
	struct FCorrespondenceSample
	{
		FMeshUVSampleInfo BaseSample;
		FVector3d BaseNormal = FVector3d::Zero();

		// The following data has an interpretation which depends on the concrete MeshMapEvaluator/IMeshBakerDetailSampler
		const void* DetailMesh = nullptr; // Arbitrary pointer to the mesh/surface being sampled
		int32 DetailTriID = IndexConstants::InvalidID;
		FVector3d DetailBaryCoords = FVector3d::Zero();
	};

	/**
	 * Evaluator context data to indicate to the baker:
	 * 1. The evaluation functions.
	 * 2. The accumulation method for data samples.
	 * 3. The number of output results/targets for this evaluator.
	 * 4. The data layout of each output result/target for this evaluator.
	 */
	struct FEvaluationContext
	{
		/**
		 * Function pointer to evaluate a mesh correspondence sample into a
		 * float buffer. This function should:
		 * 
		 * - assume that the size of the buffer is correct.
		 * - advance the buffer pointer by the DataLayout stride.
		 * 
		 * @param Out the buffer to populate.
		 * @param Sample the detail/target mesh correspondence sample data.
		 * @param EvalData custom data pointer provided by the evaluation context.
		 */
		using EvaluateFn = void(*)(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);
		EvaluateFn Evaluate = nullptr;

		/**
		 * Function pointer to evaluate the default sample into a
		 * float buffer. The default sample is used:
		 * 
		 * - to clear the output image on initialization.
		 * - for samples for which there is no mesh image occupancy nor correspondence.
		 * 
		 * This function should:
		 * 
		 * - assume that the size of the buffer is correct.
		 * - advance the buffer pointer by the DataLayout stride.
		 * 
		 * @param Out the buffer to populate.
		 * @param Sample the detail/target mesh correspondence sample data.
		 * @param EvalData custom data pointer provided by the evaluation context.
		 */
		using EvaluateDefaultFn = void(*)(float*& Out, void* EvalData);
		EvaluateDefaultFn EvaluateDefault = nullptr;

		/**
		 * Function pointer to evaluate the color representation of
		 * an evaluated sample result.
		 *
		 * This function will be invoked once per pixel after the
		 * evaluated samples have been accumulated. This function
		 * should:
		 *
		 * - assume that the size of the buffer is correct.
		 * - advance the [In] buffer pointer by the DataLayout stride.
		 *
		 * @param DataIdx the index into the DataLayout being processed.
		 * @param In the buffer containing the accumulated sample result.
		 * @param Out the output float4 color to populate
		 * @param EvalData custom data pointer provided by the evaluation context.
		 */
		using EvaluateColorFn = void(*)(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);
		EvaluateColorFn EvaluateColor = nullptr;

		/**
		 * Function pointer to evaluate the single channel representation of
		 * an evaluated sample result.
		 *
		 * This function will be invoked once per pixel channel after the
		 * evaluated samples have been accumulated. This function
		 * should:
		 *
		 * - assume that the size of the buffer is correct.
		 * - advance the [In] buffer pointer by the DataLayout stride.
		 *
		 * @param DataIdx the index into the DataLayout being processed.
		 * @param In the buffer containing the accumulated sample result.
		 * @param Out the output float channel to populate
		 * @param EvalData custom data pointer provided by the evaluation context.
		 */
		using EvaluateChannelFn = void(*)(const int DataIdx, float*& In, float& Out, void* EvalData);
		EvaluateChannelFn EvaluateChannel = nullptr;



		/** Define custom data to be passed to the evaluation function. */
		void* EvalData = nullptr;

		/** Define how the results should be accumulatd by the Baker. */
		EAccumulateMode AccumulateMode = EAccumulateMode::Add;

		/**
		 * The data layout of this evaluator including:
		 * 1. The number of outputs.
		 * 2. The number of floats (stride) per output. [1,4]
		 */
		TArray<EComponents> DataLayout;
	};

	virtual ~FMeshMapEvaluator() = default;

	/**
	 * Invoked at start of bake to setup this evaluator and return
	 * an evaluation context to the baker. The evaluation context
	 * indicates to the baker how the evaluator should be invoked.
	 * See FEvaluationContext for more details.
	 * 
	 * @param Baker the baker that owns this evaluator.
	 * @param Context [out] the evaluation context.
	 */
	virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) = 0;

	/** @return the data layout of the evaluator */
	virtual const TArray<EComponents>& DataLayout() const = 0;

	/** @return the type of evaluator. */
	virtual EMeshMapEvaluatorType Type() const = 0;

protected:
	/**
	 * Write float data to a float buffer and increment.
	 *
	 * @param Out the target float buffer.
	 * @param Data the data to write into the buffer.
	 */
	static void WriteToBuffer(float*& Out, const float Data)
	{
		*Out++ = Data;
	}

	/**
	 * Write FVector2f data to a float buffer and increment.
	 *
	 * @param Out the target float buffer.
	 * @param Data the data to write into the buffer.
	 */
	static void WriteToBuffer(float*& Out, const FVector2f& Data)
	{
		*Out++ = Data.X;
		*Out++ = Data.Y;
	}

	/**
	 * Write FVector3f data to a float buffer and increment.
	 *
	 * @param Out the target float buffer.
	 * @param Data the data to write into the buffer.
	 */
	static void WriteToBuffer(float*& Out, const FVector3f& Data)
	{
		*Out++ = Data.X;
		*Out++ = Data.Y;
		*Out++ = Data.Z;
	}

	/**
	 * Write FVector4f data to a float buffer and increment.
	 * 
	 * @param Out the target float buffer.
	 * @param Data the data to write into the buffer.
	 */
	static void WriteToBuffer(float*& Out, const FVector4f& Data)
	{
		*Out++ = Data.X;
		*Out++ = Data.Y;
		*Out++ = Data.Z;
		*Out++ = Data.W;
	}
};

} // end namespace UE::Geometry
} // end namespace UE
