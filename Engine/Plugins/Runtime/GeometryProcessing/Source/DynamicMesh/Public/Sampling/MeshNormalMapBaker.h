// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Sampling/MeshImageBaker.h"
#include "Sampling/MeshImageBakingCache.h"
#include "Image/ImageBuilder.h"
#include "DynamicMesh/MeshTangents.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FMeshNormalMapBaker : public FMeshImageBaker
{
public:
	virtual ~FMeshNormalMapBaker() = default;

	//
	// Required input data
	//

	const TMeshTangents<double>* BaseMeshTangents;

	FVector3f DefaultNormal = FVector3f(0, 0, 1);

	//
	// Compute functions
	//

	UE_API virtual void Bake() override;

	//
	// Output
	//

	const TUniquePtr<TImageBuilder<FVector3f>>& GetResult() const { return NormalsBuilder; }
	
	TUniquePtr<TImageBuilder<FVector3f>> TakeResult() { return MoveTemp(NormalsBuilder); }

protected:
	TUniquePtr<TImageBuilder<FVector3f>> NormalsBuilder;

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
