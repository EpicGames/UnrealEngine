// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Tuple.h"
#include "Containers/Map.h"
#include "Sampling/MeshMapEvaluator.h"
#include "Sampling/MeshBakerCommon.h"
#include "DynamicMesh/MeshTangents.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * A mesh evaluator for tangent space normals.
 */
class FMeshNormalMapEvaluator : public FMeshMapEvaluator
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMeshNormalMapEvaluator() = default;
	FMeshNormalMapEvaluator(const FMeshNormalMapEvaluator&) = default;
	FMeshNormalMapEvaluator& operator=( const FMeshNormalMapEvaluator& ) = default;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Begin FMeshMapEvaluator interface
	virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	UE_API virtual const TArray<EComponents>& DataLayout() const override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::Normal; }
	// End FMeshMapEvaluator interface

	template <bool bUseDetailNormalMap>
	static void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

	static UE_API void EvaluateDefault(float*& Out, void* EvalData);

	static UE_API void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);
	static UE_API void EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData);

protected:
	// Cached data
	const IMeshBakerDetailSampler* DetailSampler = nullptr;

	using FNormalTexture = IMeshBakerDetailSampler::FBakeDetailNormalTexture;
	using FNormalTextureMap = TMap<const void*, FNormalTexture>;
	FNormalTextureMap DetailNormalMaps;
	
	bool bHasDetailNormalTextures = false;
	const TMeshTangents<double>* BaseMeshTangents = nullptr;

	FVector3f DefaultNormal = FVector3f::UnitZ();

private:
	template <bool bUseDetailNormalMap>
	FVector3f SampleFunction(const FCorrespondenceSample& SampleData) const;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
