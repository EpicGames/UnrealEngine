// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"
#include "Sampling/MeshImageBaker.h"
#include "Sampling/MeshImageBakingCache.h"
#include "Image/ImageBuilder.h"
#include "DynamicMesh/MeshTangents.h"

namespace UE
{
namespace Geometry
{

enum class EOcclusionMapType : uint8
{
	None = 0,
	AmbientOcclusion = 1,
	BentNormal = 2,
	All = 3
};
ENUM_CLASS_FLAGS(EOcclusionMapType);

class FMeshOcclusionMapBaker : public FMeshImageBaker
{
public:
	virtual ~FMeshOcclusionMapBaker() = default;

	//
	// Inputs
	//

	const TMeshTangents<double>* BaseMeshTangents = nullptr;

	//
	// Options
	//

	enum class EDistribution
	{
		Uniform,
		Cosine
	};

	enum class ESpace
	{
		Tangent,
		Object
	};

	EOcclusionMapType OcclusionType = EOcclusionMapType::All;
	int32 NumOcclusionRays = 32;
	double MaxDistance = TNumericLimits<double>::Max();
	double SpreadAngle = 180.0;
	EDistribution Distribution = EDistribution::Cosine;

	// Ambient Occlusion
	double BiasAngleDeg = 15.0;
	double BlurRadius = 0.0;

	// Bent Normal
	ESpace NormalSpace = ESpace::Tangent;


	//
	// Compute functions
	//

	DYNAMICMESH_API virtual void Bake() override;

	//
	// Output
	//

	enum class EResult
	{
		AmbientOcclusion,
		BentNormal
	};

	const TUniquePtr<TImageBuilder<FVector3f>>& GetResult(EResult Result) const
	{
		switch (Result)
		{
		case EResult::AmbientOcclusion:
			return OcclusionBuilder;
		case EResult::BentNormal:
			return NormalBuilder;
		default:
			check(false);
			return OcclusionBuilder;
		}
	}

	TUniquePtr<TImageBuilder<FVector3f>> TakeResult(EResult Result)
	{
		switch (Result)
		{
		case EResult::AmbientOcclusion:
			return MoveTemp(OcclusionBuilder);
		case EResult::BentNormal:
			return MoveTemp(NormalBuilder);
		default:
			check(false);
			return MoveTemp(OcclusionBuilder);
		}
	}

	//
	// Utility
	//

	inline bool WantAmbientOcclusion() const
	{
		return ((OcclusionType & EOcclusionMapType::AmbientOcclusion) == EOcclusionMapType::AmbientOcclusion);
	}

	inline bool WantBentNormal() const
	{
		return ((OcclusionType & EOcclusionMapType::BentNormal) == EOcclusionMapType::BentNormal);
	}

protected:
	TUniquePtr<TImageBuilder<FVector3f>> OcclusionBuilder;
	TUniquePtr<TImageBuilder<FVector3f>> NormalBuilder;
};

} // end namespace UE::Geometry
} // end namespace UE
