// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshMapEvaluator.h"
#include "Templates/Tuple.h"
#include "Misc/EnumClassFlags.h"
#include "DynamicMesh/MeshTangents.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class IMeshBakerDetailSampler;	

enum class EMeshOcclusionMapType : uint8
{
	None = 0,
	AmbientOcclusion = 1,
	BentNormal = 2,
	All = 3
};
ENUM_CLASS_FLAGS(EMeshOcclusionMapType);

/**
 * A mesh evaluator for occlusion data (Ambient Occlusion & Bent Normals).
 */
class FMeshOcclusionMapEvaluator : public FMeshMapEvaluator
{
public:
	enum class EDistribution
	{
		Uniform,
		Cosine
	};

	enum class ESpace
	{
		Tangent,
		Object
	};

	EMeshOcclusionMapType OcclusionType = EMeshOcclusionMapType::All;
	int32 NumOcclusionRays = 32;
	double MaxDistance = TNumericLimits<double>::Max();
	double SpreadAngle = 180.0;
	EDistribution Distribution = EDistribution::Cosine;

	// Ambient Occlusion
	double BiasAngleDeg = 15.0;

	// Bent Normal
	ESpace NormalSpace = ESpace::Tangent;

	using FOcclusionTuple = TTuple<float, FVector3f>;

public:
	// Begin FMeshMapEvaluator interface
	UE_API virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	UE_API virtual const TArray<EComponents>& DataLayout() const override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::Occlusion; }
	// End FMeshMapEvaluator interface

	template <EMeshOcclusionMapType OcclusionType, ESpace NormalSpace>
	static void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

	template <EMeshOcclusionMapType OcclusionType, ESpace NormalSpace>
	static void EvaluateDefault(float*& Out, void* EvalData);

	template <EMeshOcclusionMapType OcclusionType, ESpace NormalSpace>
	static void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);

	template <EMeshOcclusionMapType OcclusionType, ESpace NormalSpace>
	static void EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData);

protected:
	// Cached data
	const IMeshBakerDetailSampler* DetailSampler = nullptr;
	const TMeshTangents<double>* BaseMeshTangents = nullptr;
	double BiasDotThreshold = 0.25;
	TArray<FVector3d> RayDirections;

	static UE_API const FVector3d DefaultTangentNormal;
	static UE_API const FVector3d DefaultObjectNormal;

private:
	template <EMeshOcclusionMapType OcclusionType, ESpace NormalSpace>
	FOcclusionTuple SampleFunction(const FCorrespondenceSample& SampleData, const FVector3d& DefaultNormal);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
