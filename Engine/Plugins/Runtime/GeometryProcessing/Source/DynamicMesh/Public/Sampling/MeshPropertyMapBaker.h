// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Sampling/MeshImageBaker.h"
#include "Sampling/MeshImageBakingCache.h"
#include "Image/ImageBuilder.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

enum class EMeshPropertyBakeType
{
	Position = 1,
	Normal = 2,
	FacetNormal = 3,
	UVPosition = 4,
	MaterialID = 5
};


class FMeshPropertyMapBaker : public FMeshImageBaker
{
public:
	virtual ~FMeshPropertyMapBaker() = default;

	//
	// Options
	//

	EMeshPropertyBakeType Property = EMeshPropertyBakeType::Normal;

	//
	// Compute functions
	//

	UE_API virtual void Bake() override;

	//
	// Output
	//

	const TUniquePtr<TImageBuilder<FVector3f>>& GetResult() const { return ResultBuilder; }

	TUniquePtr<TImageBuilder<FVector3f>> TakeResult() { return MoveTemp(ResultBuilder); }

protected:
	TUniquePtr<TImageBuilder<FVector3f>> ResultBuilder;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
