// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshMapEvaluator.h"
#include "Sampling/MeshBakerCommon.h"
#include "Image/ImageBuilder.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class IMeshBakerDetailSampler;	

enum class EMeshPropertyMapType
{
	Position = 1,
	Normal = 2,
	FacetNormal = 3,
	UVPosition = 4,
	MaterialID = 5,
	VertexColor = 6,
	PolyGroupID = 7
};

/**
 * A mesh evaluator for mesh properties as color data.
 */
class FMeshPropertyMapEvaluator : public FMeshMapEvaluator
{
public:
	EMeshPropertyMapType Property = EMeshPropertyMapType::Normal;

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMeshPropertyMapEvaluator() = default;
	FMeshPropertyMapEvaluator(const FMeshPropertyMapEvaluator&) = default;
	FMeshPropertyMapEvaluator& operator=(const FMeshPropertyMapEvaluator&) = default;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Begin FMeshMapEvaluator interface
	virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	UE_API virtual const TArray<EComponents>& DataLayout() const override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::Property; }
	// End FMeshMapEvaluator interface

	template <bool bUseDetailNormalMap>
	static void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

	static UE_API void EvaluateDefault(float*& Out, void* EvalData);

	static UE_API void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);
	static UE_API void EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData);

protected:
	UE_API FVector4f GetDefaultValue(EMeshPropertyMapType InProperty) const;
	
	// Cached data
	const IMeshBakerDetailSampler* DetailSampler = nullptr;

	using FNormalTexture = IMeshBakerDetailSampler::FBakeDetailNormalTexture;
	using FNormalTextureMap = TMap<const void*, FNormalTexture>;
	FNormalTextureMap DetailNormalMaps;
	
	bool bHasDetailNormalTextures = false;
	FAxisAlignedBox3d Bounds;

	UE_DEPRECATED(5.6, "DefaultValue is deprecated. Please use GetDefaultValue() instead.")
	FVector3f DefaultValue = FVector3f::Zero();
	
private:
	static FVector4f NormalToColor(const FVector3f Normal) 
	{
		return FVector4f((Normal + FVector3f::One()) * 0.5f, 1.0f);
	}

	static FVector4f UVToColor(const FVector2f UV)
	{
		const float X = FMathf::Clamp(UV.X, 0.0, 1.0);
		const float Y = FMathf::Clamp(UV.Y, 0.0, 1.0);
		return FVector4f(X, Y, 0.f, 1.f);
	}

	static FVector4f PositionToColor(const FVector3d Position, const FAxisAlignedBox3d SafeBounds)
	{
		const float X = static_cast<float>((Position.X - SafeBounds.Min.X) / SafeBounds.Width());
		const float Y = static_cast<float>((Position.Y - SafeBounds.Min.Y) / SafeBounds.Height());
		const float Z = static_cast<float>((Position.Z - SafeBounds.Min.Z) / SafeBounds.Depth());
		return FVector4f(X, Y, Z, 1.f);
	}

	template <bool bUseDetailNormalMap>
	FVector4f SampleFunction(const FCorrespondenceSample& SampleData) const;

	FVector4f DefaultValue4f = FVector4f();
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
