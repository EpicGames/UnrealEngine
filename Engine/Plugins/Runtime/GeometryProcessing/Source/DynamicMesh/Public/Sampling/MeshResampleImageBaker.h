// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Sampling/MeshImageBaker.h"
#include "Sampling/MeshImageBakingCache.h"
#include "Image/ImageBuilder.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FMeshResampleImageBaker : public FMeshImageBaker
{
public:
	virtual ~FMeshResampleImageBaker() = default;

	//
	// Required input data
	//

	TFunction<FVector4f(FVector2d)> SampleFunction = [](FVector2d Position) { return FVector4f::Zero(); };
	const FDynamicMeshUVOverlay* DetailUVOverlay = nullptr;

	FVector4f DefaultColor = FVector4f(0, 0, 0, 1);

	//
	// Compute functions
	//

	UE_API virtual void Bake() override;

	//
	// Output
	//

	const TUniquePtr<TImageBuilder<FVector4f>>& GetResult() const { return ResultBuilder; }

	TUniquePtr<TImageBuilder<FVector4f>> TakeResult() { return MoveTemp(ResultBuilder); }

protected:
	TUniquePtr<TImageBuilder<FVector4f>> ResultBuilder;
};



class FMeshMultiResampleImageBaker : public FMeshResampleImageBaker
{
public:

	TMap<int32, TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>> MultiTextures;

	UE_API virtual void Bake() override;

protected:

	UE_API void InitResult();
	UE_API void BakeMaterial(int32 MaterialID);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
