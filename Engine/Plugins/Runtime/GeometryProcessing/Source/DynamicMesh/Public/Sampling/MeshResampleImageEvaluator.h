// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshMapEvaluator.h"
#include "Sampling/MeshBaseBaker.h"
#include "Image/ImageBuilder.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class IMeshBakerDetailSampler;	

/**
 * A mesh evaluator for sampling 2D texture data. 
 */
class FMeshResampleImageEvaluator : public FMeshMapEvaluator
{
public:
	FVector4f DefaultColor = FVector4f(0, 0, 0, 1);

public:
	// Begin FMeshMapEvaluator interface
	UE_API virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	UE_API virtual const TArray<EComponents>& DataLayout() const override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::ResampleImage; }
	// End FMeshMapEvaluator interface

	static UE_API void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

	static UE_API void EvaluateDefault(float*& Out, void* EvalData);

	static UE_API void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);
	static UE_API void EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData);

protected:
	// Cached data
	const IMeshBakerDetailSampler* DetailSampler = nullptr;

private:
	UE_API FVector4f ImageSampleFunction(const FCorrespondenceSample& Sample) const;
};


// TODO: Add support for multiple color maps per mesh in IMeshBakerDetailSampler for proper MultiTexture support.	
/**
 * A mesh evaluator for sampling multiple 2D textures by material ID
 */
class FMeshMultiResampleImageEvaluator : public FMeshResampleImageEvaluator
{
public:
	/** List of textures indexed by material ID. Entries can be null. */
	TArray<TSharedPtr<UE::Geometry::TImageBuilder<FVector4f>, ESPMode::ThreadSafe>> MultiTextures;

	/** The UV channel used to sample the textures. */
	int DetailUVLayer = 0;

public:
	// Begin FMeshMapEvaluator interface
	UE_API virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::MultiResampleImage; }
	// End FMeshMapEvaluator interface

	static UE_API void EvaluateSampleMulti(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

private:
	UE_API FVector4f ImageSampleFunction(const FCorrespondenceSample& Sample) const;

	// Cached data
	int NumMultiTextures = 0;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
