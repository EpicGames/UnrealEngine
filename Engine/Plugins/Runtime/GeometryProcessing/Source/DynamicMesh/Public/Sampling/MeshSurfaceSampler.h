// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

/**
 * Wrapper around a Mesh and UV Overlay that provides UVs triangles as vertices.
 * This allows building a TMeshAABBTree3 for the UV mesh
 */
struct FDynamicMeshUVMesh
{
	const FDynamicMesh3* Mesh = nullptr;
	const FDynamicMeshUVOverlay* UV = nullptr;

	inline bool IsTriangle(int32 TriangleIndex) const
	{
		return UV->IsSetTriangle(TriangleIndex);
	}

	inline bool IsVertex(int32 VertexIndex) const
	{
		return UV->IsElement(VertexIndex);
	}

	inline int32 MaxTriangleID() const
	{
		return Mesh->MaxTriangleID();
	}

	inline int32 TriangleCount() const
	{
		return Mesh->TriangleCount();
	}

	inline int32 MaxVertexID() const
	{
		return UV->MaxElementID();
	}

	inline int32 VertexCount() const
	{
		return UV->ElementCount();
	}

	inline uint64 GetChangeStamp() const
	{
		return Mesh->GetChangeStamp();
	}

	inline FIndex3i GetTriangle(int32 TriangleIndex) const
	{
		return UV->GetTriangle(TriangleIndex);
	}

	inline FVector3d GetVertex(int32 ElementIndex) const
	{
		FVector2f Elem = UV->GetElement(ElementIndex);
		return FVector3d(Elem.X, Elem.Y, 0);
	}

	template<typename VecType>
	inline void GetTriVertices(int32 TriangleIndex, VecType& V0, VecType& V1, VecType& V2) const
	{
		FIndex3i TriIndices = UV->GetTriangle(TriangleIndex);
		V0 = GetVertex(TriIndices.A);
		V1 = GetVertex(TriIndices.B);
		V2 = GetVertex(TriIndices.C);
	}
};


/**
 * Information about a UV sample
 */
struct FMeshUVSampleInfo
{
	// Triangle containing the sample
	int32 TriangleIndex = IndexConstants::InvalidID;

	// 3D vertices
	FIndex3i MeshVertices;
	// 3D triangle
	FTriangle3d Triangle3D { FVector3d::ZeroVector, FVector3d::ZeroVector, FVector3d::ZeroVector };

	// UV overlay vertices
	FIndex3i UVVertices;
	// 2D triangle
	FTriangle2d TriangleUV { FVector2d(0.0,0.0), FVector2d(0.0,0.0), FVector2d(0.0,0.0) };

	// barycentric coords in triangle
	FVector3d BaryCoords = FVector3d::ZeroVector;
	// surface point (lying in Triangle3D)
	FVector3d SurfacePoint = FVector3d::ZeroVector;
};


/** Types of query that FMeshSurfaceUVSampler/TMeshSurfaceUVSampler supports */
enum class EMeshSurfaceSamplerQueryType : uint8
{
	/** Query with arbitrary UV value */
	UVOnly,
	/** Query with given TriangleID and UV that is assumed to lie within that Triangle */
	TriangleAndUV
};

// Hacky base class to avoid 8 bytes of padding after the vtable
class FMeshSurfaceUVSamplerFixLayout
{
public:
	virtual ~FMeshSurfaceUVSamplerFixLayout() = default;
};

/**
 * FMeshSurfaceUVSampler computes FMeshUVSampleInfo's on a given mesh at given UV-space positions, this info can then
 * be used by an external function to compute some application-specific sample.
 *
 * Note: This class is similar to TMeshSurfaceUVSampler (the "old" class) but provides a more convenient API. This class
 * computes the sample info and that's it. The old class computes the sample info internally but also immediately
 * forwards it to an application-specific sampling function which must be provided as a callback and hence has a fixed
 * signature and requires that the sample type is passed as a template parameter. Also the old class is more awkward to
 * use when sampling multiple things, see the documentation of TMeshSurfaceUVSampler for more details.
 */
class FMeshSurfaceUVSampler : public FMeshSurfaceUVSamplerFixLayout
{
public:

	/**
	 * Initialize the sampler.
	 * @param Mesh      Mesh to sample
	 * @param UVOverlay UV overlay of Mesh
	 * @param QueryType Type of query functions we will call. Some queries are simpler and do not require spatial data structures
	 */
	UE_API virtual void Initialize(const FDynamicMesh3* Mesh, const FDynamicMeshUVOverlay* UVOverlay, EMeshSurfaceSamplerQueryType QueryType);

	/**
	 * Fill in the given SampleInfo for the given UV location.
	 * @pre The Initialize function must have been called with EMeshSurfaceSamplerQueryType::UVOnly
	 * @return If the query location is not on a triangle in the Mesh then return false (SampleInfo will be invalid)
	 */
	UE_API virtual bool QuerySampleInfo(const FVector2d& UV, FMeshUVSampleInfo& SampleInfo);

	/**
	 * Fill in the given SampleInfo for the given UV location in the given UVTriangleID.
	 * @return If the query location is not on a triangle in the Mesh then return false (SampleInfo will be invalid)
	 */
	UE_API virtual bool QuerySampleInfo(int32 UVTriangleID, const FVector2d& UV, FMeshUVSampleInfo& SampleInfo);

protected:

	const FDynamicMesh3* Mesh = nullptr;
	const FDynamicMeshUVOverlay* UVOverlay = nullptr;
	EMeshSurfaceSamplerQueryType QueryType = EMeshSurfaceSamplerQueryType::TriangleAndUV;

	bool bUVMeshSpatialInitialized = false;

	// BV tree for finding triangle for a given UV. Not always initialized.
	FDynamicMeshUVMesh UVMeshAdapter;
	TMeshAABBTree3<FDynamicMeshUVMesh> UVMeshSpatial;
	void InitializeUVMeshSpatial();
};

/**
 * Consider using FMeshSurfaceUVSampler instead, it is more flexible and easier to understand!
 *
 * TMeshSurfaceUVSampler computes point samples of the given SampleType at positions on the mesh
 * based on UV-space positions. The standard use case for this class is to compute samples used
 * in building Normal Maps, AO Maps, etc.
 *
 * Note that for UVOnly sample type, an internal UV-space BVTree will be constructed, and each
 * sample will query that to find the UV/3D correspondence. If you already know the TriangleID, 
 * you can use the TriangleAndUV type to avoid the BVTree construction and queries.
 *
 * Note that if you need to sample multiple things, rather than building up an uber-SampleType, you
 * can first compute a sample with SampleType=FMeshUVSampleInfo to find the correspondence information,
 * and then construction additional samplers of type EMeshSurfaceSamplerQueryType::TriangleAndUV,
 * and call CachedSampleUV(), to avoid expensive BVTree constructions and UV-to-3D recalculation.
 */
template<typename SampleType>
class TMeshSurfaceUVSampler : public FMeshSurfaceUVSamplerFixLayout
{
public:

	/**
	 * Configure the sampler.
	 * @param MeshIn mesh to sample
	 * @param UVOverlayIn UV overlay of MeshIn to sample
	 * @param QueryTypeIn type of query functions we will call. Some queries are simpler and do not require spatial data structures.
	 * @param ZeroValueIn the value that is returned if a sample cannot be found
	 * @param SampleValueFunctionIn This function is called to compute the sample at a given UV location. SampleInfo provides the necessary UV/3D correspondence data.
	 */
	virtual void Initialize(
		const FDynamicMesh3* MeshIn, 
		const FDynamicMeshUVOverlay* UVOverlayIn,
		EMeshSurfaceSamplerQueryType QueryTypeIn,
		SampleType ZeroValueIn,
		TUniqueFunction<void(const FMeshUVSampleInfo& SampleInfo, SampleType& SampleValueOut)> SampleValueFunctionIn)
	{
		this->Mesh = MeshIn;
		this->UVOverlay = UVOverlayIn;
		this->ZeroValue = ZeroValueIn;
		this->ValueFunction = MoveTemp(SampleValueFunctionIn);

		// initialize spatial data structure if we need it
		QueryType = QueryTypeIn;
		if (QueryType == EMeshSurfaceSamplerQueryType::UVOnly)
		{
			InitializeBVTree();
		}
	}

	/**
	 * Compute a sample at the given UV location
	 * @return true if valid sample was computed
	 */
	virtual bool SampleUV(const FVector2d& UV, SampleType& ResultOut);

	/**
	 * Compute a sample at the given UV location in the given Triangle
	 * @return true if valid sample was computed
	 */
	virtual bool SampleUV(int32 UVTriangleID, const FVector2d& UV, SampleType& ResultOut);

	/**
	 * Compute a sample at the given UV/3D location specified by CachedSampleInfo, which presumably was produced by previous calls to SampleUV()
	 * @return true if valid sample was computed
	 */
	virtual bool CachedSampleUV(const FMeshUVSampleInfo& CachedSampleInfo, SampleType& ResultOut);

protected:
	EMeshSurfaceSamplerQueryType QueryType = EMeshSurfaceSamplerQueryType::TriangleAndUV;
	bool bUVSpatialValid = false;
	const FDynamicMesh3* Mesh = nullptr;
	const FDynamicMeshUVOverlay* UVOverlay = nullptr;

	TUniqueFunction<void(const FMeshUVSampleInfo& SampleInfo, SampleType& SampleValueOut)> ValueFunction;

	SampleType ZeroValue;

	// BV tree for finding triangle for a given UV. Not always initialized.
	FDynamicMeshUVMesh UVMeshAdapter;
	TMeshAABBTree3<FDynamicMeshUVMesh> UVBVTree;
	void InitializeBVTree();
};



template<typename SampleType>
void TMeshSurfaceUVSampler<SampleType>::InitializeBVTree()
{
	if (bUVSpatialValid)
	{
		return;
	}

	check(UVOverlay);
	
	UVMeshAdapter.Mesh = Mesh;
	UVMeshAdapter.UV = UVOverlay;
	UVBVTree.SetMesh(&UVMeshAdapter, true);

	bUVSpatialValid = true;
}



template<typename SampleType>
bool TMeshSurfaceUVSampler<SampleType>::SampleUV(const FVector2d& UV, SampleType& ResultOut)
{
	check(QueryType == EMeshSurfaceSamplerQueryType::UVOnly);
	check(bUVSpatialValid);

	FRay3d HitRay(FVector3d(UV.X, UV.Y, 100.0), -FVector3d::UnitZ());
	const int32 UVTriangleID = UVBVTree.FindNearestHitTriangle(HitRay);

	return SampleUV(UVTriangleID, UV, ResultOut);
}




template<typename SampleType>
bool TMeshSurfaceUVSampler<SampleType>::SampleUV(int32 UVTriangleID, const FVector2d& UV, SampleType& ResultOut)
{
	check(QueryType == EMeshSurfaceSamplerQueryType::TriangleAndUV);

	FMeshUVSampleInfo Sample;

	Sample.TriangleIndex = UVTriangleID;
	if (Mesh->IsTriangle(Sample.TriangleIndex) == false)
	{
		ResultOut = ZeroValue;
		return false;
	}
	check(UVOverlay->IsSetTriangle(Sample.TriangleIndex));

	Sample.MeshVertices = Mesh->GetTriangle(Sample.TriangleIndex);
	Sample.Triangle3D = FTriangle3d(
		Mesh->GetVertex(Sample.MeshVertices.A),
		Mesh->GetVertex(Sample.MeshVertices.B),
		Mesh->GetVertex(Sample.MeshVertices.C));

	Sample.UVVertices = UVOverlay->GetTriangle(Sample.TriangleIndex);
	Sample.TriangleUV = FTriangle2d(
		(FVector2d)UVOverlay->GetElement(Sample.UVVertices.A),
		(FVector2d)UVOverlay->GetElement(Sample.UVVertices.B),
		(FVector2d)UVOverlay->GetElement(Sample.UVVertices.C));

	Sample.BaryCoords = Sample.TriangleUV.GetBarycentricCoords(UV);
	Sample.SurfacePoint = Mesh->GetTriBaryPoint(Sample.TriangleIndex, Sample.BaryCoords.X, Sample.BaryCoords.Y, Sample.BaryCoords.Z);

	ValueFunction(Sample, ResultOut);

	return true;
}


template<typename SampleType>
bool TMeshSurfaceUVSampler<SampleType>::CachedSampleUV(const FMeshUVSampleInfo& CachedSampleInfo, SampleType& ResultOut)
{
	ValueFunction(CachedSampleInfo, ResultOut);
	return true;
}



} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API 