// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshMapEvaluator.h"
#include "Sampling/MeshBakerCommon.h"

#define UE_API DYNAMICMESH_API

namespace UE::Geometry
{

class IMeshBakerDetailSampler;	

/**
 * A mesh evaluator for mesh properties as color data.
 */
class FMeshUVShellMapEvaluator : public FMeshMapEvaluator
{
public:
	FVector2d TexelSize = FVector2d::One();
	float WireframeThickness = 1.0f;
	FVector4f WireframeColor = FLinearColor::Blue;
	FVector4f ShellColor = FLinearColor::Gray;
	FVector4f BackgroundColor = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);

	int UVLayer = 0;

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMeshUVShellMapEvaluator() = default;
	FMeshUVShellMapEvaluator(const FMeshUVShellMapEvaluator&) = default;
	FMeshUVShellMapEvaluator& operator=(const FMeshUVShellMapEvaluator&) = default;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Begin FMeshMapEvaluator interface
	virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override;

	UE_API virtual const TArray<EComponents>& DataLayout() const override;

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::UVShell; }
	// End FMeshMapEvaluator interface

	static UE_API void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData);

	static UE_API void EvaluateDefault(float*& Out, void* EvalData);

	static UE_API void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData);
	static UE_API void EvaluateChannel(const int DataIdx, float*& In, float& Out, void* EvalData);

protected:
	// Cached data
	const IMeshBakerDetailSampler* DetailSampler = nullptr;

private:
	UE_API FVector4f SampleFunction(const FCorrespondenceSample& SampleData) const;
};

} // end namespace UE::Geometry

#undef UE_API
