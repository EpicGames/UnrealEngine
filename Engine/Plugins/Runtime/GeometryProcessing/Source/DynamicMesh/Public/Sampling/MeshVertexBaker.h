// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshBaseBaker.h"
#include "Sampling/MeshMapEvaluator.h"
#include "Sampling/MeshConstantMapEvaluator.h"
#include "Image/ImageBuilder.h"
#include "Image/ImageDimensions.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FMeshVertexBaker : public FMeshBaseBaker
{
public:
	enum class EBakeMode
	{
		RGBA,       // Bake color data to RGBA
		PerChannel  // Bake scalar data into color channels
	};

	EBakeMode BakeMode = EBakeMode::RGBA;

	/** Evaluator to use for full color bakes */
	TSharedPtr<FMeshMapEvaluator, ESPMode::ThreadSafe> ColorEvaluator;

	/** Evaluators to use for per channel (RGBA) bakes */
	TSharedPtr<FMeshMapEvaluator, ESPMode::ThreadSafe> ChannelEvaluators[4];

	//
	// Analytics
	//
	double TotalBakeDuration = 0.0;

public:	
	//
	// Bake
	//

	/** Process all bakers to generate the image result. */
	UE_API void Bake();

	/** @return the bake result image. */
	UE_API const TImageBuilder<FVector4f>* GetBakeResult() const;

	/** if this function returns true, we should abort calculation */
	TFunction<bool(void)> CancelF = []() { return false; };

	//
	// Parameters
	//

protected:
	/** Template bake implementation. */
	template<EBakeMode ComputeMode>
	static void BakeImpl(void* Data);

protected:
	const bool bParallel = true;

	/** Internally cached image dimensions proportional to the number of unique vertex color elements. */
	FImageDimensions Dimensions;

	/** Bake output image. */
	TUniquePtr<TImageBuilder<FVector4f>> BakeResult;

	/** Internal list of bake evaluators. */
	TArray<FMeshMapEvaluator*> Bakers;

	/** Evaluation contexts for each mesh evaluator. */
	TArray<FMeshMapEvaluator::FEvaluationContext> BakeContexts;

	/** Internal cached default bake data. */
	FVector4f BakeDefaults;

	/** The total size of the temporary float buffer for BakeSample. */
	int32 BakeSampleBufferSize = 0;

	/** Function pointer to internal bake implementation. */
	using BakeFn = void(*)(void* Data);
	BakeFn BakeInternal = nullptr;

	/** Constant evaluator for empty channels. */
	static UE_API FMeshConstantMapEvaluator ZeroEvaluator;
	static UE_API FMeshConstantMapEvaluator OneEvaluator;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
