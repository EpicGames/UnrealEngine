// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sampling/MeshMapEvaluator.h"

namespace UE
{
namespace Geometry
{

// TODO Consolidate this with ERenderCaptureType
enum class ERenderCaptureChannel
{
	BaseColor = 1,
	Roughness = 2,
	Metallic = 4,
	Specular = 8,
	Emissive = 16,
	WorldNormal = 32,
	DeviceDepth = 64,
	CombinedMRS = 128,
	Opacity = 256,
	SubsurfaceColor = 512
};

template <typename DataType>
class FRenderCaptureMapEvaluator : public FMeshMapEvaluator
{
public:

	ERenderCaptureChannel Channel = ERenderCaptureChannel::BaseColor;

	// Analytics
	// TODO Add some analytics to collect

	DataType DefaultResult;
	TFunction<DataType(const FCorrespondenceSample& Sample)> EvaluateSampleCallback;
	TFunction<FVector4f(const int DataIdx, float*& In)> EvaluateColorCallback;

public:
	virtual ~FRenderCaptureMapEvaluator() = default;

	// Begin FMeshMapEvaluator interface
	virtual void Setup(const FMeshBaseBaker& Baker, FEvaluationContext& Context) override
	{
		Context.Evaluate = &EvaluateSample;
		Context.EvaluateDefault = &EvaluateDefault;
		Context.EvaluateColor = &EvaluateColor;
		Context.EvalData = this;
		Context.AccumulateMode = EAccumulateMode::Add;
		Context.DataLayout = DataLayout();
	}

	virtual const TArray<EComponents>& DataLayout() const override {
		static const TArray<EComponents> Layout{(Channel == ERenderCaptureChannel::WorldNormal
			? EComponents::Float3
			: EComponents::Float4) };
		return Layout;
	}

	virtual EMeshMapEvaluatorType Type() const override { return EMeshMapEvaluatorType::RenderCapture; }
	// End FMeshMapEvaluator interface

	static void EvaluateSample(float*& Out, const FCorrespondenceSample& Sample, void* EvalData)
	{
		const FRenderCaptureMapEvaluator* Eval = static_cast<FRenderCaptureMapEvaluator*>(EvalData);
		const DataType SampleResult = Eval->EvaluateSampleCallback(Sample);
		WriteToBuffer(Out, SampleResult);
	}

	static void EvaluateDefault(float*& Out, void* EvalData)
	{
		const FRenderCaptureMapEvaluator* Eval = static_cast<FRenderCaptureMapEvaluator*>(EvalData);
		WriteToBuffer(Out, Eval->DefaultResult);
	}

	static void EvaluateColor(const int DataIdx, float*& In, FVector4f& Out, void* EvalData)
	{
		const FRenderCaptureMapEvaluator* Eval = static_cast<FRenderCaptureMapEvaluator*>(EvalData);
		Out = Eval->EvaluateColorCallback(DataIdx, In);
	}
};


} // end namespace UE::Geometry
} // end namespace UE
