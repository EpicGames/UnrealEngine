// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/IntVector.h"
#include "TriangleTypes.h"
#include "SegmentTypes.h"
#include "BoxTypes.h"
#include "DynamicMesh/InfoTypes.h"

#define UE_API DYNAMICMESH_API


namespace UE
{
namespace Geometry
{


/**
 * FGeoSelectionID provides a pair of 32-bit unsigned integers that can
 * be packed into a 64-bit unsigned integer for use with FGeometrySelection.
 * This is generally intended to be used to encode a mesh geometry ID
 * (eg TriangleID, EdgeID, VertexID) combined with a "Topology ID",
 * eg something like a Face Group ID. However none of this is enforced
 * and so a caller can use these two integers for any purpose.
 * 
 * Note that since the ints are unsigned, IndexConstants::InvalidID
 * is not directly representable (-1 will become positive 0xFFFFFFFF). 
 */
struct FGeoSelectionID
{
	/** Topology ID, stored in upper 32 bits when packed into 64-bits */
	uint32 TopologyID;
	/** Geometry ID, stored in lower 32 bits when packed into 64-bits */
	uint32 GeometryID;

	FGeoSelectionID()
	{
		GeometryID = 0;
		TopologyID = 0;
	}

	/**
	 * Initialize the TopologyID and GeometryID with the given values
	 */
	explicit FGeoSelectionID(uint32 GeometryIDIn, uint32 TopologyIDIn = 0)
	{
		GeometryID = GeometryIDIn;
		TopologyID = TopologyIDIn;
	}

	/**
	 * Initialize the TopologyID and GeometryID by unpacking the 64-bit packed EncodedID
	 */
	explicit FGeoSelectionID(uint64 EncodedID)
	{
		GeometryID = EncodedID & 0x00000000FFFFFFFF;
		TopologyID = (EncodedID & 0xFFFFFFFF00000000) >> 32;
	}

	/**
	 * @return the packed 64-bit representation of [TopologyID, GeometryID]
	 */
	uint64 Encoded() const
	{
		return ((uint64)TopologyID << 32) | (uint64)GeometryID;
	}

	/**
	 * @return a FGeoSelectionID initialized with the given TriangleID, and a TopologyID of 0
	 */
	static FGeoSelectionID MeshTriangle(int32 TriangleID)
	{
		return FGeoSelectionID((uint32)TriangleID, 0);
	}

	/**
	 * @return a FGeoSelectionID initialized with the given VertexID, and a TopologyID of 0
	 */
	static FGeoSelectionID MeshVertex(int32 VertexID)
	{
		return FGeoSelectionID((uint32)VertexID, 0);
	}

	/**
	 * @return a FGeoSelectionID initialized with the given 32-bit encoding of the MeshTriEdgeID, and a TopologyID of 0
	 */
	static FGeoSelectionID MeshEdge(UE::Geometry::FMeshTriEdgeID EdgeKey)
	{
		return FGeoSelectionID(EdgeKey.Encoded());
	}

	/**
	 * @return a FGeoSelectionID initialized with the given TriangleID and GroupID used as the TopologyID
	 */
	static FGeoSelectionID GroupFace(int32 TriangleID, int32 GroupID)
	{
		return FGeoSelectionID((uint32)TriangleID, (uint32)GroupID);
	}

	friend uint32 GetTypeHash(const FGeoSelectionID& TopoKey)
	{
		return ::GetTypeHash(TopoKey.Encoded());
	}
};




/**
 * Type of selected Elements in a FGeometrySelection
 */
enum class EGeometryElementType
{
	/** Mesh Vertices, Polygroup Corners, ... */
	Vertex = 1,
	/** Mesh Edges, Polygroup Edges, ... */
	Edge = 2,
	/** Mesh Triangles, Polygroup Faces, ... */
	Face = 4
};

/**
 * Type of selected Topology in a FGeometrySelection.
 */
enum class EGeometryTopologyType
{
	Triangle = 1,
	Polygroup = 2
};


/**
 * FGeometrySelection represents a subset of geometric elements of a larger
 * object, for example a Mesh (currently the only use case). The main selection
 * is represented via 64-bit unsigned integers. The integers are stored in a TSet
 * for efficient unique adds and removals. No assumptions are made about the values,
 * they could be (eg) mesh indices, IDs of some type, or even pointer values.
 */
struct FGeometrySelection
{
	/** Type of geometric element represented by this selection, if applicable */
	EGeometryElementType ElementType = EGeometryElementType::Face;

	/** Type of geometric topology this selection is defined relative to, if applicable */
	EGeometryTopologyType TopologyType = EGeometryTopologyType::Triangle;

	/** Set of selected items/elements */
	TSet<uint64> Selection;

	/** Map of soft selected vertices and their weights*/
	TMap<uint64, double> SoftSelectedVertices;
	
	/**
	 * @return true if Selection is empty
	 */
	bool IsEmpty() const
	{
		return Selection.Num() == 0;
	}

	/**
	 * @return number of elements in Selection
	 */
	int32 Num() const
	{
		return Selection.Num();
	}

	/**
	 * Clear the Selection (may not release memory)
	 */
	void Reset()
	{
		Selection.Reset();
	}

	/**
	* Initialize the Element and Topology types for this Selection
	*/
	void InitializeTypes(EGeometryElementType ElementTypeIn, EGeometryTopologyType TopologyTypeIn)
	{
		ElementType = ElementTypeIn;
		TopologyType = TopologyTypeIn;
	}

	/**
	 * Initialize the Element and Topology types for this Selection based on another Selection
	 */
	void InitializeTypes(const FGeometrySelection& FromSelection)
	{
		ElementType = FromSelection.ElementType;
		TopologyType = FromSelection.TopologyType;
	}

	/**
	 * @return true if the two selections have the same type
	 */
	bool IsSameType(const FGeometrySelection& OtherSelection) const
	{
		return ElementType == OtherSelection.ElementType && TopologyType == OtherSelection.TopologyType;
	}

	/**
	 * @return a FGeometrySelection containing the soft selected vertices
	 */
	FGeometrySelection GetSoftSelectedVertices() const
	{
		FGeometrySelection Result;
		Result.TopologyType = TopologyType;
		Result.ElementType = EGeometryElementType::Vertex;

		SoftSelectedVertices.GetKeys(Result.Selection);
	
		return Result;
	}
};


/**
 * FGeometrySelectionHitQueryConfig defines the desired settings for a "hit query" on
 * selected meshes/objects. 
 */
struct FGeometrySelectionHitQueryConfig
{
	/** Type of object topology to query */
	EGeometryTopologyType TopologyType = EGeometryTopologyType::Polygroup;
	/** Type of object topological element to query */
	EGeometryElementType ElementType = EGeometryElementType::Face;
	
	/** If true, only "visible" elements are considered, otherwise query may return obscured elements (eg, hidden edges occluded by mesh surface) */
	bool bOnlyVisible = true;

	/**
	 * When false, triangles that are facing away from the camera are not considered.
	 * This can be useful when working with inside-out meshes, where we usually want to select the farther, visible wall.
	 */
	bool bHitBackFaces = false;
};



/**
 * Type of change, relative to a FGeometrySelection
 */
enum class EGeometrySelectionChangeType
{
	/** Elements Added to Selection */
	Add,
	/** Elements Removed from Selection */
	Remove,
	/** Selected Elements replaced with a new set of Selected Elements */
	Replace
};


/**
 * FGeometrySelectionUpdateConfig is passed to various Selection Editing functions/classes
 * to indicate what type of change should be applied to a FGeometrySelection
 * (based on additional parameters, generally)
 */
struct FGeometrySelectionUpdateConfig
{
	EGeometrySelectionChangeType ChangeType = EGeometrySelectionChangeType::Add;
};


/**
 * FGeometrySelectionDelta represents a change to the set of elements in a FGeometrySelection.
 * The delta is ordered, ie if the Delta was to be re-applied, the Removed elements should
 * be removed before the Added elements are added.
 * 
 * (Currently there is no way to swap the order)
 */
struct FGeometrySelectionDelta
{
	/**
	 * Elements removed from a FGeometrySelection during some selection edit
	 */
	TArray<uint64> Removed;
	/**
	 * Elements added to a FGeometrySelection during some selection edit
	 */
	TArray<uint64> Added;

	/** @return true if the Delta is empty (nothing Removed or Added) */
	bool IsEmpty() const { return Removed.Num() == 0 && Added.Num() == 0; }
};



/**
 * 3D Bounding information for a FGeometrySelection
 */
struct FGeometrySelectionBounds
{
	UE::Geometry::FAxisAlignedBox3d WorldBounds = UE::Geometry::FAxisAlignedBox3d::Empty();
};


/**
 * 3D Geometry representing a FGeometrySelection, for example
 * suitable for passing to rendering code, etc
 */
struct FGeometrySelectionElements
{
	TArray<UE::Geometry::FTriangle3d> Triangles;
	TArray<UE::Geometry::FSegment3d> Segments;
	TArray<FVector3d> Points;

	void Reset()
	{
		Triangles.Reset();
		Segments.Reset();
		Points.Reset();
	}
};


struct FGeometrySelectionUpdateResult
{
	bool bSelectionMissed = false;
	bool bSelectionModified = false;

	FGeometrySelectionDelta SelectionDelta;
};

/**
 * FGeometrySelectionEditor is a utility/helper class used for modifying
 * a FGeometrySelection. The various functions can be used to add or remove
 * to the Selection, while also tracking what changed, returned via
 * FGeometrySelectionDelta structs. 
 * 
 * In some cases (eg Polygroup selections on faces and edges), only the TopologyIDs
 * are unique, and the same TopologyID may be paired with an arbitrary TriangleID/EdgeID.
 * FGeometrySelection is currently not aware of this distinction, however it means when
 * (for example) adding to a selection, a new uint64 ID should only be added if the 
 * TopologyID is unique, ie the ElementID bit should be ignored. FGeometrySelectionEditor can
 * do this, by enabling TopologyIDFiltering in the various setup/config functions.
 * 
 * Note, however, this means that FGeometrySelectionEditor.IsSelected() must be used
 * to determine whether an ID is selected, rather than Selection.Contains()
 * 
 */
class FGeometrySelectionEditor
{
public:
	/**
	 * Initialize the Editor with the given Selection. The
	 * TargetSelectionIn must live longer than the FGeometrySelectionEditor
	 */
	UE_API void Initialize(
		FGeometrySelection* TargetSelectionIn,
		const FGeometrySelectionHitQueryConfig& QueryConfigIn,
		bool bEnableTopologyIDFiltering);

	/**
	 * Initialize the Editor with the given Selection. The
	 * TargetSelectionIn must live longer than the FGeometrySelectionEditor
	 */
	UE_API void Initialize(
		FGeometrySelection* TargetSelectionIn,
		bool bEnableTopologyIDFiltering);

	/** @return the Element Type of the Target Selection */
	EGeometryElementType GetElementType() const { return TargetSelection->ElementType; }
	/** @return the Topology Type of the Target Selection */
	EGeometryTopologyType GetTopologyType() const { return TargetSelection->TopologyType; }

	/** @return the active configuration for this Selection Editor, eg what type of element/topology is being selected. This is redundant w/ the above fields.  */
	const FGeometrySelectionHitQueryConfig& GetQueryConfig() const { return QueryConfig; }

	bool GetIsTopologyIDFilteringEnabled() const { return bEnableTopologyIDFiltering; }

	/** 
	 * Update the active QueryConfig for this SelectionEditor. This is necessary to keep it in sync w/ the TargetSelection, if
	 * the ElementType or TopologyType are modified. This perhaps should be revisited as they basically
	 * always need to be the same...
	 */
	UE_API void UpdateQueryConfig(const FGeometrySelectionHitQueryConfig& NewConfig, bool bEnableTopologyIDFilteringIn);

	/** @return true if the given ID is currently selected in the Target Selection */
	UE_API bool IsSelected(uint64 ID) const;

	/** Clear the Target Selection and return change information in DeltaOut */
	UE_API void ClearSelection(FGeometrySelectionDelta& DeltaOut);

	/** Remove ID from Selection */
	UE_API bool RemoveFromSelection(uint64 ID);

	/** Access the Selection object this Editor is modifying */
	const FGeometrySelection& GetSelection() const { return *TargetSelection; }

	/** Add the items to the Target Selection */
	bool Select(uint64 ID)
	{
		if (IsSelected(ID) == false)
		{
			TargetSelection->Selection.Add(ID);
			return true;
		}
		return false;
	}

	/** Add the items in the List to the Target Selection and return change information in DeltaOut */
	template<typename ListType>
	bool Select(const ListType& List, FGeometrySelectionDelta& DeltaOut)
	{
		int32 NumAdded = 0;
		for (uint64 ID : List)
		{
			if (IsSelected(ID) == false)
			{
				TargetSelection->Selection.Add(ID);
				DeltaOut.Added.Add(ID);
				NumAdded++;
			}
		}
		return (NumAdded > 0);
	}

	/** Remove the items in the List from the Target Selection and return change information in DeltaOut */
	template<typename ListType>
	bool Deselect(const ListType& List, FGeometrySelectionDelta& DeltaOut)
	{
		int32 TotalRemoved = 0;
		for (uint64 ID : List)
		{
			if (RemoveFromSelection(ID))
			{
				DeltaOut.Removed.Add(ID);
				TotalRemoved++;
			}
		}
		return (TotalRemoved > 0);
	}

	/** Replace the current set of selected items with those in the NewSelection, and return change information in DeltaOut */
	UE_API bool Replace(const FGeometrySelection& NewSelection, FGeometrySelectionDelta& DeltaOut);


protected:
	FGeometrySelection* TargetSelection = nullptr;
	FGeometrySelectionHitQueryConfig QueryConfig;

	bool bEnableTopologyIDFiltering = false;
	UE_API uint64 RemapToExistingTopologyID(uint64 ID, bool& bFoundOut) const;
};


/**
 * FGeometrySelectionPreview is a combined FGeometrySelection and FGeometrySelectionEditor.
 * The purpose of this class is to support things like hover-highlighting, where the Selection 
 * system needs to be queried but the result is going to be discarded. FGeometrySelectionPreview can 
 * only be constructed based on a parent FGeometrySelectionEditor, from which it derives it's
 * configuration, which is then how the rest of the system knows what geometry to provide as a preview/etc
 */
class FGeometrySelectionPreview : public FGeometrySelectionEditor
{
public:
	FGeometrySelection PreviewSelection;

	FGeometrySelectionPreview(const FGeometrySelectionEditor& ActiveEditor)
	{
		PreviewSelection.InitializeTypes(ActiveEditor.GetSelection());
		Initialize(&PreviewSelection, ActiveEditor.GetQueryConfig(), ActiveEditor.GetIsTopologyIDFilteringEnabled());
	}

	bool IsEmpty() const { return PreviewSelection.IsEmpty(); }
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
