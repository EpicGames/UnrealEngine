// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Selections/GeometrySelection.h"
#include "TriangleTypes.h"
#include "SegmentTypes.h"
#include "FrameTypes.h"
#include "Polygroups/PolygroupSet.h"

PREDECLARE_GEOMETRY(class FDynamicMesh3);

namespace UE
{
namespace Geometry
{

class FColliderMesh;
class FGroupTopology;
struct FGroupTopologySelection;
	
enum class EEnumerateSelectionMapping : uint8
{
	None         = 0,
	
	FacesToFaces = 1 << 0,
	FacesToEdges = 1 << 1,

	Default = FacesToFaces,
};
ENUM_CLASS_FLAGS(EEnumerateSelectionMapping);
	
enum class EEnumerateSelectionConversionParams
{
	/** Converts to components whose constituent parts are all contained in the initial selection. */
	ContainSelection,
	/** Converts to components where any of its constituent parts are included in the initial selection. */
	ExpandSelection
};
	
/**
 * Test if SelectionA and SelectionB are the same selection.
 * This is currently relatively expensive on Polygroup selections due to how they are encoded
 * @return true if the selections are iddentical
 */
DYNAMICMESH_API bool AreSelectionsIdentical(
	const FGeometrySelection& SelectionA,
	const FGeometrySelection& SelectionB);



/**
 * Assuming that the uint64 values in the GeometrySelection are encoded FGeoSelectionID's, 
 * find the item that has a matching TopologyID, ignoring the GeometryID.
 * @param FoundValue the hash value found 
 * @return true if an item was found
 */
DYNAMICMESH_API bool FindInSelectionByTopologyID(
	const FGeometrySelection& GeometrySelection,
	uint32 TopologyID,
	uint64& FoundValue);


/**
 * Update a FGeometrySelection (via the Editor) of mesh vertices/edges/triangles
 * based on a raycast
 * @param UpdateConfig type of update (add/remove/replace) and other update controls
 * @param ResultOut selection delta and information on hit/miss are returned here
 */
DYNAMICMESH_API void UpdateTriangleSelectionViaRaycast(
	const FColliderMesh* ColliderMesh,
	FGeometrySelectionEditor* Editor,
	const FRay3d& LocalRay,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut);

/**
 * Update a FGeometrySelection (via the Editor) of mesh polygroup faces/edges/corners,
 * based on a raycast
 * @param UpdateConfig type of update (add/remove/replace) and other update controls
 * @param ResultOut selection delta and information on hit/miss are returned here
 */
DYNAMICMESH_API void UpdateGroupSelectionViaRaycast(
	const FColliderMesh* ColliderMesh,
	const FGroupTopology* GroupTopology,
	FGeometrySelectionEditor* Editor,
	const FRay3d& LocalRay,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut);


/**
 * Update a FGeometrySelection (via the Editor)
 * @param ChangeType type of change to make (add/remove/replace)
 * @param NewIDs set of new IDs to use to update the selection
 * @param Delta selection delta will be stored here, if non-nullptr is provided
 */
DYNAMICMESH_API bool UpdateSelectionWithNewElements(
	FGeometrySelectionEditor* Editor,
	EGeometrySelectionChangeType ChangeType,
	const TArray<uint64>& NewIDs,
	FGeometrySelectionDelta* DeltaOut = nullptr);


/**
 * Prefer pointer version for ApplyTransform in EnumerateTriangleSelectionVertices
 */
UE_DEPRECATED(5.5, "EnumerateTriangleSelectionVertices with ApplyTransform passed a reference is deprecated. "
				   "Please use the function of the same name which takes ApplyTransform as a pointer instead")
DYNAMICMESH_API bool EnumerateTriangleSelectionVertices(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FTransform& ApplyTransform,
	TFunctionRef<void(uint64, const FVector3d&)> VertexFunc
);

/**
 * Call VertexFunc for each selected Mesh element (vertex/edge/tri) in MeshSelection.
 * ApplyTransform will be applied to Vertex Positions before calling VertexFunc
 *
 * For Triangles: includes all 3 points which make up each selected triangle
 * For Edges: includes both vertex endpoints of each selected edge
 * For Vertices: includes each selected vertex
 */
DYNAMICMESH_API bool EnumerateTriangleSelectionVertices(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FTransform* ApplyTransform,
	TFunctionRef<void(uint64, const FVector3d&)> VertexFunc
);

/**
 * Call VertexFunc for each selected Mesh element (vertex/edge/tri) in the set
 * of polygroup faces/edges/corners specified by GroupSelection (relative to GroupTopology parameter)
 * ApplyTransform will be applied to Vertex Positions before calling VertexFunc
 *
 * For Faces: enumerates all vertices in the selected polygroup(s), including internal, non-Corner vertices
 * For Edges/Boundaries: enumerates the TriVerts which make up the selected PolyEdge(s)
 * For Verts/Corners: returns the TriVert that corresponds to the selected Corner(s)
 *
 * For rendering purposes, use EnumeratePolygroupSelectionElements
 */
DYNAMICMESH_API bool EnumeratePolygroupSelectionVertices(
	const FGeometrySelection& GroupSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FTransform& ApplyTransform,
	TFunctionRef<void(uint64, const FVector3d&)> VertexFunc
);


/**
 * Call TriangleFunc for each mesh TriangleID included in MeshSelection.
 * TriangleFunc may be called multiple times for the same TriangleID.
 * This will forward to EnumerateTriangleSelectionTriangles() or 
 * EnumeratePolygroupSelectionTriangles() depending on the selection topology type.
 * If UseGroupSet and MeshSelection is for polygroups, the default Mesh group layer will be used.
 */
DYNAMICMESH_API bool EnumerateSelectionTriangles(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> TriangleFunc,
	const UE::Geometry::FPolygroupSet* UseGroupSet = nullptr
);
/**
 * Call TriangleFunc for each mesh TriangleID included in MeshSelection.
 * For Edges, both connected edges are included.
 * For Vertices, all triangles in the vertex one-ring are included.
 */
DYNAMICMESH_API bool EnumerateTriangleSelectionTriangles(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> TriangleFunc
);
/**
 * Call TriangleFunc for each mesh TriangleID included in MeshSelection, where MeshSelection has polygroup topology.
 * For Polygroup Faces, all triangles in the face are included.
 * For Polygroup Edges, currently all triangles in any group adjacent to the edge
 * For Polygroup Corners, currently all triangles in any group touching the corner
 */
DYNAMICMESH_API bool EnumeratePolygroupSelectionTriangles(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FPolygroupSet& GroupSet,
	TFunctionRef<void(int32)> TriangleFunc
);
/**
 * Call EdgeFunc for each mesh EdgeID included in MeshSelection.
 * EdgeFunc may be called multiple times for the same EdgeID.
 * This will forward to EnumerateTriangleSelectionEdges() or 
 * EnumeratePolygroupSelectionEdges() depending on the selection topology type.
 * If UseGroupSet and MeshSelection is for polygroups, the default Mesh group layer will be used.
 * For polygroups - all edges and vertices within the polygroup are included, not exclusively Group Boundaries or Group Corners
 */
DYNAMICMESH_API bool EnumerateSelectionEdges(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> EdgeFunc,
	const UE::Geometry::FPolygroupSet* UseGroupSet = nullptr
);
/**
 * Call EdgeFunc for each mesh EdgeID included in MeshSelection.
 * For Triangles, all 3 edges are included.
 * For Edges, all selected edges are included
 * For Vertices, all edges touching each selected vertex are included.
 */
DYNAMICMESH_API bool EnumerateTriangleSelectionEdges(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32)> EdgeFunc
);
/**
 * Call EdgeFunc for each mesh EdgeID included in MeshSelection, where MeshSelection has polygroup topology.
 * For Polygroup Faces, all edges (including internal, non-border edges) in the polygroup are included.
 * For Polygroup Borders, includes all mesh edges which are a part of the selected GroupEdge(s)/BorderEdge(s)
 * For Polygroup Corners, includes all mesh edges in all GroupEdge(s) which touch the selected Corner(s)
 *
 * For rendering purposes, use EnumeratePolygroupSelectionElements
 */
DYNAMICMESH_API bool EnumeratePolygroupSelectionEdges(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FPolygroupSet& GroupSet,
	TFunctionRef<void(int32)> EdgeFunc
);

/**
 * Version of EnumeratePolygroupSelectionEdges which uses GroupTopology
 *
* Call EdgeFunc for each mesh EdgeID included in MeshSelection, where MeshSelection has polygroup topology.
 * For Polygroup Faces, all edges (including internal, non-border edges) in the polygroup are included.
 * For Polygroup Borders, includes all mesh edges which are a part of the selected GroupEdge(s)/BorderEdge(s)
 * For Polygroup Corners, includes all mesh edges in all GroupEdge(s) which touch the selected Corner(s)
 */
DYNAMICMESH_API bool EnumeratePolygroupSelectionEdges(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FGroupTopology& GroupTopology,
	TFunctionRef<void(int32)> EdgeFunc
);	
	
/** Prefer EnumerateTriangleSelectionElements with Flags parameter. */
DYNAMICMESH_API bool EnumerateTriangleSelectionElements(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32, const FVector3d&)> VertexFunc,
	TFunctionRef<void(int32, const FSegment3d&)> EdgeFunc,
	TFunctionRef<void(int32, const FTriangle3d&)> TriangleFunc,
	const FTransform* ApplyTransform = nullptr,
	const bool bMapFacesToEdgeLoops = false
);


/**
 * Enumerates the renderable geometry for a given MeshSelection when in Triangle Topology mode.
 *
 * Calls VertexFunc/EdgeFunc/TriangleFunc for the vertices/edges/triangles
 * identified by MeshSelection. Since a MeshSelection only stores vertices
 * edges, or triangles, but not combined, only one of these functions will be
 * invoked during a call to this function.
 *
 * This function employs the component specific variants (ex.
 * EnumerateTriangleSelectionTriangles) but the overall intention of this function
 * is to collect the renderable geometry representations of the selection.
 * In triangle topology mode only, there is not a difference between 
 * all selection elements and the desired rendered elements. However, this function
 * is still used for render representation for consistency and clarity.
 * @param ApplyTransform if non-null, transform is applied to the 3D geometry
 */
DYNAMICMESH_API bool EnumerateTriangleSelectionElements(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	TFunctionRef<void(int32, const FVector3d&)> VertexFunc,
	TFunctionRef<void(int32, const FSegment3d&)> EdgeFunc,
	TFunctionRef<void(int32, const FTriangle3d&)> TriangleFunc,
	const FTransform* ApplyTransform = nullptr,
	const EEnumerateSelectionMapping Flags = EEnumerateSelectionMapping::Default
);

/** Prefer EnumeratePolygroupSelectionElements with Flags parameter. */
DYNAMICMESH_API bool EnumeratePolygroupSelectionElements(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	TFunctionRef<void(int32, const FVector3d&)> VertexFunc,
	TFunctionRef<void(int32, const FSegment3d&)> EdgeFunc,
	TFunctionRef<void(int32, const FTriangle3d&)> TriangleFunc,
	const FTransform* ApplyTransform = nullptr,
	const bool bMapFacesToEdgeLoops = false
);
/**
 * Enumerates the renderable geometry for a given MeshSelection when in Polygroup Topology mode.
 *
 * Calls VertexFunc/EdgeFunc/TriangleFunc for the vertices/edges/triangles
 * identified by MeshSelection. Since a MeshSelection only stores vertices
 * edges, or triangles, but not combined, only one of these functions will be
 * invoked during a call to this function.
 *
 * This function differs from the component specific variants (ex.
 * EnumeratePolygroupSelectionTriangles) by only collecting the renderable
 * geometry representations of the selection. For instance, a polygroup
 * face may only want to render the triangles and border edges, while
 * EnumeratePolygroupSelectionTriangles would collect all triangles and
 * edges within that polygroup face. This is the difference between render
 * representation and component conversion.
 */
DYNAMICMESH_API bool EnumeratePolygroupSelectionElements(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	TFunctionRef<void(int32, const FVector3d&)> VertexFunc,
	TFunctionRef<void(int32, const FSegment3d&)> EdgeFunc,
	TFunctionRef<void(int32, const FTriangle3d&)> TriangleFunc,
	const FTransform* ApplyTransform = nullptr,
	const EEnumerateSelectionMapping Flags = EEnumerateSelectionMapping::Default
);


/**
 * Convert a MeshSelection with Polygroup topology type to a FGroupTopologySelection
 */
DYNAMICMESH_API bool ConvertPolygroupSelectionToTopologySelection(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	FGroupTopologySelection& TopologySelectionOut
);


/**
 * Convert Triangle IDs to target Selection type
 */
DYNAMICMESH_API bool InitializeSelectionFromTriangles(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	TArrayView<const int> Triangles,
	FGeometrySelection& SelectionOut);


/**
 * Convert Selection from one type to another, based on geometry/topology types in FromSelectionIn and ToSelectionOut.
 * 
 * The following table describes the conversions, the FromSelectionIn/ToSelectionOut type are rows/columns respectively:
 *
 *   ================================================================
 *                 To:    Triangle               Polygroup           
 *   From:                Vertex  Edge    Face   Vertex  Edge    Face
 *   ----------------------------------------------------------------
 *   Triangle Vertex      1       1       7      4#      7#       7#   
 *   Triangle Edge        1       1       8      4#      1#       8#   
 *   Triangle Face        1       1       1      4#      3#       2#  
 *   Polygroup Vertex     6#      9#      9      1       7#       7#   
 *   Polygroup Edge       5#      1#      8      1#      1        8#   
 *   Polygroup Face       1#      1#      1      1#      1#       1   
 *   ================================================================
 *
 *   Key:
 *   .  These conversions are not implemented... yet? GroupTopology is ignored
 *   1  - The implementation is obvious/unambiguous
 *   2  - Polygroup faces containing any input triangle are selected
 *   3  - Polygroup edges containing any input triangle edge are selected, but
 *                 polygroup edges containing only input triangle vertices are not.
 *   4  - Polygroup corners coinciding with any input triangle face/edge/vertex are selected
 *   5  - All mesh vertices along the polygroup edge are selected
 *   6  - All mesh vertices coinciding with polygroup corners are selected
 *   7  - All Elements which immediately touch any input vertex/corner are selected
 *   8  - All triangles or faces on either side of the any input edge are selected
 *			PolyEdge->TriFace includes all triangles in both groups which the edge borders
 *   9  - All edges in each PolyEdge with touch any input Corner are selected
 *		  All triangles in each PolyFace which touch any input Corner are selected
 *   #  indicates GroupTopology must not be null for this combination. If this symbol is missing GroupTopology is ignored
 *
 * @return true if conversion is supported and was computed successfully, return false otherwise
 */
UE_DEPRECATED(5.5, "This version of ConvertSelection is deprecated, please use the version with the EENumerateSelectionConversionParams parameter going forward.")	
DYNAMICMESH_API bool ConvertSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& FromSelectionIn,
	FGeometrySelection& ToSelectionOut);

DYNAMICMESH_API bool ConvertSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& FromSelectionIn,
	FGeometrySelection& ToSelectionOut,
	const EEnumerateSelectionConversionParams ConversionParams);

/**
 * Convert the given MeshSelection to a list of Triangles and Vertices into the Mesh,
 * which can be used to represent a selection of overlay elements. This is always possible
 * since any FGeometrySelection can be represented as an overlay element selection because
 * any overlay element can be represented as a (Triangle,Vertex) pair.
 *
 * If TriangleVertexSelectionIncidentToEdgeSelection is not null and MeshSelection is an Edge Selection it will
 * be set to a Vertex selection with Triangle Topology corresponding to the vertices touched by the edge selection. This
 * is useful when users expect an edge selection to behave similarly to the incident vertex selection.
 *
 * @note it is not necessarily the case that all vertices of triangles in TrianglesOut will be in VerticesOut.
 * @return false if the MeshSelection topology type is not Triangle and true otherwise
 */
DYNAMICMESH_API bool ConvertTriangleSelectionToOverlaySelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGeometrySelection& MeshSelection,
	TSet<int>& TrianglesOut,
	TSet<int>& VerticesOut,
	FGeometrySelection* TriangleVertexSelectionIncidentToEdgeSelection = nullptr);

/**
 * Convert the given MeshSelection to a list of Triangles and Vertices into the Mesh,
 * which can be used to represent a selection of overlay elements. This is always possible
 * since any FGeometrySelection can be represented as an overlay element selection because
 * any overlay element can be represented as a (Triangle,Vertex) pair.
 * 
 * For Polygroup Faces, all triangles in the face are included.
 * For Polygroup Edges, all triangles in any group adjacent to the edge are included.
 * For Polygroup Corners, all triangles in any group touching the corner are included.
 * See ConvertPolygroupSelectionToIncidentOverlaySelection for a similar function which only includes triangles
 * immediately incident to the polygroup element
 * 
 * @return false if the MeshSelection topology type is not Polygroup and true otherwise
 */
DYNAMICMESH_API bool ConvertPolygroupSelectionToOverlaySelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FPolygroupSet& GroupSet,
	const FGeometrySelection& MeshSelection,
	TSet<int>& TrianglesOut,
	TSet<int>& VerticesOut);

/**
 * Like ConvertPolygroupSelectionToOverlaySelection but only includes overlay elements that are immediately incident to
 * Polygroup Vertices/Edges.
 *
 * If TriangleVertexSelectionIncidentToEdgeOrVertexSelection is not null and MeshSelection is an Edge or Vertex
 * selection it will be set to a Vertex selection with Triangle Topology corresponding to the vertices touched by the
 * edge selection. This is useful when users expect an edge selection to behave similarly to the incident vertex
 * selection.
 *
 * @return false if the MeshSelection topology type is not Polygroup and true otherwise
 */
DYNAMICMESH_API bool ConvertPolygroupSelectionToIncidentOverlaySelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology& GroupTopology,
	const FGeometrySelection& MeshSelection,
	TSet<int>& TrianglesOut,
	TSet<int>& VerticesOut,
	FGeometrySelection* TriangleVertexSelectionIncidentToEdgeOrVertexSelection = nullptr);


/**
 * Select all elements of the provided Mesh and GroupTopology that pass the provided SelectionIDPredicate, 
 * and store in the output AllSelection. The type of elements selected is defined by the existing configured
 * type of the AllSelection parameter. 
 * @param GroupTopology precomputed group topology for Mesh, can be passed as null for EGeometryTopologyType::Triangle selections
 * @return true if AllSelection had a known geometry/topology type pair and was populated
 */
DYNAMICMESH_API bool MakeSelectAllSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	TFunctionRef<bool(FGeoSelectionID)> SelectionIDPredicate,
	FGeometrySelection& AllSelection);

/**
 * Expand the input ReferenceSelection to include all "connected" elements and return in AllConnectedSelection.
 * The type of selected element is defined by ReferenceSelection.
 * @param GroupTopology precomputed group topology for Mesh, can be passed as null for EGeometryTopologyType::Triangle selections
 * @param SelectionIDPredicate only elements that pass this filter will be expanded "to"  (but elements of ReferenceSelection that fail the filter will still be included in output)
 * @param IsConnectedPredicate this function determines if "A" should be considered connected to "B", ie can "expand" along that connection
 * @return true if ReferenceSelection had a known geometry/topology type pair and AllConnectedSelection was populated
 */
DYNAMICMESH_API bool MakeSelectAllConnectedSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& ReferenceSelection,
	TFunctionRef<bool(FGeoSelectionID)> SelectionIDPredicate,
	TFunctionRef<bool(FGeoSelectionID A, FGeoSelectionID B)> IsConnectedPredicate,
	FGeometrySelection& AllConnectedSelection);

/**
 * Create a selection of the elements adjacent to the "Border" of the given ReferenceSelection and return in BoundaryConnectedSelection.
 * The type of selected element is defined by ReferenceSelection.
 * Currently "adjacency" is defined as "included in the one-ring of the boundary vertices of the ReferenceSelection", ie first the 
 * vertices on boundary edges are found, and then their one-rings are enumerated. Note that this will include "inside" and "outside" adjacent elements,
 * and for vertices, the boundary vertices will still also be included. The main purpose of this function is to implement expand/contract selection
 * operations, which would typically involve first finding the boundary-connected set and then using CombineSelectionInPlace to modify the original selection.
 * @param GroupTopology precomputed group topology for Mesh, can be passed as null for EGeometryTopologyType::Triangle selections
 * @param SelectionIDPredicate only elements that pass this filter will be expanded "to"  (but elements of ReferenceSelection that fail the filter will still be included in output)
 * @return true if ReferenceSelection had a known geometry/topology type pair and BoundaryConnectedSelection was populated
 */
DYNAMICMESH_API bool MakeBoundaryConnectedSelection(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection& ReferenceSelection,
	TFunctionRef<bool(FGeoSelectionID)> SelectionIDPredicate,
	FGeometrySelection& BoundaryConnectedSelection);

/**
 * Given a selection, return the vertex IDs of the vertices on the boundary of this selection. A selected vertex 
 * is considered to be on the boundary either if it is on the actual mesh boundary (for an open mesh) or it is
 * connected to a triangle element that is not part of the selection (i.e. if the vertex has a neighbor vertex 
 * not in selection for a selection of type EGeometryElementType::Vertex, or an adjacent edge not in selection
 * for a selection of type EGeometryElementType::Edge, an adjacent triangle not in the selection for a selection 
 * of type EGeometryElementType::Face).
 * 
 * For selections of type EGeometryTopologyType::Polygroup, the results are equivalent to first converting the
 * selection to corresponding EGeometryTopologyType::Triangle selection and then finding the boundary vertices.
 * This gives the intuitive result for face selections, but may or may not be what is desired for polygroup 
 * vertex/edge selections, because vertices/edges that seem to be on the interior of the polygroup selection 
 * may be considered border vertices if the tesselation is such that they are adjacent to unselected triangle
 * vertices/edges.
 * 
 * @param GroupTopology Must not be null for selections of type EGeometryTopologyType::Polygroup
 * @param BorderVidsOut Output vertex IDs of border vertices
 * @param CurVerticesOut Output vertex IDs of all vertices in the current selection
 * @return true if successful. For instance, could fail if GroupTopology was null for a EGeometryTopologyType::Polygroup selection
 */
DYNAMICMESH_API bool GetSelectionBoundaryVertices(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FGroupTopology* GroupTopology,
	const UE::Geometry::FGeometrySelection& ReferenceSelection,
	TSet<int32>& BorderVidsOut, TSet<int32>& CurVerticesOut);

/**
 * Given a EGeometryTopologyType::Polygroup selection, return the corner IDs of the polygroup corners on the
 * boundary of the selection. A selected corner is considered to be on the boundary either if it is on the actual 
 * mesh boundary (for an open mesh) or it is connected to an element that is not part of the selection (i.e. if
 * there is a neighbor corner not in the selection for a selection of type EGeometryElementType::Vertex, or an
 * adjoining edge not in the selection for a selection of type EGeometryElementType::Edge, or an adjoining group not
 * in the selection for a selection of type EGeometryElementType::Face).
 * 
 * Selection must be of type EGeometryTopologyType::Polygroup, and GroupTopology must not be null.
 *
 * @param GroupTopology Must not be null
 * @param BorderCornerIDsOut Output corner IDs of border corners
 * @param CurCornerIDsOut Output corner IDs of all corners included in the current selection
 * @return true if successful. For instance, could fail if GroupTopology was null
 */
DYNAMICMESH_API bool GetSelectionBoundaryCorners(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const UE::Geometry::FGroupTopology* GroupTopology,
	const UE::Geometry::FGeometrySelection& ReferenceSelection,
	TSet<int32>& BorderCornerIDsOut, TSet<int32>& CurCornerIDsOut);



enum class EGeometrySelectionCombineModes : uint8
{
	Add,
	Subtract,
	Intersection
};


/**
 * Combine the elements of SelectionA and SelectionB using the provided CombineMode, and store the result in SelectionA.
 * @return true if the selectins were compatible (ie both the same type) and of supported geometry/topology type.
 */
DYNAMICMESH_API bool CombineSelectionInPlace(
	FGeometrySelection& SelectionA,
	const FGeometrySelection& SelectionB, 
	EGeometrySelectionCombineModes CombineMode );



/**
 * Compute a 3D Frame suitable for use as a 3D transform gizmo position/orientation
 * for the given MeshSelection
 */
DYNAMICMESH_API bool GetTriangleSelectionFrame(
	const FGeometrySelection& MeshSelection,
	const UE::Geometry::FDynamicMesh3& Mesh,
	FFrame3d& SelectionFrameOut);


/**
 * Computes and smooths per-triangle integer values across a specified region of a mesh.
 *
 * Each triangle in the mesh is assigned an initial value via the user-provided `ValueForTriangleFunc`.
 * Triangles that are on the boundary of the mesh or on the boundary of the specified `InsideTriangles` region are assigned a value of zero.
 * Triangle values are then adjusted within the region to ensure that each triangle's value does not exceed one more than the minimum value of its neighboring triangles.
 * This produces a smoothly varying integer field that respects the boundary constraints.
 * Note the field varies with topological distance (triangle connectivity), not distance along the surface or Euclidean distance
 *
 * @param TriangleROI          Triangles to consider
 * @param LevelForTriangleFunc Function that takes a triangle ID and returns the maximum value for that triangle
 * @param OutTriangleLevels    Output array of triangle values
 */
DYNAMICMESH_API void TaperPerTriangleValues(const UE::Geometry::FDynamicMesh3& Mesh,
	const TSet<int32>& TriangleROI,
	TFunction<int32(int32)> ValueForTriangleFunc,
	TArray<int>& OutTriangleValues);

}
}
