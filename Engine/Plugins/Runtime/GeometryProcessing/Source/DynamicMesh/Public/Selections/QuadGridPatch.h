// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IndexTypes.h"
#include "Templates/Function.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * FQuadGridPatch represents a set of quads, formed by pairs of triangles, arranged in a 2D grid pattern.
 * U is the "across"/column direction and "V" is the "up"/row direction, and if you were thinking about it in terms
 * of a piece of graph paper, (U=0,V=0) would be in the bottom-left.
 * 
 * Each row of Vertices is stored, as well as each row of Quads. In the case of a Quad grid that is a loop,
 * the last vertex in each row is a duplicate of the first. Note, however, that there is currently no explicit 
 * tracking of whether or not there is a Loop in U or V - this must be tracked externally. 
 * 
 * Generally the quad triangles will not be assumed to form any consistent pattern. This complicates some of
 * the functions however, and perhaps this assumption could be optionally provided in the future
 */
class FQuadGridPatch
{
public:
	int NumVertexColsU = 0;
	int NumVertexRowsV = 0;

	/** NumVertexRowsV rows of NumVertexColsU VertexIDs, may contain repeated element if the patch forms a loop */
	TArray<TArray<int32>> VertexSpans;

	/** Quads stored as pairs of triangle indices, (NumVertexRowsV-1) rows of (NumVertexColsU-1) */
	TArray<TArray<FIndex2i>> QuadTriangles;


	int NumVertexCols() const { return NumVertexColsU; }
	int NumVertexRows() const { return NumVertexRowsV; }
	int NumQuadCols() const { return NumVertexColsU-1; }
	int NumQuadRows() const { return NumVertexRowsV-1; }

	bool IsEmpty() const { return NumVertexColsU == 0 || NumVertexRowsV == 0; }

	/**
	 * Initialize an Nx1 grid from a list of quads (represented as a pair of triangles) and the row of vertices on the "bottom" (ie first span)
	 */
	UE_API void InitializeFromQuadStrip(const FDynamicMesh3& Mesh, const TArray<FIndex2i>& SequentialQuadsIn, const TArray<int32>& FirstVertexSpan );

	/**
	 * Initialize an NxM grid from a list of quads (represented as a pair of triangles) and corresponding rows of vertices. 
	 * For N quad-rows, there must be N+1 vertex spans. 
	 * @return false if the input data is detected to be invalid, in this case the patch is also cleared
	 */
	UE_API bool InitializeFromQuadPatch(const FDynamicMesh3& Mesh, const TArray<TArray<FIndex2i>>& QuadRowsIn, const TArray<TArray<int32>>& VertexSpansIn);

	/**
	 * Reverse the order of the vertex/quad rows, ie "flip" vertically
	 */
	UE_API void ReverseRows();

	/**
	 * Append the rows of NextPatch to the rows of the current QuadGrid.
	 * NextPatch must have the same number of vertex-columns as this Patch.
	 * Requires that the last vertex-row of this Patch be the same as the first
	 * vertex-row of NextPatch. 
	 * @param bChecked if true, function verifies requirements and returns false if a mismatch is detected
	 * @return true if append succeeded
	 */
	UE_API bool AppendQuadPatchRows(FQuadGridPatch&& NextPatch, bool bChecked);


	/** @return the total number of quads in the grid */
	int32 GetNumQuads() const { return (NumVertexRowsV-1) * (NumVertexColsU-1); }

	/** @return a list of all the triangles in the grid */
	UE_API void GetAllTriangles(TArray<int32>& AllTrianglesOut) const;

	/** 
	 * Map a row and column index pair into a single int32. 11 Bits are available for the row, and 20 for the column 
	 */
	int32 EncodeRowColumnIndex(int32 Row, int32 Column ) const
	{
		return (Row << 20) | Column;
	}
	/**
	 * Recover row and column indices from an int32 encoded via EncodeRowColumnIndex
	 */
	FIndex2i DecodeRowColumnIndex(int32 Encoded) const
	{
		return FIndex2i( (Encoded >> 20), Encoded & 0xFFFFF );
	}

	/**
	 * In cases like assigning UVs or Normals to a QuadGridPatch, we want to assign a unique element
	 * to each of the vertices in the grid, and then generate "element triangles" that index into those
	 * new elements. To do this we need to know the (row,column) pair ("Vertex Span Index") for each triangle vertex. 
	 * This function returns a 3-tuple of vertex span indices for a given quad triangle.
	 * The function EncodeRowColumnIndex() is used to map the (row,column) to a single integer, 
	 * and callers can use DecodeRowColumnIndex() to recover the (row,column)
	 * 
	 * @param QuadRow the row of quads the requested triangle is in
	 * @param QuadIndex the "column" index of the quad containing the request triangle
	 * @param TriIndex the index of the triangle in the quad, only 0 and 1 are valid
	 * @return tuple of encoded (row,column) vertex span indices, in the same ordering as the identified triangle
	 */
	UE_API FIndex3i GetQuadTriMappedToSpanIndices(const FDynamicMesh3& Mesh, int QuadRow, int QuadIndex, int TriIndex) const;


	/**
	 * Call ApplyFunc for each quad, passing it's grid index and triangle pair
	 */
	void ForEachQuad(TFunctionRef<void(int32 QuadRow, int32 QuadCol, FIndex2i QuadTris)> ApplyFunc) const
	{
		for ( int32 Row = 0; Row < NumVertexRowsV-1; ++Row )
		{
			for (int32 Column = 0; Column < NumVertexColsU - 1; ++Column)
			{
				ApplyFunc(Row, Column, QuadTriangles[Row][Column]);
			}
		}
	}

	/**
	 * Return vertices in specified Column
	 */
	UE_API bool GetVertexColumn(int32 ColumnIndex, TArray<int32>& VerticesOut) const;

	/**
	 * Find column that contains vertex, or InvalidID
	 */
	UE_API int32 FindColumnIndex(int32 VertexID) const;

	/**
	 * Construct a new QuadGridPatch that is a subset of the current QuadGridPatch
	 * The values (Start|End)Quad(Row|Col) define the sub-patch, the values are inclusive,
	 * so passing (1,2,1,2) defines a 2x2 quad patch
	 */
	UE_API void GetSubPatchByQuadRange(int StartQuadRow, int EndQuadRow, int StartQuadCol, int EndQuadCol, FQuadGridPatch& PatchOut ) const;
	

	/**
	 * Split the Columns of the QuadGridPatch into new QuadGridPatches where the columns
	 * are grouped via the PredicateFunc. 
	 */
	UE_API void SplitColumnsByPredicate( 
		TFunctionRef<bool(int32 QuadColumn0, int32 QuadColumn1)> PredicateFunc,
		TArray<FQuadGridPatch>& SplitPatchesOut) const;
	


	/**
	 * Compute the opening angle between the two quads in Column0 and Column1 of the given Row.
	 * This is based on the face normals of the two triangles connected to the edge between Column0 and Column1,
	 * so any non-planarity of the quads is ignored.
	 * @return unsigned opening angle in degrees in range [0,180], where 0 means quads are parallel
	 */
	UE_API double GetQuadOpeningAngleDeg(const FDynamicMesh3& Mesh, int32 Column0, int32 Column1, int32 Row) const;

};




} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
