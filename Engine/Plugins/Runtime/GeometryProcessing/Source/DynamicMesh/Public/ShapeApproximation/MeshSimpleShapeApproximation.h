// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ShapeApproximation/SimpleShapeSet3.h"

#define UE_API DYNAMICMESH_API

class FProgressCancel;

namespace UE {
namespace Geometry {

/**
 * EDetectedSimpleShapeType is used to identify auto-detected simple shapes for a mesh/etc
 */
enum class EDetectedSimpleShapeType
{
	/** Object is not a simple shape */
	None = 0,
	/** Object has been identified as a sphere */
	Sphere = 2,
	/** Object has been identified as a box */
	Box = 4,
	/** Object has been identified as a capsule */
	Capsule = 8,
	/** Object has been identified as a Convex */
	Convex = 16
};



/**
 * FMeshSimpleShapeApproximation can calculate various "simple" shape approximations for a set of meshes,
 * by fitting various primitives/hulls/etc to each mesh. The assumption is that the input mesh(es) are
 * already partitioned into pieces.
 *
 * There are various Generate_X() functions which apply different strategies, generally to fit a containing
 * simple shape or hull to the mesh. However in addition to these explicit strategies, input meshes that 
 * are very close to approximations of spheres/boxes/capsules (ie basically meshed versions of these primitives)
 * can be identified and used directly, skipping the fitting process.
 *
 */
class FMeshSimpleShapeApproximation
{
public:
	
	//
	// configuration parameters
	//

	/** Should spheres be auto-detected */
	bool bDetectSpheres = true;
	/** Should boxes be auto-detected */
	bool bDetectBoxes = true;
	/** Should capsules be auto-detected */
	bool bDetectCapsules = true;
	/** Should convex be auto-detected */
	bool bDetectConvexes = true;

	/** minimal dimension of fit shapes, eg thickness/radius/etc (currently only enforced in certain cases) */
	double MinDimension = 0.0;

	/** should hulls be simplified as a post-process */
	bool bSimplifyHulls = true;
	/** target number of triangles when simplifying 3D convex hulls */
	int32 HullTargetFaceCount = 50;
	/** simplification tolerance for 3D convex hulls, if > 0, applies *after* hull target face count is reached to futher simplify */
	double HullGeometricToleranceAfterTarget = 0.;

	/** simplification tolerance when simplifying 2D (swept/projected) convex hulls; note HullGeometricToleranceAfterTarget is used instead for 3D hulls */
	double HullSimplifyTolerance = 1.0;

	/** Whether to use apply an edge-length based simplification to the input before running convex decompositions. Useful for very dense input meshes where the decomposition will be slow to compute. */
	bool bDecompositionPreSimplifyWithEdgeLength = false;
	/** If enabled by the above bool flag, pre-simplify input geometry to this edge length before computing convex decompositions. */
	double DecompositionPreSimplifyEdgeLength = 1.0;

	/** How many convex pieces to target per mesh when creating convex decompositions. Ignored if < 1. If ErrorTolerance or ProtectNegativeSpace are used, can create fewer pieces. */
	int32 ConvexDecompositionMaxPieces = 1;
	/** Whether to use the above Max Pieces to drive the convex decomposition. Otherwise, will allow the error tolerances / negative space protection settings to drive the number of pieces generated. */
	bool bUseConvexDecompositionMaxPieces = true;
	/** 
	 * How much additional decomposition decomposition + merging to do, as a fraction of max pieces.  Larger values can help better-cover small features, while smaller values create a cleaner decomposition with less overlap between hulls. 
	 * Note: Not used if bConvexDecompositionProtectNegativeSpace is true.
	 */
	float ConvexDecompositionSearchFactor = .5;
	/** Error tolerance to guide convex decomposition (in cm); we stop adding new parts if the volume error is below the threshold.  For volumetric errors, value will be cubed. */
	double ConvexDecompositionErrorTolerance = 0;
	/** Minimum part thickness for convex decomposition (in cm); hulls thinner than this will be merged into adjacent hulls, if possible. */
	double ConvexDecompositionMinPartThickness = .1;
	/** Whether to use 'navigation-driven' convex decomposition -- using NegativeSpaceTolerance and NegativeSpaceMinRadius to define space that the decomposition hulls cannot occupy */
	bool bConvexDecompositionProtectNegativeSpace = false;

	/** Negative space closer to the input than this tolerance distance can be filled in */
	double NegativeSpaceTolerance = 3;
	/** Minimum radius of negative space to protect; tunnels with radius smaller than this could be filled in */
	double NegativeSpaceMinRadius = 10;
	/** Whether to ignore negative space that is not accessible by traversing from the convex hull (via paths w/ radius of at least Negative Space Tolerance) */
	bool bIgnoreInternalNegativeSpace = true;

	bool bUseExactComputationForBox = false;

	/** Level Set Grid resolution along longest axis */
	int32 LevelSetGridResolution = 10;

	//
	// setup/initialization
	//


	/**
	 * Initialize internal mesh sets. This also detects/caches the precise simple shape fits controlled
	 * by bDetectSpheres/etc above, so those cannot be modified without calling InitializeSourceMeshes() again.
	 * The TSharedPtrs are stored, rather than making a copy of the input meshes
	 * @param bIsParallelSafe if true
	 */
	UE_API void InitializeSourceMeshes(const TArray<const FDynamicMesh3*>& InputMeshSet);


	//
	// approximation generators
	//

	/**
	 * Fit containing axis-aligned boxes to each input mesh and store in ShapeSetOut
	 */
	UE_API void Generate_AlignedBoxes(FSimpleShapeSet3d& ShapeSetOut);

	/**
	 * Fit containing minimal-volume oriented boxes to each input mesh and store in ShapeSetOut
	 */
	UE_API void Generate_OrientedBoxes(FSimpleShapeSet3d& ShapeSetOut, FProgressCancel* Progress = nullptr);

	/**
	 * Fit containing minimal-volume spheres to each input mesh and store in ShapeSetOut
	 */
	UE_API void Generate_MinimalSpheres(FSimpleShapeSet3d& ShapeSetOut);

	/**
	 * Fit containing approximate-minimum-volume capsules to each input mesh and store in ShapeSetOut
	 * @warning the capsule is fit by first fitting a line to the vertices, and then containing the points, so the fit can deviate quite a bit from a truly "minimal" capsule
	 */
	UE_API void Generate_Capsules(FSimpleShapeSet3d& ShapeSetOut);

	/**
	 * Calculate 3D Convex Hulls for each input mesh and store in ShapeSetOut.
	 * Each convex hull is stored as a triangle mesh, and optionally simplified if bSimplifyHulls=true
	 */
	UE_API void Generate_ConvexHulls(FSimpleShapeSet3d& ShapeSetOut, FProgressCancel* Progress = nullptr);

	/**
	 * Calculate multiple 3D Convex Hulls for each input mesh and store in ShapeSetOut.
	 * Each convex hull is stored as a triangle mesh, and optionally simplified if bSimplifyHulls=true
	 */
	UE_API void Generate_ConvexHullDecompositions(FSimpleShapeSet3d& ShapeSetOut, FProgressCancel* Progress = nullptr);


	/** Type/Mode for deciding 3D axis to use in Generate_ProjectedHulls() */
	enum class EProjectedHullAxisMode
	{
		/** Use Unit X axis */
		X = 0,
		/** Use Unit Y axis */
		Y = 1,
		/** Use Unit Z axis */
		Z = 2,
		/** Use X/Y/Z axis with smallest axis-aligned-bounding-box dimension */
		SmallestBoxDimension = 3,
		/** Compute projected hull for each of X/Y/Z axes and use the one that has the smallest volume  */
		SmallestVolume = 4
	};

	/**
	 * Calculate Projected Convex Hulls for each input mesh and store in ShapeSetOut.
	 * A Projected Hull is computed by first projecting all the mesh vertices to a plane, computing a 2D convex hull polygon,
	 * and then sweeping the polygon in 3D to contain all the mesh vertices. The 2D convex hull polygons
	 * are optionally simplified if bSimplifyHulls=true.
	 * @param AxisMode which axis to use for the planar projection
	 */
	UE_API void Generate_ProjectedHulls(FSimpleShapeSet3d& ShapeSetOut, EProjectedHullAxisMode AxisMode);

	UE_API void Generate_LevelSets(FSimpleShapeSet3d& ShapeSetOut, FProgressCancel* Progress = nullptr);
	

	/**
	 * Fit containing axis-aligned box, oriented box, capsule, and sphere to each input mesh, and
	 * store the one with smallest volume in ShapeSetOut
	 */
	UE_API void Generate_MinVolume(FSimpleShapeSet3d& ShapeSetOut);



protected:
	TArray<const FDynamicMesh3*> SourceMeshes;

	struct FSourceMeshCache
	{
		EDetectedSimpleShapeType DetectedType = EDetectedSimpleShapeType::None;

		FSphere3d DetectedSphere;
		FOrientedBox3d DetectedBox;
		FCapsule3d DetectedCapsule;
	};
	TArray<FSourceMeshCache> SourceMeshCaches;


	UE_API void DetectAndCacheSimpleShapeType(const FDynamicMesh3* SourceMesh, FSourceMeshCache& CacheOut);
	UE_API bool GetDetectedSimpleShape(const FSourceMeshCache& Cache,
		FSimpleShapeSet3d& ShapeSetOut, FCriticalSection& ShapeSetLock);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
