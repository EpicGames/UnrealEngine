// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "FrameTypes.h"
#include "SphereTypes.h"
#include "OrientedBoxTypes.h"
#include "CapsuleTypes.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE
{
	namespace Geometry
	{
		/**
		 * Detect if input Mesh is a meshed approximation of an analytic Sphere, and if so return best guess in SphereOut.
		 * Fits a sphere to input points with several rounds of incremental improvement, then measures chordal deviation of edge midpoints.
		 * @param RelativeDeviationTol Scaled by sphere diameter. The allowed difference in the distance from edge midpoints to the surface of the sphere vs the ideal distance for an edge of the same length.
		 * @param MaxAngleRangeDegrees Maximum angle difference in vectors from center of sphere to the endpoints of any surface edge. Controls how coarsely tessellated a sphere can be before it is not considered a sphere.
		 * @return true if mesh is a Sphere and SphereOut is initialized
		 */
		bool DYNAMICMESH_API IsSphereMesh(const FDynamicMesh3& Mesh, FSphere3d& SphereOut, double RelativeDeviationTol = 0.025, double MaxAngleRangeDegrees = 72);


		/**
		 * Detect if input Mesh is a meshed box, and if so return analytic box in BoxOut.
		 * Clusters face planes, looking to find 6 unique planes with normals grouped into 3 opposite-direction pairs.
		 * If this configuraiton is found, computing minimal box is trivial.
		 * @param AngleToleranceDeg normals are allowed to deviate by this amount and still be considered coplanar
		 * @param PlaneDistanceTolerance planes with the same normal are allowed to be this far apart and still be considered coplanar
		 * @return true if mesh is a Box and BoxOut is initialized
		 */
		bool DYNAMICMESH_API IsBoxMesh(const FDynamicMesh3& Mesh, FOrientedBox3d& BoxOut, double AngleToleranceDeg = 0.1, double PlaneDistanceTolerance = 1e-2);


		/**
		 * Detect if input Mesh is a meshed approximation of an analytic Capsule, and if so return best guess in CapsuleOut.
		 * Fits a capsule to input points, then measures chordal deviation of edge midpoints.
		 * @param RelativeDeviationTol Scaled by capsule diameter. The allowed difference in the distance from edge midpoints to the surface of the capsule vs the ideal distance for an edge of the same length.
		 * @param MaxAngleRangeDegrees Maximum angle range an edge can cover over the cylinder or the spherical endcaps. Controls how coarsely tessellated a capsule can be before it is not considered a capsule.
		 * @return true if mesh is a Capsule and CapsuleOut is initialized
		 */
		bool DYNAMICMESH_API IsCapsuleMesh(const FDynamicMesh3& Mesh, FCapsule3d& CapsuleOut, double RelativeDeviationTol = 0.025, double MaxAngleRangeDegrees = 72);

	}
}