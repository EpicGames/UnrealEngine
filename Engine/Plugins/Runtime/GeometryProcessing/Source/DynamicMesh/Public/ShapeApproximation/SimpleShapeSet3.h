// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "OrientedBoxTypes.h"
#include "SegmentTypes.h"
#include "CapsuleTypes.h"
#include "SphereTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Spatial/DenseGrid3.h"
#include "TransformTypes.h"
#include "TransformSequence.h"

#define UE_API DYNAMICMESH_API


namespace UE {
namespace Geometry {


/**
 * Supported/known types of Simple Shapes
 */
enum class ESimpleShapeType
{
	Sphere = 2,
	Box = 4,
	Capsule = 8,
	Convex = 16,
	LevelSet = 32
};
ENUM_CLASS_FLAGS(ESimpleShapeType);


/**
 * FSphereShape is a 3D sphere
 */
class FSphereShape3d
{
public:
	FSphere3d Sphere;

	FSphereShape3d() = default;

	FSphereShape3d(const FSphere3d& SphereIn)
		: Sphere(SphereIn)
	{
	}

	ESimpleShapeType GetShapeType() const { return ESimpleShapeType::Sphere; }
};


/**
 * FBoxShape is a 3D oriented box
 */
struct FBoxShape3d
{
	FOrientedBox3d Box;

	FBoxShape3d() = default;

	FBoxShape3d(const FOrientedBox3d& BoxIn)
		: Box(BoxIn)
	{
	}

	ESimpleShapeType GetShapeType() const { return ESimpleShapeType::Box; }
};


/**
 * FCapsuleShape is a 3D oriented capsule/sphyl
 */
struct FCapsuleShape3d
{
	FCapsule3d Capsule;

	FCapsuleShape3d() = default;

	FCapsuleShape3d(const FCapsule3d& CapsuleIn)
		: Capsule(CapsuleIn)
	{
	}

	ESimpleShapeType GetShapeType() const { return ESimpleShapeType::Capsule; }
};


/**
 * FConvexShape is a 3D convex hull, currently stored as a triangle mesh
 */
struct FConvexShape3d
{
	FDynamicMesh3 Mesh;

	FConvexShape3d() = default;

	FConvexShape3d(const FDynamicMesh3& MeshIn)
		: Mesh(MeshIn)
	{
	}

	FConvexShape3d(FDynamicMesh3&& MeshIn)
		: Mesh(MoveTemp(MeshIn))
	{
	}

	ESimpleShapeType GetShapeType() const { return ESimpleShapeType::Convex; }
};

/**
 * FLevelSetShape is a 3D signed distance function sampled on a regular grid
 */

struct FLevelSetShape3d
{
	FTransform3d GridTransform;
	FDenseGrid3f Grid;
	float CellSize;

	FLevelSetShape3d() = default;

	FLevelSetShape3d(const FTransform3d& GridTransform, const FDenseGrid3f& Grid, float CellSize) :
		GridTransform(GridTransform),
		Grid(Grid),
		CellSize(CellSize)
	{
	}

	FLevelSetShape3d(FTransform3d&& GridTransform, FDenseGrid3f&& Grid, float CellSize) :
		GridTransform(MoveTemp(GridTransform)),
		Grid(MoveTemp(Grid)),
		CellSize(CellSize)
	{
	}

	ESimpleShapeType GetShapeType() const { return ESimpleShapeType::LevelSet; }

	FOrientedBox3d GetGridBox() const
	{
		const UE::Geometry::FVector3i& Dims = Grid.GetDimensions();
		const FVector3d BoxExtents = (0.5 * CellSize) * FVector3d(Dims[0], Dims[1], Dims[2]);
		const FFrame3d BoxFrame(GridTransform.GetTranslation(), GridTransform.GetRotation());
		return FOrientedBox3d(BoxFrame, BoxExtents);
	}
};


/**
 * FSimpleShapeSet stores a set of simple geometry shapes useful for things like collision detection/etc.
 * Various functions set-processing operations are supported.
 */
struct FSimpleShapeSet3d
{
	TArray<FSphereShape3d> Spheres;
	TArray<FBoxShape3d> Boxes;
	TArray<UE::Geometry::FCapsuleShape3d> Capsules;
	TArray<FConvexShape3d> Convexes;
	TArray<FLevelSetShape3d> LevelSets;

	/** @return total number of elements in all sets */
	int32 TotalElementsNum() const { return Spheres.Num() + Boxes.Num() + Capsules.Num() + Convexes.Num() + LevelSets.Num(); }

	/**
	 * Append elements of another shape set
	 */
	UE_API void Append(const FSimpleShapeSet3d& OtherShapeSet);


	/**
	 * Append elements of another shape set with given transform applied
	 */
	UE_API void Append(const FSimpleShapeSet3d& OtherShapeSet, const FTransform3d& Transform);


	/**
	 * Append elements of another shape set with given transforms applied
	 */
	UE_API void Append(const FSimpleShapeSet3d& OtherShapeSet, const TArray<FTransform3d>& TransformSequence);

	/**
	 * Append elements of another shape set with given transforms applied
	 */
	UE_API void Append(const FSimpleShapeSet3d& OtherShapeSet, const FTransformSequence3d& TransformSequence);

	/**
	 * Remove any of the elements that are fully contained in larger elements
	 */
	UE_API void RemoveContainedGeometry();

	/**
	 * Sort the elements by volume and then discard all but the largest MaximumCount elements
	 */
	UE_API void FilterByVolume(int32 MaximumCount);

	struct FMergeShapesSettings
	{
		/** Whether to use the negative space protection settings when merging collision shapes */
		bool bMergeShapesProtectNegativeSpace = false;

		/** Negative space closer to the input than this tolerance distance can be filled in */
		double NegativeSpaceTolerance = 3;
		/** Minimum radius of negative space to protect; tunnels with radius smaller than this could be filled in */
		double NegativeSpaceMinRadius = 10;
		/** Whether to ignore negative space that is not accessible by traversing from the convex hull (via paths w/ radius of at least Negative Space Tolerance) */
		bool bIgnoreInternalNegativeSpace = true;
	};
	// Attempt to reduce the number of simple collision shapes by merging neighboring shapes, while still optionally protecting negative space
	UE_API bool MergeShapes(int32 MergeAboveCount, const FMergeShapesSettings& MergeSettings);

	/**
	 * Transform shape elements. This will be a best-effort as if there is non-uniform scaling only Convexes can be transformed correctly
	 */
	UE_API void ApplyTransform(const FTransform3d& Transform);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
