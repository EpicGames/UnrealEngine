// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/UniquePtr.h"
#include "Solvers/ConstrainedMeshSolver.h"
#include "Solvers/MeshLaplacian.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Curve/DynamicGraph3.h"

namespace UE
{
	namespace MeshDeformation
	{
		using namespace UE::Geometry;

		/**
		*  Solves the linear system for p_vec
		*
		*         ( Transpose(L) * L   + (0  0      )  ) p_vec = source_vec + ( 0              )
		*		  (                      (0 lambda^2)  )                      ( lambda^2 c_vec )
		*
		*   where:  L := laplacian for the mesh,
		*           source_vec := Transpose(L)*L mesh_vertex_positions
		*           lambda := weights
		*           c_vec := constrained positions
		*
		* Expected Use:
		*
		*   // Create Deformation Solver from Mesh
		*   TUniquePtr<IConstrainedMeshSolver>  MeshDeformer = ConstructConstrainedMeshDeformer(ELaplacianWeightScheme::ClampedCotangent, DynamicMesh);
		*
		*   // Add constraints.
		*   for..
		*   {
		*   	int32 VtxId = ..; double Weight = ..; FVector3d TargetPos = ..;  bool bPostFix = ...;
		*   	MeshDeformer->AddConstraint(VtxId, Weight, TargetPos, bPostFix);
		*   }
		*
		*   // Solve for new mesh vertex locations
		*   TArray<FVector3d> PositionBuffer;
		*   MeshDeformer->Deform(PositionBuffer);
		*
		*   // Update Mesh? for (int32 VtxId : DynamicMesh.VertexIndices()) DynamicMesh.SetVertex(VtxId, PositionBuffer[VtxId]);
		*   ...
		*
		*   // Update constraint positions.
		*   for ..
		*   {
		*   	int32 VtxId = ..;  FVector3d TargetPos = ..; bool bPostFix = ...;
		*	    MeshDeformer->UpdateConstraintPosition(VtxId, TargetPos, bPostFix);
		*   }
		*
		*   // Solve for new vertex locations.
		*   MeshDeformer->Deform(PositionBuffer);
		*   // Update Mesh?
		*/
		TUniquePtr<UE::Solvers::IConstrainedMeshSolver> DYNAMICMESH_API ConstructConstrainedMeshDeformer(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& DynamicMesh);


		/**
		 * Interface for a mesh with only edges and vertices
		 */
		//class IVertexEdgeGraph
		//{
		//public:
		//	virtual const FVector3d& GetVertex(int) const = 0;
		//	virtual int VertexCount() const = 0;
		//	virtual int MaxVertexID() const = 0;
		//	virtual FRefCountVector::IndexEnumerable VertexIndicesItr() const = 0;
		//	virtual FSmallListSet::MappedValueEnumerable VtxVerticesItr(int VID) const = 0;
		//	virtual int GetVtxEdgeCount(int VID) const = 0;

		//	// Required for FVertexLinearization
		//	virtual bool IsBoundaryVertex(int VID) const = 0;
		//};

		/**
		 * Construct a Mesh Deformer object for the given generic mesh that uses Biharmonic Laplacian Mesh Deformation 
		 * to solve for the deformed vertex positions.
		 *
		 * Similar to ConstructConstrainedMeshDeformer() however a uniform-weighted Laplacian is always used.
		 * This allows for the solution of deformation problems on a more generic mesh type (triangles or faces are
		 * not required.)
		 * 
		 */
		TUniquePtr<UE::Solvers::IConstrainedMeshSolver> DYNAMICMESH_API ConstructUniformConstrainedMeshDeformer(const FDynamicGraph3d& Graph);


		/**
		 * Construct a Mesh Deformer object for the given mesh that uses Biharmonic Laplacian Mesh Deformation to solve
		 * for the deformed vertex positions.
		 * 
		 * Similar to ConstructConstrainedMeshDeformer() however (1) a Voronoi-Area Weighted Clamped Cotangent Laplacian 
		 * is always used and (2) the boundary positions are included in the system. This allows for the solution of
		 * deformation problems where the boundary also moves, however it also means that constraints should be added
		 * for all boundary vertices or the deformation may be unstable. 
		 *
		 */
		TUniquePtr<UE::Solvers::IConstrainedLaplacianMeshSolver> DYNAMICMESH_API ConstructSoftMeshDeformer(const FDynamicMesh3& DynamicMesh);

	}
}