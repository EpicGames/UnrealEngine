// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/UniquePtr.h"
#include "Solvers/ConstrainedMeshSolver.h"
#include "Solvers/MeshLaplacian.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE
{
	namespace MeshDeformation
	{
		using namespace UE::Geometry;

		/**
		*  Solves the linear system for p_vec
		*
		*         ( Transpose(L) * L   + (0  0      )  ) p_vec = ( 0              )
		*		  (                      (0 lambda^2)  )         ( lambda^2 c_vec )
		*
		*   where:  L := laplacian for the mesh,
		*           lambda := weights
		*           c_vec := constrained positions
		*
		* Expected Use: same as the ConstrainedMeshDeformer above.
		*
		*/
		TUniquePtr<UE::Solvers::IConstrainedMeshSolver> DYNAMICMESH_API ConstructConstrainedMeshSmoother(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& DynamicMesh);
	}
}

