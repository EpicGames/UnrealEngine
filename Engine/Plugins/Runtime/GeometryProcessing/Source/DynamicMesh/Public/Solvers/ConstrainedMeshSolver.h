// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"

PREDECLARE_GEOMETRY(class FDynamicMesh3);

namespace UE
{
	namespace Solvers
	{
		using namespace UE::Geometry;

		/**
		 * Interface to a index-based deformation solver for a 3D mesh vertex set that supports weighted point constraints.
		 */
		class IConstrainedMeshSolver
		{
		public:

			virtual ~IConstrainedMeshSolver() = default;

			// Add or update a weighted positional constraint associated with VtxId
			// @param bPostFix if true, this constraint will be explicitly applied after the solve
			virtual void AddConstraint(const int32 VtxId, const double Weight, const FVector3d& Position, const bool bPostFix) = 0;

			// Update or Create a constraint position associated with @param VtxId
			// @return true if a constraint weight is associated with @param VtxId
			virtual bool UpdateConstraintPosition(const int32 VtxId, const FVector3d& Position, const bool bPostFix) = 0;

			// Update or Create a constraint weight associated with @param VtxId
			// @return true if a constraint position is associated with @param VtxId
			virtual bool UpdateConstraintWeight(const int32 VtxId, const double Weight) = 0;

			// Clear all constraints (Positions and Weights)
			virtual void ClearConstraints() = 0;

			// Clear all Constraint Weights
			virtual void ClearConstraintWeights() = 0;

			// Clear all Constraint Positions
			virtual void ClearConstraintPositions() = 0;


			// Test if a non-zero weighted constraint is associated with VtxId 
			virtual bool IsConstrained(const int32 VtxId) const = 0;

			// Returns the vertex locations of the deformed mesh. 
			// Note the array may have empty elements as the index matches the Mesh based VtxId.  PositionBuffer[VtxId] = Pos.
			virtual bool Deform(TArray<FVector3d>& PositionBuffer) = 0;
		};


		/**
		 * Extension of IConstrainedMeshSolver that supports manipulating the underlying Laplacian vectors
		 * used in Laplacian-based Deformation Solvers.
		 */
		class IConstrainedLaplacianMeshSolver : public IConstrainedMeshSolver
		{
		public:
			virtual ~IConstrainedLaplacianMeshSolver() = default;

			// Update global scale applied to Laplacian vectors before solve.
			virtual void UpdateLaplacianScale(double UniformScale) = 0;
		};


		/**
		 * Interface to a index-based UV solver for a 3D triangle mesh that supports weighted point constraints.
		 */
		class IConstrainedMeshUVSolver
		{
		public:

			virtual ~IConstrainedMeshUVSolver() = default;

			// Add or update a weighted positional constraint associated with VtxId
			// @param bPostFix if true, this constraint will be explicitly applied after the solve
			virtual void AddConstraint(const int32 VtxId, const double Weight, const FVector2d& Position, const bool bPostFix) = 0;

			// Update or Create a constraint position associated with @param VtxId
			// @return true if a constraint weight is associated with @param VtxId
			virtual bool UpdateConstraintPosition(const int32 VtxId, const FVector2d& Position, const bool bPostFix) = 0;

			// Update or Create a constraint weight associated with @param VtxId
			// @return true if a constraint position is associated with @param VtxId
			virtual bool UpdateConstraintWeight(const int32 VtxId, const double Weight) = 0;

			// Clear all constraints (Positions and Weights)
			virtual void ClearConstraints() = 0;

			// Test if a non-zero weighted constraint is associated with VtxId 
			virtual bool IsConstrained(const int32 VtxId) const = 0;

			// Returns the vertex locations of the deformed mesh. 
			// Note the array may have empty elements as the index matches the Mesh based VtxId.  PositionBuffer[VtxId] = Pos.
			virtual bool SolveUVs(const FDynamicMesh3* DynamicMesh, TArray<FVector2d>& UVBuffer) = 0;
		};

	}
}

