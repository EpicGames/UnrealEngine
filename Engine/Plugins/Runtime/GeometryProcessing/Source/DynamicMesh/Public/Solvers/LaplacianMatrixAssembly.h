// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Util/IndexUtil.h"
#include "Solvers/MatrixInterfaces.h"
#include "Solvers/MeshLinearization.h"
#include "Solvers/MeshLaplacian.h"
#include "Solvers/PrecomputedMeshWeightData.h"
#include "Operations/IntrinsicTriangulationMesh.h"


namespace UE
{
	namespace MeshDeformation
	{
		using namespace UE::Geometry;

		/**
		* Construct a sparse matrix representation of a uniform weighted Laplacian.
		* The uniform weighted Laplacian is defined solely in terms of the connectivity
		* of the mesh.  Note, by construction this should be a symmetric matrix.
		*
		* The mesh itself is assumed to have N interior vertices, and M boundary vertices.
		*
		* Row i represents the Laplacian at vert_i, the non-zero entries correspond to
		* the incident one-ring vertices vert_j.
		*
		* L_{ij} = 1                      if vert_j is in the one-ring of vert_i
		* L_{ii} = -Sum{ L_{ij}, j != i}
		*
		*
		* @param DynamicMesh        The triangle mesh
		* @param VertexMap          On return, Additional arrays used to map between vertexID and offset in a linear array (i.e. the row).
		*                           The vertices are ordered so that last M ( = VertexMap.NumBoundaryVerts() )  correspond to those on the boundary.
		* @param LaplacianInterior  On return, the laplacian operator that acts on the interior vertices: sparse  N x N matrix
		* @param LaplacianBoundary  On return, the portion of the operator that acts on the boundary vertices: sparse  N x M matrix
		*
		*   LaplacianInterior * Vector_InteriorVerts + LaplacianBoundary * Vector_BoundaryVerts = Full Laplacian applied to interior vertices.
		*/
		template<typename RealType, typename MeshType>
		void ConstructUniformLaplacian(const MeshType& Mesh,
									   const FVertexLinearization& VertexMap,
									   UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior,
									   UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary);

		/**
		* Construct a sparse matrix representation of an umbrella weighted Laplacian.
		* This Laplacian is defined solely in terms of the connectivity
		* of the mesh.  Note, there is no expectation that the resulting matrix will be symmetric.
		*
		* The mesh itself is assumed to have N interior vertices, and M boundary vertices.
		*
		* Row i represents the Laplacian at vert_i, the non-zero entries correspond to
		* the incident one-ring vertices vert_j.
		*
		* L_{ij} = 1 / valence(of i)                      if vert_j is in the one-ring of vert_i
		* L_{ii} = -Sum{ L_{ij}, j != i} = -1
		*
		*
		* @param DynamicMesh        The triangle mesh
		* @param VertexMap          Additional arrays used to map between vertexID and offset in a linear array (i.e. the row).
		*                           The vertices are ordered so that last M ( = VertexMap.NumBoundaryVerts() )  correspond to those on the boundary.
		* @param LaplacianInterior  On return, the laplacian operator that acts on the interior vertices: sparse  N x N matrix
		* @param LaplacianBoundary  On return, the portion of the operator that acts on the boundary vertices: sparse  N x M matrix
		*
		*   LaplacianInterior * Vector_InteriorVerts + LaplacianBoundary * Vector_BoundaryVerts = Full Laplacian applied to interior vertices.
		*/
		template<typename RealType>
		void ConstructUmbrellaLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary);

		/**
		* Construct a sparse matrix representation of a valence-weighted Laplacian.
		* The valence weighted Laplacian is defined solely in terms of the connectivity
		* of the mesh.  Note, by construction this should be a symmetric matrix.
		*
		* The mesh itself is assumed to have N interior vertices, and M boundary vertices.
		*
		* Row i represents the Laplacian at vert_i, the non-zero entries correspond to
		* the incident one-ring vertices vert_j.
		*
		* L_{ij} = 1/\sqrt(valence(i) + valence(j))   if vert_j is in the one-ring of vert_i
		* L_{ii} = -Sum{ L_{ij}, j != i}
		*
		* @param DynamicMesh        The triangle mesh
		* @param VertexMap          Additional arrays used to map between vertexID and offset in a linear array (i.e. the row).
		*                           The vertices are ordered so that last M ( = VertexMap.NumBoundaryVerts() )  correspond to those on the boundary.
		* @param LaplacianInterior  On return, the laplacian operator that acts on the interior vertices: sparse  N x N matrix
		* @param LaplacianBoundary  On return, the portion of the operator that acts on the boundary vertices: sparse  N x M matrix
		*
		*   LaplacianInterior * Vector_InteriorVerts + LaplacianBoundary * Vector_BoundaryVerts = Full Laplacian applied to interior vertices.
		*/
		template<typename RealType>
		void ConstructValenceWeightedLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary);

		/**
		* Construct a sparse matrix representation using a meanvalue-weighted  Laplacian.
		* NB: there is no reason to expect this to be a symmetric matrix.
		*
		* The mesh itself is assumed to have N interior vertices, and M boundary vertices.
		*
		* @param DynamicMesh        The triangle mesh
		* @param VertexMap          Additional arrays used to map between vertexID and offset in a linear array (i.e. the row).
		*                           The vertices are ordered so that last M ( = VertexMap.NumBoundaryVerts() )  correspond to those on the boundary.
		* @param LaplacianInterior  On return, the laplacian operator that acts on the interior vertices: sparse  N x N matrix
		* @param LaplacianBoundary  On return, the portion of the operator that acts on the boundary vertices: sparse  N x M matrix
		*
		*   LaplacianInterior * Vector_InteriorVerts + LaplacianBoundary * Vector_BoundaryVerts = Full Laplacian applied to interior vertices.
		*/
		template<typename RealType>
		void ConstructMeanValueWeightLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary);


		/**
		* Construct a sparse matrix representation using a cotangent-weighted  Laplacian.
		* but returns the result in two symmetric parts.
		*
		* The mesh itself is assumed to have N interior vertices, and M boundary vertices.
		*
		*  (AreaMatrix^{-1}) * L_hat  = Cotangent weighted Laplacian.
		*
		*
		* @param DynamicMesh        The triangle mesh
		* @param VertexMap          Additional arrays used to map between vertexID and offset in a linear array (i.e. the row).
		*                           The vertices are ordered so that last M ( = VertexMap.NumBoundaryVerts() )  correspond to those on the boundary.
		* @param AreaMatrix         On return, the mass matrix for the internal vertices.  sparse N x N matrix
		* @param LaplacianInterior  On return, the laplacian operator that acts on the interior vertices: sparse  N x N matrix - symmetric
		* @param LaplacianBoundary  On return, the portion of the operator that acts on the boundary vertices: sparse  N x M matrix
		*
		*   AreaMatrix^{-1} * ( LaplacianInterior * Vector_InteriorVerts + LaplacianBoundary * Vector_BoundaryVerts) = Full Laplacian applied to interior vertices.
		*/
		template<typename RealType>
		void ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
			UE::Solvers::TSparseMatrixAssembler<RealType>& AreaMatrix,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary);


		/**
		* Construct a sparse matrix representation using a pre-multiplied cotangent-weighted Laplacian.
		* NB: there is no reason to expect this to be a symmetric matrix.
		*
		* This computes the laplacian scaled by the average area A_ave:  ie.  LScaled =   A_ave/(2A_i) ( Cot alpha_ij + Cot beta_ij )
		*
		* The mesh itself is assumed to have N interior vertices, and M boundary vertices.
		*
		* @param DynamicMesh        The triangle mesh
		* @param VertexMap          Additional arrays used to map between vertexID and offset in a linear array (i.e. the row).
		*                           The vertices are ordered so that last M ( = VertexMap.NumBoundaryVerts() )  correspond to those on the boundary.
		* @param LaplacianInterior  On return, scaled laplacian operator that acts on the interior vertices: sparse  N x N matrix
		* @param LaplacianBoundary  On return, scaled portion of the operator that acts on the boundary vertices: sparse  N x M matrix
		* @param bClampAreas        Indicates if (A_ave / A_i) should be clamped to (0.5, 5) range.
		*                           in practice this is desirable when creating the biharmonic operator, but not the mean curvature flow operator
		*
		*   LaplacianInterior * Vector_InteriorVerts + LaplacianBoundary * Vector_BoundaryVerts = Full Laplacian applied to interior vertices.
		*/
		template<typename RealType>
		void ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary,
			const bool bClampWeights);

		/**
		* Construct a sparse matrix representation using a pre-multiplied cotangent-weighted Laplacian, using an intrinsic Delaunay mesh internally 
		* NB: there is no reason to expect this to be a symmetric matrix.
		*
		* This computes the laplacian scaled by the average area A_ave:  ie.  LScaled =   A_ave/(2A_i) ( Cot alpha_ij + Cot beta_ij )
		*
		* The mesh itself is assumed to have N interior vertices, and M boundary vertices.
		*
		* @param DynamicMesh        The triangle mesh
		* @param VertexMap          Additional arrays used to map between vertexID and offset in a linear array (i.e. the row).
		*                           The vertices are ordered so that last M ( = VertexMap.NumBoundaryVerts() )  correspond to those on the boundary.
		* @param LaplacianInterior  On return, scaled laplacian operator that acts on the interior vertices: sparse  N x N matrix
		* @param LaplacianBoundary  On return, scaled portion of the operator that acts on the boundary vertices: sparse  N x M matrix
		* @param bClampAreas        Indicates if (A_ave / A_i) should be clamped to (0.5, 5) range.
		*                           in practice this is desirable when creating the biharmonic operator, but not the mean curvature flow operator
		*
		*   LaplacianInterior * Vector_InteriorVerts + LaplacianBoundary * Vector_BoundaryVerts = Full Laplacian applied to interior vertices.
		*/
		template<typename RealType>
		void ConstructIDTCotangentLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary,
			const bool bClampWeights);
		
		
		enum class ECotangentWeightMode
		{
			/** Standard cotangent weights */
			Default = 0,
			/** Magnitude of matrix entries clamped to [-1e5,1e5], scaled by area weight */
			ClampedMagnitude = 1,
			/** Divide cotangent weights by the area of the triangle */
			TriangleArea = 2
		};


		enum class ECotangentAreaMode
		{
			/** uniform-weighted cotangents */
			NoArea = 0,
			/** weight each vertex/row by 1/voronoi_area */
			VoronoiArea = 1
		};

		/**
		 * Construct sparse Cotangent Laplacian matrix.
		 * This variant combines the N interior and M boundary vertices into a single (N+M) matrix and does not do any special treatment of boundaries,
		 * they just get standard Cotan weights
		 *
		 * If you don't want to move the boundary vertices to the end, then initialize the VertexMap with bRemapBoundary 
		 * set to false.
		 * 
		 * 
		 * Example usage:
		 * 
		 * If you want to get the standard cotangent weight matrix for a mesh with no boundary reordering:
		 * 
		 * FDynamicMesh3 DynamicMesh;
		 * const bool bRemapBoundary = false;
		 * FVertexLinearization VertexMap(DynamicMesh, bRemapBoundary);
		 * ConstructFullCotangentLaplacian(DynamicMesh, VertexMap, ECotangentWeightMode::Default, ECotangentAreaMode::NoArea);
		 */
		template<typename RealType>
		void ConstructFullCotangentLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianMatrix,
			ECotangentWeightMode WeightMode = ECotangentWeightMode::ClampedMagnitude,
			ECotangentAreaMode AreaMode = ECotangentAreaMode::NoArea );
		/**
		* Use intrinsic Delaunay mesh to construct sparse Cotangent Laplacian matrix.
		* This variant combines the N interior and M boundary vertices into a single (N+M) matrix and does not do any special treatment of boundaries,
		* they just get standard Cotan weights.  
		*/
		template<typename RealType>
		void ConstructFullIDTCotangentLaplacian(const FDynamicMesh3& Mesh, const FVertexLinearization& VertexMap,
			UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianMatrix,
			ECotangentWeightMode WeightMode = ECotangentWeightMode::ClampedMagnitude,
			ECotangentAreaMode AreaMode = ECotangentAreaMode::NoArea);

	}
}





//
//
//  Implementations
//
//



template<typename RealType, typename MeshType>
void UE::MeshDeformation::ConstructUniformLaplacian(const MeshType& Mesh, 
													const FVertexLinearization& VertexMap,
													UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior,
													UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary)
{
	//check(VertexMap_is_good)
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	// pre-allocate space when possible
	int32 NumMatrixEntries = ComputeNumMatrixElements(Mesh, ToMeshV);
	LaplacianInterior.ReserveEntriesFunc(NumMatrixEntries);

	// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.
	// NB: the vertices are ordered with the interior verts first.
	for (int32 i = 0; i < NumInteriorVerts; ++i)
	{
		const int32 VertId = ToMeshV[i];
		RealType CenterWeight = RealType(0); // equal and opposite the sum of the neighbor weights

		checkSlow(!Mesh.IsBoundaryVertex(VertId));  // we should only be looping over the internal verts

		for (int32 NeighborVertId : Mesh.VtxVerticesItr(VertId))
		{
			const int32 j = ToIndex[NeighborVertId];
			RealType NeighborWeight = RealType(1);
			CenterWeight += NeighborWeight;

			if (j < NumInteriorVerts)
			{
				// add the neighbor 
				LaplacianInterior.AddEntryFunc(i, j, NeighborWeight);
			}
			else
			{
				int32 jBoundary = j - NumInteriorVerts;
				LaplacianBoundary.AddEntryFunc(i, jBoundary, NeighborWeight);
			}
		}
		// add the center
		LaplacianInterior.AddEntryFunc(i, i, -CenterWeight);
	}
}




template<typename RealType>
void UE::MeshDeformation::ConstructUmbrellaLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary)
{
	//check(VertexMap_is_good)
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	// pre-allocate space when possible
	int32 NumMatrixEntries = ComputeNumMatrixElements(DynamicMesh, ToMeshV);
	LaplacianInterior.ReserveEntriesFunc(NumMatrixEntries);

	// Cache valency of each vertex.
	// Number of non-zero elements in the i'th row = 1 + OneRingSize(i)
	TArray<int32> OneRingSize;
	{
		OneRingSize.SetNumUninitialized(NumVerts);
		for (int32 i = 0; i < NumVerts; ++i)
		{
			const int32 VertId = ToMeshV[i];
			OneRingSize[i] = DynamicMesh.GetVtxEdgeCount(VertId);
		}
	}

	// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.

	for (int32 i = 0; i < NumInteriorVerts; ++i)
	{
		const int32 VertId = ToMeshV[i];
		const int32 Valence = OneRingSize[i];
		RealType InvValence = (Valence != 0) ? (RealType)1.0 / RealType(Valence) : 0.;

		checkSlow(!DynamicMesh.IsBoundaryVertex(VertId));

		for (int32 NeighborVertId : DynamicMesh.VtxVerticesItr(VertId))
		{
			const int32 j = ToIndex[NeighborVertId];

			// add the neighbor 
			if (j < NumInteriorVerts)
			{
				LaplacianInterior.AddEntryFunc(i, j, InvValence);
			}
			else
			{
				int32 jBoundary = j - NumInteriorVerts;
				LaplacianBoundary.AddEntryFunc(i, jBoundary, InvValence);
			}
		}
		// add the center
		LaplacianInterior.AddEntryFunc(i, i, -RealType(1));

	}
}







template<typename RealType>
void UE::MeshDeformation::ConstructValenceWeightedLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary)
{
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	// pre-allocate space when possible
	int32 NumMatrixEntries = ComputeNumMatrixElements(DynamicMesh, ToMeshV);
	LaplacianInterior.ReserveEntriesFunc(NumMatrixEntries);

	// Cache valency of each vertex.
	// Number of non-zero elements in the i'th row = 1 + OneRingSize(i)
	TArray<int32> OneRingSize;
	{
		OneRingSize.SetNumUninitialized(NumVerts);

		for (int32 i = 0; i < NumVerts; ++i)
		{
			const int32 VertId = ToMeshV[i];
			OneRingSize[i] = DynamicMesh.GetVtxEdgeCount(VertId);
		}
	}

	// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.
	for (int32 i = 0; i < NumInteriorVerts; ++i)
	{
		const int32 VertId = ToMeshV[i];
		const int32 IOneRingSize = OneRingSize[i];

		RealType CenterWeight = RealType(0); // equal and opposite the sum of the neighbor weights
		for (int32 NeighborVertId : DynamicMesh.VtxVerticesItr(VertId))
		{
			const int32 j = ToIndex[NeighborVertId];
			const int32 JOneRingSize = OneRingSize[j];

			RealType NeighborWeight = RealType(1) / TMathUtil<RealType>::Sqrt(IOneRingSize + JOneRingSize);
			CenterWeight += NeighborWeight;

			if (j < NumInteriorVerts)
			{
				// add the neighbor 
				LaplacianInterior.AddEntryFunc(i, j, NeighborWeight);
			}
			else
			{
				int32 jBoundary = j - NumInteriorVerts;
				LaplacianBoundary.AddEntryFunc(i, jBoundary, NeighborWeight);
			}
		}
		// add the center
		LaplacianInterior.AddEntryFunc(i, i, -CenterWeight);
	}
}




template<typename RealType>
void UE::MeshDeformation::ConstructMeanValueWeightLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary)
{
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	// pre-allocate space when possible
	int32 NumMatrixEntries = ComputeNumMatrixElements(DynamicMesh, ToMeshV);
	LaplacianInterior.ReserveEntriesFunc(NumMatrixEntries);

	// Map the triangles.
	FTriangleLinearization TriangleMap(DynamicMesh);
	const TArray<int32>& ToMeshTri = TriangleMap.ToId();
	const TArray<int32>& ToTriIdx = TriangleMap.ToIndex();
	const int32 NumTris = TriangleMap.NumTris();

	// Create an array that holds all the geometric information we need for each triangle.
	TArray<MeanValueTriangleData> TriangleDataArray;
	ConstructTriangleDataArray<MeanValueTriangleData>(DynamicMesh, TriangleMap, TriangleDataArray);

	// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.
	//                             skipping the boundary verts for later use.
	for (int32 i = 0; i < NumInteriorVerts; ++i)
	{
		const int32 IVertId = ToMeshV[i]; // I - the row
		double WeightII = 0.; // accumulate to equal and opposite the sum of the neighbor weights

		// for each connecting edge
		for (int32 EdgeId : DynamicMesh.VtxEdgesItr(IVertId))
		{
			// [v0, v1, t0, t1]:  NB: both t0 & t1 exist since IVert isn't a boundary vert.
			const FDynamicMesh3::FEdge Edge = DynamicMesh.GetEdge(EdgeId);

			// the other vert in the edge - identifies the matrix column
			const int32 JVertId = (Edge.Vert[0] == IVertId) ? Edge.Vert[1] : Edge.Vert[0];  // J - the column

			// Get the cotangents for this edge.
			const int32 Tri0Idx = ToTriIdx[Edge.Tri[0]];
			const auto& Tri0Data = TriangleDataArray[Tri0Idx];
			double TanHalfAngleSum = Tri0Data.GetTanHalfAngle(IVertId);
			double EdgeLength = FMathd::Max(1.e-5, Tri0Data.GetEdgeLength(EdgeId)); // Clamp the length

			// The second triangle will be invalid if this is an edge!
			TanHalfAngleSum += (Edge.Tri[1] != FDynamicMesh3::InvalidID) ? TriangleDataArray[ToTriIdx[Edge.Tri[1]]].GetTanHalfAngle(IVertId) : 0.;

			double WeightIJ = TanHalfAngleSum / EdgeLength;
			WeightII += WeightIJ;

			const int32 j = ToIndex[JVertId];
			if (j < NumInteriorVerts)
			{
				LaplacianInterior.AddEntryFunc(i, j, (RealType)WeightIJ);
			}
			else
			{
				int32 jBoundary = j - NumInteriorVerts;
				LaplacianBoundary.AddEntryFunc(i, jBoundary, (RealType)WeightIJ);
			}
		}
		LaplacianInterior.AddEntryFunc(i, i, (RealType)-WeightII);
	}
}





template<typename RealType>
void UE::MeshDeformation::ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
	UE::Solvers::TSparseMatrixAssembler<RealType>& AreaMatrix,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary)
{
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	// pre-allocate space when possible
	int32 NumMatrixEntries = ComputeNumMatrixElements(DynamicMesh, ToMeshV);
	LaplacianInterior.ReserveEntriesFunc(NumMatrixEntries);

	// Create the mapping of triangles
	FTriangleLinearization TriangleMap(DynamicMesh);
	const TArray<int32>& ToMeshTri = TriangleMap.ToId();
	const TArray<int32>& ToTriIdx = TriangleMap.ToIndex();
	const int32 NumTris = TriangleMap.NumTris();

	// Clear space for the areas
	AreaMatrix.ReserveEntriesFunc(NumVerts);

	// Create an array that holds all the geometric information we need for each triangle.
	TArray<CotanTriangleData> CotangentTriangleDataArray;
	ConstructTriangleDataArray<CotanTriangleData>(DynamicMesh, TriangleMap, CotangentTriangleDataArray);

	// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.
	//                             store the id of the boundary verts for later use.
	for (int32 i = 0; i < NumInteriorVerts; ++i)
	{
		const int32 IVertId = ToMeshV[i]; // I - the row

		// Compute the Voronoi area for this vertex.
		double WeightArea = 0.;
		for (int32 TriId : DynamicMesh.VtxTrianglesItr(IVertId))
		{
			const int32 TriIdx = ToTriIdx[TriId];
			const CotanTriangleData& TriData = CotangentTriangleDataArray[TriIdx];

			// The three VertIds for this triangle.
			const FIndex3i TriVertIds = DynamicMesh.GetTriangle(TriId);

			// Which of the corners is IVertId?
			int32 Offset = 0;
			while (TriVertIds[Offset] != IVertId)
			{
				Offset++;
				checkSlow(Offset < 3);
			}

			WeightArea += TriData.VoronoiArea[Offset];
		}

		double WeightII = 0.; // accumulate to equal and opposite the sum of the neighbor weights

		// for each connecting edge
		for (int32 EdgeId : DynamicMesh.VtxEdgesItr(IVertId))
		{
			// [v0, v1, t0, t1]:  NB: both t0 & t1 exist since IVert isn't a boundary vert.
			const FDynamicMesh3::FEdge Edge = DynamicMesh.GetEdge(EdgeId);


			// the other vert in the edge - identifies the matrix column
			const int32 JVertId = (Edge.Vert[0] == IVertId) ? Edge.Vert[1] : Edge.Vert[0];  // J - the column

			checkSlow(JVertId != IVertId);

			// Get the cotangents for this edge.
			const int32 Tri0Idx = ToTriIdx[Edge.Tri[0]];
			const CotanTriangleData& Tri0Data = CotangentTriangleDataArray[Tri0Idx];
			const double CotanAlpha = Tri0Data.GetOpposingCotangent(EdgeId);

			// The second triangle will be invalid if this is an edge!
			const double CotanBeta = (Edge.Tri[1] != FDynamicMesh3::InvalidID) ? CotangentTriangleDataArray[ToTriIdx[Edge.Tri[1]]].GetOpposingCotangent(EdgeId) : 0.;

			double WeightIJ = 0.5 * (CotanAlpha + CotanBeta);
			WeightII += WeightIJ;

			const int32 j = ToIndex[JVertId];
			if (j < NumInteriorVerts)
			{
				LaplacianInterior.AddEntryFunc(i, j, (RealType)WeightIJ);
			}
			else
			{
				int32 jBoundary = j - NumInteriorVerts;
				LaplacianBoundary.AddEntryFunc(i, jBoundary, (RealType)WeightIJ);
			}
		}
		LaplacianInterior.AddEntryFunc(i, i, (RealType)-WeightII);
		AreaMatrix.AddEntryFunc(i, i, (RealType)WeightArea);
	}
}





template<typename RealType>
void UE::MeshDeformation::ConstructCotangentLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior, 
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary,
	const bool bClampWeights)
{
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	// pre-allocate space when possible
	int32 NumMatrixEntries = ComputeNumMatrixElements(DynamicMesh, ToMeshV);
	LaplacianInterior.ReserveEntriesFunc(NumMatrixEntries);

	// Map the triangles.
	FTriangleLinearization TriangleMap(DynamicMesh);
	const TArray<int32>& ToMeshTri = TriangleMap.ToId();
	const TArray<int32>& ToTriIdx = TriangleMap.ToIndex();
	const int32 NumTris = TriangleMap.NumTris();

	// Create an array that holds all the geometric information we need for each triangle.
	TArray<CotanTriangleData> CotangentTriangleDataArray;
	ConstructTriangleDataArray<CotanTriangleData>(DynamicMesh, TriangleMap, CotangentTriangleDataArray);

	// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.
	//                             skipping the boundary verts for later use.
	for (int32 i = 0; i < NumInteriorVerts; ++i)
	{
		const int32 IVertId = ToMeshV[i]; // I - the row

		// Compute the Voronoi area for this vertex.
		double WeightArea = 0.;
		for (int32 TriId : DynamicMesh.VtxTrianglesItr(IVertId))
		{
			const int32 TriIdx = ToTriIdx[TriId];
			const CotanTriangleData& TriData = CotangentTriangleDataArray[TriIdx];

			// The three VertIds for this triangle.
			const FIndex3i TriVertIds = DynamicMesh.GetTriangle(TriId);

			// Which of the corners is IVertId?
			int32 Offset = 0;
			while (TriVertIds[Offset] != IVertId)
			{
				Offset++;
				checkSlow(Offset < 3);
			}

			WeightArea += TriData.VoronoiArea[Offset];
		}

		double WeightII = 0.; // accumulate to equal and opposite the sum of the neighbor weights

		// for each connecting edge
		for (int32 EdgeId : DynamicMesh.VtxEdgesItr(IVertId))
		{
			// [v0, v1, t0, t1]:  NB: both t0 & t1 exist since IVert isn't a boundary vert.
			const FDynamicMesh3::FEdge Edge = DynamicMesh.GetEdge(EdgeId);

			// the other vert in the edge - identifies the matrix column
			const int32 JVertId = (Edge.Vert[0] == IVertId) ? Edge.Vert[1] : Edge.Vert[0];  // J - the column
			checkSlow(JVertId != IVertId);

			// Get the cotangents for this edge.
			const int32 Tri0Idx = ToTriIdx[Edge.Tri[0]];
			const CotanTriangleData& Tri0Data = CotangentTriangleDataArray[Tri0Idx];
			const double CotanAlpha = Tri0Data.GetOpposingCotangent(EdgeId);

			// The second triangle will be invalid if this is an edge!
			const double CotanBeta = (Edge.Tri[1] != FDynamicMesh3::InvalidID) ? CotangentTriangleDataArray[ToTriIdx[Edge.Tri[1]]].GetOpposingCotangent(EdgeId) : 0.;

			double WeightIJ = 0.5 * (CotanAlpha + CotanBeta);
			if (bClampWeights)
			{
				WeightIJ = FMathd::Clamp(WeightIJ, -1.e5 * WeightArea, 1.e5 * WeightArea);
			}

			WeightII += WeightIJ;

			const int32 j = ToIndex[JVertId];
			if (j < NumInteriorVerts)
			{
				LaplacianInterior.AddEntryFunc(i, j, (RealType)(WeightIJ / WeightArea));
			}
			else
			{
				int32 jBoundary = j - NumInteriorVerts;
				LaplacianBoundary.AddEntryFunc(i, jBoundary, (RealType)(WeightIJ / WeightArea));
			}
		}
		LaplacianInterior.AddEntryFunc(i, i, (RealType)(-WeightII / WeightArea));
	}
}

template<typename RealType>
void UE::MeshDeformation::ConstructIDTCotangentLaplacian(const FDynamicMesh3& DynamicMesh, const FVertexLinearization& VertexMap,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianInterior,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianBoundary,
	const bool bClampWeights)
{
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();
	const int32 NumBoundaryVerts = VertexMap.NumBoundaryVerts();
	const int32 NumInteriorVerts = NumVerts - NumBoundaryVerts;

	// create intrinsic mesh with delaunay triangulation
	UE::Geometry::FSimpleIntrinsicEdgeFlipMesh IntrinsicMesh(DynamicMesh);

	TSet<int32> Uncorrected; // some edges can't be flipped.
	const int32 NumFlips = UE::Geometry::FlipToDelaunay(IntrinsicMesh, Uncorrected);

	// pre-allocate space when possible
	int32 NumMatrixEntries = ComputeNumMatrixElements(IntrinsicMesh, ToMeshV);
	LaplacianInterior.ReserveEntriesFunc(NumMatrixEntries);

	// cache the cotan weight for each edge (i.e. 1/2 ( cot(a_ij) + cot(b_ij)  for edge(i,j) ) 
	TArray<double> EdgeCotanWeights;
	{
		const int32 MaxEID = IntrinsicMesh.MaxEdgeID();
		EdgeCotanWeights.AddZeroed(MaxEID);
		for (int32 EdgeId = 0; EdgeId < MaxEID; ++EdgeId)
		{
			if (!IntrinsicMesh.IsEdge(EdgeId))
			{
				continue;
			}
			const double CTWeight = IntrinsicMesh.EdgeCotanWeight(EdgeId);
			EdgeCotanWeights[EdgeId] = FMathd::Max(0., CTWeight);  // roundoff errors can produce very, very small negative values when the angles opposite the edge are each 90-degrees. 
		}
	}

	// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.
	//                             skipping the boundary verts for later use.
	for (int32 i = 0; i < NumInteriorVerts; ++i)
	{
		const int32 IVertId = ToMeshV[i]; // I - the row

		// Compute the Voronoi area for this vertex.
		// 1/8 Sum ( cot(a) + cot(b) )*length^2
		// see for example http://www.geometry.caltech.edu/pubs/DMSB_III.pdf
		double WeightArea = 0.0; 
		bool bUseUniform = false;
		{
			for (int32 EdgeId : IntrinsicMesh.VtxEdgesItr(IVertId))
			{

				const double EdgeLength = IntrinsicMesh.GetEdgeLength(EdgeId);
				const double CTWeight = EdgeCotanWeights[EdgeId];
				const double CTWeightLSqr = CTWeight * EdgeLength * EdgeLength;
				
				if (Uncorrected.Contains(EdgeId)) // this part of the intrinsic mesh is not Delaunay.
				{
					bUseUniform = true; 
				}

				WeightArea += CTWeightLSqr;
			}
			WeightArea *= 0.25;
			WeightArea = FMathd::Max(WeightArea, 1.e-5);
		}		

		double WeightII = 0.; // accumulate to equal and opposite the sum of the neighbor weights

		if (bUseUniform)
		{ 
			// use uniform stencil for this vertex
			for (int32 EdgeId : IntrinsicMesh.VtxEdgesItr(IVertId))
			{
				// note: both sides of the edge exist since this isn't a boundary vert
				const FIndex2i EdgeV = IntrinsicMesh.GetEdgeV(EdgeId);
				// the other vert in the edge - identifies the matrix column
				const int32 JVertId = (EdgeV[0] == IVertId) ? EdgeV[1] : EdgeV[0];  // J - the column
				const int32 j = ToIndex[JVertId];

				double  WeightIJ = 1.;
				WeightII += WeightIJ;

				if (j < NumInteriorVerts)
				{
					LaplacianInterior.AddEntryFunc(i, j, (RealType)(WeightIJ));
				}
				else
				{
					int32 jBoundary = j - NumInteriorVerts;
					LaplacianBoundary.AddEntryFunc(i, jBoundary, (RealType)(WeightIJ));
				}
			}
			LaplacianInterior.AddEntryFunc(i, i, (RealType)(-WeightII));
		}
		else
		{ 

			// for each connecting edge
			for (int32 EdgeId : IntrinsicMesh.VtxEdgesItr(IVertId))
			{
				// note: both sides of the edge exist since this isn't a boundary vert
				const FIndex2i EdgeV = IntrinsicMesh.GetEdgeV(EdgeId);

				if (EdgeV.A == EdgeV.B)
				{
					continue; // the weights cancle. 
				}

				double WeightIJ = EdgeCotanWeights[EdgeId];
				if (bClampWeights)
				{
					WeightIJ = FMathd::Clamp(WeightIJ, -1.e5 * WeightArea, 1.e5 * WeightArea);
				}

				double Weight = (WeightIJ / WeightArea);
				WeightII += Weight;

				// the other vert in the edge - identifies the matrix column
				const int32 JVertId = (EdgeV[0] == IVertId) ? EdgeV[1] : EdgeV[0];  // J - the column
				const int32 j = ToIndex[JVertId];
				
				if (j < NumInteriorVerts)
				{
					LaplacianInterior.AddEntryFunc(i, j, (RealType)(Weight));
				}
				else
				{
					int32 jBoundary = j - NumInteriorVerts;
					LaplacianBoundary.AddEntryFunc(i, jBoundary, (RealType)(Weight));
				}

			}

			LaplacianInterior.AddEntryFunc(i, i, (RealType)(-WeightII));
		}
	}

}



template<typename RealType>
void UE::MeshDeformation::ConstructFullCotangentLaplacian(const FDynamicMesh3& Mesh, const FVertexLinearization& VertexMap,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianMatrix, 
	ECotangentWeightMode WeightMode,
	ECotangentAreaMode AreaMode)
{
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();


	// Map the triangles.
	FTriangleLinearization TriangleMap(Mesh);
	const TArray<int32>& ToTriIdx = TriangleMap.ToIndex();
	const int32 NumTris = TriangleMap.NumTris();

	const int32 NumTasks = FMath::Clamp(FTaskGraphInterface::Get().GetNumWorkerThreads()/2, 1, NumVerts);
	constexpr int32 MinVerticesByTask = 60;
	const int32 VerticesByTask = FMath::Max(FMath::DivideAndRoundUp(NumVerts, NumTasks), MinVerticesByTask);
	const int32 NumBatches = FMath::DivideAndRoundUp(NumVerts, VerticesByTask);
	TArray<UE::Tasks::FTask> PendingTasks;
	PendingTasks.Reserve(NumBatches+1);

	UE::Tasks::FTask FirstPendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&Mesh, &ToMeshV, &LaplacianMatrix]()
	{
		// pre-allocate space when possible
		const int32 NumMatrixEntries = ComputeNumMatrixElements(Mesh, ToMeshV);
		LaplacianMatrix.ReserveEntriesFunc(NumMatrixEntries);
	});
	PendingTasks.Add(FirstPendingTask);

	// Create an array that holds all the geometric information we need for each triangle.
	TArray<CotanTriangleData> CotangentTriangleDataArray;
	ConstructTriangleDataArray<CotanTriangleData>(Mesh, TriangleMap, CotangentTriangleDataArray);


	using MatrixEntry = typename UE::Solvers::TSparseMatrixAssembler<RealType>::FTupleData;
	TArray<TArray<MatrixEntry>> TriplesBatches;
	TriplesBatches.SetNum(NumBatches);

	for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
	{
		const int32 StartIndex = BatchIndex * VerticesByTask;
		int32 EndIndex = (BatchIndex + 1) * VerticesByTask;
		EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(NumVerts, EndIndex) : EndIndex;

		TArray<MatrixEntry>& TriplesBatch = TriplesBatches[BatchIndex];

		UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [StartIndex, EndIndex, &ToMeshV, &ToIndex, &ToTriIdx,
			&CotangentTriangleDataArray, &Mesh, AreaMode, WeightMode, &TriplesBatch, &LaplacianMatrix]()
		{
			// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.
			for (int32 i = StartIndex; i < EndIndex; ++i)
			{
				const int32 IVertId = ToMeshV[i]; // I - the row

				double WeightArea = 1.0;
				if (AreaMode == ECotangentAreaMode::VoronoiArea)
				{
					WeightArea = 0.0;
					Mesh.EnumerateVertexTriangles(IVertId, [&](int32 TriId) {
						const int32 TriIdx = ToTriIdx[TriId];
						const CotanTriangleData& TriData = CotangentTriangleDataArray[TriIdx];
						const FIndex3i TriVertIds = Mesh.GetTriangle(TriId);
						int32 TriVertIndex = IndexUtil::FindTriIndex(IVertId, TriVertIds);
						WeightArea += TriData.VoronoiArea[TriVertIndex];
						});
				}

				double WeightII = 0.; // accumulate to equal and opposite the sum of the neighbor weights

				for (int32 EdgeId : Mesh.VtxEdgesItr(IVertId))
				{
					const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeId);

					// the other vert in the edge - identifies the matrix column
					const int32 JVertId = (Edge.Vert[0] == IVertId) ? Edge.Vert[1] : Edge.Vert[0];  // J - the column

					// Get the cotangents for this edge.
					const int32 Tri0Idx = ToTriIdx[Edge.Tri[0]];
					const CotanTriangleData& Tri0Data = CotangentTriangleDataArray[Tri0Idx];
					double CotanAlpha = Tri0Data.GetOpposingCotangent(EdgeId);

					// The second triangle will be invalid if this is an edge!
					double CotanBeta = (Edge.Tri[1] != FDynamicMesh3::InvalidID) ? CotangentTriangleDataArray[ToTriIdx[Edge.Tri[1]]].GetOpposingCotangent(EdgeId) : 0.0;

					if (WeightMode == ECotangentWeightMode::TriangleArea)
					{
						CotanAlpha /= Tri0Data.Area;

						if (Edge.Tri[1] != FDynamicMesh3::InvalidID)
						{
							const CotanTriangleData& Tri1Data = CotangentTriangleDataArray[ToTriIdx[Edge.Tri[1]]];
							CotanBeta /= Tri1Data.Area;
						}
					}

					// do not need to multiply by 0.5 here...
					//double WeightIJ = 0.5 * (CotanAlpha + CotanBeta);
					double WeightIJ = (CotanAlpha + CotanBeta);

					if (WeightMode == ECotangentWeightMode::ClampedMagnitude)
					{
						WeightIJ = FMathd::Clamp(WeightIJ, -1.e5 * WeightArea, 1.e5 * WeightArea);
					}

					WeightII += WeightIJ;

					const int32 j = ToIndex[JVertId];

					TriplesBatch.Emplace(i, j, (RealType)(WeightIJ / WeightArea));
				}
				TriplesBatch.Emplace(i, i, (RealType)(-WeightII / WeightArea));
			}
		});
		PendingTasks.Add(PendingTask);
	}

	UE::Tasks::Wait(PendingTasks);

	for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
	{
		TArray <MatrixEntry>& Triplets = TriplesBatches[BatchIndex];
		LaplacianMatrix.AddEntriesFunc(Triplets.GetData(), Triplets.Num());
	}
}


template<typename RealType>
void UE::MeshDeformation::ConstructFullIDTCotangentLaplacian(const FDynamicMesh3& Mesh, const FVertexLinearization& VertexMap,
	UE::Solvers::TSparseMatrixAssembler<RealType>& LaplacianMatrix,
	ECotangentWeightMode WeightMode,
	ECotangentAreaMode AreaMode)
{
	const TArray<int32>& ToMeshV = VertexMap.ToId();
	const TArray<int32>& ToIndex = VertexMap.ToIndex();
	const int32 NumVerts = VertexMap.NumVerts();

	// create intrinsic mesh with delaunay triangulation
	UE::Geometry::FSimpleIntrinsicEdgeFlipMesh IntrinsicMesh(Mesh);
	TSet<int32> Uncorrected;
	const int32 NumFlips = UE::Geometry::FlipToDelaunay(IntrinsicMesh, Uncorrected);

	// pre-allocate space when possible
	int32 NumMatrixEntries = ComputeNumMatrixElements(IntrinsicMesh, ToMeshV);
	LaplacianMatrix.ReserveEntriesFunc(NumMatrixEntries);

	// cache the cotan weight for each edge (i.e. 1/2 ( cot(a_ij) + cot(b_ij)  for edge(i,j) ) 
	TArray<double> EdgeCotanWeights;
	{
		const int32 MaxEID = IntrinsicMesh.MaxEdgeID();
		EdgeCotanWeights.AddUninitialized(MaxEID);
		
		for (int32 e = 0; e < MaxEID; ++e)
		{
			if (!IntrinsicMesh.IsEdge(e))
			{
				continue;
			}
			EdgeCotanWeights[e] = IntrinsicMesh.EdgeCotanWeight(e);
		}
	}

	// Construct Laplacian Matrix: loop over verts constructing the corresponding matrix row.
	for (int32 i = 0; i < NumVerts; ++i)
	{
		const int32 IVertId = ToMeshV[i]; // I - the row
		
		bool bUseUniform = false;
		double WeightArea = 1.0;
		if (AreaMode == ECotangentAreaMode::VoronoiArea)
		{
			// 1/8 Sum ( cot(a) + cot(b) )*length^2
			// see for example http://www.geometry.caltech.edu/pubs/DMSB_III.pdf
			WeightArea = 0.0;
			for (int32 EdgeId : IntrinsicMesh.VtxEdgesItr(IVertId))
			{
				const double EdgeLength = IntrinsicMesh.GetEdgeLength(EdgeId);
				const double CTWeight = EdgeCotanWeights[EdgeId];
				const double CTWeightLSqr = CTWeight * EdgeLength * EdgeLength;

				if (Uncorrected.Contains(EdgeId) || CTWeightLSqr < 1.e-1)
				{
					bUseUniform = true;
				}

				WeightArea += CTWeightLSqr;
			}
			WeightArea *= 0.25;  
		}
		if (bUseUniform)
		{
			WeightArea = 1.0;
		}

		double WeightII = 0.; // accumulate to equal and opposite the sum of the neighbor weights

		for (int32 EdgeId : IntrinsicMesh.VtxEdgesItr(IVertId))
		{
			const FIndex2i EdgeV = IntrinsicMesh.GetEdgeV(EdgeId);
			if (EdgeV.A == EdgeV.B)
			{
				continue; // weights will cancle.
			}
			// the other vert in the edge - identifies the matrix column
			const int32 JVertId = (EdgeV[0] == IVertId) ? EdgeV[1] : EdgeV[0];  // J - the column

			double WeightIJ = (bUseUniform) ? 1. : EdgeCotanWeights[EdgeId];

			if (WeightMode == ECotangentWeightMode::ClampedMagnitude)
			{
				WeightIJ = FMathd::Clamp(WeightIJ, -1.e5 * WeightArea, 1.e5 * WeightArea);
			}

			double Weight = WeightIJ / WeightArea;
			WeightII += Weight;

			const int32 j = ToIndex[JVertId];
			LaplacianMatrix.AddEntryFunc(i, j, (RealType)(Weight));
		}
		LaplacianMatrix.AddEntryFunc(i, i, (RealType)(-WeightII));
	}
}