// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Types of Laplacian weights. 
 * This enum is used to indicate which type of Mesh Laplacian should be used in various Mesh Deformation solvers.
 * See LaplacianMatrixAssembly.h for specifics
 */
enum class ELaplacianWeightScheme
{
	/** Weight on each vertex is 1 */
	Uniform,
	/** Weight on each vertex is 1/N, where N is valence of vertex */
	Umbrella,
	/** Weight on each vertex is 1/ Sqrt(Valence + Nbr_Valence) */
	Valence,
	/** Mean-Value weight on each vertex, where Mean Value weight is defined by formula 9 in https://www.inf.usi.ch/hormann/papers/Floater.2006.AGC.pdf */
	MeanValue,
	/**  */
	Cotangent,
	/**  */
	ClampedCotangent,
	/** Cotan for Intrinsic Delaunay triangulation*/
	IDTCotanget,
};


/**
* Utility to map the enum names for debuging etc.
*/
FString DYNAMICMESH_API LaplacianSchemeName(const ELaplacianWeightScheme Scheme);


/**
* @return true if Laplacian scheme produces a symmetric matrix
*/
bool DYNAMICMESH_API bIsSymmetricLaplacian(const ELaplacianWeightScheme Scheme);
