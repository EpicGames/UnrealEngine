// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ElementLinearization.h"
#include "DynamicMesh/DynamicMesh3.h"


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

/**
 * Used to linearize the VtxIds in a mesh as a single array and allow mapping from array offset to mesh VtxId.
 * Generally, the array offset will correspond to a matrix row when forming a Laplacian.
 * The last NumBoundaryVerts are the boundary verts. This may be zero.
 */
class FVertexLinearization : public FElementLinearization
{
public:
	FVertexLinearization() = default;

	/**
	 * @param Mesh 			  The mesh to lineralize.
	 * @param bRemapBoundary  If true, will move the boundary vertices to the end of the arrays and record the number of 
	 * 						  boundary vertices.
	 */
	template<typename MeshType>
	FVertexLinearization(const MeshType& Mesh, bool bRemapBoundary = true)
	{
		Reset(Mesh, bRemapBoundary);
	}

	template<typename MeshType>
	void Reset(const MeshType& Mesh, bool bRemapBoundary = true)
	{
		Empty();
		FElementLinearization::Populate(Mesh.MaxVertexID(), Mesh.VertexCount(), Mesh.VertexIndicesItr());

		if (bRemapBoundary) 
		{
			RemapBoundaryVerts(Mesh); 
		}
	}

	int32 NumVerts() const { return FElementLinearization::NumIds(); }

	int32 NumBoundaryVerts() const { return NumBndryVerts; }

private:

	// Moves the boundary verts to the end of the arrays and records the number of boundary verts
	template<typename MeshType>
	void RemapBoundaryVerts(const MeshType& DynamicMesh)
	{
		int32 VertCount = NumVerts();

		// Collect the BoundaryVerts and the internal verts in two array
		TArray<int32> BoundaryVertIds;
		TArray<int32> TmpToIdMap;
		TmpToIdMap.Reserve(VertCount);
		for (int32 i = 0, I = ToIdMap.Num(); i < I; ++i)
		{
			int32 VtxId = ToIdMap[i];

			// Test if the vertex has a one ring
			bool bEmptyOneRing = DynamicMesh.GetVtxEdgeCount(VtxId) < 1;

			if (bEmptyOneRing || DynamicMesh.IsBoundaryVertex(VtxId))
			{
				BoundaryVertIds.Add(VtxId);
			}
			else
			{
				TmpToIdMap.Add(VtxId);
			}
		}

		// The number of boundary verts
		NumBndryVerts = BoundaryVertIds.Num();

		// Merge the boundary verts at the tail 
		// Add the Boundary verts at the end of the array.
		TmpToIdMap.Append(BoundaryVertIds);
	
		// rebuild the 'to' Index
		for (int32 i = 0, I = ToIndexMap.Num(); i < I; ++i)
		{
			ToIndexMap[i] = FDynamicMesh3::InvalidID;
		}
		for (int32 i = 0, I = TmpToIdMap.Num(); i < I; ++i)
		{
			int32 Id = TmpToIdMap[i];
			ToIndexMap[Id] = i;
		}

		// swap the temp
		Swap(TmpToIdMap, ToIdMap);
	}

private:

	FVertexLinearization(const FVertexLinearization&);

	int32 NumBndryVerts = 0;
};



/**
* Used linearize the TriIds in a mesh as a single array and allow mapping from array offset to mesh TriId.
*
*/
class FTriangleLinearization : public FElementLinearization
{
public:
	FTriangleLinearization() = default;

	FTriangleLinearization(const FDynamicMesh3& DynamicMesh)
	{
		Reset(DynamicMesh);
	}

	void Reset(const FDynamicMesh3& DynamicMesh)
	{
		Empty();
		FElementLinearization::Populate(DynamicMesh.MaxTriangleID(), DynamicMesh.TriangleCount(), DynamicMesh.TriangleIndicesItr());
	}

	int32 NumTris() const { return FElementLinearization::NumIds(); }


private:
	FTriangleLinearization(const FTriangleLinearization&);
};


} // end namespace UE::Geometry
} // end namespace UE
