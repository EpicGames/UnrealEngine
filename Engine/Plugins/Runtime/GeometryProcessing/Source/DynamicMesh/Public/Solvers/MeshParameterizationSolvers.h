// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/UniquePtr.h"
#include "Solvers/ConstrainedMeshSolver.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE
{
	namespace MeshDeformation
	{
		using namespace UE::Geometry;

		/**
		 * Create solver for the Least Squares Conformal UV Parameterization for this mesh.
		 * @warning Assumption is that mesh is a single connected component.
		 */
		TUniquePtr<UE::Solvers::IConstrainedMeshUVSolver> DYNAMICMESH_API ConstructNaturalConformalParamSolver(const FDynamicMesh3& DynamicMesh);
	
		/**
		 * Create solver for the Spectral Conformal UV Parameterization for this mesh.
		 * @warning Assumption is that mesh is a single connected component.
		 */
		TUniquePtr<UE::Solvers::IConstrainedMeshUVSolver> DYNAMICMESH_API ConstructSpectralConformalParamSolver(const FDynamicMesh3& DynamicMesh, bool bPreserveIrregularity);
	}
}

