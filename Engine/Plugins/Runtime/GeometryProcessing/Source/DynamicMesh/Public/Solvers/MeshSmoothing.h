// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Solvers/MeshLaplacian.h"
#include "Util/ProgressCancel.h"

namespace UE
{
	namespace MeshDeformation
	{
		using namespace UE::Geometry;

		/**
		*
		* Note: for discussion of implicit / explicit integration of diffusion and biharmonic equations
		*       see "Implicit Fairing of Irregular Meshes using Diffusion and Curvature Flow" - M Desbrun 99.
		*       although the following suggests an additional source term could be included in the implicit solve for better accuracy.
		*       or "Generalized Surface Flows for Mesh Processing" Eckstein et al. 2007
		*/

		/**
		* This is equivalent to taking a single backward Euler time step of bi-harmonic diffusion
		* where L is the Laplacian (Del^2) , and L^T L is an approximation of the Del^4.
		*
		* dp/dt = - k*k L^T L[p]
		*
		* p^{n+1} + dt * k * k L^TL [p^{n+1}] = p^{n}
		*
		* re-write as
		* L^TL[p^{n+1}] + weight * weight p^{n+1} = weight * weight p^{n}
		* with
		* weight = 1 / (k * Sqrt[dt] )
		*
		* The result is returned in the PositionArray
		* Note: if canceled using the optional FProgressCancel, the results may be in an unusable state. 
		*/
		void DYNAMICMESH_API ComputeSmoothing_BiHarmonic(const ELaplacianWeightScheme WeightingScheme, const FDynamicMesh3& OriginalMesh,
			const double Speed, const double Weight, const int32 NumIterations, TArray<FVector3d>& PositionArray, FProgressCancel* Progress = nullptr);

		void  DYNAMICMESH_API ComputeSmoothing_ImplicitBiHarmonicPCG(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& OriginalMesh,
			const double Speed, const double Weight, const int32 MaxIterations, TArray<FVector3d>& PositionArray);

		/**
		* This is equivalent to forward or backward Euler time steps of the diffusion equation
		*
		* dp/dt = L[p]
		*
		* p^{n+1} = p^{n} + dt L[p^{n}]
		*
		* with dt = Speed / Max(|w_ii|)
		*
		* here w_ii are the diagonal values of L
		* Note: if canceled using the optional FProgressCancel, the results may be in an unusable state. 
		*/
		void  DYNAMICMESH_API ComputeSmoothing_Diffusion(const ELaplacianWeightScheme WeightScheme, const FDynamicMesh3& OriginalMesh, bool bForwardEuler,
			const double Speed, double Weight, const int32 NumIterations, TArray<FVector3d>& PositionArray, FProgressCancel* Progress = nullptr);


		/**
		 * Simple iterative smoothing with an optional weight map
		 * Each iteration Lerps towards the weighted neighbor centroid by the GetSmoothingAlpha amount
		 * 
		 * @param bUniformWeightScheme If true, uses uniform weights for centroid; else uses cotan weights
		 * @param bSmoothBoundary If true, boundary vertices are smoothed (toward the centroid of neighboring boundary vertices); else boundary is fixed
		 * @param OriginalMesh The source mesh, used for topology
		 * @param GetSmoothingAlpha The amount to lerp each vertex towards the centroid, per iteration
		 * @param NumIterations Number of iterations to smooth
		 * @param PositionsArray Vertex positions (input and output)
		 * @param Progress An optional interrupter that can terminate the iterations, leaving the results in an unusable state. 
		 */
		void DYNAMICMESH_API ComputeSmoothing_Forward(bool bUniformWeightScheme, bool bSmoothBoundary, const FDynamicMesh3& OriginalMesh,
			TFunctionRef<double(int VID, bool bBoundary)> GetSmoothingAlpha, const int32 NumIterations, TArray<FVector3d>& PositionArray, 
			class FProgressCancel* Progress = nullptr);

	}
}