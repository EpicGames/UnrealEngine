// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Solvers/MeshLinearization.h"
#include "Tasks/Task.h"
#include "Async/TaskGraphInterfaces.h"

#define UE_API DYNAMICMESH_API

namespace UE
{
	namespace MeshDeformation
	{
		using namespace UE::Geometry;

		// Utility to compute the number of elements in the sparse laplacian matrix
		template<typename MeshT>
		int32 ComputeNumMatrixElements(const MeshT& DynamicMesh, const TArray<int32>& ToVtxId)
		{
			const int32 NumVerts = ToVtxId.Num();
			TArray<int32> OneRingSize;
			{
				OneRingSize.SetNumUninitialized(NumVerts);

				for (int32 i = 0; i < NumVerts; ++i)
				{
					const int32 VertId = ToVtxId[i];
					OneRingSize[i] = DynamicMesh.GetVtxEdgeCount(VertId);
				}
			}

			// Compute the total number of entries in the sparse matrix
			int32 NumMatrixEntries = 0;
			{
				for (int32 i = 0; i < NumVerts; ++i)
				{
					NumMatrixEntries += 1 + OneRingSize[i]; // myself plus my neighbors
				}

			}

			return NumMatrixEntries;
		}


		/**
		* The per-triangle data used in constructing the cotangent weighted laplacian.
		*
		*/
		class CotanTriangleData
		{
		public:

			typedef FVector3d TriangleVertices[3];

			CotanTriangleData() = default;

			CotanTriangleData(const FDynamicMesh3& DynamicMesh, int32 TriId)
			{
				Initialize(DynamicMesh, TriId);
			}

			UE_API void Initialize(const FDynamicMesh3& DynamicMesh, int32 SrcTriId);

			int32 GetLocalEdgeIdx(const int32 DynamicsMeshEdgeId) const
			{
				int32 Result = -1;
				if (DynamicsMeshEdgeId == OppositeEdge[0])
				{
					Result = 0;
				}
				else if (DynamicsMeshEdgeId == OppositeEdge[1])
				{
					Result = 1;
				}
				else if (DynamicsMeshEdgeId == OppositeEdge[2])
				{
					Result = 2;
				}
				return Result;
			}

			/** helper to return the cotangent of the angle opposite the given edge
			*
			*   @param DynamicsMeshEdgeId is the id used by FDynamicMesh3 for this edge.
			*   @param bValid will false on return if the requested edge is not part of this triangle
			*   @return Cotangent of the opposite angle.
			*/
			double GetOpposingCotangent(const int32 DynamicsMeshEdgeId, bool& bValid) const
			{

				double OpposingCotangent = -1.;
				bValid = false;
				int32 LocalEdgeIdx = GetLocalEdgeIdx(DynamicsMeshEdgeId);
				if (LocalEdgeIdx > -1)
				{
					OpposingCotangent = Cotangent[LocalEdgeIdx];
					bValid = true;
				}

				return OpposingCotangent;
			}

			double GetOpposingCotangent(const int32 DynamicsMeshEdgeId) const
			{
				bool bValid;
				double OpposingCotangent = GetOpposingCotangent(DynamicsMeshEdgeId, bValid);
				checkSlow(bValid);
				return OpposingCotangent;
			}

			bool bIsObtuse() const
			{
				return (Cotangent[0] < 0.f || Cotangent[1] < 0.f || Cotangent[2] < 0.f);
			}


			// The "floor" for triangle area.
			// NB: the cotan laplacian has terms ~ 1/TriArea
			//     and the deformation matrix has terms ~ 1/TriArea**2

			static constexpr double SmallTriangleArea = SMALL_NUMBER; //1.e-8;

			// testing
			int32 TriId = -1;

			/** Total byte count: 6 double + 3 int32 = 60 bytes.   */

			// Cotangent[i] is cos(theta)/sin(theta) at the i'th vertex.

			double Cotangent[3] = { 0. };

			// VoronoiArea[i] is the voronoi area about the i'th vertex in this triangle.

			double VoronoiArea[3] = { 0. };

			// Area of the triangle 
			
			double Area = -1;

			// OppositeEdge[i] = Corresponding DynamicsMesh3::EdgeId for the edge that is opposite
			// the i'th vertex in this triangle

			int32 OppositeEdge[3] = { -1 };

		};


		/**
		* The per-triangle data used in constructing the mean-value weighted laplacian.
		*
		*/
		class MeanValueTriangleData
		{

		public:

			UE_API void Initialize(const FDynamicMesh3& DynamicMesh, int32 SrcTriId);

			// return Tan(angle / 2) for the corner indicated by this vert id.
			double GetTanHalfAngle(int32 VtxId) const
			{
				int32 Offset = 0;

				while (VtxId != TriVtxIds[Offset])
				{
					Offset++;
					checkSlow(Offset < 3);
				}

				return TanHalfAngle[Offset];
			}

			// return the length of the indicated edge
			double GetEdgeLength(int32 EdgeId) const
			{
				int32 Offset = 0;

				while (EdgeId != TriEdgeIds[Offset])
				{
					Offset++;
					checkSlow(Offset < 3);
				}

				return EdgeLength[Offset];
			}


			int32 TriId = -1;
			FIndex3i TriVtxIds;
			FIndex3i TriEdgeIds;

			bool bDegenerate = true;

			double EdgeLength[3] = { 0. };
			double TanHalfAngle[3] = { 0. };
		};




		/**
		* Return and array in triangle order that holds the per-triangle derived data needed
		*/
		template <typename TriangleDataType>
		void ConstructTriangleDataArray(const FDynamicMesh3& DynamicMesh, const FTriangleLinearization& TriangleLinearization, TArray<TriangleDataType>& TriangleDataArray)
		{
			const int32 NumTris = TriangleLinearization.NumTris();
			TriangleDataArray.SetNumUninitialized(NumTris);

			const auto& ToTriId = TriangleLinearization.ToId();

			const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads()/2, NumTris), 1);
			constexpr int32 MinTrianglesByTask = 60;
			const int32 TrianglesByTask = FMath::Max(FMath::Max(FMath::DivideAndRoundUp(NumTris, NumTasks), MinTrianglesByTask), 1);
			const int32 NumBatches = FMath::DivideAndRoundUp(NumTris, TrianglesByTask);
			TArray<UE::Tasks::FTask> PendingTasks;
			PendingTasks.Reserve(NumBatches);
			for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
			{
				const int32 StartIndex = BatchIndex * TrianglesByTask;
				int32 EndIndex = (BatchIndex + 1) * TrianglesByTask;
				EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(NumTris, EndIndex) : EndIndex;

				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [StartIndex, EndIndex, &ToTriId, &TriangleDataArray, &DynamicMesh]()
				{
					for (int32 i = StartIndex; i < EndIndex; ++i)
					{
						// Current triangle
						const int32 TriId = ToTriId[i];

						// Compute all the geometric data needed for this triangle.
						TriangleDataArray[i].Initialize(DynamicMesh, TriId);
					}
				});
				PendingTasks.Add(PendingTask);
			}
			UE::Tasks::Wait(PendingTasks);
		}


	  	/**
		 * @return Edge cotanget weights.
		 * @todo Add an option to return the weights computed on an intrinsic mesh.
		 */
		void DYNAMICMESH_API ConstructEdgeCotanWeightsDataArray(const FDynamicMesh3& Mesh, TArray<double>& EdgeWeightsDataArray, double ClampMin = -1.e5, double ClampMax = 1.e5);
		
	}
}

#undef UE_API
