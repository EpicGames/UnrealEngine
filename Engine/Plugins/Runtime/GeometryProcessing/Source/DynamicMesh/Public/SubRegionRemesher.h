// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Remesher.h"
#include "Util/BufferUtil.h"
#include "Util/UniqueIndexSet.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "MeshWeights.h"
#include "Async/ParallelFor.h"

namespace UE
{
namespace Geometry
{

/**
 * FSubRegionRemesher is an extension of FRemesher that allows for constraining remeshing to
 * a localized region of a mesh. Currently this is initialized from a Vertex ROI.
 * 
 * @warning Currently "boundary" edges of the ROI that are split will result in the ROI
 * growing to include both new edges created by the split. 
 * 
 */
class FSubRegionRemesher : public FRemesher
{
protected:

	/**
	 * Set of vertices in ROI. You add vertices here initially, then
	 * we will update the list during each Remesh pass
	 */
	TSet<int> VertexROI;

	// active set of edges we will consider for remeshing. This set is updated on 
	// each edge flip/split/collapse, but is not used during the pass.
	TSet<int> EdgeROI;

	// Active set of triangles. This is conservative, ideally contains one-rings of all
	// edge-vertices of EdgeROI set, but may include additional triangles accumulated over time
	// (should *not* contain any triangles that no longer exist)
	TSet<int> TriangleROI;

	// static list of edges to consider during a pass (set in UpdateROI which must be called each pass)
	TArray<int> Edges;

	// index of current edge in .Edges we are at in StartEdges/GetNextEdge modulo-iteration
	int CurEdge = -1;

	// set of triangles removed in last pass. Enable this by calling BeginTrackRemovedTrisInPass()
	TSet<int> RemovedLastPassTris;

	// When deleted, remove PRAGMA_DISABLE_DEPRECATION_WARNINGS / PRAGMA_ENABLE_DEPRECATION_WARNINGS from ~FSubRegionRemesher()
	UE_DEPRECATED(5.6, "This member is no longer used")
	FUniqueIndexSet InitialROIEdgeVertices;

	// temporary data structure used to quickly convert vertex set to triangle set. Currenlty it is faster
	// to construct this and then copy to a properly-sized TSet, than to incrementally build the TSet
	FUniqueIndexSet InitialROITriangles;

	// controls whether RemovedLastPass will be populated
	bool bTrackRemoved = false;

	// counters for making sure that UpdateROI has been called
	uint32 LastUpdateROICounter = 0;
	uint32 LastRemeshPassCounter = 0;


public:


	FSubRegionRemesher(FDynamicMesh3* Mesh) : FRemesher(Mesh)
	{
		VertexControlF = [this](int vid) {
			return this->VertexFilter(vid);
		};
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FSubRegionRemesher() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void Reset()
	{
		VertexROI.Reset();
		EdgeROI.Reset();
		TriangleROI.Reset();
		Edges.Reset();
		RemovedLastPassTris.Reset();
		CurEdge = -1;
		LastUpdateROICounter = LastRemeshPassCounter = 0;
	}


	void SetInitialVertexROI(const TSet<int>& Vertices)
	{
		VertexROI = Vertices;
	}

	void SetInitialVertexROI(TSet<int>&& Vertices)
	{
		VertexROI = MoveTemp(Vertices);
	}

	void SetInitialVertexROI(const TArray<int>& Vertices)
	{
		VertexROI.Reset();
		VertexROI.Reserve(Vertices.Num());
		for (int32 tid : Vertices)
		{
			VertexROI.Add(tid);
		}
	}

	/**
	 * Initialize edge-subregion ROI from the VertexROI member that has been externally initialized.
	 * Edge ROI is initialized to fully-contained edges (i.e. both endpoints are in VertexROI).
	 */
	void InitializeFromVertexROI()
	{
		const FDynamicMesh3* UseMesh = GetMesh();

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InitializeFromVertexROI_1Init);
			EdgeROI.Reset();
			TriangleROI.Reset();

			// Convert vertex ROI to set of contained edges.
			for (int VertIdx : VertexROI)
			{
				UseMesh->EnumerateVertexEdges(VertIdx, [this, UseMesh, VertIdx](int32 eid)
				{
					FIndex2i EdgeV = UseMesh->GetEdgeV(eid);
					int32 OtherVid = IndexUtil::FindEdgeOtherVertex(EdgeV, VertIdx);
					if (VertexROI.Contains(OtherVid))
					{
						EdgeROI.Add(eid);
					}
				});
			}
			UpdateROI();
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InitializeFromVertexROI_4Triangles);

			// Convert new vertex ROI built above into new one-ring triangle ROI. 
			// This conservatively guarantees that any triangle affected by a mesh edge op 
			// (eg split/collapse/flip) is contained within this triangle set
			InitialROITriangles.Initialize(UseMesh->MaxTriangleID());
			for (int vid : VertexROI)
			{
				UseMesh->EnumerateVertexEdges(vid, [this](int32 eid)
				{
					FIndex2i EdgeT = Mesh->GetEdgeT(eid);
					InitialROITriangles.Add(EdgeT.A);
					if (EdgeT.B != IndexConstants::InvalidID) InitialROITriangles.Add(EdgeT.B);
				});
			}
			InitialROITriangles.Collect(TriangleROI);
		}
	}

	void InitializeFromTriangleROI(const TSet<int>& InTriangleROI)
	{
		EdgeROI.Reset();
		VertexROI.Reset();
		TriangleROI = InTriangleROI;

		for (int TriangleIndex : TriangleROI)
		{
			FIndex3i Vertices = Mesh->GetTriangle(TriangleIndex);
			VertexROI.Add(Vertices[0]);
			VertexROI.Add(Vertices[1]);
			VertexROI.Add(Vertices[2]);

			FIndex3i TriEdges = Mesh->GetTriEdges(TriangleIndex);
			EdgeROI.Add(TriEdges[0]);
			EdgeROI.Add(TriEdges[1]);
			EdgeROI.Add(TriEdges[2]);
		}
	}

	/**
	 * Update the internal data structures in preparation for a call to FRemesher::BasicRemeshPass.
	 * This must be called before each remesh pass!
	 */
	void UpdateROI()
	{
		Edges.Reset(EdgeROI.Num());
		for (int eid : EdgeROI)
		{
			Edges.Add(eid);
		}
		LastUpdateROICounter++;
	}

	/**
	 * Call before BasicRemeshPass() to enable tracking of removed triangles
	 */
	void BeginTrackRemovedTrisInPass()
	{
		RemovedLastPassTris.Reset();
		bTrackRemoved = true;
	}

	/**
	 * Call after BasicRemeshPass() to disable and return tracking of removed triangles
	 * @return set of removed triangles. This set will be cleared on next call to BeginTrackRemovedTrisInPass()
	 */
	const TSet<int32>& EndTrackRemovedTrisInPass()
	{
		bTrackRemoved = false;
		return RemovedLastPassTris;
	}



	/**
	 * forwards to FRemesher::BasicRemeshPass
	 */
	virtual void BasicRemeshPass() override
	{
		check(LastRemeshPassCounter != LastUpdateROICounter);
		LastRemeshPassCounter = LastUpdateROICounter;

		FRemesher::BasicRemeshPass();
	}



	/**
	 * Tell a MeshChangeTracker about the set of triangles that we might modify in the next remesh pass.
	 * This could include one-rings of either side of an edge in the ROI, if we collapse.
	 */
	void SaveActiveROI(FDynamicMeshChangeTracker* Change)
	{
		for (int tid : TriangleROI)
		{
			Change->SaveTriangle(tid, true);
		}
	}

	/**
	 * @return 
	 */
	const TSet<int>& GetCurrentVertexROI() const
	{
		return VertexROI;
	}

	/**
	 * @return set of triangles that contains edge ROI (note: may also contain additional triangles)
	 */
	const TSet<int>& GetCurrentTriangleROI() const
	{
		return TriangleROI;
	}

	/**
	 * @return set of edges in current edge ROI
	 */
	const TSet<int>& GetCurrentEdgeROI() const
	{
		return EdgeROI;
	}

	/**
	 * @return current edge array. 
	 * @warning This is only valid after calling UpdateROI() and before calling BasicRemeshPass().
	 */
	const TArray<int>& GetCurrentEdgeArray() const
	{
		return Edges;
	}



	/**
	 * Extract final triangle ROI. After calling this function, the ROI will be cleared, and must be re-initialized.
	 * This can be used to avoid having to copy the TSet.
	 * @return set of triangles that contains edge ROI (note: may also contain additional triangles)
	 */
	TSet<int>&& ExtractFinalTriangleROI()
	{
		EdgeROI.Reset();
		VertexROI.Reset();
		return MoveTemp(TriangleROI);
	}


	//
	// specialization of Remesher functionality
	// 
protected:
	virtual int StartEdges() override
	{
		CurEdge = 0;
		return (Edges.Num() > 0) ? Edges[CurEdge] : IndexConstants::InvalidID;
	}

	virtual int GetNextEdge(int CurEdgeID, bool& bDone) override
	{
		CurEdge++;

		while ((CurEdge < Edges.Num()) && (!EdgeROI.Contains(Edges[CurEdge])))		// EdgeROI may have lost some edges due to earlier operations
		{
			++CurEdge;
		}

		if (CurEdge >= Edges.Num())
		{
			bDone = true;
			return -1;
		}
		else
		{
			bDone = false;
			return Edges[CurEdge];
		}
	}




	virtual void OnEdgeSplit(int EdgeID, int VertexA, int VertexB, const FDynamicMesh3::FEdgeSplitInfo& SplitInfo) override
	{
		VertexROI.Add(SplitInfo.NewVertex);
		EdgeROI.Add(SplitInfo.NewEdges.A);

		// By always adding new edges to ROI, we are potentially 'growing' the ROI here.
		// Could filter out these edges by checking if other vtx is in VertexROI?

		EdgeROI.Add(SplitInfo.NewEdges.B);
		AddEdgeToTriangleROI(SplitInfo.NewEdges.B);
		if (SplitInfo.NewEdges.C != FDynamicMesh3::InvalidID)
		{
			EdgeROI.Add(SplitInfo.NewEdges.C);
			AddEdgeToTriangleROI(SplitInfo.NewEdges.C);
		}

		// these two triangles should be already added by AddEdgeToTriangleROI() calls above...
		TriangleROI.Add(SplitInfo.NewTriangles.A);
		if (SplitInfo.NewTriangles.B != FDynamicMesh3::InvalidID)
		{
			TriangleROI.Add(SplitInfo.NewTriangles.B);
		}
		//AddVertexToTriangleROI(SplitInfo.NewVertex);
		//for (int tid : GetMesh()->VtxTrianglesItr(SplitInfo.NewVertex))
		//	check(TriangleROI.Contains(tid));
	}


	virtual void OnEdgeCollapse(int EdgeID, int VertexA, int VertexB, const FDynamicMesh3::FEdgeCollapseInfo& CollapseInfo) override
	{
		// remove triangles from ROI
		TriangleROI.Remove(CollapseInfo.RemovedTris.A);
		if (bTrackRemoved)
		{
			RemovedLastPassTris.Add(CollapseInfo.RemovedTris.A);
		}
		if (CollapseInfo.RemovedTris.B != FDynamicMesh3::InvalidID)
		{
			TriangleROI.Remove(CollapseInfo.RemovedTris.B);
			if (bTrackRemoved)
			{
				RemovedLastPassTris.Add(CollapseInfo.RemovedTris.B);
			}
		}

		// remove vtx
		VertexROI.Remove(CollapseInfo.RemovedVertex);

		// remove edges
		EdgeROI.Remove(CollapseInfo.CollapsedEdge);
		EdgeROI.Remove(CollapseInfo.RemovedEdges.A);
		if (CollapseInfo.RemovedEdges.B != FDynamicMesh3::InvalidID)
		{
			EdgeROI.Remove(CollapseInfo.RemovedEdges.B);
		}

		// Note: we're not removing CollapsedEdge or RemovedEdges from the Edges array
	}


	virtual void OnEdgeFlip(int EdgeID, const FDynamicMesh3::FEdgeFlipInfo& FlipInfo)
	{
		// flipping an edge potentially connects new verts to the ROI

		FIndex2i EdgeV = GetMesh()->GetEdgeV(EdgeID);
		VertexROI.Add(EdgeV.A);
		VertexROI.Add(EdgeV.B);

		AddVertexToTriangleROI(EdgeV.A);
		AddVertexToTriangleROI(EdgeV.B);
	}



	void AddVertexToTriangleROI(int VertexID)
	{
		GetMesh()->EnumerateVertexTriangles(VertexID, [this](int tid)
		{
			TriangleROI.Add(tid);
		});
	}

	void AddEdgeToTriangleROI(int EdgeID)
	{
		FIndex2i EdgeV = GetMesh()->GetEdgeV(EdgeID);
		AddVertexToTriangleROI(EdgeV.A);
		AddVertexToTriangleROI(EdgeV.B);
	}



	EVertexControl VertexFilter(int VertexID)
	{
		return (VertexROI.Contains(VertexID) == false) ? EVertexControl::NoMovement : EVertexControl::AllowAll;
	}





	//
	// localized smoothing
	// 
	
	TArray<int32> SmoothIndices;
	TArray<FVector3d> SmoothedPositions;

	virtual void FullSmoothPass_Buffer(bool bParallel) override
	{
		SmoothIndices.Reset();
		SmoothIndices.Reserve(VertexROI.Num());
		for (int32 vid : VertexROI)
		{
			SmoothIndices.Add(vid);
		}
		int32 NumVertices = SmoothIndices.Num();

		SmoothedPositions.SetNum(NumVertices, EAllowShrinking::No);

		TFunction<FVector3d(const FDynamicMesh3&, int, double)> UseSmoothFunc = GetSmoothFunction();
		
		ParallelFor(NumVertices, [&](int32 i)
		{
			int32 VertIdx = SmoothIndices[i];
			bool bModified = false;
			SmoothedPositions[i] = ComputeSmoothedVertexPos(VertIdx, UseSmoothFunc, bModified);
			SmoothIndices[i] = (bModified) ? SmoothIndices[i] : -1;
		});

		ParallelFor(NumVertices, [&](int32 i)
		{
			if (SmoothIndices[i] >= 0)
			{
				Mesh->SetVertex(SmoothIndices[i], SmoothedPositions[i], false);
			}
		});
		Mesh->UpdateChangeStamps(true, false);
	}
};


/**
 * FRestrictedSubRegionRemesher is similar to FSubRegionRemesher but does not allow the ROI to grow outside of the 
 * original ROI boundary.
 *
 * It is initialized from a set of Triangles and the initial ROI boundary is inferred from those triangles.
 */
class FRestrictedSubRegionRemesher : public FSubRegionRemesher
{

public:		

	FRestrictedSubRegionRemesher(FDynamicMesh3* Mesh, const TSet<int>& InTriangleROI) :
		FSubRegionRemesher(Mesh)
	{
		InitializeFromTriangleROI(InTriangleROI);
	}


	void OnEdgeSplit(int EdgeID, int VertexA, int VertexB, const FDynamicMesh3::FEdgeSplitInfo& SplitInfo) override
	{
		check(EdgeROI.Contains(EdgeID));

		VertexROI.Add(SplitInfo.NewVertex);
		EdgeROI.Add(SplitInfo.NewEdges.A);

		bool bSplitEdgeIsROIBoundary = EdgeIsROIBoundary(EdgeID);

		// Don't grow the ROI here, by filtering out edges whose two vertices are not both in the ROI.

		if (EdgeVerticesAreInVertexROI(SplitInfo.NewEdges.B))
		{
			EdgeROI.Add(SplitInfo.NewEdges.B);
		}

		if (SplitInfo.NewEdges.C != FDynamicMesh3::InvalidID)
		{
			if (EdgeVerticesAreInVertexROI(SplitInfo.NewEdges.C))
			{
				EdgeROI.Add(SplitInfo.NewEdges.C);
			}
		}

		// Add Triangles to TriangleROI if all their edges are in EdgeROI
		for (int NewTID : { SplitInfo.NewTriangles.A, SplitInfo.NewTriangles.B })
		{
			if (Mesh->IsTriangle(NewTID) && TriangleEdgesAreInEdgeROI(NewTID))
			{
				TriangleROI.Add(NewTID);
			}
		}
	}


	void OnEdgeFlip(int EdgeID, const FDynamicMesh3::FEdgeFlipInfo& FlipInfo) override
	{
		// flipping an edge should not introduce new verts to the Vertex ROI!
		check(VertexROI.Contains(GetMesh()->GetEdgeV(EdgeID).A));
		check(VertexROI.Contains(GetMesh()->GetEdgeV(EdgeID).B));

		check(Mesh->IsEdge(FlipInfo.EdgeID));
		EdgeROI.Add(FlipInfo.EdgeID);

		check(Mesh->IsTriangle(FlipInfo.Triangles[0]));
		check(Mesh->IsTriangle(FlipInfo.Triangles[1]));
		TriangleROI.Add(FlipInfo.Triangles[0]);
		TriangleROI.Add(FlipInfo.Triangles[1]);
	}

private:

	bool EdgeVerticesAreInVertexROI(int EdgeID) const
	{
		FIndex2i EdgeVertices = Mesh->GetEdgeV(EdgeID);
		return VertexROI.Contains(EdgeVertices[0]) && VertexROI.Contains(EdgeVertices[1]);
	}

	bool TriangleEdgesAreInEdgeROI(int TriangleID) const
	{
		FIndex3i TriEdges = Mesh->GetTriEdges(TriangleID);
		return EdgeROI.Contains(TriEdges[0]) && EdgeROI.Contains(TriEdges[1]) && EdgeROI.Contains(TriEdges[2]);
	}

	bool EdgeIsROIBoundary(int EdgeID) const
	{
		FIndex2i EdgeTriangles = Mesh->GetEdgeT(EdgeID);
		return (TriangleROI.Contains(EdgeTriangles[0]) != TriangleROI.Contains(EdgeTriangles[1]));
	}

	void InitializeFromTriangleROI(const TSet<int>& InTriangleROI)
	{
		EdgeROI.Reset();
		VertexROI.Reset();
		TriangleROI = InTriangleROI;

		for (int TriangleIndex : TriangleROI)
		{
			FIndex3i Vertices = Mesh->GetTriangle(TriangleIndex);
			VertexROI.Add(Vertices[0]);
			VertexROI.Add(Vertices[1]);
			VertexROI.Add(Vertices[2]);

			FIndex3i TriEdges = Mesh->GetTriEdges(TriangleIndex);
			EdgeROI.Add(TriEdges[0]);
			EdgeROI.Add(TriEdges[1]);
			EdgeROI.Add(TriEdges[2]);
		}
	}

};


} // end namespace UE::Geometry
} // end namespace UE
