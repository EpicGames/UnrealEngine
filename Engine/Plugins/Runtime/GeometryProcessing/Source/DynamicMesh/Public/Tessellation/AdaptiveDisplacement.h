// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

namespace UE {
namespace Geometry {

class FDynamicMesh3;
struct FAdaptiveTessellatorOptions;
class FMeshConstraints;
class FDisplacementMap;

/**
 * Apply adaptive displacement on given mesh.
 * 
 * @param Mesh               mesh to be displaced (requires per-vertex normals to be present)
 * @param UVOverlay          UVs on which to evaluate displacement map
 * @param DisplacementMap    displacement map
 * @param Options            control sample rate, error, etc.
 * @param FeatureSensitivity value in [0, 1] to control additional adaptive refinement at sharp features
 * @param MeshConstraints    constraints to preserve
 * @param ActiveTriangles    optional per-triangle selection
 * @param bApplyPostOptimization whether to apply some post regularization
 * @return                   whether the adaptive displacement succeeded
 */

DYNAMICMESH_API
bool ApplyAdaptiveDisplacement(
	FDynamicMesh3& Mesh,
	FDynamicMeshUVOverlay* UVOverlay,
	const FDisplacementMap& DisplacementMap,
	const FAdaptiveTessellatorOptions& Options,
	const float FeatureSensitivity,
	FMeshConstraints& MeshConstraints,
	TBitArray<>& ActiveTriangles,
	const bool bApplyPostOptimization = true );
	
} // namespace Geometry
} // namespace UE
