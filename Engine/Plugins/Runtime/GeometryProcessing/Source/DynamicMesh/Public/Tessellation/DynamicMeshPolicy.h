// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Async/ParallelFor.h"
#include "IndexTypes.h"
#include "VectorUtil.h"
#include "Templates/UnrealTemplate.h"

#include "Tessellation/AdaptiveTessellator.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshConstraints.h"

namespace UE {
namespace Geometry {

// Mesh policy on FDynamicMesh3 for use with FAdaptiveTesselator2
class FDynamicMesh3Policy
{
	inline static constexpr int32 NextEdge(int32 e) { return (e+1)%3; }
    inline static constexpr int32 PrevEdge(int32 e) { return (e+2)%3; }

  public:

	using VecType = FVector3d;
	using RealType = double;

	FDynamicMesh3Policy(
		FDynamicMesh3& InMesh,
		FMeshConstraints& InConstraints) //< optional constraints to be respected on mesh operations. these constraints are updated when the mesh is modified (e.g. edge splits)
		: Mesh(InMesh)
		, Constraints(InConstraints)
		, bHasEdgeConstraints(!InConstraints.GetEdgeConstraints().IsEmpty())
	{
	}

	inline int32 GetVertexIndex(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		return Mesh.GetTriangle(TriIndex)[TriEdgeIndex];
	}

	inline FIndex3i GetTriangle(const int32 TriIndex) const
	{
		return Mesh.GetTriangle(TriIndex);
	}

	inline FVector3d GetVertexPosition(const int TriIndex, const int TriEdgeIndex) const
	{
		return Mesh.GetTriVertex(TriIndex, TriEdgeIndex);
	}

	inline FVector3d GetVertexPosition(const int32 VertexIndex) const
	{
		return Mesh.GetVertex(VertexIndex);
	}

	inline void SetVertexPosition(const int32 VertexIndex, const FVector3d Position) const
	{
		return Mesh.SetVertex(VertexIndex, FVector3d(Position));
	}

	inline void GetTriVertices(int TriIndex, FVector3d& P0, FVector3d& P1, FVector3d& P2) const
	{
		Mesh.GetTriVertices(TriIndex, P0, P1, P2);
	}
	
	inline int32 MaxVertexID() const
	{
		return Mesh.MaxVertexID();
	}

	inline bool IsValidVertex(int32 VertexID) const
	{
		return Mesh.IsVertex(VertexID);
	}

	inline int32 TriangleCount2() const
	{
		return Mesh.MaxTriangleID();
	}

	inline int32 MaxTriID() const
	{
		return Mesh.MaxTriangleID();
	}

	inline bool IsValidTri(int32 TriIndex) const
	{
		return Mesh.IsTriangle(TriIndex);
	}

	inline int32 GetAdjTriangle(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		check(Mesh.IsTriangle(TriIndex));
		check(TriEdgeIndex >= 0 && TriEdgeIndex < 3);

		const int EdgeIndex = Mesh.GetTriEdge(TriIndex, TriEdgeIndex);
		const auto& edge = Mesh.GetEdgeRef(EdgeIndex);

		check(edge.Tri[0] == TriIndex || edge.Tri[1] == TriIndex);
		return edge.Tri[0] == TriIndex ? edge.Tri[1] : edge.Tri[0];
	}

	// (triangle index, edge index in 0..2)
	inline TPair<int32,int32> GetAdjEdge(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		check(Mesh.IsTriangle(TriIndex));
		check(TriEdgeIndex >= 0 && TriEdgeIndex < 3);

		const int EdgeIndex = Mesh.GetTriEdge(TriIndex, TriEdgeIndex);
		const auto& edge = Mesh.GetEdgeRef(EdgeIndex);

		check(edge.Tri[0] == TriIndex || edge.Tri[1] == TriIndex);
		const int32 AdjTriIndex = (edge.Tri[0] == TriIndex) ? edge.Tri[1] : edge.Tri[0];
		
		if (AdjTriIndex < 0)
		{
			return { IndexConstants::InvalidID, IndexConstants::InvalidID };
		}

		const FIndex3i& AdjEdges = Mesh.GetTriEdgesRef(AdjTriIndex);

		int32 AdjEdgeIndex = 0;
		if (AdjEdges[0] == EdgeIndex) 
		{
			AdjEdgeIndex = 0;
		}
		else if (AdjEdges[1] == EdgeIndex)
		{
			AdjEdgeIndex = 1;
		}
		else 
		{
			check(AdjEdges[2] == EdgeIndex);
			AdjEdgeIndex = 2;
		}

		check( Mesh.GetTriangle(TriIndex)[TriEdgeIndex] == Mesh.GetTriangle(AdjTriIndex)[NextEdge(AdjEdgeIndex)] );
		check( Mesh.GetTriangle(TriIndex)[NextEdge(TriEdgeIndex)] == Mesh.GetTriangle(AdjTriIndex)[AdjEdgeIndex] );

		return { AdjTriIndex, AdjEdgeIndex };
	}

	inline bool EdgeManifoldCheck(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		const TPair<int32,int32> AdjEdge = GetAdjEdge(TriIndex, TriEdgeIndex);
		check( Mesh.GetTriangle(TriIndex)[TriEdgeIndex] == Mesh.GetTriangle(AdjEdge.Get<0>())[NextEdge(AdjEdge.Get<1>())] );
		check( Mesh.GetTriangle(TriIndex)[NextEdge(TriEdgeIndex)] == Mesh.GetTriangle(AdjEdge.Get<0>())[AdjEdge.Get<1>()]);
		return true;
	}

	inline bool AllowEdgeFlip(const int32 TriIndex, const int32 TriEdgeIndex, const int32 AdjTriIndex ) const
	{
		if (!bHasEdgeConstraints)
		{
			return true;
		}

		const int EdgeID = Mesh.GetTriEdge(TriIndex, TriEdgeIndex);
		UE::Geometry::FEdgeConstraint EdgeConstraint = Constraints.GetEdgeConstraint(EdgeID);

		return EdgeConstraint.CanFlip();
	}

	inline bool AllowEdgeSplit(const int32 TriIndex, const int32 TriEdgeIndex ) const
	{
		if (!bHasEdgeConstraints)
		{
			return true;
		}

		const int EdgeID = Mesh.GetTriEdge(TriIndex, TriEdgeIndex);
		UE::Geometry::FEdgeConstraint EdgeConstraint = Constraints.GetEdgeConstraint(EdgeID);

		return EdgeConstraint.CanSplit();
	}

	inline FVector3d GetTriangleNormal(const int32 TriIndex) const
	{
		check(Mesh.IsTriangle(TriIndex));

		FVector3d V0, V1, V2;
		Mesh.GetTriVertices(TriIndex, V0, V1, V2);
		return Normalized(VectorUtil::NormalDirection(V0, V1, V2));

		// this would call Normalize 3x
		// return Mesh.GetTriNormal(TriIndex);
	}

	UE::Geometry::FEdgeSplitInfo SplitEdge(const int32 TriIndex, const int32 TriEdgeIndex, const double SplitWeight )
	{
		const int EdgeID = Mesh.GetTriEdge(TriIndex, TriEdgeIndex);

		const FIndex3i Tri = Mesh.GetTriangle(TriIndex);

		// vertices along split edge
		const int32 vA = Tri[TriEdgeIndex];
		const int32 vB = Tri[NextEdge(TriEdgeIndex)];

		const double Weight = (vA < vB) ? SplitWeight : (1. - SplitWeight);
		
		FDynamicMesh3::FEdgeSplitInfo EdgeSplitInfo;
		const UE::Geometry::EMeshResult Result = Mesh.SplitEdge(EdgeID, EdgeSplitInfo, Weight);

		UE::Geometry::FEdgeSplitInfo SplitInfoResult;
		if (Result != UE::Geometry::EMeshResult::Ok)
		{
			SplitInfoResult.NewTriIndex[0] = IndexConstants::InvalidID;
			return SplitInfoResult;
		}

		if (bHasEdgeConstraints)
		{
			// update the edge constraints, the constraints on the edge remain
			// the first entry in NewEdges is along the split edge and inherits from the unsplit edge
			// the other 2 edges are new and don't inherit any constraints.
			Constraints.SetOrUpdateEdgeConstraint(EdgeSplitInfo.NewEdges[0], Constraints.GetEdgeConstraint(EdgeID));
		}
				
		SplitInfoResult.OldTriIndex = EdgeSplitInfo.OriginalTriangles;
		SplitInfoResult.NewTriIndex = EdgeSplitInfo.NewTriangles;
		SplitInfoResult.NewVertIndex[0] = EdgeSplitInfo.NewVertex;
		SplitInfoResult.NewVertIndex[1] = EdgeSplitInfo.NewVertex;

		return SplitInfoResult;
	}

	UE::Geometry::FPokeInfo PokeTriangle(const int32 TriIndex, const FVector3d Barycentrics)
	{
		check(Mesh.IsTriangle(TriIndex));

		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

		FDynamicMesh3::FPokeTriangleInfo PokeTriangleInfo;

		UE::Geometry::FPokeInfo PokeInfoResult;

		const UE::Geometry::EMeshResult Result = Mesh.PokeTriangle(TriIndex, FVector3d(Barycentrics), PokeTriangleInfo);
		if (Result != UE::Geometry::EMeshResult::Ok)
		{
			PokeInfoResult.NewTriIndex[0] = IndexConstants::InvalidID;
			return PokeInfoResult;
		}
				
		PokeInfoResult.NewTriIndex = { TriIndex, PokeTriangleInfo.NewTriangles[0], PokeTriangleInfo.NewTriangles[1] };
		PokeInfoResult.EdgeIndex = { 0, 0, 0 };

#if DO_CHECK
		// validation
		for (int32 e = 0; e < 3; ++e)
		{
			const int Edge = Mesh.GetTriEdge(PokeInfoResult.NewTriIndex[e], PokeInfoResult.EdgeIndex[e] );
			check(Mesh.GetEdge(Edge).Vert[0] == FMath::Min(Triangle[e], Triangle[(e+1)%3]));
			check(Mesh.GetEdge(Edge).Vert[1] == FMath::Max(Triangle[e], Triangle[(e+1)%3]));

			// the next and prev edge should contain the new vertex
			const int NextEdge = Mesh.GetTriEdge(PokeInfoResult.NewTriIndex[e], (PokeInfoResult.EdgeIndex[e]+1)%3);
			check(Mesh.GetEdge(NextEdge).Vert[0] == FMath::Min(PokeTriangleInfo.NewVertex, Triangle[(e+1)%3]));
			check(Mesh.GetEdge(NextEdge).Vert[1] == FMath::Max(PokeTriangleInfo.NewVertex, Triangle[(e+1)%3]));

			const int PrevEdge = Mesh.GetTriEdge(PokeInfoResult.NewTriIndex[e], (PokeInfoResult.EdgeIndex[e]+2)%3);
			check(Mesh.GetEdge(PrevEdge).Vert[0] == FMath::Min(PokeTriangleInfo.NewVertex, Triangle[e]));
			check(Mesh.GetEdge(PrevEdge).Vert[1] == FMath::Max(PokeTriangleInfo.NewVertex, Triangle[e]));
		}
#endif

		PokeInfoResult.NewVertIndex = PokeTriangleInfo.NewVertex;
		
		return PokeInfoResult;
	}

	UE::Geometry::FFlipEdgeInfo FlipEdge(const int32 TriIndex, const int32 TriEdgeIndex)
	{
		FDynamicMesh3::FEdgeFlipInfo DMEdgeFlipInfo;

		const auto EdgeID = Mesh.GetTriEdge(TriIndex, TriEdgeIndex);

		check(!bHasEdgeConstraints || Constraints.GetEdgeConstraint(EdgeID).CanFlip());

		UE::Geometry::FFlipEdgeInfo FlipEdgeInfo;

		const UE::Geometry::EMeshResult Result = Mesh.FlipEdge(EdgeID, DMEdgeFlipInfo);

		if (Result != UE::Geometry::EMeshResult::Ok)
		{
			FlipEdgeInfo.Triangles = FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID);
			return FlipEdgeInfo;
		}

		// remove constraints from flipped edge, it's a new edge that can't inherit in a meaningful way
		// from the original edge. if there were split constraints, it would have implied also setting a flip constraint
		// but it's up to the user to decide
		if (bHasEdgeConstraints)
		{
			Constraints.ClearEdgeConstraint(EdgeID);
		}

		const auto FlippedEdge = Mesh.GetEdge(DMEdgeFlipInfo.EdgeID);
		check(FlippedEdge.Vert[0] == FMath::Min(DMEdgeFlipInfo.OpposingVerts[0], DMEdgeFlipInfo.OpposingVerts[1]));
		check(FlippedEdge.Vert[1] == FMath::Max(DMEdgeFlipInfo.OpposingVerts[0], DMEdgeFlipInfo.OpposingVerts[1]));

		
		FlipEdgeInfo.Triangles = DMEdgeFlipInfo.Triangles;

		// FDynamicMesh3::FlipEdge always constructs the triangles with shared edges first.
		FlipEdgeInfo.SharedEdge = FIndex2i(0, 0);

		return FlipEdgeInfo;
	}

	static constexpr bool IsTriangleSoup() { return false; }

  private:
	FDynamicMesh3& Mesh;
	FMeshConstraints& Constraints;
	const bool bHasEdgeConstraints;
};

} // namespace Geometry
} // namespace UE