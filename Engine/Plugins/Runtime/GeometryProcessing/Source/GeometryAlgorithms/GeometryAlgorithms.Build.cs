// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryAlgorithms : ModuleRules
{	
	public GeometryAlgorithms(ReadOnlyTargetRules Target) : base(Target)
	{
		IWYUSupport = IWYUSupport.KeepAsIsForNow;

        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// Note: The module purposefully doesn't have a dependency on CoreUObject.
		// If possible, we would like avoid having UObjects in GeometryProcessing
		// modules to keep the door open for writing standalone command-line programs
		// (which won't have UObject garbage collection).
        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"GeometryCore"
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Eigen",
			}
		);
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		// fTetWild can optinally use TBB for multithreading, but it is not consistently faster to do so; currently default-disabled
		bool bUseTBB = false;
		if (bUseTBB && // Note TBB is not supported on all platforms, so we only enable it on windows and mac
			(Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) || Target.Platform == UnrealTargetPlatform.Mac))
		{
			PrivateDefinitions.Add("FLOAT_TETWILD_USE_TBB");
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"IntelTBB"
			);
		}

		bDisableAutoRTFMInstrumentation = true;

		AddEngineThirdPartyPrivateStaticDependencies(Target, "MikkTSpace");
	}
}
