// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp Arrangement2d

#include "Arrangement2d.h"

#include "CompGeom/Delaunay2.h"

using namespace UE::Geometry;


namespace
{
//#define DEBUG_FILE_DUMPING 1
#ifndef DEBUG_FILE_DUMPING
	void DumpGraphForDebug(const FDynamicGraph2d& Graph, const FString& PathBase)
	{
	}
	void DumpGraphForDebugAsOBJ(const FDynamicGraph2d& Graph, const FString& PathBase)
	{
	}
	void DumpTriangulationForDebug(const FDynamicGraph2d& Graph, const TArray<FIntVector>& Triangles, const FString& PathBase)
	{
	}
#else
#include <fstream>
	static int num = 0;
	void DumpGraphForDebug(const FDynamicGraph2d& Graph, const FString& PathBase)
	{
		num++;
		FString Path = PathBase + FString::FromInt(num) + ".txt";
		std::ofstream f(*Path);

		for (int32 VertexIdx = 0; VertexIdx < Graph.MaxVertexID(); VertexIdx++)
		{
			const FVector2d& Vertex = Graph.GetVertex(VertexIdx);
			f << "v " << Vertex.X << " " << Vertex.Y << " " << std::endl;
		}
		for (const FDynamicGraph::FEdge& Edge : Graph.Edges())
		{
			f << "e " << Edge.A << " " << Edge.B << std::endl;
		}
		f.close();
	}
	void DumpGraphForDebugAsOBJ(const FDynamicGraph2d& Graph, const FString& PathBase)
	{
		num++;
		FString Path = PathBase + FString::FromInt(num) + ".obj";
		std::ofstream f(*Path);

		for (int32 VertexIdx = 0; VertexIdx < Graph.MaxVertexID(); VertexIdx++)
		{
			const FVector2d& Vertex = Graph.GetVertex(VertexIdx);
			f << "v " << Vertex.X << " " << Vertex.Y << " 0" << std::endl;
		}
		for (int32 VertexIdx = 0; VertexIdx < Graph.MaxVertexID(); VertexIdx++)
		{
			const FVector2d& Vertex = Graph.GetVertex(VertexIdx);
			f << "v " << Vertex.X << " " << Vertex.Y << " .5" << std::endl;
		}
		for (const FDynamicGraph::FEdge& Edge : Graph.Edges())
		{
			f << "f " << Edge.A + 1 << " " << Edge.B + 1 << " " << 1 + Edge.A + Graph.MaxVertexID() << std::endl;
		}
		f.close();
	}
	void DumpTriangulationForDebug(const FDynamicGraph2d& Graph, const TArray<FIntVector>& Triangles, const FString& PathBase)
	{
		num++;
		FString Path = PathBase + FString::FromInt(num) + ".obj";
		std::ofstream f(*Path);
		for (int32 VertexIdx = 0; VertexIdx < Graph.MaxVertexID(); VertexIdx++)
		{
			const FVector2d& Vertex = Graph.GetVertex(VertexIdx);
			f << "v " << Vertex.X << " " << Vertex.Y << " 0" << std::endl;
		}
		for (const FIntVector& Tri : Triangles)
		{
			f << "f " << 1 + Tri.X << " " << 1 + Tri.Y << " " << 1 + Tri.Z << std::endl;
		}
		f.close();
	}
#endif
}

bool FArrangement2d::AttemptTriangulate(TArray<FIntVector>& Triangles, TArray<int32>& SkippedEdges, int32 BoundaryEdgeGroupID)
{
	// TODO: Currently this just constructs an FIndex3i array and then copies it to an FIntVector array; if we need to construct FIntVector versions faster, we could directly construct them here instead
	// However this would require writing FIntVector versions of the GetTriangles functions in FDelaunay2; ideally we would instead mainly call the FIndex3i variant
	TArray<FIndex3i> TrianglesInd3i;
	bool bResult = TriangulateInternal(TrianglesInd3i, true, BoundaryEdgeGroupID, false, -1, true, &SkippedEdges);
	Triangles.SetNumUninitialized(TrianglesInd3i.Num());
	for (int32 Idx = 0; Idx < TrianglesInd3i.Num(); Idx++)
	{
		Triangles[Idx] = (FIntVector)TrianglesInd3i[Idx];
	}
	return bResult;
}

bool FArrangement2d::TriangulateInternal(TArray<FIndex3i>& Triangles, bool bHasBoundaryGroupID, int32 BoundaryEdgeGroupID, bool bHasHoleGroupID, int32 HoleEdgeGroupID, bool bLegacyKeepTrianglesIfBoundaryNotFound, TArray<int32>* SkippedEdges)
{
	Triangles.Empty();

	// A flat array of the vertices, copied out of the graph
	TArray<FVector2d> InputVertices;

	// If there are unused vertices, define a vertex index remapping so the delaunay code won't have to understand that
	bool bNeedsRemap = Graph.MaxVertexID() != Graph.VertexCount();
	TArray<int> InputIndices, OutputIndices;
	if (bNeedsRemap)
	{
		InputIndices.SetNumZeroed(Graph.MaxVertexID());
		OutputIndices.SetNumZeroed(Graph.VertexCount());
		int TotalOffset = 0;
		for (int i = 0; i < Graph.MaxVertexID(); i++)
		{
			if (Graph.IsVertex(i))
			{
				OutputIndices[i - TotalOffset] = i;
				InputIndices[i] = i - TotalOffset;
				FVector2d Vertex = Graph.GetVertex(i);
				InputVertices.Add(Vertex);
			}
			else
			{
				InputIndices[i] = -1;
				TotalOffset++;
			}
		}
	}
	else
	{	// no index remapping needed; just copy the vertices
		for (int i = 0; i < Graph.MaxVertexID(); i++)
		{
			FVector2d Vertex = Graph.GetVertex(i);
			InputVertices.Add(Vertex);
		}
	}

	FDelaunay2 Delaunay;
	Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = false; // Arrangement will remove duplicates already

	if (!Delaunay.Triangulate(InputVertices))
	{
		return false;
	}

	Delaunay.bValidateEdges = false;
	Delaunay.bKeepFastEdgeAdjacencyData = true;

	bool bInsertConstraintFailure = false;
	bool bBoundaryTrackingFailure = false;

	TArray<FIndex2i> AllEdges, BoundaryEdges, HoleEdges;

	for (int EdgeIdx : Graph.EdgeIndices())
	{
		FDynamicGraph::FEdge Edge = Graph.GetEdge(EdgeIdx);
		if (bNeedsRemap)
		{
			Edge.A = InputIndices[Edge.A];
			Edge.B = InputIndices[Edge.B];
		}
		FIndex2i& AddedEdge = AllEdges.Emplace_GetRef(Edge.A, Edge.B);
		if (bHasBoundaryGroupID && Edge.Group == BoundaryEdgeGroupID)
		{
			BoundaryEdges.Add(AddedEdge);
		}
		if (bHasHoleGroupID && Edge.Group == HoleEdgeGroupID)
		{
			HoleEdges.Add(AddedEdge);
		}
	}

	Delaunay.ConstrainEdges(InputVertices, AllEdges);

	// Verify all edges after all constraints are in -- to ensure that inserted edges were also not removed by subsequent edge insertion
	for (int EdgeIdx : Graph.EdgeIndices())
	{
		FDynamicGraph::FEdge Edge = Graph.GetEdge(EdgeIdx);
		if (bNeedsRemap)
		{
			Edge.A = InputIndices[Edge.A];
			Edge.B = InputIndices[Edge.B];
		}
		if (!Delaunay.HasEdge(FIndex2i(Edge.A, Edge.B), false))
		{
			bInsertConstraintFailure = true;
			if (SkippedEdges)
			{
				SkippedEdges->Add(EdgeIdx);
			}
			if (Edge.Group == BoundaryEdgeGroupID)
			{
				bBoundaryTrackingFailure = true;
			}
		}
	}

	if (bHasBoundaryGroupID && (!bLegacyKeepTrianglesIfBoundaryNotFound || (!bBoundaryTrackingFailure && BoundaryEdges.Num() > 0)))
	{
		if (bHasHoleGroupID)
		{
			Delaunay.GetFilledTriangles(Triangles, BoundaryEdges, HoleEdges);
		}
		else
		{
			Triangles = Delaunay.GetFilledTriangles(BoundaryEdges, FDelaunay2::EFillMode::Solid);
		}
	}
	else
	{
		Triangles = Delaunay.GetTriangles();
	}

	if (bNeedsRemap)
	{
		for (FIndex3i& Face : Triangles)
		{
			Face.A = OutputIndices[Face.A];
			Face.B = OutputIndices[Face.B];
			Face.C = OutputIndices[Face.C];
		}
	}
	return !bInsertConstraintFailure;
}

bool FArrangement2d::AttemptTriangulate(TArray<FIndex3i>& Triangles, TArray<int32>& SkippedEdges, int32 BoundaryEdgeGroupID)
{
	return TriangulateInternal(Triangles, true, BoundaryEdgeGroupID, false, -1, true, &SkippedEdges);
}

bool FArrangement2d::TriangulateWithBoundaryAndHoles(TArray<FIndex3i>& Triangles, int32 BoundaryEdgeGroupID, int32 HoleEdgeGroupID)
{
	return TriangulateInternal(Triangles, true, BoundaryEdgeGroupID, true, HoleEdgeGroupID, false, nullptr);
}

bool FArrangement2d::TriangulateWithBoundary(TArray<FIndex3i>& Triangles, int32 BoundaryEdgeGroupID)
{
	return TriangulateInternal(Triangles, true, BoundaryEdgeGroupID, false, -1, false, nullptr);
}

bool FArrangement2d::Triangulate(TArray<FIndex3i>& Triangles)
{
	return TriangulateInternal(Triangles, false, -1, false, -1, false, nullptr);
}
