// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstrainedDelaunay2.h"
#include "CompGeom/Delaunay2.h"
#include "Async/ParallelFor.h"
#include "Curve/PolygonIntersectionUtils.h"

using namespace UE::Geometry;

//namespace
//{
//#define DEBUG_FILE_DUMPING 1
//#ifndef DEBUG_FILE_DUMPING
//	void DumpDelaunayInputForDebugAsOBJ(const FConstrainedDelaunay2d& Delaunay, const FString& PathBase)
//	{
//	}
//	void DumpDelaunayTriangulationForDebug(const FConstrainedDelaunay2d& Delaunay, const FString& PathBase)
//	{
//	}
//#else
//#include <fstream>
//	static int num = 0;
//	template <typename RealType>
//	void DumpDelaunayInputForDebugAsOBJ(const TConstrainedDelaunay2<RealType>& Delaunay, const FString& PathBase)
//	//void DumpGraphForDebugAsOBJ(const FDynamicGraph2d& Graph, const FString& PathBase)
//	{
//		num++;
//		FString Path = PathBase + FString::FromInt(num) + ".obj";
//		std::ofstream f(*Path);
//
//		for (int32 VertexIdx = 0; VertexIdx < Delaunay.Vertices.Num(); VertexIdx++)
//		{
//			const TVector2<RealType>& Vertex = Delaunay.Vertices[VertexIdx];
//			f << "v " << Vertex.X << " " << Vertex.Y << " 0" << std::endl;
//		}
//		for (int32 VertexIdx = 0; VertexIdx < Delaunay.Vertices.Num(); VertexIdx++)
//		{
//			const TVector2<RealType>& Vertex = Delaunay.Vertices[VertexIdx];
//			f << "v " << Vertex.X << " " << Vertex.Y << " .5" << std::endl;
//		}
//		for (const FIndex2i& Edge : Delaunay.Edges)
//		{
//			f << "f " << Edge.A + 1 << " " << Edge.B + 1 << " " << 1 + Edge.A + Delaunay.Vertices.Num() << std::endl;
//		}
//		f.close();
//	}
//	//void DumpTriangulationForDebug(const FDynamicGraph2d& Graph, const TArray<FIntVector>& Triangles, const FString& PathBase)
//	template <typename RealType>
//	void DumpDelaunayTriangulationForDebug(const TConstrainedDelaunay2<RealType>& Delaunay, const FString& PathBase)
//	{
//		num++;
//		FString Path = PathBase + FString::FromInt(num) + ".obj";
//		std::ofstream f(*Path);
//		for (int32 VertexIdx = 0; VertexIdx < Delaunay.Vertices.Num(); VertexIdx++)
//		{
//			const TVector2<RealType>& Vertex = Delaunay.Vertices[VertexIdx];
//			f << "v " << Vertex.X << " " << Vertex.Y << " 0" << std::endl;
//		}
//		for (const FIndex3i& Tri : Delaunay.Triangles)
//		{
//			f << "f " << 1 + Tri.A << " " << 1 + Tri.B << " " << 1 + Tri.C << std::endl;
//		}
//		f.close();
//	}
//#endif
//}


void AddOrderedEdge(TMap<TPair<int, int>, bool>& EdgeMap, int VertA, int VertB)
{
	bool bReversed = VertA > VertB;
	if (bReversed)
	{
		Swap(VertA, VertB);
	}
	EdgeMap.Add(TPair<int, int>(VertA, VertB), bReversed);
}

/**
 * Compute the change in winding number from crossing an oriented edge connecting VertA to VertB
 *
 * @param EdgeMap Map of known edges & orientations
 * @param VertA First vertex on edge
 * @param VertB Second vertex on edge
 * @return  -1 if reverse edge (B-A) found, 1 if forward edge (A-B) found, 0 otherwise
 */
int WindingAcross(const TMap<TPair<int, int>, bool>& EdgeMap, int VertA, int VertB)
{
	bool bReversed = VertA > VertB;
	if (bReversed)
	{
		Swap(VertA, VertB);
	}
	TPair<int, int> EdgeKey(VertA, VertB);
	const bool *bFoundReversed = EdgeMap.Find(EdgeKey);
	if (!bFoundReversed)
	{
		return 0;
	}
	bool bSameDir = bReversed == *bFoundReversed;
	return bSameDir ? 1 : -1;
}

/**
 * Check if any edge in edge map connects VertA to VertB (or VertB to VertA)
 * 
 * @param EdgeMap Map of known edges & orientations
 * @param VertA First vertex on edge
 * @param VertB Second vertex on edge
 * @return  true if VertA and VertB are connected by an edge (in either direction)
 */
bool HasUnorderedEdge(const TMap<TPair<int, int>, bool>& EdgeMap, int VertA, int VertB)
{
	return EdgeMap.Contains(TPair<int, int>(FMath::Min(VertA, VertB), FMath::Max(VertA, VertB)));
}

namespace ConstrainedDelaunay2Internal
{
	template<class RealType>
	void SplitBowtiesHelper(TArray<TPair<int, int>>& NeedUpdates, TArray<TVector2<RealType>>& Vertices, TArray<int8>& Keep, const TArray<FIndex3i>& Indices, const TArray<FIndex3i>& Adj)
	{
		int TriNum = Adj.Num();
		int32 OrigNumVertices = Vertices.Num();
		// track all wedge verts that are seen by walking local tris
		auto OtherEdgeOnTri = [&Indices](int VertID, int TriID, int EdgeIdx)
		{
			int StepNext = 1;
			if (Indices[TriID][EdgeIdx] == VertID) {
				StepNext = 2;
			}
			return (EdgeIdx + StepNext) % 3;
		};
		// helper to find new edge idx of the edge you crossed to go from FromTriID over to ToTriID
		auto CrossEdge = [&Adj](int FromTriID, int ToTriID)
		{
			for (int AdjEdgeIdx = 0; AdjEdgeIdx < 3; AdjEdgeIdx++)
			{
				if (Adj[ToTriID][AdjEdgeIdx] == FromTriID)
				{
					return AdjEdgeIdx;
				}
			}
			checkSlow(false);
			return -1;
		};
		auto GetVertSubIdx = [&Indices](int VertID, int TriID)
		{
			for (int VertSubIdx = 0; VertSubIdx < 3; VertSubIdx++)
			{
				if (Indices[TriID][VertSubIdx] == VertID)
				{
					return VertSubIdx;
				}
			}
			checkSlow(false);
			return -1;
		};
		auto Walk = [&Adj, &OtherEdgeOnTri](int VertID, int TriID, int EdgeSubIdx)
		{
			int OtherEdgeSubIdx = OtherEdgeOnTri(VertID, TriID, EdgeSubIdx);
			int AdjTri = Adj[TriID][OtherEdgeSubIdx];
			return AdjTri;
		};
		TArray<bool> Seen, SeenSource; Seen.SetNumZeroed(TriNum * 3); SeenSource.SetNumZeroed(Vertices.Num());
		for (int TriID = 0; TriID < TriNum; TriID++)
		{
			if (Keep[TriID] != 1)
			{
				continue;
			}
			for (int SubIdx = 0, OtherSubIdx = 2; SubIdx < 3; OtherSubIdx = SubIdx++)
			{
				int WedgeIdx = TriID * 3 + SubIdx;
				int VertID = Indices[TriID][SubIdx];

				if (Seen[WedgeIdx]) // already been walked over & therefore covered by previous pass
				{
					continue;
				}

				// if seen source but haven't seen specific wedge, then we need to duplicate the vertex and re-link
				bool bSeenSource = SeenSource[VertID];

				int NewVertID = -1;
				if (bSeenSource)
				{
					TVector2<RealType> VertexToCopy = Vertices[VertID];
					NewVertID = Vertices.Add(VertexToCopy);
				}

				// process all triangles starting from the given tri ID and edge idx; return true if looped, false otherwise
				auto WalkAll = [&TriNum, &Indices, &Seen, &SeenSource, &bSeenSource, &NeedUpdates, &NewVertID, &Keep,
					&CrossEdge, &GetVertSubIdx, &Walk](int WalkVertID, int WalkTriID, int WalkSubIdx)
				{
					int StartTriID = WalkTriID;
					int SafetyCounter = 0;
					while (true)
					{
						int VertSubIdx = GetVertSubIdx(WalkVertID, WalkTriID);
						int WalkWedgeIdx = WalkTriID * 3 + VertSubIdx;
						ensure(!Seen[WalkWedgeIdx]);
						checkSlow(Indices[WalkTriID][VertSubIdx] == WalkVertID);
						Seen[WalkWedgeIdx] = true;
						if (bSeenSource)
						{
							NeedUpdates.Add(TPair<int, int>(WalkWedgeIdx, NewVertID));
						}

						int NextTriID = Walk(WalkVertID, WalkTriID, WalkSubIdx);
						if (NextTriID < 0 || Keep[NextTriID] != 1)
						{
							return false;
						}
						WalkSubIdx = CrossEdge(WalkTriID, NextTriID);
						WalkTriID = NextTriID;
						if (WalkTriID == StartTriID)
						{
							return true;
						}
						check(SafetyCounter++ < TriNum * 2); // infinite loop catcher
					}
				};
				bool bLooped = WalkAll(VertID, TriID, SubIdx);
				if (!bLooped)
				{
					// if it didn't loop around, also walk the other direction
					int OtherWayTriID = Walk(VertID, TriID, OtherSubIdx);
					if (OtherWayTriID >= 0 && Keep[OtherWayTriID] == 1)
					{
						int OtherWayTriSubIdx = CrossEdge(TriID, OtherWayTriID);
						ensure(!WalkAll(VertID, OtherWayTriID, OtherWayTriSubIdx));
					}
				}

				SeenSource[VertID] = true;
			}
		}
	}

	void BuildFinalTriangles(TArray<FIndex3i>& Triangles, TArray<TPair<int, int>>& NeedUpdates, int& AddedVerticesStartIndex, TArray<int8>& Keep, const TArray<FIndex3i>& Indices, int OrigNumVertices, bool bOutputCCW)
	{
		int TriNum = Indices.Num();

		// function to build output triangles out of an indices array
		// normally called directly on the const indices from the CDT, but will be called on an updated copy if bowtie splits happen
		auto BuildTriangles = [&Triangles, &TriNum, &Keep, &bOutputCCW](const TArray<FIndex3i>& IndicesIn)
		{
			for (int i = 0; i < TriNum; i++)
			{
				if (Keep[i] > 0)
				{
					FIndex3i& Tri = Triangles.Emplace_GetRef(IndicesIn[i].A, IndicesIn[i].B, IndicesIn[i].C);
					if (!bOutputCCW)
					{
						Swap(Tri.B, Tri.C);
					}
				}
			}
		};
		if (NeedUpdates.Num() > 0)
		{
			TArray<FIndex3i> UpdatedIndices = Indices;
			for (const TPair<int, int>& Update : NeedUpdates)
			{
				int TriID = (int)Update.Key / 3;
				int SubIdx = Update.Key % 3;
				UpdatedIndices[TriID][SubIdx] = Update.Value;
			}

			AddedVerticesStartIndex = OrigNumVertices;

			BuildTriangles(UpdatedIndices);
		}
		else
		{
			BuildTriangles(Indices);
		}
	}
}

template<class RealType>
bool TConstrainedDelaunay2<RealType>::AddWithIntersectionResolution(const TPolygon2<RealType>& Polygon)
{
	TGeneralPolygon2<RealType> GPolygon(Polygon);
	return AddWithIntersectionResolution(GPolygon);
}

template<class RealType>
bool TConstrainedDelaunay2<RealType>::AddWithIntersectionResolution(const TGeneralPolygon2<RealType>& GPolygon)
{
	TGeneralPolygon2<RealType> Empty;
	TUnionPolygon2Polygon2<TGeneralPolygon2<RealType>, RealType> Union(GPolygon, Empty);
	bool bResolveSuccess = Union.ComputeResult();
	if (bResolveSuccess)
	{
		for (const TGeneralPolygon2<RealType>& ResultGPoly : Union.Result)
		{
			Add(ResultGPoly);
		}
	}
	return bResolveSuccess;
}


template<class RealType>
bool TConstrainedDelaunay2<RealType>::Triangulate(TFunctionRef<bool(const TArray<TVector2<RealType>>&, const FIndex3i&)> KeepTriangle)
{
	Triangles.Empty();

	FDelaunay2 Delaunay;
	Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = true;
	if (!Delaunay.Triangulate(Vertices))
	{
		return false;
	}
	Delaunay.bKeepFastEdgeAdjacencyData = true;
	bool bEdgesFailed = !Delaunay.ConstrainEdges(Vertices, Edges);
	bool bHoleEdgesFailed = !Delaunay.ConstrainEdges(Vertices, HoleEdges);
	bool bBoundaryTrackingFailure = bEdgesFailed || bHoleEdgesFailed;

	TArray<FIndex3i> Indices, Adj;
	Delaunay.GetTrianglesAndAdjacency(Indices, Adj);

	int TriNum = Adj.Num();
	TArray<int8> Keep;  // values: 0->unprocessed (delete), 1->yes keep, -1->processed, delete
	Keep.SetNumZeroed(TriNum);

	ParallelFor(TriNum, [this, &KeepTriangle, &Indices, &Keep](int32 Index)
	{
		bool bKeepTri = KeepTriangle(Vertices, Indices[Index]);
		Keep[Index] = bKeepTri ? 1 : -1;
	});

	TArray<TPair<int, int>> NeedUpdates; // stores all wedge indices and the corresponding new vertices they require
	int32 OrigNumVertices = Vertices.Num();
	if (bSplitBowties)
	{
		ConstrainedDelaunay2Internal::SplitBowtiesHelper(NeedUpdates, Vertices, Keep, Indices, Adj);
	}

	ConstrainedDelaunay2Internal::BuildFinalTriangles(Triangles, NeedUpdates, AddedVerticesStartIndex, Keep, Indices, OrigNumVertices, bOutputCCW);

	return !bBoundaryTrackingFailure;
}

template<class RealType>
bool TConstrainedDelaunay2<RealType>::Triangulate()
{
	Triangles.Empty();

	check(FillRule <= EFillRule::Odd || bOrientedEdges);

	FDelaunay2 Delaunay;
	Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = true;
	if (!Delaunay.Triangulate(Vertices))
	{
		return false;
	}

	Delaunay.bKeepFastEdgeAdjacencyData = true;
	Delaunay.bValidateEdges = false; // edge validation will be done manually later
	Delaunay.ConstrainEdges(Vertices, Edges);
	Delaunay.ConstrainEdges(Vertices, HoleEdges);

	TMap<TPair<int, int>, bool> BoundaryMap, HoleMap; // tracks all the boundary edges as they are added, so we can later flood fill across them for inside/outside decisions
	TMap<TPair<int, int>, bool>* EdgeAndHoleMaps[2] = { &BoundaryMap, &HoleMap };
	bool bBoundaryTrackingFailure = false;

	TArray<FIndex2i>* InputEdgesAndHoles[2] = { &Edges, &HoleEdges };
	for (int EdgeOrHole = 0; EdgeOrHole < 2; EdgeOrHole++)
	{
		TArray<FIndex2i>& Input = *InputEdgesAndHoles[EdgeOrHole];
		TMap<TPair<int, int>, bool>& InputMap = *EdgeAndHoleMaps[EdgeOrHole];
		for (FIndex2i Edge : Input)
		{
			Delaunay.FixDuplicatesOnEdge(Edge);
			if (!Delaunay.HasEdge(Edge, false))
			{
				// Note the failed edge; we will try to proceed anyway, just without this edge.  This can happen for example if the edge is exactly on top of another edge
				bBoundaryTrackingFailure = true;
			}
			else
			{
				AddOrderedEdge(InputMap, Edge.A, Edge.B);
			}
		}
	}

	TArray<FIndex3i> Indices, Adj;
	Delaunay.GetTrianglesAndAdjacency(Indices, Adj);

	int TriNum = Adj.Num();
	TArray<int8> Keep;  // values: 0->unprocessed (delete), 1->yes keep, -1->processed, delete
	Keep.SetNumZeroed(TriNum);

	TArray<TPair<int, int>> ToWalkQ; // Pair of tri index, winding number
	// seed the queue with all triangles that are on the boundary of the convex hull
	// note: need *all* not just *one* because of the strategy of refusing to cross hole edges; if using pure winding number classification would just need one boundary triangle to start
	for (int TriIdx = 0; TriIdx < TriNum; TriIdx++)
	{
		for (int SubIdx = 0, NextIdx = 2; SubIdx < 3; NextIdx = SubIdx++)
		{
			if (Adj[TriIdx][NextIdx] < 0) // on hull
			{
				int VertA = Indices[TriIdx][SubIdx], VertB = Indices[TriIdx][NextIdx];
				if (HasUnorderedEdge(HoleMap, VertA, VertB))
				{
					continue; // cannot cross hole edges
				}
				// note we negate the winding across for these hull triangles because we're not actually crossing the edge; we're already on the 'inside' of the hull edge
				int Winding = -WindingAcross(BoundaryMap, VertA, VertB);
				ToWalkQ.Add(TPair<int, int>(TriIdx, Winding));
				Keep[TriIdx] = ClassifyFromRule(Winding) ? 1 : -1;
				break; // don't check any more edges once in queue
			}
		}
	}

	int SelIdx = 0; // Index of item to Pop next; used to make the traversal less depth-first in shape, so a little more robust to bad data
	while (ToWalkQ.Num())
	{
		SelIdx = (SelIdx + 1) % ToWalkQ.Num();
		TPair<int, int> TriWithWinding = ToWalkQ[SelIdx];
		ToWalkQ.RemoveAtSwap(SelIdx);
		int TriIdx = TriWithWinding.Key;
		int LastWinding = TriWithWinding.Value;
		for (int SubIdx = 0, NextIdx = 2; SubIdx < 3; NextIdx = SubIdx++)
		{
			int VertA = Indices[TriIdx][SubIdx], VertB = Indices[TriIdx][NextIdx];
			if (HasUnorderedEdge(HoleMap, VertA, VertB))
			{
				continue; // cannot cross hole edges
			}
			int AdjTri = Adj[TriIdx][NextIdx];
			if (AdjTri >= 0 && Keep[AdjTri] == 0)
			{
				int WindingChange = WindingAcross(BoundaryMap, VertA, VertB);
				int Winding = LastWinding + WindingChange;
				ToWalkQ.Add(TPair<int, int>(AdjTri, Winding));
				Keep[AdjTri] = ClassifyFromRule(Winding) ? 1 : -1;
			}
		}
	}




	TArray<TPair<int, int>> NeedUpdates; // stores all wedge indices and the corresponding new vertices they require
	int32 OrigNumVertices = Vertices.Num();
	if (bSplitBowties)
	{
		ConstrainedDelaunay2Internal::SplitBowtiesHelper(NeedUpdates, Vertices, Keep, Indices, Adj);
	}


	ConstrainedDelaunay2Internal::BuildFinalTriangles(Triangles, NeedUpdates, AddedVerticesStartIndex, Keep, Indices, OrigNumVertices, bOutputCCW);

	return !bBoundaryTrackingFailure;
}

template<typename RealType>
TArray<FIndex3i> GEOMETRYALGORITHMS_API UE::Geometry::ConstrainedDelaunayTriangulate(const TGeneralPolygon2<RealType>& GeneralPolygon)
{
	TConstrainedDelaunay2<RealType> Triangulation;
	Triangulation.FillRule = TConstrainedDelaunay2<RealType>::EFillRule::Positive;
	Triangulation.Add(GeneralPolygon);
	Triangulation.Triangulate();
	return Triangulation.Triangles;
}
template<typename RealType>
TArray<FIndex3i> GEOMETRYALGORITHMS_API UE::Geometry::ConstrainedDelaunayTriangulateWithVertices(const TGeneralPolygon2<RealType>& GeneralPolygon, TArray<TVector2<RealType>>& OutVertices)
{
	TConstrainedDelaunay2<RealType> Triangulation;
	Triangulation.FillRule = TConstrainedDelaunay2<RealType>::EFillRule::Positive;
	Triangulation.Add(GeneralPolygon);
	Triangulation.Triangulate();
	OutVertices = MoveTemp(Triangulation.Vertices);
	return Triangulation.Triangles;
}

namespace UE
{
namespace Geometry
{

template TArray<FIndex3i> GEOMETRYALGORITHMS_API UE::Geometry::ConstrainedDelaunayTriangulate(const TGeneralPolygon2<double>& GeneralPolygon);
template TArray<FIndex3i> GEOMETRYALGORITHMS_API UE::Geometry::ConstrainedDelaunayTriangulate(const TGeneralPolygon2<float>& GeneralPolygon);
template TArray<FIndex3i> GEOMETRYALGORITHMS_API UE::Geometry::ConstrainedDelaunayTriangulateWithVertices(const TGeneralPolygon2<double>& GeneralPolygon, TArray<TVector2<double>>& Vertices);
template TArray<FIndex3i> GEOMETRYALGORITHMS_API UE::Geometry::ConstrainedDelaunayTriangulateWithVertices(const TGeneralPolygon2<float>& GeneralPolygon, TArray<TVector2<float>>& Vertices);



template struct TConstrainedDelaunay2<float>;
template struct TConstrainedDelaunay2<double>;

} // end namespace UE::Geometry
} // end namespace UE