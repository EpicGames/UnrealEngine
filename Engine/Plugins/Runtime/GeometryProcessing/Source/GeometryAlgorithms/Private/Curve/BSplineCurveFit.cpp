// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curve/BSplineCurveFit.h"

#include "ThirdParty/GTEngine/Mathematics/GteBSplineCurveFit.h"
#include "ThirdParty/GTEngine/LowLevel/GteLogger.h"

namespace UE::Geometry
{

namespace SplineCurveFitUtils
{

bool HasValidParameters(const int NumSamples, const int Degree, const int NumControlPoints)
{
	if (Degree < 1)
	{
		return false;
	}

	if (Degree >= NumControlPoints)
	{
		return false;
	}

	if (NumControlPoints > NumSamples)
	{
		return false;
	}

	return true;
}
	
class ScopedListener : public gte::Logger::Listener
{
public:
	typedef  gte::Logger::Listener  MyBase;
	ScopedListener( int flags = MyBase::LISTEN_FOR_ALL) 
	: MyBase(flags)
	, bHasReport(false)
	{
		gte::Logger::Subscribe(this);
	}

	virtual ~ScopedListener()
	{
		gte::Logger::Unsubscribe(this);
	}

	bool HasReport() const { return bHasReport;}
private:
	virtual void Report(std::string const& message) override
	{
		bHasReport= true;
	}
	
	bool bHasReport;
};

} }

bool UE::Geometry::BSplineCurveFit(const TArray<FVector2f>& DataPoints, const int32 SplineDegree, const int32 NumControlPoints, TArray<FVector2f>& ControlPointsOut)
{
	typedef FVector2f VectorType;
	bool bSuccess = false;

	using FReal = float;
	const int32 Dim = 2;
	const int32 NumSamples = DataPoints.Num();

	if (!SplineCurveFitUtils::HasValidParameters(NumSamples, SplineDegree, NumControlPoints))
	{
		return bSuccess;
	}

	TArray<float> Data;
	Data.SetNumUninitialized(DataPoints.Num()* Dim);
	for (int i = 0, I = DataPoints.Num(); i < I; ++i)
	{
		Data[Dim * i] = DataPoints[i].X;
		Data[Dim * i + 1] = DataPoints[i].Y;
	}

	// listener will catch errors with the curve fit.
	SplineCurveFitUtils::ScopedListener Listener;

	// fit the curve
	gte::BSplineCurveFit<float>  CurveFit(Dim, NumSamples, Data.GetData(), SplineDegree, NumControlPoints);


	if (!Listener.HasReport())
	{ 
		bSuccess = true;
		const float* ControlData = CurveFit.GetControlData();

		ControlPointsOut.SetNumUninitialized(NumControlPoints);
		for (int i = 0, I = NumControlPoints; i < I; ++i)
		{
			ControlPointsOut[i] = VectorType(ControlData[Dim * i] , ControlData[Dim * i + 1]);
		}	
	}

	return bSuccess;
}

bool  UE::Geometry::BSplineCurveFit(const TArray<FVector3f>& DataPoints, const int32 SplineDegree, const int32 NumControlPoints, TArray<FVector3f>& ControlPointsOut)
{
	typedef FVector3f VectorType;

	bool bSuccess = false;

	using FReal = float;
	const int32 Dim = 3;
	const int32 NumSamples = DataPoints.Num();

	if (!SplineCurveFitUtils::HasValidParameters(NumSamples, SplineDegree, NumControlPoints))
	{
		return bSuccess;
	}

	TArray<float> Data;
	Data.SetNumUninitialized(DataPoints.Num() * Dim);
	for (int i = 0, I = DataPoints.Num(); i < I; ++i)
	{
		Data[Dim * i] = DataPoints[i].X;
		Data[Dim * i + 1] = DataPoints[i].Y;
		Data[Dim * i + 2] = DataPoints[i].Z;
	}

	// listener will catch errors with the curve fit.
	SplineCurveFitUtils::ScopedListener Listener;

	gte::BSplineCurveFit<float>  CurveFit(Dim, NumSamples, Data.GetData(), SplineDegree, NumControlPoints);


	if (!Listener.HasReport())
	{ 
		bSuccess = true;

		const float* ControlData = CurveFit.GetControlData();

		ControlPointsOut.SetNumUninitialized(NumControlPoints);
		for (int i = 0, I = NumControlPoints; i < I; ++i)
		{
			ControlPointsOut[i] = VectorType(ControlData[Dim * i], ControlData[Dim * i + 1], ControlData[Dim * i + 2]);
		}
	}
	return bSuccess;
}
