// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curve/GeneralPolygon2.h"
#include "Math/NumericLimits.h"
THIRD_PARTY_INCLUDES_START
#include "ThirdParty/clipper/clipper.h"
THIRD_PARTY_INCLUDES_END

namespace UE::Geometry::Private
{
	// Matches what's in the clipper lib, precision can change per platform and int64 doesn't reflect this
	using IntegralType = int64;

	static constexpr IntegralType IntMin = 0;
	static constexpr IntegralType IntMax = 9007199254740991; // fit to double bits, product of 2^53 - 1
	static constexpr IntegralType IntRange = IntMax - IntMin;
	
	template <typename RealType, typename OutputType>
	static Clipper2Lib::Point<OutputType> PackVector(const TVector2<RealType> InVector, const TVector2<RealType>& InMin, const RealType& InRange);

	template <typename InputType, typename RealType>
	static TVector2<RealType> UnpackVector(const Clipper2Lib::Point<InputType>& InPoint, const TVector2<RealType>& InMin, const RealType& InRange);
	
	template <typename RealType, typename OutputType>
	static Clipper2Lib::Path<OutputType> ConvertPolygonToPath(const TPolygon2<RealType>& InPolygon, const TVector2<RealType>& InMin = {}, const RealType& InRange = {});

	template <typename RealType, typename OutputType>
	static Clipper2Lib::Path<OutputType> ConvertPolygonToPath(const TArrayView<UE::Math::TVector2<RealType>>& InPolygon, const TVector2<RealType>& InMin = {}, const RealType& InRange = {});
		
	template <typename RealType, typename OutputType>
	static Clipper2Lib::Paths<OutputType> ConvertPolygonsToPaths(const TArray<TArrayView<UE::Math::TVector2<RealType>>>& InPolygons, const TVector2<RealType>& InMin = {}, const RealType& InRange = {});

	template <typename RealType, typename OutputType>
	static Clipper2Lib::Paths<OutputType> ConvertGeneralizedPolygonToPath(const UE::Geometry::TGeneralPolygon2<RealType>& InPolygon, const TVector2<RealType>& InMin = {}, const RealType& InRange = {});

	template <typename RealType, typename OutputType>
	static void AddGeneralizedPolygonToPaths(Clipper2Lib::Paths<OutputType>& OutPaths, const UE::Geometry::TGeneralPolygon2<RealType>& InPolygon, const TVector2<RealType>& InMin = {}, const RealType& InRange = {});

	template <typename InputType, typename RealType>
	static TPolygon2<RealType> ConvertPathToPolygon(const Clipper2Lib::Path<InputType>& InPath, const TVector2<RealType>& InMin = {}, const RealType& InRange = {});
	 
	template <typename InputType, typename RealType>
	static void ConvertPolyTreeToPolygons(const Clipper2Lib::PolyPath<InputType>* InPaths, TArray<UE::Geometry::TGeneralPolygon2<RealType>>& OutPolygons, const TVector2<RealType>& InMin = {} , const RealType& InRange = {}, int ParentPolygonIdx = -1);

	template<typename InputType, typename RealType>
	TArray<TVector2<RealType>> ConvertPathToUEPath(const Clipper2Lib::Path<InputType>& InPath, const TVector2<RealType>& InMin, const RealType& InRange);

	template<typename InputType, typename RealType>
	TArray<TArray<TVector2<RealType>>> ConvertPathsToUEPaths(const Clipper2Lib::Paths<InputType>& InPaths, const TVector2<RealType>& InMin, const RealType& InRange);
}

#include "ClipperUtils.inl"
