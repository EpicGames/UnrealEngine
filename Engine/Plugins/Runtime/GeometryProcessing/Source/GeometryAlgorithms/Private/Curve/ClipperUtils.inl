// Copyright Epic Games, Inc. All Rights Reserved.

#if USING_CODE_ANALYSIS
#include "Curve/ClipperUtils.h"
#endif

namespace UE::Geometry::Private
{
	template <typename RealType, typename OutputType>
	Clipper2Lib::Point<OutputType> PackVector(const TVector2<RealType> InVector, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		constexpr RealType RealTypeRange = static_cast<RealType>(IntRange);
		constexpr RealType RealTypeMin = static_cast<RealType>(IntMin);
		
		return
		Clipper2Lib::Point<OutputType>{
			static_cast<OutputType>((((InVector.X - InMin.X) / InRange) * RealTypeRange) + RealTypeMin),
			static_cast<OutputType>((((InVector.Y - InMin.Y) / InRange) * RealTypeRange) + RealTypeMin)
		};
	}

	template <typename InputType, typename RealType>
	TVector2<RealType> UnpackVector(const Clipper2Lib::Point<InputType>& InPoint, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		const Clipper2Lib::Point<RealType> RealTypePoint(static_cast<RealType>(InPoint.x), static_cast<RealType>(InPoint.y));
		constexpr RealType RealTypeRange = static_cast<RealType>(IntRange);
		constexpr RealType RealTypeMin = static_cast<RealType>(IntMin);

		return
		{
			(((static_cast<RealType>(RealTypePoint.x - RealTypeMin) / RealTypeRange) * InRange) + InMin.X),
			(((static_cast<RealType>(RealTypePoint.y - RealTypeMin) / RealTypeRange) * InRange) + InMin.Y)
		};
	}

	template <typename RealType, typename OutputType>
	static Clipper2Lib::Path<OutputType> ConvertPolygonToPath(const TPolygon2<RealType>& InPolygon, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		Clipper2Lib::Path<OutputType> Path;
		Path.reserve(InPolygon.VertexCount());
		for(int32 Idx = 0; Idx < InPolygon.VertexCount(); ++Idx)
		{
			// Integral output normalized the input floating point values
			if constexpr (TIsIntegral<OutputType>::Value)
			{
				Path.emplace_back(PackVector<RealType, OutputType>(InPolygon[Idx], InMin, InRange));
			}
			else if constexpr (TIsFloatingPoint<OutputType>::Value)
			{
				Path.emplace_back(InPolygon[Idx].X, InPolygon[Idx].Y);
			}
		}
		return Path;
	}

	template <typename RealType, typename OutputType>
	static Clipper2Lib::Path<OutputType> ConvertPolygonToPath(const TArrayView<UE::Math::TVector2<RealType>>& InPolygon, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		Clipper2Lib::Path<OutputType> Path;
		Path.reserve(InPolygon.Num());
		for(int32 Idx = 0; Idx < InPolygon.Num(); ++Idx)
		{
			// Integral output normalized the input floating point values
			if constexpr (TIsIntegral<OutputType>::Value)
			{
				Path.emplace_back(PackVector<RealType, OutputType>(InPolygon[Idx], InMin, InRange));
			}
			else if constexpr (TIsFloatingPoint<OutputType>::Value)
			{
				Path.emplace_back(InPolygon[Idx].X, InPolygon[Idx].Y);
			}
		}
		return Path;
	}

	template <typename RealType, typename OutputType>
	Clipper2Lib::Paths<OutputType> ConvertPolygonsToPaths(const TArray<TArrayView<UE::Math::TVector2<RealType>>>& InPolygons, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		Clipper2Lib::Paths<OutputType> Paths;
		Paths.reserve(InPolygons.Num());
		for(const TArrayView<UE::Math::TVector2<RealType>>& Polygon : InPolygons)
		{
			Paths.push_back(ConvertPolygonToPath<RealType, OutputType>(Polygon, InMin, InRange));
		}
		return Paths;
	}

	template <typename RealType, typename OutputType>
	void AddGeneralizedPolygonToPaths(Clipper2Lib::Paths<OutputType>& OutPaths, const UE::Geometry::TGeneralPolygon2<RealType>& InPolygon, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		OutPaths.reserve(OutPaths.size() + 1 + InPolygon.GetHoles().Num());
		OutPaths.push_back(ConvertPolygonToPath<RealType, OutputType>(InPolygon.GetOuter(), InMin, InRange));
		for(const TPolygon2<RealType>& HolePath : InPolygon.GetHoles())
		{
			OutPaths.push_back(ConvertPolygonToPath<RealType, OutputType>(HolePath, InMin, InRange));
		}
	}
	template <typename RealType, typename OutputType>
	Clipper2Lib::Paths<OutputType> ConvertGeneralizedPolygonToPath(const UE::Geometry::TGeneralPolygon2<RealType>& InPolygon, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		Clipper2Lib::Paths<OutputType> Paths;
		AddGeneralizedPolygonToPaths(Paths, InPolygon, InMin, InRange);
		return Paths;
	}

	template <typename InputType, typename RealType>
	TPolygon2<RealType>	ConvertPathToPolygon(const Clipper2Lib::Path<InputType>& InPath, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		TPolygon2<RealType> Polygon;
		for(int32 PointIdx = 0; PointIdx < InPath.size(); ++PointIdx)
		{
			const Clipper2Lib::Point<InputType> Point = InPath[PointIdx];
			if constexpr (TIsIntegral<InputType>::Value)
			{
				Polygon.AppendVertex(UnpackVector<InputType, RealType>(Point, InMin, InRange));
			}
			else if constexpr (TIsFloatingPoint<InputType>::Value)
			{
				Polygon.AppendVertex({static_cast<RealType>(Point.x), static_cast<RealType>(Point.y)});
			}
		}
		return Polygon;
	}


	template <typename InputType, typename RealType>
	void ConvertPolyTreeToPolygons(const Clipper2Lib::PolyPath<InputType>* InPaths, TArray<UE::Geometry::TGeneralPolygon2<RealType>>& OutPolygons, const TVector2<RealType>& InMin, const RealType& InRange, int ParentPolygonIdx)
	{
		if (OutPolygons.IsEmpty())
		{
			ParentPolygonIdx = -1;
		}

		const Clipper2Lib::Path<InputType>& InputPolygon = InPaths->Polygon();
		TPolygon2<RealType> OutputPolygon = ConvertPathToPolygon<InputType, RealType>(InputPolygon, InMin, InRange);

		if(OutputPolygon.VertexCount() > 0)
		{
			if (InPaths->IsHole() && ensure(OutPolygons.IsValidIndex(ParentPolygonIdx)))
			{
				OutPolygons[ParentPolygonIdx].AddHole(OutputPolygon, false, false);
			}
			else
			{
				ParentPolygonIdx = OutPolygons.Emplace(OutputPolygon);
			}
		}

		for(const Clipper2Lib::PolyPath<InputType>* const ChildPath : *InPaths)
		{
			ConvertPolyTreeToPolygons<InputType, RealType>(ChildPath, OutPolygons, InMin, InRange, ParentPolygonIdx);
		}
	}

	template<typename InputType, typename RealType>
	TArray<TVector2<RealType>> ConvertPathToUEPath(const Clipper2Lib::Path<InputType>& InPath, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		TArray<TVector2<RealType>> Path;
		Path.Reserve(InPath.size());
		for (const Clipper2Lib::Point<InputType>& Point : InPath)
		{
			if constexpr (TIsIntegral<InputType>::Value)
			{
				Path.Add(UnpackVector<InputType, RealType>(Point, InMin, InRange));
			}
			else if constexpr (TIsFloatingPoint<InputType>::Value)
			{
				Path.Add({ static_cast<RealType>(Point.x), static_cast<RealType>(Point.y) });
			}
		}

		return Path;
	}

	template<typename InputType, typename RealType>
	TArray<TArray<TVector2<RealType>>> ConvertPathsToUEPaths(const Clipper2Lib::Paths<InputType>& InPaths, const TVector2<RealType>& InMin, const RealType& InRange)
	{
		TArray<TArray<TVector2<RealType>>> Paths;
		Paths.Reserve(InPaths.size());
		for (const Clipper2Lib::Path<InputType>& InPath : InPaths)
		{
			Paths.Add(ConvertPathToUEPath<InputType, RealType>(InPath, InMin, InRange));
		}

		return Paths;
	}
}
