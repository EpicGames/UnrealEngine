// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curve/PolygonIntersectionUtils.h"

#include "Curve/ClipperUtils.h"

using namespace UE::Geometry;

namespace UE::Geometry::Private
{
	template <typename RealType>
	static bool	Clip(
		const Clipper2Lib::ClipType& InClipType,
		const UE::Geometry::TGeneralPolygon2<RealType>& InPolygonA,
		const UE::Geometry::TGeneralPolygon2<RealType>& InPolygonB,
		TArray<UE::Geometry::TGeneralPolygon2<RealType>>& OutResult)
	{
		// Get combined bounds (min, max) of points
		TAxisAlignedBox2<RealType> InputBounds = InPolygonA.GetOuter().Bounds();
		TAxisAlignedBox2<RealType> InputBoundsB = InPolygonB.GetOuter().Bounds();
		InputBounds.Contain(InputBoundsB);

		RealType InputRange = (InputBounds.Extents() * 2).GetMax();
		
		const Clipper2Lib::Paths<IntegralType> PathsA = ConvertGeneralizedPolygonToPath<RealType, IntegralType>(InPolygonA, InputBounds.Min, InputRange); // Corresponds with PolygonA
		const Clipper2Lib::Paths<IntegralType> PathsB = ConvertGeneralizedPolygonToPath<RealType, IntegralType>(InPolygonB, InputBounds.Min, InputRange); // Corresponds with PolygonB
 
		Clipper2Lib::Clipper64 Clipper;
		Clipper.AddSubject(PathsA);
		Clipper.AddClip(PathsB);

		Clipper2Lib::PolyTree<IntegralType> ResultTree;
		Clipper2Lib::Paths64 ResultOpenPaths;
		const bool bExecuteResult = Clipper.Execute(InClipType, Clipper2Lib::FillRule::NonZero, ResultTree, ResultOpenPaths);

		if(bExecuteResult)
		{
			ConvertPolyTreeToPolygons<IntegralType, RealType>(&ResultTree, OutResult, InputBounds.Min, InputRange);
		}

		return bExecuteResult;
	}

	// Clip polygons in the subject array by the polygons in the clip array
	template <typename RealType>
	static bool ClipArrays(
		const Clipper2Lib::ClipType& InClipType,
		TArrayView<const FGeneralPolygon2d> SubjPolygons,
		TArrayView<const FGeneralPolygon2d> ClipPolygons,
		TArray<FGeneralPolygon2d>& ResultOut)
	{
		TAxisAlignedBox2<RealType> InputBounds;
		for (const FGeneralPolygon2d& Polygon : SubjPolygons)
		{
			InputBounds.Contain(Polygon.GetOuter().Bounds());
		}
		for (const FGeneralPolygon2d& Polygon : ClipPolygons)
		{
			InputBounds.Contain(Polygon.GetOuter().Bounds());
		}
		RealType InputRange = (InputBounds.Extents() * 2).GetMax();

		Clipper2Lib::Clipper64 Clipper;
		for (const FGeneralPolygon2d& Polygon : SubjPolygons)
		{
			Clipper.AddSubject(ConvertGeneralizedPolygonToPath<RealType, IntegralType>(Polygon, InputBounds.Min, InputRange));
		}
		for (const FGeneralPolygon2d& Polygon : ClipPolygons)
		{
			Clipper.AddClip(ConvertGeneralizedPolygonToPath<RealType, IntegralType>(Polygon, InputBounds.Min, InputRange));
		}

		Clipper2Lib::PolyTree<IntegralType> ResultTree;
		Clipper2Lib::Paths64 ResultOpenPaths;
		const bool bExecuteResult = Clipper.Execute(InClipType, Clipper2Lib::FillRule::NonZero, ResultTree, ResultOpenPaths);

		if (bExecuteResult)
		{
			ResultOut.Reset();
			ConvertPolyTreeToPolygons<IntegralType, RealType>(&ResultTree, ResultOut, InputBounds.Min, InputRange);
		}

		return bExecuteResult;
	}

	// Clip paths in the subject array by the polygons in the clip array
	template <typename RealType>
	static bool ClipPaths(
		const Clipper2Lib::ClipType& InClipType,
		const TArray<TArrayView<FVector2D>>& SubjPaths,
		TArrayView<const FGeneralPolygon2d> ClipPolygons,
		TArray<TArray<FVector2D>>& ResultOut)
	{
		TAxisAlignedBox2<RealType> InputBounds;
		for (TArrayView<FVector2D> Path : SubjPaths)
		{
			InputBounds.Contain(Path);
		}
		for (const FGeneralPolygon2d& Polygon : ClipPolygons)
		{
			InputBounds.Contain(Polygon.GetOuter().Bounds());
		}
		RealType InputRange = (InputBounds.Extents() * 2).GetMax();

		Clipper2Lib::Clipper64 Clipper;

		Clipper.AddOpenSubject(ConvertPolygonsToPaths<RealType, IntegralType>(SubjPaths, InputBounds.Min, InputRange));

		for (const FGeneralPolygon2d& Polygon : ClipPolygons)
		{
			Clipper.AddClip(ConvertGeneralizedPolygonToPath<RealType, IntegralType>(Polygon, InputBounds.Min, InputRange));
		}

		Clipper2Lib::PolyTree<IntegralType> ResultTree;
		Clipper2Lib::Paths64 ResultOpenPaths;
		const bool bExecuteResult = Clipper.Execute(InClipType, Clipper2Lib::FillRule::NonZero, ResultTree, ResultOpenPaths);

		if (bExecuteResult)
		{
			ResultOut = ConvertPathsToUEPaths(ResultOpenPaths, InputBounds.Min, InputRange);
		}

		return bExecuteResult;
	}
};


// Array-based operations

namespace UE::Geometry
{
bool PolygonsUnion(TArrayView<const FGeneralPolygon2d> Polygons, TArray<FGeneralPolygon2d>& ResultOut, bool bCopyInputOnFailure)
{
	// always put everything in the 'subject' array for unions
	bool bResult = Private::ClipArrays<double>(Clipper2Lib::ClipType::Union, Polygons, TArrayView<const FGeneralPolygon2d>(), ResultOut);
	// optionally fall back to copying the input polygons on failure
	if (bCopyInputOnFailure && !bResult && ResultOut.GetData() != Polygons.GetData())
	{
		ResultOut.Reset(Polygons.Num());
		ResultOut.Append(Polygons);
	}
	return bResult;
}
bool PolygonsDifference(TArrayView<const FGeneralPolygon2d> SubjPolygons, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<FGeneralPolygon2d>& ResultOut)
{
	return Private::ClipArrays<double>(Clipper2Lib::ClipType::Difference, SubjPolygons, ClipPolygons, ResultOut);
}
bool PolygonsIntersection(TArrayView<const FGeneralPolygon2d> SubjPolygons, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<FGeneralPolygon2d>& ResultOut)
{
	return Private::ClipArrays<double>(Clipper2Lib::ClipType::Intersection, SubjPolygons, ClipPolygons, ResultOut);
}
bool PolygonsExclusiveOr(TArrayView<const FGeneralPolygon2d> SubjPolygons, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<FGeneralPolygon2d>& ResultOut)
{
	return Private::ClipArrays<double>(Clipper2Lib::ClipType::Xor, SubjPolygons, ClipPolygons, ResultOut);
}
bool PathsIntersection(const TArray<TArrayView<FVector2d>>& SubjPaths, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<TArray<FVector2d>>& PathsOut)
{
	return Private::ClipPaths<double>(Clipper2Lib::ClipType::Intersection, SubjPaths, ClipPolygons, PathsOut);
}
bool PathsDifference(const TArray<TArrayView<FVector2d>>& SubjPaths, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<TArray<FVector2d>>& PathsOut)
{
	return Private::ClipPaths<double>(Clipper2Lib::ClipType::Difference, SubjPaths, ClipPolygons, PathsOut);
}
} // namespace UE::Geometry


// Two-polygon operations

template <EPolygonBooleanOp OperationType, typename GeometryType, typename RealType>
TBooleanPolygon2Polygon2<OperationType, GeometryType, RealType>::TBooleanPolygon2Polygon2(
	const GeometryType& InPolygonA, 
	const GeometryType& InPolygonB)
	: PolygonA(InPolygonA)
	, PolygonB(InPolygonB)
	, Result()
{
}

template <EPolygonBooleanOp OperationType, typename GeometryType, typename RealType>
bool TBooleanPolygon2Polygon2<OperationType, GeometryType, RealType>::ComputeResult()
{
	if constexpr (OperationType == EPolygonBooleanOp::Union)
	{
		return Private::Clip<RealType>(Clipper2Lib::ClipType::Union, PolygonA, PolygonB, Result);
	}
	else if constexpr (OperationType == EPolygonBooleanOp::Difference)
	{
		return Private::Clip<RealType>(Clipper2Lib::ClipType::Difference, PolygonA, PolygonB, Result);
	}
	else if constexpr (OperationType == EPolygonBooleanOp::Intersect)
	{
		return Private::Clip<RealType>(Clipper2Lib::ClipType::Intersection, PolygonA, PolygonB, Result);
	}
	else if constexpr (OperationType == EPolygonBooleanOp::ExclusiveOr)
	{
		return Private::Clip<RealType>(Clipper2Lib::ClipType::Xor, PolygonA, PolygonB, Result);
	}
	else
	{
		return false;
	}	
}

namespace UE::Geometry
{
	template class GEOMETRYALGORITHMS_API TBooleanPolygon2Polygon2<UE::Geometry::EPolygonBooleanOp::Union, TGeneralPolygon2<float>, float>;
	template class GEOMETRYALGORITHMS_API TBooleanPolygon2Polygon2<UE::Geometry::EPolygonBooleanOp::Union, TGeneralPolygon2<double>, double>;

	template class GEOMETRYALGORITHMS_API TBooleanPolygon2Polygon2<UE::Geometry::EPolygonBooleanOp::Difference, TGeneralPolygon2<float>, float>;
	template class GEOMETRYALGORITHMS_API TBooleanPolygon2Polygon2<UE::Geometry::EPolygonBooleanOp::Difference, TGeneralPolygon2<double>, double>;

	template class GEOMETRYALGORITHMS_API TBooleanPolygon2Polygon2<UE::Geometry::EPolygonBooleanOp::Intersect, TGeneralPolygon2<float>, float>;
	template class GEOMETRYALGORITHMS_API TBooleanPolygon2Polygon2<UE::Geometry::EPolygonBooleanOp::Intersect, TGeneralPolygon2<double>, double>;

	template class GEOMETRYALGORITHMS_API TBooleanPolygon2Polygon2<UE::Geometry::EPolygonBooleanOp::ExclusiveOr, TGeneralPolygon2<float>, float>;
	template class GEOMETRYALGORITHMS_API TBooleanPolygon2Polygon2<UE::Geometry::EPolygonBooleanOp::ExclusiveOr, TGeneralPolygon2<double>, double>;
}
