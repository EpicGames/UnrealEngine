// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curve/PolygonOffsetUtils.h"

#include "Curve/ClipperUtils.h"
THIRD_PARTY_INCLUDES_START
#include "ThirdParty/clipper/clipper.h"
THIRD_PARTY_INCLUDES_END

using namespace UE::Geometry;

namespace UE::Geometry::Private
{
	template <typename RealType>
	static bool	Offset(
		const TArray<TArrayView<TVector2<RealType>>>& InPolygons,
		const EPolygonOffsetJoinType& InJoinType,
		const EPolygonOffsetEndType& InEndType,
		const RealType& InOffset,
		const RealType& InMiterLimit,
		TArray<UE::Geometry::TGeneralPolygon2<RealType>>& OutResult,
		double MaxStepsPerRadian,
		double DefaultStepsPerRadianScale)
	{
		// Get combined bounds (min, max) of points 
		UE::Math::TBox2<RealType> InputBounds(EForceInit::ForceInitToZero);
		for(const TArrayView<TVector2<RealType>>& Polygon : InPolygons) 
		{
			if(!Polygon.IsEmpty())
			{
				UE::Math::TBox2<RealType> PolygonBounds(&Polygon[0], Polygon.Num());
				InputBounds += PolygonBounds;
			}
		}

		Clipper2Lib::ClipperOffset ClipperOffset;

		bool bOneSidedOffset = InEndType == EPolygonOffsetEndType::Polygon;

		RealType OffsetExpand = bOneSidedOffset ? FMath::Max(0, InOffset) : FMath::Abs(InOffset);
		RealType MaxMiterExpand = InJoinType == EPolygonOffsetJoinType::Miter ? InMiterLimit : 0;
		InputBounds = InputBounds.ExpandBy(OffsetExpand + MaxMiterExpand); // pad to allow for offset geometry
		RealType InputRange = FMath::Max(TMathUtil<RealType>::ZeroTolerance, InputBounds.GetSize().GetMax());
		ClipperOffset.SetRoundScaleFactors(static_cast<double>(InputRange) / static_cast<double>(IntRange), DefaultStepsPerRadianScale, MaxStepsPerRadian);
		
		const Clipper2Lib::Paths<IntegralType> Paths = ConvertPolygonsToPaths<RealType, IntegralType>(InPolygons, InputBounds.Min, InputRange);

		ClipperOffset.MergeGroups(false); // This disables union clipping so we can perform it later to get a polytree
		ClipperOffset.AddPaths(Paths, static_cast<Clipper2Lib::JoinType>(InJoinType), static_cast<Clipper2Lib::EndType>(InEndType));
		ClipperOffset.MiterLimit(InMiterLimit);

		RealType ScaledOffset = FMath::Floor(static_cast<RealType>((InOffset / InputRange) * static_cast<RealType>(IntRange))); // scale to account for value normalization
		Clipper2Lib::Paths64 OffsetResultPaths = ClipperOffset.Execute(ScaledOffset);

		if(OffsetResultPaths.size() <= 0)
		{
			// if the offset removed everything, result should be empty and there's nothing else to do
			OutResult.Reset();
			return true;
		}

		bool bExecuteResult = true;
		if(OffsetResultPaths.size() > 1)
		{
			// ...then union to merge and get polytree
			Clipper2Lib::Clipper64 Clipper;
			Clipper.PreserveCollinear = false;
			Clipper.AddSubject(OffsetResultPaths);
		
			Clipper2Lib::PolyTree64 UnionResultPolyTree;
			Clipper2Lib::Paths64 UnionResultOpenPaths;
			bExecuteResult = Clipper.Execute(Clipper2Lib::ClipType::Union, Clipper2Lib::FillRule::NonZero, UnionResultPolyTree, UnionResultOpenPaths);

			if(bExecuteResult)
			{
				OutResult.Reset();
				ConvertPolyTreeToPolygons<IntegralType, RealType>(&UnionResultPolyTree, OutResult, InputBounds.Min, InputRange);
			}
		}
		else
		{
			OutResult.Reset();
			OutResult.Add(ConvertPathToPolygon(OffsetResultPaths.back(), InputBounds.Min, InputRange));
		}

		return bExecuteResult;
	}

	template <typename RealType>
	static bool	ApplyOffsetsToGeneralPolygons(
		const TArrayView<const UE::Geometry::TGeneralPolygon2<RealType>> InPolygons,
		const EPolygonOffsetJoinType& InJoinType,
		const EPolygonOffsetEndType& InEndType,
		TArrayView<const RealType> InOffsets,
		const RealType& InMiterLimit,
		TArray<UE::Geometry::TGeneralPolygon2<RealType>>& OutResult,
		bool bCopyInputOnFailure,
		double MaxStepsPerRadian,
		double DefaultStepsPerRadianScale)
	{
		// Get combined bounds (min, max) of points 
		FAxisAlignedBox2d AccumBounds = FAxisAlignedBox2d::Empty();
		for (const FGeneralPolygon2d& Polygon : InPolygons)
		{
			AccumBounds.Contain(Polygon.Bounds());
		}

		bool bOneSidedOffset = InEndType == EPolygonOffsetEndType::Polygon;

		RealType MaxExpandBounds = 0;
		for (const RealType& Offset : InOffsets)
		{
			MaxExpandBounds = FMath::Max(MaxExpandBounds, bOneSidedOffset ? FMath::Max(0, Offset) : FMath::Abs(Offset));
		}
		RealType MaxMiterExpand = InJoinType == EPolygonOffsetJoinType::Miter ? InMiterLimit : 0;
		AccumBounds.Expand(MaxExpandBounds + MaxMiterExpand); // pad to allow for offset geometry
		RealType InputRange = FMath::Max(TMathUtil<RealType>::ZeroTolerance, (AccumBounds.Extents() * 2).GetMax());

		Clipper2Lib::Paths64 CurrentPaths;
		for (const FGeneralPolygon2d& Polygon : InPolygons)
		{
			AddGeneralizedPolygonToPaths<RealType, IntegralType>(CurrentPaths, Polygon, AccumBounds.Min, InputRange);
		}

		int NumOffsets = InOffsets.Num();
		for (int OffsetIdx = 0; OffsetIdx < NumOffsets; ++OffsetIdx)
		{
			Clipper2Lib::ClipperOffset ClipperOffset;
			ClipperOffset.SetRoundScaleFactors(static_cast<double>(InputRange) / static_cast<double>(IntRange), DefaultStepsPerRadianScale, MaxStepsPerRadian);
			ClipperOffset.MergeGroups(OffsetIdx + 1 != NumOffsets); // This disables union clipping for the final offset, so we can perform it later to get a polytree
			ClipperOffset.AddPaths(CurrentPaths, static_cast<Clipper2Lib::JoinType>(InJoinType), static_cast<Clipper2Lib::EndType>(InEndType));
			ClipperOffset.MiterLimit(InMiterLimit);

			RealType ScaledOffset = FMath::Floor(static_cast<RealType>((InOffsets[OffsetIdx] / InputRange) * static_cast<RealType>(IntRange))); // scale to account for value normalization
			CurrentPaths = ClipperOffset.Execute(ScaledOffset);

			if (CurrentPaths.size() <= 0)
			{
				// if the offset removed everything, result should be empty and there's nothing else to do
				OutResult.Reset();
				return true;
			}
		}

		
		bool bExecuteResult = true;
		if (CurrentPaths.size() > 1)
		{
			// ...then union to merge and get polytree
			Clipper2Lib::Clipper64 Clipper;
			Clipper.PreserveCollinear = false;
			Clipper.AddSubject(CurrentPaths);

			Clipper2Lib::PolyTree64 UnionResultPolyTree;
			Clipper2Lib::Paths64 UnionResultOpenPaths;
			bExecuteResult = Clipper.Execute(Clipper2Lib::ClipType::Union, Clipper2Lib::FillRule::NonZero, UnionResultPolyTree, UnionResultOpenPaths);

			if (bExecuteResult)
			{
				OutResult.Reset();
				ConvertPolyTreeToPolygons<IntegralType, RealType>(&UnionResultPolyTree, OutResult, AccumBounds.Min, InputRange);
			}
			else
			{
				if (bCopyInputOnFailure && OutResult.GetData() != InPolygons.GetData())
				{
					OutResult.Reset();
					OutResult.Append(InPolygons);
				}
			}
		}
		else
		{
			OutResult.Reset();
			OutResult.Add(ConvertPathToPolygon(CurrentPaths.back(), AccumBounds.Min, InputRange));
		}

		return bExecuteResult;
	}
}; 

template <typename GeometryType, typename RealType>
TOffsetPolygon2<GeometryType, RealType>::TOffsetPolygon2(
	const TArray<GeometryType>& InPolygons)
		: Polygons(InPolygons)
		, Result()
{
}

template <typename GeometryType, typename RealType>
bool TOffsetPolygon2<GeometryType, RealType>::ComputeResult()
{
	MiterLimit = FMath::Max<RealType>(static_cast<RealType>(2.0), MiterLimit); // Clamps lower value to 2.0
	return Private::Offset<RealType>(Polygons, JoinType, EndType, Offset, MiterLimit, Result, MaxStepsPerRadian, DefaultStepsPerRadianScale);
}

namespace UE::Geometry
{
	template class GEOMETRYALGORITHMS_API TOffsetPolygon2<TArrayView<TVector2<float>>, float>;
	template class GEOMETRYALGORITHMS_API TOffsetPolygon2<TArrayView<TVector2<double>>, double>;


	bool PolygonsOffset(double Offset, 
		TArrayView<const FGeneralPolygon2d> Polygons, TArray<FGeneralPolygon2d>& ResultOut,
		bool bCopyInputOnFailure,
		double MiterLimit,
		EPolygonOffsetJoinType JoinType,
		EPolygonOffsetEndType EndType,
		double MaxStepsPerRadian,
		double DefaultStepsPerRadianScale)
	{
		using namespace UE::Geometry::Private;

		TArrayView<const double> OffsetsView(&Offset, 1);
		bool bSuccess = ApplyOffsetsToGeneralPolygons<double>(Polygons, JoinType, EndType, OffsetsView, MiterLimit, ResultOut, bCopyInputOnFailure, MaxStepsPerRadian, DefaultStepsPerRadianScale);

		return bSuccess;
	}

	bool PolygonsOffsets(double FirstOffset, double SecondOffset, 
		TArrayView<const FGeneralPolygon2d> Polygons, TArray<FGeneralPolygon2d>& ResultOut,
		bool bCopyInputOnFailure,
		double MiterLimit,
		EPolygonOffsetJoinType JoinType,
		EPolygonOffsetEndType EndType,
		double MaxStepsPerRadian,
		double DefaultStepsPerRadianScale)
	{
		using namespace UE::Geometry::Private;

		double Offsets[2]{ FirstOffset, SecondOffset };
		TArrayView<const double> OffsetsView(Offsets, 2);
		bool bSuccess = ApplyOffsetsToGeneralPolygons<double>(Polygons, JoinType, EndType, OffsetsView, MiterLimit, ResultOut, bCopyInputOnFailure, MaxStepsPerRadian, DefaultStepsPerRadianScale);

		return bSuccess;
	}
}