// Copyright Epic Games, Inc. All Rights Reserved.

#include "Distance/DistLine3Circle3.h"
#include "ThirdParty/GTEngine/Mathematics/GteDistLine3Circle3.h"
#include "GteUtil.h"


using namespace UE::Geometry;

template <typename Real>
Real TDistLine3Circle3<Real>::ComputeResult()
{
	if (DistanceSquared >= 0)
	{
		return DistanceSquared;
	}

	gte::Line3<Real> gteLine = Convert(Line);
	gte::Circle3<Real> gteCircle = Convert(Circle);

	gte::DCPQuery<Real, gte::Line3<Real>, gte::Circle3<Real>> Query;
	auto Result = Query(gteLine, gteCircle);

	DistanceSquared = Result.sqrDistance;
	NumClosestPairs = Result.numClosestPairs;
	LineClosest[0] = Convert(Result.lineClosest[0]);
	LineClosest[1] = Convert(Result.lineClosest[1]);
	CircleClosest[0] = Convert(Result.circleClosest[0]);
	CircleClosest[1] = Convert(Result.circleClosest[1]);
	bIsEquiDistant = Result.equidistant;

	return DistanceSquared;
}



namespace UE
{
namespace Geometry
{

template class GEOMETRYALGORITHMS_API TDistLine3Circle3<float>;
template class GEOMETRYALGORITHMS_API TDistLine3Circle3<double>;

} // end namespace UE::Geometry
} // end namespace UE
