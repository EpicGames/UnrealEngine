// Copyright Epic Games, Inc. All Rights Reserved.


#include "DynamicMeshMikkTWrapper.h"

#include "DynamicMesh/DynamicMesh3.h"

// on platforms that build the mikkt library (win/mac/linux) its build.cs will define this:
#if WITH_MIKKTSPACE
#include "mikktspace.h"
#endif //WITH_MIKKTSPACE

#if WITH_MIKKTSPACE
namespace DynamicMeshMikkTWrapper
{
	using namespace UE::Geometry;

	namespace DynamicMeshMikkTInterface
	{
		struct FDynamicMeshInfo
		{
			const FDynamicMesh3* Mesh;
			const FDynamicMeshNormalOverlay* NormalOverlay;
			const FDynamicMeshUVOverlay* UVOverlay;
			FMeshTangentsd* TangentsOut;
		};


		int MikkGetNumFaces(const SMikkTSpaceContext* Context)
		{
			const FDynamicMeshInfo* MeshInfo = (const FDynamicMeshInfo*)(Context->m_pUserData);
			return MeshInfo->Mesh->MaxTriangleID();
		}

		int MikkGetNumVertsOfFace(const SMikkTSpaceContext* Context, const int FaceIdx)
		{
			const FDynamicMeshInfo* MeshInfo = (const FDynamicMeshInfo*)(Context->m_pUserData);
			return MeshInfo->Mesh->IsTriangle(FaceIdx) ? 3 : 0;
		}

		void MikkGetPosition(const SMikkTSpaceContext* Context, float Position[3], const int FaceIdx, const int VertIdx)
		{
			const FDynamicMeshInfo* MeshInfo = (const FDynamicMeshInfo*)(Context->m_pUserData);
			FIndex3i Triangle = MeshInfo->Mesh->GetTriangle(FaceIdx);
			FVector3d VertexPos = MeshInfo->Mesh->GetVertex(Triangle[VertIdx]);
			Position[0] = (float)VertexPos.X;
			Position[1] = (float)VertexPos.Y;
			Position[2] = (float)VertexPos.Z;
		}

		void MikkGetNormal(const SMikkTSpaceContext* Context, float Normal[3], const int FaceIdx, const int VertIdx)
		{
			const FDynamicMeshInfo* MeshInfo = (const FDynamicMeshInfo*)(Context->m_pUserData);
			FIndex3i NormTriangle = MeshInfo->NormalOverlay->GetTriangle(FaceIdx);
			FVector3f Normalf;
			if (NormTriangle.A == IndexConstants::InvalidID)
			{
				Normalf = FVector3f::ZAxisVector;
			}
			else
			{
				MeshInfo->NormalOverlay->GetElement(NormTriangle[VertIdx], Normalf);
			}
			Normal[0] = Normalf.X;
			Normal[1] = Normalf.Y;
			Normal[2] = Normalf.Z;
		}

		void MikkGetTexCoord(const SMikkTSpaceContext* Context, float UV[2], const int FaceIdx, const int VertIdx)
		{
			const FDynamicMeshInfo* MeshInfo = (const FDynamicMeshInfo*)(Context->m_pUserData);
			FIndex3i UVTriangle = MeshInfo->UVOverlay->GetTriangle(FaceIdx);
			FVector2f UVf;
			if (UVTriangle.A == IndexConstants::InvalidID)
			{
				UV[0] = UV[1] = 0.f;
			}
			else
			{
				MeshInfo->UVOverlay->GetElement(UVTriangle[VertIdx], UVf);
				UV[0] = UVf.X;
				UV[1] = UVf.Y;
			}
		}

		void MikkSetTSpaceBasic(const SMikkTSpaceContext* Context, const float Tangent[3], const float BitangentSign, const int FaceIdx, const int VertIdx)
		{
			FDynamicMeshInfo* MeshInfo = (FDynamicMeshInfo*)(Context->m_pUserData);
			FVector3d Tangentd(Tangent[0], Tangent[1], Tangent[2]);
			FIndex3i NormTriangle = MeshInfo->NormalOverlay->GetTriangle(FaceIdx);
			FVector3d Normald = (NormTriangle.A != IndexConstants::InvalidID) ? (FVector3d)MeshInfo->NormalOverlay->GetElement(NormTriangle[VertIdx]) : FVector3d::ZAxisVector;
			FVector3d Bitangentd = VectorUtil::Bitangent(Normald, Tangentd, -(double)BitangentSign);

			MeshInfo->TangentsOut->SetPerTriangleTangent(FaceIdx, VertIdx, Tangentd, Bitangentd);
		}


		void MikkSetTSpace(const SMikkTSpaceContext* Context, const float Tangent[3], const float BiTangent[3],
			const float MagS, const float MagT, const tbool bIsOrientationPreserving,
			const int FaceIdx, const int VertIdx)
		{
			FDynamicMeshInfo* MeshInfo = (FDynamicMeshInfo*)(Context->m_pUserData);
			FVector3d Tangentd(Tangent[0], Tangent[1], Tangent[2]);
			FVector3d Bitangentd(BiTangent[0], BiTangent[1], BiTangent[2]);
			MeshInfo->TangentsOut->SetPerTriangleTangent(FaceIdx, VertIdx, Tangentd, Bitangentd);
		}

	}

	bool IsSupported()
	{
		return true;
	}
	bool ComputeTangents(FMeshTangentsd& MeshTangents, int32 TargetUVLayer)
	{
		if (!MeshTangents.GetMesh())
		{
			return false;
		}

		const FDynamicMesh3& SourceMesh = *MeshTangents.GetMesh();
		if (!SourceMesh.HasAttributes() || SourceMesh.Attributes()->NumNormalLayers() == 0 || TargetUVLayer < 0 || TargetUVLayer >= SourceMesh.Attributes()->NumUVLayers())
		{
			return false;
		}

		MeshTangents.InitializeTriVertexTangents(true);

		DynamicMeshMikkTInterface::FDynamicMeshInfo MeshInfo;
		MeshInfo.Mesh = &SourceMesh;
		MeshInfo.NormalOverlay = SourceMesh.Attributes()->PrimaryNormals();
		MeshInfo.UVOverlay = SourceMesh.Attributes()->GetUVLayer(TargetUVLayer);
		MeshInfo.TangentsOut = &MeshTangents;

		SMikkTSpaceInterface MikkTInterface;
		MikkTInterface.m_getNormal = DynamicMeshMikkTInterface::MikkGetNormal;
		MikkTInterface.m_getNumFaces = DynamicMeshMikkTInterface::MikkGetNumFaces;
		MikkTInterface.m_getNumVerticesOfFace = DynamicMeshMikkTInterface::MikkGetNumVertsOfFace;
		MikkTInterface.m_getPosition = DynamicMeshMikkTInterface::MikkGetPosition;
		MikkTInterface.m_getTexCoord = DynamicMeshMikkTInterface::MikkGetTexCoord;
		MikkTInterface.m_setTSpaceBasic = DynamicMeshMikkTInterface::MikkSetTSpaceBasic;
		MikkTInterface.m_setTSpace = nullptr;

		SMikkTSpaceContext MikkTContext;
		MikkTContext.m_pInterface = &MikkTInterface;
		MikkTContext.m_pUserData = (void*)(&MeshInfo);
		MikkTContext.m_bIgnoreDegenerates = false;

		// execute mikkt
		genTangSpaceDefault(&MikkTContext);

		return true;
	}
}
#else

namespace DynamicMeshMikkTWrapper
{
	bool IsSupported()
	{
		return false;
	}
	bool ComputeTangents(UE::Geometry::FMeshTangentsd& ComputedTangents, int32 UVLayer)
	{
		return false;
	}
}

#endif
