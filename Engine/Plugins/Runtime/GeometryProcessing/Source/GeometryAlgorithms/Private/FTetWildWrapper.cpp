// Copyright Epic Games, Inc. All Rights Reserved.


#include "FTetWildWrapper.h"

THIRD_PARTY_INCLUDES_START
#include "ThirdParty/fTetWild/FloatTetwild.h"
THIRD_PARTY_INCLUDES_END

namespace UE::Geometry::FTetWild
{
	/*
	* 	enum class EFilterOutsideMethod
	{
		None, // Do not remove outside tets from the result
		FloodFill, // Floodfill from tagged boundary faces of the active tet mesh
		InputSurface, // Use the winding number of the input surface mesh
		TrackedSurface // Use the winding number of tagged boundary faces on the active tet mesh
	};

	struct FFloatTetWildParameters
	{
		bool bSmoothOpenBoundary = false;

		EFilterOutsideMethod OutsideFilterMethod = EFilterOutsideMethod::TrackedSurface;
		
		bool bCoarsen = false;
		bool bExtractManifoldBoundarySurface = false; // TODO: this is just for the final extraction

		bool bApplySizing = false;
		TArray<FVector> SizingFieldVertices;
		TArray<FIntVector4> SizingFieldTets;
		TArray<double> SizingFieldValues;
	};

	*/

// Helpers to set up for fTetWild algorithm
namespace
{
	floatTetWild::Parameters ConvertParameters(const FTetMeshParameters& InParams)
	{
		floatTetWild::Parameters Params;
		Params.apply_sizing_field = InParams.bApplySizing;
		if (Params.apply_sizing_field)
		{
			Params.V_sizing_field.resize(InParams.SizingFieldVertices.Num() * 3, 1);
			Params.T_sizing_field.resize(InParams.SizingFieldTets.Num() * 4, 1);
			Params.values_sizing_field.resize(InParams.SizingFieldValues.Num(), 1);
			for (int32 Idx = 0; Idx < InParams.SizingFieldVertices.Num(); ++Idx)
			{
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					Params.V_sizing_field(Idx * 3 + SubIdx) = InParams.SizingFieldVertices[Idx][SubIdx];
				}
			}
			for (int32 Idx = 0; Idx < InParams.SizingFieldTets.Num(); ++Idx)
			{
				for (int32 SubIdx = 0; SubIdx < 4; ++SubIdx)
				{
					Params.T_sizing_field(Idx * 4 + SubIdx) = InParams.SizingFieldTets[Idx][SubIdx];
				}
			}
			for (int32 Idx = 0; Idx < InParams.SizingFieldValues.Num(); ++Idx)
			{
				Params.values_sizing_field[Idx] = InParams.SizingFieldValues[Idx];
			}
		}
		Params.ideal_edge_length_rel = InParams.IdealEdgeLengthRel;
		Params.eps_rel = InParams.EpsRel;
		Params.max_its = InParams.MaxIts;
		Params.stop_energy = InParams.StopEnergy;
		Params.coarsen = InParams.bCoarsen;
		Params.manifold_surface = InParams.bExtractManifoldBoundarySurface;
		switch (InParams.OutsideFilterMethod)
		{
		case EFilterOutsideMethod::None:
			Params.disable_filtering = true;
			break;
		case EFilterOutsideMethod::FloodFill:
			Params.use_floodfill = true;
			break;
		case EFilterOutsideMethod::TrackedSurface:
			// default parameters will use this method
			break;
		case EFilterOutsideMethod::InputSurface:
			Params.use_input_for_wn = true;
			break;
		case EFilterOutsideMethod::SmoothOpenBoundary:
			Params.smooth_open_boundary = true;
			break;
		default: // unsupported filter method
			ensure(false);
		}
		return Params;
	}

	void ConvertIndexMesh(const TArray<FVector>& InVertices, const TArray<FIntVector3>& InFaces,
		std::vector<floatTetWild::Vector3>& OutVertices, std::vector<floatTetWild::Vector3i>& OutTris)
	{
		OutVertices.reserve(InVertices.Num());
		OutTris.reserve(InFaces.Num());
		for (const FVector& V : InVertices)
		{
			OutVertices.emplace_back(V[0], V[1], V[2]);
		}
		for (const FIntVector3& F : InFaces)
		{
			OutTris.emplace_back(F[0], F[1], F[2]);
		}
	}
}

bool ComputeTetMesh(
	const FTetMeshParameters& InParams,
	const TArray<FVector>& InVertices,
	const TArray<FIntVector3>& InFaces,
	TArray<FVector>& OutVertices,
	TArray<FIntVector4>& OutTets,
	FProgressCancel* Progress)
{
	floatTetWild::Parameters Params = ConvertParameters(InParams);
	std::vector<floatTetWild::Vector3> Vertices;
	std::vector<floatTetWild::Vector3i> Faces;
	std::vector<int> Tags;
	ConvertIndexMesh(InVertices, InFaces, Vertices, Faces);
	return floatTetWild::tetrahedralization(Vertices, Faces, Tags, Params, InParams.bInvertOutputTets, OutVertices, OutTets, nullptr, nullptr, -1, InParams.bSkipSimplification, Progress);
}

} // namespace UE::Geometry::FTetWildWrapper
