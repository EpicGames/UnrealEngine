// Copyright Epic Games, Inc. All Rights Reserved.

#include "FitCapsule3.h"
#include "ThirdParty/GTEngine/Mathematics/GteCapsule.h"
#include "ThirdParty/GTEngine/Mathematics/GteContCapsule3.h"

using namespace UE::Geometry;
using namespace UE::Math;

template<typename RealType>
bool TFitCapsule3<RealType>::Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc)
{
	using ComputeType = double;

	TArray<gte::Vector3<ComputeType>> PointList;
	PointList.SetNum(NumPoints);
	for (int32 k = 0; k < NumPoints; ++k)
	{
		TVector<RealType> Point = GetPointFunc(k);
		PointList[k] = { {(ComputeType)Point.X, (ComputeType)Point.Y, (ComputeType)Point.Z} };
	}

	gte::Capsule3<ComputeType> FitCapsule;
	bResultValid = GetContainer(NumPoints, &PointList[0], FitCapsule);
	if (bResultValid)
	{
		gte::Vector3<ComputeType> Center, Direction;
		ComputeType Extent;
		FitCapsule.segment.GetCenteredForm(Center, Direction, Extent);

		Capsule.Segment = TSegment3<RealType>(
			TVector<RealType>((RealType)Center[0], (RealType)Center[1], (RealType)Center[2]),
			TVector<RealType>((RealType)Direction[0], (RealType)Direction[1], (RealType)Direction[2]),
			(RealType)Extent);

		Capsule.Radius = (RealType)FitCapsule.radius;
	}

	return bResultValid;
}


namespace UE
{
namespace Geometry
{

template class GEOMETRYALGORITHMS_API TFitCapsule3<float>;
template class GEOMETRYALGORITHMS_API TFitCapsule3<double>;

} // end namespace UE::Geometry
} // end namespace UE
