// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "LineTypes.h"
#include "CircleTypes.h"

THIRD_PARTY_INCLUDES_START
#include "ThirdParty/GTEngine/Mathematics/GteVector2.h"
#include "ThirdParty/GTEngine/Mathematics/GteVector3.h"
#include "ThirdParty/GTEngine/Mathematics/GteLine.h"
#include "ThirdParty/GTEngine/Mathematics/GteCircle3.h"
THIRD_PARTY_INCLUDES_END

namespace UE
{
namespace Geometry
{

template<typename Real>
gte::Vector2<Real> Convert(const UE::Math::TVector2<Real>& Vec)
{
	return gte::Vector2<Real>({ Vec.X, Vec.Y});
}

template<typename Real>
UE::Math::TVector2<Real> Convert(const gte::Vector2<Real>& Vec)
{
	return UE::Math::TVector2<Real>(Vec[0], Vec[1]);
}

template<typename Real>
gte::Vector3<Real> Convert(const UE::Math::TVector<Real>& Vec)
{
	return gte::Vector3<Real>({ Vec.X, Vec.Y, Vec.Z });
}

template<typename Real>
UE::Math::TVector<Real> Convert(const gte::Vector3<Real>& Vec)
{
	return UE::Math::TVector<Real>(Vec[0], Vec[1], Vec[2]);
}

template<typename Real>
gte::Line3<Real> Convert(const TLine3<Real>& Line)
{
	return gte::Line3<Real>(Convert(Line.Origin), Convert(Line.Direction));
}

template<typename Real>
gte::Circle3<Real> Convert(const TCircle3<Real>& Circle)
{
	return gte::Circle3<Real>(
		Convert(Circle.Frame.Origin), Convert(Circle.GetNormal()), Circle.Radius);
}

} // end namespace UE::Geometry
} // end namespace UE