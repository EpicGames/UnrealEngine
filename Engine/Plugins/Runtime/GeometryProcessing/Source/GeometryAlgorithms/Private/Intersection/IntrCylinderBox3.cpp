// Copyright Epic Games, Inc. All Rights Reserved.

#include "Intersection/IntrCylinderBox3.h"

#include "BoxTypes.h"

#include "GteUtil.h"

THIRD_PARTY_INCLUDES_START
#include "ThirdParty/GTEngine/Mathematics/GteIntrAlignedBox3Cylinder3.h"
THIRD_PARTY_INCLUDES_END

template <typename RealType>
bool UE::Geometry::DoesCylinderIntersectBox(
	const UE::Geometry::TAxisAlignedBox3<RealType>& Box,
	const UE::Geometry::TVector<RealType>& CylinderCenter,
	const UE::Geometry::TVector<RealType>& CylinderDirection,
	RealType CylinderRadius,
	RealType CylinderHeight)
{
	using namespace UE::Geometry;

	if (Box.IsEmpty())
	{
		return false;
	}

	typedef gte::TIQuery<RealType, gte::AlignedBox3<RealType>, gte::Cylinder3<RealType>> QueryType;
	QueryType Query;
	struct QueryType::Result Output = Query(
		gte::AlignedBox3<RealType>(Convert(Box.Min), Convert(Box.Max)),
		gte::Cylinder3(gte::Line3<RealType>(Convert(CylinderCenter), Convert(CylinderDirection)), CylinderRadius, CylinderHeight));
	return Output.intersect;
}

template bool GEOMETRYALGORITHMS_API UE::Geometry::DoesCylinderIntersectBox(
	const UE::Geometry::TAxisAlignedBox3<float>& Box,
	const UE::Geometry::TVector<float>& CylinderCenter,
	const UE::Geometry::TVector<float>& CylinderDirection,
	float CylinderRadius,
	float CylinderHeight);
template bool GEOMETRYALGORITHMS_API UE::Geometry::DoesCylinderIntersectBox(
	const UE::Geometry::TAxisAlignedBox3<double>& Box,
	const UE::Geometry::TVector<double>& CylinderCenter,
	const UE::Geometry::TVector<double>& CylinderDirection,
	double CylinderRadius,
	double CylinderHeight);