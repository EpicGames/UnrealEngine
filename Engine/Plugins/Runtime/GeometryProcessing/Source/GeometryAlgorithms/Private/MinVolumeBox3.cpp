// Copyright Epic Games, Inc. All Rights Reserved.

#include "MinVolumeBox3.h"

#include "CompGeom/ConvexHull3.h"
#include "CompGeom/FitOrientedBox3.h"
#include "CompGeom/DiTOrientedBox.h"

#include "Util/ProgressCancel.h"
#include "Util/IteratorUtil.h"

using namespace UE::Geometry;
using namespace UE::Math;

namespace UE {
namespace Geometry {

template <typename RealType>
struct TMinVolumeBox3Internal
{
	TOrientedBox3<RealType> Result;
	bool bSolutionOK = false;
	
	// Settings

	RealType SameNormalTolerance = (RealType)0.01;
	int32 OptimizeIterations = 10;
	EDiTO DiTODirections = EDiTO::DiTO_26;
	bool bMostAccurateFit = false;
	EBox3FitCriteria FitCriteria = EBox3FitCriteria::Volume;

	void SetSolution(TOrientedBox3<RealType> ResultIn)
	{
		Result = ResultIn;
		bSolutionOK = true;
	}

	bool ComputeResult(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, FProgressCancel* Progress)
	{
		if (bMostAccurateFit)
		{
			Result = FitOrientedBox3Points<RealType>(NumPoints, GetPointFunc, [](int32)->bool {return true;}, FitCriteria, SameNormalTolerance, Progress);
			if (Progress && Progress->Cancelled())
			{
				return false;
			}
		}
		else
		{
			TOrientedBox3<RealType> InitialBox = ComputeOrientedBBox<RealType>(DiTODirections, NumPoints, GetPointFunc);
			Result = OptimizeOrientedBox3Points<RealType>(InitialBox, OptimizeIterations, NumPoints, GetPointFunc, [](int32)->bool {return true;}, FitCriteria, Progress);
			if (Progress && Progress->Cancelled())
			{
				return false;
			}
		}

		// if resulting box is not finite, something went wrong, just return an empty box
		if (!FMathd::IsFinite(Result.Extents.SquaredLength()))
		{
			bSolutionOK = false;
			return false;
		}

		bSolutionOK = true;
		return true;
	}

};

} // end namespace UE::Geometry
} // end namespace UE

template<typename RealType>
bool TMinVolumeBox3<RealType>::Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, bool bMostAccurateFit, FProgressCancel* Progress )
{
	Initialize(bMostAccurateFit);
	check(Internal);

	return Internal->ComputeResult(NumPoints, GetPointFunc, Progress);
}


template<typename RealType>
bool TMinVolumeBox3<RealType>::SolveSubsample(int32 NumPoints, int32 MaxPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, bool bMostAccurateFit, FProgressCancel* Progress)
{
	if (NumPoints <= MaxPoints)
	{
		return Solve(NumPoints, GetPointFunc, bMostAccurateFit, Progress);
	}

	Initialize(bMostAccurateFit);
	check(Internal);

	int32 k = 0;
	FModuloIteration Iter(NumPoints);
	int32 Index;
	TArray<TVector<RealType>> ReducedPoints;
	ReducedPoints.Reserve(NumPoints);
	while (Iter.GetNextIndex(Index) && k < MaxPoints)
	{
		TVector<RealType> Point = GetPointFunc(Index);
		ReducedPoints.Add(Point);
	}

	return Internal->ComputeResult(ReducedPoints.Num(), [&ReducedPoints](int32 PtIdx) {return ReducedPoints[PtIdx];}, Progress);
}

template<typename RealType>
bool TMinVolumeBox3<RealType>::IsSolutionAvailable() const
{
	return Internal && Internal->bSolutionOK;
}

template<typename RealType>
void TMinVolumeBox3<RealType>::GetResult(TOrientedBox3<RealType>& BoxOut)
{
	ensure(IsSolutionAvailable());
	BoxOut = Internal->Result;
}


template<typename RealType>
void TMinVolumeBox3<RealType>::Initialize(bool bMostAccurateFit)
{
	Internal = MakePimpl<TMinVolumeBox3Internal<RealType>>();
	Internal->bMostAccurateFit = bMostAccurateFit;
}


namespace UE
{
namespace Geometry
{

template class GEOMETRYALGORITHMS_API TMinVolumeBox3<float>;
template class GEOMETRYALGORITHMS_API TMinVolumeBox3<double>;

} // end namespace UE::Geometry
} // end namespace UE
