// Copyright Epic Games, Inc. All Rights Reserved.

#include "MinVolumeSphere3.h"

#include "ThirdParty/GTEngine/Mathematics/GteBSNumber.h"
#include "ThirdParty/GTEngine/Mathematics/GteBSRational.h"
#include "ThirdParty/GTEngine/Mathematics/GteUIntegerFP32.h"
#include "ThirdParty/GTEngine/Mathematics/GteUIntegerAP32.h"
#include "ThirdParty/GTEngine/Mathematics/GteMinimumVolumeSphere3.h"

using namespace UE::Geometry;
using namespace UE::Math;

namespace UE {
namespace Geometry {

template <typename RealType>
struct TMinVolumeSphere3Internal
{
	//using PreciseNumberType = gte::BSRational<gte::UIntegerFP32<197>>;
	using PreciseNumberType = gte::BSRational<gte::UIntegerAP32>;
	using DVector3 = gte::Vector3<double>;

	bool bUseExact;
	TArray<DVector3> DoubleInput;

	TVector<RealType> Center;
	RealType Radius;
	bool bIsMinimalSphere;
	bool bSolutionOK;

	void SetPoint(int32 Index, const TVector<RealType>& Point)
	{
		DoubleInput[Index] = DVector3{ {(double)Point.X, (double)Point.Y, (double)Point.Z} };
	}

	gte::MinimumVolumeSphere3<double, double> DoubleCompute;
	gte::MinimumVolumeSphere3<double, PreciseNumberType> PreciseCompute;

	bool ComputeResult()
	{
		gte::Sphere3<double> MinimalSphere;

		if (bUseExact)
		{
			bIsMinimalSphere = PreciseCompute(DoubleInput.Num(), &DoubleInput[0], MinimalSphere);
			bSolutionOK = true;
		}
		else
		{
			bIsMinimalSphere = DoubleCompute(DoubleInput.Num(), &DoubleInput[0], MinimalSphere);
			bSolutionOK = true;
		}

		Center = TVector<RealType>((RealType)MinimalSphere.center[0], (RealType)MinimalSphere.center[1], (RealType)MinimalSphere.center[2]);
		Radius = (RealType)MinimalSphere.radius;

		return true;
	}

};

} // end namespace UE::Geometry
} // end namespace UE



template<typename RealType>
bool TMinVolumeSphere3<RealType>::Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, bool bUseExactComputation)
{
	Initialize(NumPoints, bUseExactComputation);
	check(Internal);

	for (int32 k = 0; k < NumPoints; ++k)
	{
		TVector<RealType> Point = GetPointFunc(k);
		Internal->SetPoint(k, Point);
	}
	
	return Internal->ComputeResult();
}

template<typename RealType>
bool TMinVolumeSphere3<RealType>::IsSolutionAvailable() const
{
	return Internal && Internal->bSolutionOK;
}

template<typename RealType>
void TMinVolumeSphere3<RealType>::GetResult(TSphere3<RealType>& SphereOut)
{
	ensure(IsSolutionAvailable());
	SphereOut.Center = Internal->Center;
	SphereOut.Radius = Internal->Radius;
}


template<typename RealType>
void TMinVolumeSphere3<RealType>::Initialize(int32 NumPoints, bool bUseExactComputation)
{
	Internal = MakePimpl<TMinVolumeSphere3Internal<RealType>>();

	Internal->bUseExact = bUseExactComputation;
	Internal->DoubleInput.SetNum(NumPoints);
}



namespace UE
{
namespace Geometry
{

template class GEOMETRYALGORITHMS_API TMinVolumeSphere3<float>;
template class GEOMETRYALGORITHMS_API TMinVolumeSphere3<double>;

} // end namespace UE::Geometry
} // end namespace UE
