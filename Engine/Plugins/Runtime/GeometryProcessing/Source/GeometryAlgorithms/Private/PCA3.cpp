// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCA3.h"
#include "BoxTypes.h"


// Magic to include the allowed subset of eigen and ward against ensuing compiler warnings
#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif

#if defined(__clang__) && __clang_major__ > 9
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wimplicit-int-float-conversion"
#endif

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Eigenvalues>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

#if defined(__clang__) && __clang_major__ > 9
#pragma clang diagnostic pop
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif
// End of wards needed to include Eigen


namespace UE::Private::PCAHelper
{
	using namespace ::UE::Math;
	using namespace ::UE::Geometry;

	template<typename RealType>
	static bool ComputePCA3Helper(
		TConstArrayView<TVector<RealType>> Points, 
		TVector<RealType>& OutMean, TStaticArray<TVector<RealType>, 3>& OutEigenvectors, TVector<RealType>& OutEigenvalues, TVector<RealType>& OutScaleFactor, 
		const FComputePCA3Options& Options)
	{
		OutMean = TVector<RealType>::Zero();

		if (Points.IsEmpty())
		{
			return false;
		}

		TAxisAlignedBox3<RealType> Bounds;
		
		for (const TVector<RealType>& P : Points)
		{
			OutMean += P;
			Bounds.Contain(P);
		}
		OutMean /= (RealType)Points.Num();

		RealType Scale = (RealType)1.0;
		if (Options.bScaleDataToUnitCube)
		{
			// Inverse scale by bounds max dim (with max to avoid failure for degenerate/collapsed input)
			Scale /= FMath::Max(Bounds.MaxDim(), (RealType)UE_SMALL_NUMBER);
		}
		// Update the vector scale factor w/ the scale that was used. (Using a vector scale factor in case we want per-dimension scales at some point.)
		OutScaleFactor = TVector<RealType>(Scale);

		using EigMatrix3 = Eigen::Matrix<RealType, 3, 3>;
		using EigVec3 = Eigen::Vector<RealType, 3>;

		Eigen::SelfAdjointEigenSolver<EigMatrix3> EigenSystem;
		EigMatrix3 X = EigMatrix3::Zero();
		for (const TVector<RealType>& P : Points)
		{
			EigVec3 V((P.X - OutMean.X) * Scale, (P.Y - OutMean.Y) * Scale, (P.Z - OutMean.Z) * Scale);
			X += V.lazyProduct(V.transpose());
		}
		// Note computeDirect only works for fixed size 2d or 3d matrices; otherwise we'd need compute(X) here instead
		EigenSystem.computeDirect(X);

		const EigVec3& Eigenvalues = EigenSystem.eigenvalues();

		// The Eigen::SelfAdjointEigenSolver puts the eigenvalues in increasing order, so we just reverse to sort in decreasing order if requested
		// (note other eigensolver methods may return an unsorted result; revisit this if changing the solver)
		FIndex3i Order = Options.bSortEigenvalues ? FIndex3i(2, 1, 0) : FIndex3i(0, 1, 2);

		// copy out the sorted eigenvalues and eigenvectors
		for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
		{
			OutEigenvalues[SubIdx] = Eigenvalues[Order[SubIdx]];
		}

		const EigMatrix3& Eigenvectors = EigenSystem.eigenvectors();
		for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
		{
			int32 Col = Order[SubIdx];
			OutEigenvectors[SubIdx] = TVector<RealType>(
				Eigenvectors.coeff(0, Col),
				Eigenvectors.coeff(1, Col),
				Eigenvectors.coeff(2, Col)
			);
		}

		return true;
	}
}

namespace UE::Geometry 
{
	template<typename RealType>
	bool TPCA3<RealType>::Compute(TConstArrayView<UE::Math::TVector<RealType>> Points, const FComputePCA3Options& Options)
	{
		return UE::Private::PCAHelper::ComputePCA3Helper<RealType>(Points, Mean, Eigenvectors, Eigenvalues, ScaleFactor, Options);
	}

	template class GEOMETRYALGORITHMS_API TPCA3<float>;
	template class GEOMETRYALGORITHMS_API TPCA3<double>;
} // end namespace UE::Geometry
