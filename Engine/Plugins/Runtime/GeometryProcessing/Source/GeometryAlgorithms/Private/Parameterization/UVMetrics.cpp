// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parameterization/UVMetrics.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;

namespace UVMetricsLocals
{
	double Triangle2DArea(FVector2f p1, FVector2f p2, FVector2f p3)
	{
		return FMath::Abs(((p2.X - p1.X) * (p3.Y - p1.Y) - (p3.X - p1.X) * (p2.Y - p1.Y)) / 2.0);
	}

	double Triangle3DArea(FVector3d q1, FVector3d q2, FVector3d q3)
	{
		FVector3d Q1 = FVector3d(q2.X - q1.X, q2.Y - q1.Y, q2.Z - q1.Z);
		FVector3d Q2 = FVector3d(q3.X - q1.X, q3.Y - q1.Y, q3.Z - q1.Z);

		FVector3d U = Q1.Cross(Q2);

		return U.Length() / 2.0;
	}

	double Triangle2DPerimeter(FVector2f p1, FVector2f p2, FVector2f p3)
	{
		FVector2f Edge1 = p2 - p1;
		FVector2f Edge2 = p3 - p1;
		FVector2f Edge3 = p3 - p2;
		return Edge1.Length() + Edge2.Length() + Edge3.Length();

	}

	double Triangle3DPerimeter(FVector3d q1, FVector3d q2, FVector3d q3)
	{
		FVector3d Edge1 = q2 - q1;
		FVector3d Edge2 = q3 - q1;
		FVector3d Edge3 = q3 - q2;
		return Edge1.Length() + Edge2.Length() + Edge3.Length();
	}

	FVector2f GetTriangleSingularValues(FVector3d V1, FVector3d V2, FVector3d V3, FVector2f UV1, FVector2f UV2, FVector2f UV3)
	{
		FVector2f SingularValues;
		FVector3d Ss, St;
		double A;

		A = Triangle2DArea(UV1, UV2, UV3);
		Ss = (V1 * (UV2.Y - UV3.Y) + V2 * (UV3.Y - UV1.Y) + V3 * (UV1.Y - UV2.Y)) / A;
		St = (V1 * (UV3.X - UV2.X) + V2 * (UV1.X - UV3.X) + V3 * (UV2.X - UV1.X)) / A;

		double a, b, c;
		a = Ss.Dot(Ss);
		b = Ss.Dot(St);
		c = St.Dot(St);

		double Discriminant = FMath::Sqrt(FMath::Square(a - c) + 4 * FMath::Square(b));

		SingularValues.X = (float)(FMath::Sqrt(0.5 * ((a + c) + Discriminant)));
		SingularValues.Y = (float)(FMath::Sqrt(0.5 * ((a + c) - Discriminant)));

		return SingularValues;
	}
}


double FUVMetrics::ReedBeta(const FDynamicMesh3& Mesh, int32 UVChannel, int32 Tid)
{
	// Inspiration for the math used to compute the eccentricity comes from here:
	// https://www.reedbeta.com/blog/conformal-texture-mapping/

	FVector3d V1, V2, V3;
	FVector2f UV1, UV2, UV3;
	Mesh.GetTriVertices(Tid, V1, V2, V3);
	Mesh.Attributes()->GetUVLayer(UVChannel)->GetTriElements(Tid, UV1, UV2, UV3);

	FVector2f SingularValues = UVMetricsLocals::GetTriangleSingularValues(V1, V2, V3, UV1, UV2, UV3);
	if (TMathUtil<double>::Abs(SingularValues.X) < TMathUtil<double>::Epsilon)
	{
		return 0;
	}
	else
	{
		double Eccentricity = FMath::Sqrt(1 - FMath::Square(SingularValues.Y / SingularValues.X));
		return Eccentricity;
	}
}

double FUVMetrics::Sander(const FDynamicMesh3& Mesh, int32 UVChannel, int32 Tid, bool bUseL2)
{
	// These metrics are defined by this paper:
	// "Texture Mapping Progressive Meshes"
	// https://dl.acm.org/doi/10.1145/383259.383307
	// https://hhoppe.com/tmpm.pdf

	FVector3d q1, q2, q3;
	FVector2f p1, p2, p3;
	Mesh.GetTriVertices(Tid, q1, q2, q3);
	Mesh.Attributes()->GetUVLayer(UVChannel)->GetTriElements(Tid, p1, p2, p3);

	double GammaMax, GammaMin, L2_Norm, L_Inf;
	double L2_Norm_Normalized, L_Inf_Normalized;
	{
		FVector3d Ss, St;
		double A, Aq;
		A = UVMetricsLocals::Triangle2DArea(p1, p2, p3);
		Aq = UVMetricsLocals::Triangle3DArea(q1, q2, q3);
		Ss = (q1 * (p2.Y - p3.Y) + q2 * (p3.Y - p1.Y) + q3 * (p1.Y - p2.Y)) / (2 * A);
		St = (q1 * (p3.X - p2.X) + q2 * (p1.X - p3.X) + q3 * (p2.X - p1.X)) / (2 * A);

		double a, b, c;
		a = Ss.Dot(Ss);
		b = Ss.Dot(St);
		c = St.Dot(St);

		double D = FMath::Sqrt(FMath::Square(a - c) + 4 * FMath::Square(b));

		GammaMax = FMath::Sqrt(0.5 * ((a + c) + D));
		GammaMin = FMath::Sqrt(0.5 * ((a + c) - D));

		L2_Norm = FMath::Sqrt(0.5 * (a + c));
		L_Inf = GammaMax;

		L2_Norm_Normalized = L2_Norm / FMath::Sqrt(Aq / A);
		L_Inf_Normalized = L_Inf / FMath::Sqrt(Aq / A);
	}
	if (bUseL2)
	{
		return L2_Norm_Normalized;
	}
	else
	{
		return L_Inf_Normalized;
	}
}

double FUVMetrics::TexelDensity(const FDynamicMesh3& Mesh, int32 UVChannel, int32 Tid, int32 MapSize)
{
	// Computes a metric describing the difference between the density of a single UV triangle relative to the desired density
	FVector3d q1, q2, q3;
	FVector2f p1, p2, p3;
	Mesh.GetTriVertices(Tid, q1, q2, q3);
	Mesh.Attributes()->GetUVLayer(UVChannel)->GetTriElements(Tid, p1, p2, p3);

	double UVArea = UVMetricsLocals::Triangle2DArea(p1, p2, p3);
	double WorldSpaceArea = UVMetricsLocals::Triangle3DArea(q1, q2, q3);
	double Density = FMath::Sqrt(UVArea / WorldSpaceArea) * MapSize; // Since we're comparing area ratios here, we want to take the square root to get the linear scaling factor back out.
																		// Technically Mapsize is Mapsize^2 in terms of pixels per unit area, so we implicitly take it's square root here.
	return Density;
}
