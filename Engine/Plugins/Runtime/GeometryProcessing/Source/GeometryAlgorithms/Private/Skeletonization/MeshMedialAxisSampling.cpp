// Copyright Epic Games, Inc. All Rights Reserved.

#include "Skeletonization/MeshMedialAxisSampling.h"

#include <atomic>

#include "Algo/RemoveIf.h"
#include "Algo/MaxElement.h"
#include "Algo/MinElement.h"
#include "VertexConnectedComponents.h"
#include "MeshAdapter.h"
#include "MeshAdapter/MeshVertexNormals.h"
#include "Misc/CoreMiscDefines.h"

#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"

#include "Util/SizedDisjointSet.h"
#include "VectorUtil.h"

// Include Eigen/Dense for a 4x4 matrix solve
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

namespace MeshMedialAxisSamplingLocals
{
	using FEigMat4 = Eigen::Matrix4d;
	using FEigVec4 = Eigen::Vector4d;

	// Struct to sum/track/eval the spherical quadric error metric
	struct FSQEM
	{
		// Terms for quadric error, s^T A s - s^T B + C
		FEigMat4 A;
		FEigVec4 B;
		double C;
		FSQEM() : A(FEigMat4::Zero()), B(FEigVec4::Zero()), C(0) {}
		FSQEM(const FVector3d& Centroid, const FVector3d& Normal, double Wt)
		{
			FEigVec4 N_1(Normal.X, Normal.Y, Normal.Z, 1.);
			A = (Wt * N_1) * N_1.transpose();
			double NdP = Normal.Dot(Centroid);
			B = N_1 * (2. * NdP * Wt);
			C = NdP * NdP * Wt;
		}
		static FEigVec4 SphereToEig(const FSphere& Sphere)
		{
			return FEigVec4(Sphere.Center.X, Sphere.Center.Y, Sphere.Center.Z, Sphere.W);
		}
		double Eval(const FSphere& Sphere) const
		{
			return Eval(SphereToEig(Sphere));
		}
		double Eval(const FEigVec4& EigSphere) const
		{
			// Compute the spherical quadric error, s^T A s - s^T B + C
			double Aterm = EigSphere.dot(A * EigSphere);
			double Bterm = EigSphere.dot(B);
			return Aterm - Bterm + C;
		}
		void Add(const FSQEM& Other)
		{
			A += Other.A;
			B += Other.B;
			C += Other.C;
		}
		void Add(const FSQEM& Other, double Wt)
		{
			A += Other.A * Wt;
			B += Other.B * Wt;
			C += Other.C * Wt;
		}
		// 'regularization' term biasing towards a specific sphere -- to avoid singular solve
		void AddRegularizationToSphere(const FSphere& Sphere, double Wt)
		{
			A += FEigMat4::Identity() * Wt;
			B += SphereToEig(Sphere) * (2. * Wt);
		}
		// plane error for sphere center; used to keep close to the original skeleton triangles
		void AddCenterNormalQEM(const FVector3d& Point, const FVector3d& Normal, double Wt)
		{
			FEigVec4 N_0(Normal.X, Normal.Y, Normal.Z, 0.);
			A += (Wt * N_0) * N_0.transpose();
			double NdP = Normal.Dot(Point);
			B += N_0 * (2. * NdP * Wt);
			C += NdP * NdP * Wt;
		}
		// line error for sphere center; used to keep close to original skeleton edges
		void AddCenterLineQEM(const FVector3d& Point, const FVector3d& NormalizedLineDir, double Wt)
		{
			FVector3d N1, N2;
			UE::Geometry::VectorUtil::MakePerpVectors(NormalizedLineDir, N1, N2);
			AddCenterNormalQEM(Point, N1, Wt * .5);
			AddCenterNormalQEM(Point, N2, Wt * .5);
		}
		[[nodiscard]] bool SolveSphere(FSphere& OutSphere, double MinRadius = FMathd::ZeroTolerance) const
		{
			Eigen::LDLT<FEigMat4> LDLT = A.ldlt();
			if (LDLT.info() != Eigen::Success)
			{
				return false;
			}
			FEigVec4 EigSphere = LDLT.solve(.5 * B);
			if (!EigSphere.allFinite())
			{
				return false;
			}
			EigSphere[3] = FMath::Max(MinRadius, EigSphere[3]);
			OutSphere = FSphere(FVector(EigSphere[0], EigSphere[1], EigSphere[2]), EigSphere[3]);
			return true;
		}
	};

	// helpers to get an edge or tri in canonical sorted form
	UE::Geometry::FIndex2i AsEdge(int32 A, int32 B)
	{
		UE::Geometry::FIndex2i Edge(A, B); Edge.Sort();
		return Edge;
	}
	UE::Geometry::FIndex3i AsTri(int32 A, int32 B, int32 C)
	{
		UE::Geometry::FIndex3i Tri(A, B, C); Tri.Sort();
		return Tri;
	}
	// helper to interpolate two spheres via lerp of centers and radii
	FSphere LerpSphere(const FSphere& A, const FSphere& B, double T)
	{
		return FSphere(
			FMath::Lerp(A.Center, B.Center, T),
			FMath::Lerp(A.W, B.W, T)
		);
	}

	// Info about a split edge: original edge A-B was split, inserting intermediate nodes along it
	struct FSplitEdgeInfo
	{
		FSplitEdgeInfo(int32 InA, int32 InB) : A(InA), B(InB) {}
		int32 A, B;
		TArray<int32, TInlineAllocator<2>> IntermediateNodes; // nodes inserted between A and B, in order
		TArray<int32, TInlineAllocator<2>> OppositeVerts; // vertices opposite the source AB edge
	};

	// Request to split an edge at specified parametric positions
	struct FEdgeSplitRequest
	{
		UE::Geometry::FIndex2i Edge;
		TArray<double, TInlineAllocator<4>> SplitTs; // parameter values in (0,1), sorted, where intermediate nodes should be added
	};

	// Split skeleton edges according to provided split requests, updating topology
	// CollectCandidates takes Skeleton and EdgeToOppVerts map, and returns-by-reference the split candidates
	void SplitSkeletonEdgesImpl(
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton,
		TFunctionRef<void(const UE::Geometry::MedialAxis::FMedialSkeleton&, const TMap<UE::Geometry::FIndex2i, TArray<int32>>&, TArray<FEdgeSplitRequest>&)> CollectCandidates,
		int32& OutStartNewClusters,
		TSet<int32>& OutClusterCanRemapVertices,
		TArray<FSplitEdgeInfo>* OutSplitEdges = nullptr)
	{
		using namespace UE::Geometry;

		// Build edge to opposite vertex map for all tris
		TMap<FIndex2i, TArray<int32>> EdgeToOppVerts;
		for (const FIndex3i& Tri : Skeleton.ClusterTriangles)
		{
			for (int32 Idx = 0, Prev = 2, Next = 1; Idx < 3; Next = Prev, Prev = Idx++)
			{
				EdgeToOppVerts.FindOrAdd(AsEdge(Tri[Next], Tri[Prev])).AddUnique(Tri[Idx]);
			}
		}

		// helper to replace a target edge-opposite vertex with a new vertex
		// no-op if the edge was already removed
		auto ReplaceEdgeOppVert = [&EdgeToOppVerts](FIndex2i Edge, int32 OldV, int32 NewV)
		{
			if (TArray<int32>* OppVerts = EdgeToOppVerts.Find(Edge))
			{
				int32 OldVIdx = OppVerts->Find(OldV);
				if (ensure(OldVIdx != INDEX_NONE))
				{
					(*OppVerts)[OldVIdx] = NewV;
				}
			}
		};

		// Helper to add edges from Skeleton data
		auto AddEdge = [&Skeleton](int32 A, int32 B)
		{
			Skeleton.ClusterNeighbors[A].AddUnique(B);
			Skeleton.ClusterNeighbors[B].AddUnique(A);
		};

		// SplitEdge book-keeping data -- used in a post-process to fix up skeleton data
		// The skeleton triangle changes to apply after
		TSet<FIndex3i> TrianglesToRemove;
		TArray<FIndex3i> TrianglesToAdd;

		// Collect candidates via caller-provided callback
		TArray<FEdgeSplitRequest> Candidates;
		CollectCandidates(Skeleton, EdgeToOppVerts, Candidates);

		OutStartNewClusters = Skeleton.Spheres.Num();

		// Process each candidate: split edge into chain of intermediate nodes
		for (const FEdgeSplitRequest& Cand : Candidates)
		{
			if (!ensure(!Cand.SplitTs.IsEmpty()))
			{
				continue;
			}
			const int32 A = Cand.Edge.A, B = Cand.Edge.B;


			// Build chain: [A, I1, I2, ..., I_k, B]
			TArray<int32, TInlineAllocator<8>> ChainNodes;
			ChainNodes.Add(A);
			for (double T : Cand.SplitTs)
			{
				int32 I = Skeleton.Spheres.Add(LerpSphere(Skeleton.Spheres[A], Skeleton.Spheres[B], T));
				Skeleton.ClusterNeighbors.AddDefaulted();
				ChainNodes.Add(I);
			}
			ChainNodes.Add(B);

			// Get opposite vertices (copy since we modify EdgeToOppVerts below)
			TArray<int32> OppVerts;
			if (const TArray<int32>* OppVertsPtr = EdgeToOppVerts.Find(Cand.Edge))
			{
				OppVerts = *OppVertsPtr;
			}

			// Fan triangulation from opposite vertices
			for (int32 Cx : OppVerts)
			{
				TrianglesToRemove.Add(AsTri(A, B, Cx));

				for (int32 Seg = 0; Seg < ChainNodes.Num() - 1; ++Seg)
				{
					TrianglesToAdd.Add(AsTri(ChainNodes[Seg], ChainNodes[Seg + 1], Cx));
				}

				for (int32 Seg = 1; Seg < ChainNodes.Num() - 1; ++Seg)
				{
					AddEdge(Cx, ChainNodes[Seg]);
				}

				ReplaceEdgeOppVert(AsEdge(A, Cx), B, ChainNodes[1]);
				ReplaceEdgeOppVert(AsEdge(B, Cx), A, ChainNodes[ChainNodes.Num() - 2]);
			}

			// Update ClusterNeighbors: remove A-B, add chain edges
			Skeleton.ClusterNeighbors[A].Remove(B);
			Skeleton.ClusterNeighbors[B].Remove(A);
			for (int32 Seg = 0; Seg < ChainNodes.Num() - 1; ++Seg)
			{
				AddEdge(ChainNodes[Seg], ChainNodes[Seg + 1]);
			}

			OutClusterCanRemapVertices.Add(A);
			OutClusterCanRemapVertices.Add(B);
			OutClusterCanRemapVertices.Append(OppVerts);

			// Fill optional split edge info
			if (OutSplitEdges)
			{
				FSplitEdgeInfo& Info = OutSplitEdges->Emplace_GetRef(A, B);
				for (int32 Seg = 1; Seg < ChainNodes.Num() - 1; ++Seg)
				{
					Info.IntermediateNodes.Add(ChainNodes[Seg]);
				}
				Info.OppositeVerts.Append(OppVerts);
			}

			EdgeToOppVerts.Remove(Cand.Edge);
		}

		// Apply accumulated triangle modifications
		Skeleton.ClusterTriangles.SetNum(Algo::RemoveIf(Skeleton.ClusterTriangles,
			[&TrianglesToRemove](const FIndex3i& Tri) { return TrianglesToRemove.Contains(Tri); }), EAllowShrinking::No);
		Skeleton.ClusterTriangles.Append(TrianglesToAdd);
	}

	// Shared re-clustering helper: reassign vertices near split edges to new intermediate clusters
	template<typename MeshType>
	void ReclusterVerticesAfterSplit(
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton,
		const MeshType& Mesh,
		TConstArrayView<FSQEM> VtxSQEMs,
		TConstArrayView<float> VtxAreas,
		double PosErrorWt,
		int32 StartNewClusters,
		const TSet<int32>& ClusterCanRemapVertices)
	{
		const double UsePosErrorWt = FMath::Clamp(PosErrorWt, UE_DOUBLE_KINDA_SMALL_NUMBER, 1.0);

		// Error metric that produces equivalent rankings to ComputeSkeleton's vertex-cluster assignment method, using PosErrorWt
		// Note: area-scaled (not normalized), so only valid for comparing across clusters for the same vertex
		auto ClusterVertexError = [&Mesh, &VtxAreas, &VtxSQEMs, UsePosErrorWt](int32 VID, const FSphere& S) -> double
		{
			const double PosErr = (Mesh.GetVertex(VID) - S.Center).Length() - S.W;
			return UsePosErrorWt * VtxAreas[VID] * PosErr * PosErr + (1. - UsePosErrorWt) * VtxSQEMs[VID].Eval(S);
		};
		// Pass through vertices and remap cluster assignments to new vertices, if they are on a marked-as-can-remap cluster
		ParallelFor(Mesh.MaxVertexID(), [&Skeleton, &ClusterCanRemapVertices, &ClusterVertexError, StartNewClusters](int32 VID)
		{
			const int32 CurCluster = Skeleton.VIDtoClusterIndex[VID];
			if (CurCluster == INDEX_NONE)
			{
				return;
			}
			if (ClusterCanRemapVertices.Contains(CurCluster))
			{
				double BestClusterErr = ClusterVertexError(VID, Skeleton.Spheres[CurCluster]);
				int32 BestCluster = CurCluster;
				for (int32 NbrCluster : Skeleton.ClusterNeighbors[CurCluster])
				{
					// Only consider remapping to new clusters (assume vertex was already mapped to the 'best' cluster of previously-existing ones)
					if (NbrCluster >= StartNewClusters)
					{
						double NbrClusterErr = ClusterVertexError(VID, Skeleton.Spheres[NbrCluster]);
						if (NbrClusterErr < BestClusterErr)
						{
							BestClusterErr = NbrClusterErr;
							BestCluster = NbrCluster;
						}
					}
				}
				Skeleton.VIDtoClusterIndex[VID] = BestCluster;
			}
		});
	}

	// Helper to split skinny triangles -- to give the simplifier more DOFs to collapse away surfaces
	template<typename MeshType>
	void SplitThinSkeletonTriEdges(
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton,
		TArray<FSplitEdgeInfo>& OutSplitEdges,
		const MeshType& Mesh,
		TConstArrayView<FSQEM> VtxSQEMs,
		TConstArrayView<float> VtxAreas,
		double AngleThresholdDeg,
		double PosErrorWt)
	{
		using namespace UE::Geometry;

		const double UseCosAngleThreshold = FMath::Cos(FMath::Clamp(AngleThresholdDeg, 90. + FMathd::ZeroTolerance, 180. - FMathd::ZeroTolerance) * FMathd::DegToRad);

		int32 StartNewClusters = 0;
		TSet<int32> ClusterCanRemapVertices;
		SplitSkeletonEdgesImpl(Skeleton,
			[&UseCosAngleThreshold](const UE::Geometry::MedialAxis::FMedialSkeleton& InSkeleton, const TMap<FIndex2i, TArray<int32>>& EdgeToOppVerts, TArray<FEdgeSplitRequest>& OutRequests)
			{
				// Get candidate edges w/ attached skinny triangles to split
				struct FCandidateSplit
				{
					FIndex2i Edge;
					double CosAngle, ProjT;
				};
				TArray<FCandidateSplit> RawCandidates;
				for (const auto& Entry : EdgeToOppVerts)
				{
					const FIndex2i& Edge = Entry.Key;
					const TArray<int32>& OppVerts = Entry.Value;
					const FVector3d& PA = InSkeleton.Spheres[Edge.A].Center;
					const FVector3d& PB = InSkeleton.Spheres[Edge.B].Center;
					const FVector3d AB = PB - PA;
					const double ABLenSq = AB.SquaredLength();
					if (ABLenSq < FMathd::ZeroTolerance)
					{
						continue;
					}
					double MinCosAngle = UseCosAngleThreshold;
					int32 MinCosOppV = INDEX_NONE;
					for (int32 Cx : OppVerts)
					{
						const FVector3d& PC = InSkeleton.Spheres[Cx].Center;
						const double CosAngleAtCx = Normalized(PA - PC).Dot(Normalized(PB - PC));
						if (CosAngleAtCx < MinCosAngle)
						{
							MinCosAngle = CosAngleAtCx;
							MinCosOppV = Cx;
						}
					}
					if (MinCosOppV != INDEX_NONE)
					{
						double ProjT = (InSkeleton.Spheres[MinCosOppV].Center - PA).Dot(AB) / ABLenSq;
						if (ProjT > FMathd::ZeroTolerance && ProjT < 1. - FMathd::ZeroTolerance)
						{
							RawCandidates.Add({ Edge, MinCosAngle, ProjT });
						}
					}
				}

				// Process largest-angle (smallest cosine) triangles first, since split order can affect results
				RawCandidates.Sort([](const FCandidateSplit& A, const FCandidateSplit& B) { return A.CosAngle < B.CosAngle; });

				OutRequests.Reserve(RawCandidates.Num());
				for (const FCandidateSplit& Cand : RawCandidates)
				{
					FEdgeSplitRequest& Req = OutRequests.AddDefaulted_GetRef();
					Req.Edge = Cand.Edge;
					Req.SplitTs.Add(Cand.ProjT);
				}
			},
			StartNewClusters,
			ClusterCanRemapVertices,
			&OutSplitEdges);

		ReclusterVerticesAfterSplit(Skeleton, Mesh, VtxSQEMs, VtxAreas, PosErrorWt, StartNewClusters, ClusterCanRemapVertices);
	}

	// Collect edges that exceed the target length and produce split requests with uniform T values for refinement
	void CollectSubdivideCandidates(
		const UE::Geometry::MedialAxis::FMedialSkeleton& InSkeleton,
		const TMap<UE::Geometry::FIndex2i, TArray<int32>>& EdgeToOppVerts,
		TArray<FEdgeSplitRequest>& OutRequests,
		TFunctionRef<double(const FSphere&, const FSphere&)> GetTargetEdgeLength,
		bool bSubdivideOnSurfaces)
	{
		using namespace UE::Geometry;

		const int32 NumClusters = InSkeleton.Spheres.Num();
		for (int32 ClusterIdx = 0; ClusterIdx < NumClusters; ++ClusterIdx)
		{
			for (int32 NbrIdx : InSkeleton.ClusterNeighbors[ClusterIdx])
			{
				if (NbrIdx <= ClusterIdx)
				{
					continue;
				}
				FIndex2i Edge = AsEdge(ClusterIdx, NbrIdx);

				if (!bSubdivideOnSurfaces)
				{
					const TArray<int32>* OppVerts = EdgeToOppVerts.Find(Edge);
					if (OppVerts && OppVerts->Num() > 0)
					{
						continue;
					}
				}

				const FSphere& SA = InSkeleton.Spheres[Edge.A];
				const FSphere& SB = InSkeleton.Spheres[Edge.B];
				const double EdgeLen = FVector3d::Dist(SA.Center, SB.Center);
				const double TargetLen = GetTargetEdgeLength(SA, SB);

				if (TargetLen <= 0 || EdgeLen <= TargetLen)
				{
					continue;
				}

				const int32 N = FMath::CeilToInt32(EdgeLen / TargetLen);
				if (N > 1)
				{
					FEdgeSplitRequest& Req = OutRequests.AddDefaulted_GetRef();
					Req.Edge = Edge;
					Req.SplitTs.Reserve(N - 1);
					for (int32 Seg = 1; Seg < N; ++Seg)
					{
						Req.SplitTs.Add((double)Seg / (double)N);
					}
				}
			}
		}
	}

	// Subdivide skeleton edges, splitting those that exceed the target length. Returns whether any edges were split.
	bool SubdivideSkeletonEdges(
		const UE::Geometry::MedialAxis::FSkeletonSubdivider& Subdivider,
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton,
		int32& OutStartNewClusters,
		TSet<int32>& OutClusterCanRemapVertices)
	{
		SplitSkeletonEdgesImpl(Skeleton,
			[&Subdivider](const UE::Geometry::MedialAxis::FMedialSkeleton& InSkeleton, const TMap<UE::Geometry::FIndex2i, TArray<int32>>& EdgeToOppVerts, TArray<FEdgeSplitRequest>& OutRequests)
			{
				CollectSubdivideCandidates(InSkeleton, EdgeToOppVerts, OutRequests, Subdivider.GetTargetEdgeLength, Subdivider.bSubdivideOnSurfaces);
			},
			OutStartNewClusters,
			OutClusterCanRemapVertices);

		return OutStartNewClusters < Skeleton.Spheres.Num();
	}

	// Build up standard mesh QEM error terms for the skeleton geometry
	// Uses cluster areas rather than the skeleton geometry for weighting
	void AccumulateSkeletonDistanceQEMs(
		const UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton,
		TConstArrayView<double> ClusterAreas,
		int32 NumClusters,
		TFunctionRef<bool(const UE::Geometry::FIndex2i&)> EdgeHasTris,
		double SkeletonDistanceWt,
		TArrayView<FSQEM> ClusterSQEMs)
	{
		using namespace UE::Geometry;

		// Sum weights of all neighbors, so we can normalize QEM weights below
		TArray<double> NeighborFeatureAreaSums;
		NeighborFeatureAreaSums.SetNumZeroed(NumClusters);

		for (const FIndex3i& Tri : Skeleton.ClusterTriangles)
		{
			for (int32 Idx = 0, Prev = 2, Next = 1; Idx < 3; Next = Prev, Prev = Idx++)
			{
				NeighborFeatureAreaSums[Tri[Idx]] += ClusterAreas[Tri[Prev]] + ClusterAreas[Tri[Next]];
			}
		}
		for (int32 ClusterIdx = 0; ClusterIdx < NumClusters; ++ClusterIdx)
		{
			for (int32 NbrIdx : Skeleton.ClusterNeighbors[ClusterIdx])
			{
				if (NbrIdx <= ClusterIdx)
				{
					continue;
				}
				FIndex2i Edge(ClusterIdx, NbrIdx);
				if (!EdgeHasTris(Edge))
				{
					NeighborFeatureAreaSums[ClusterIdx] += ClusterAreas[NbrIdx];
					NeighborFeatureAreaSums[NbrIdx] += ClusterAreas[ClusterIdx];
				}
			}
		}

		// Add the triangle QEM terms
		for (const FIndex3i& Tri : Skeleton.ClusterTriangles)
		{
			const FVector3d PA = Skeleton.Spheres[Tri.A].Center;
			const FVector3d PB = Skeleton.Spheres[Tri.B].Center;
			const FVector3d PC = Skeleton.Spheres[Tri.C].Center;
			FVector3d Normal = VectorUtil::Normal(PA, PB, PC);
			const FVector3d Centroid = (PA + PB + PC) / 3.0;
			for (int32 Idx = 0, Prev = 2, Next = 1; Idx < 3; Next = Prev, Prev = Idx++)
			{
				const int32 VID = Tri[Idx];
				if (NeighborFeatureAreaSums[VID] > 0.)
				{
					double OtherAreaSum = ClusterAreas[Tri[Prev]] + ClusterAreas[Tri[Next]];
					const double W = (OtherAreaSum / NeighborFeatureAreaSums[VID]) * ClusterAreas[VID] * SkeletonDistanceWt;
					ClusterSQEMs[VID].AddCenterNormalQEM(Centroid, Normal, W);
				}
			}
		}
		// Add the edge QEM terms (only on edge's w/ no triangle attached)
		for (int32 ClusterIdx = 0; ClusterIdx < NumClusters; ++ClusterIdx)
		{
			for (int32 NbrIdx : Skeleton.ClusterNeighbors[ClusterIdx])
			{
				if (NbrIdx <= ClusterIdx)
				{
					continue;
				}
				FIndex2i Edge(ClusterIdx, NbrIdx);
				if (EdgeHasTris(Edge))
				{
					continue;
				}
				const FVector3d PA = Skeleton.Spheres[ClusterIdx].Center;
				const FVector3d PB = Skeleton.Spheres[NbrIdx].Center;
				FVector3d EdgeDir = PB - PA;
				const double EdgeLen = Normalize(EdgeDir);
				if (EdgeLen < FMathd::ZeroTolerance)
				{
					continue;
				}
				if (NeighborFeatureAreaSums[ClusterIdx] > 0.)
				{
					const double WA = (ClusterAreas[NbrIdx] / NeighborFeatureAreaSums[ClusterIdx]) * ClusterAreas[ClusterIdx] * SkeletonDistanceWt;
					ClusterSQEMs[ClusterIdx].AddCenterLineQEM(PA, EdgeDir, WA);
				}
				if (NeighborFeatureAreaSums[NbrIdx] > 0.)
				{
					const double WB = (ClusterAreas[ClusterIdx] / NeighborFeatureAreaSums[NbrIdx]) * ClusterAreas[NbrIdx] * SkeletonDistanceWt;
					ClusterSQEMs[NbrIdx].AddCenterLineQEM(PA, EdgeDir, WB);
				}
			}
		}
	}

	template<typename MeshType>
	bool EdgeIntersectsMesh(const FVector3d EdgeA, const FVector3d EdgeB, const UE::Geometry::TMeshAABBTree3<MeshType>& MeshBVH)
	{
		using namespace UE::Geometry;

		FVector3d EdgeVec = EdgeB - EdgeA;
		double Len = Normalize(EdgeVec);
		if (Len > 0)
		{
			FRay3d Ray(EdgeA, EdgeVec);
			IMeshSpatial::FQueryOptions Options;
			Options.MaxDistance = Len;
			if (MeshBVH.FindNearestHitTriangle(Ray, Options) != INDEX_NONE)
			{
				return true;
			}
		}
		return false;
	}

	// Helper to compute the 'weld mapping' for vertices, if needed (ImplicitVertexNormalWeldThreshold > 0)
	// (returns an empty array otherwise, we fall back to identity mapping in that case)
	template<typename MeshType>
	TArray<int32> ComputeWeldIdx(
		const MeshType& Mesh,
		TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter,
		double ImplicitVertexNormalWeldThreshold)
	{
		using namespace UE::Geometry;
		TArray<int32> WeldIdx;
		if (ImplicitVertexNormalWeldThreshold > 0)
		{
			FVertexConnectedComponents WeldVerts(Mesh.MaxVertexID());
			WeldVerts.ConnectCloseFilteredVertices(Mesh, ImplicitVertexNormalWeldThreshold, OptionalBoundaryVertexFilter);
			WeldIdx.SetNumUninitialized(Mesh.MaxVertexID());
			ParallelFor(Mesh.MaxVertexID(), [&WeldIdx, &WeldVerts](int32 VID) { WeldIdx[VID] = WeldVerts.GetComponentThreadsafe(VID); });
		}
		return WeldIdx;
	}
	// Helper to compute per-triangle normals/areas and per-vertex area sums and SQEMs.
	// OutTriNormals and OutTriAreas are filled for use by other parts of ComputeSkeleton.
	// Returns total mesh surface area.
	template<typename MeshType>
	double ComputeVertexAreasAndSQEMs(
		const MeshType& Mesh,
		TFunctionRef<int32(int32)> VIDtoWeldVIDFn,
		TArray<FVector3f>& OutTriNormals,
		TArray<float>& OutTriAreas,
		TArray<FSQEM>& OutVtxSQEMs,
		TArray<float>& OutVtxAreas)
	{
		using namespace UE::Geometry;
		OutTriNormals.SetNumUninitialized(Mesh.MaxTriangleID());
		OutTriAreas.SetNumUninitialized(Mesh.MaxTriangleID());
		OutVtxSQEMs.SetNumZeroed(Mesh.MaxVertexID());
		OutVtxAreas.SetNumZeroed(Mesh.MaxVertexID());
		double TotalArea = 0;
		for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
		{
			if (!Mesh.IsTriangle(TID))
			{
				continue;
			}
			FVector3d Centroid, Normal;
			double Area;
			TMeshQueries<MeshType>::GetTriNormalAreaCentroid(Mesh, TID, Normal, Area, Centroid);
			TotalArea += Area;
			FIndex3i Tri = Mesh.GetTriangle(TID);
			OutTriNormals[TID] = (FVector3f)Normal;
			OutTriAreas[TID] = (float)Area;
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				// Note: vertex areas are per-triangle sums (not divided by 3); 1/3 factors cancel out in usages
				OutVtxAreas[VIDtoWeldVIDFn(Tri[SubIdx])] += OutTriAreas[TID];
				OutVtxSQEMs[VIDtoWeldVIDFn(Tri[SubIdx])].Add(FSQEM(Centroid, Normal, Area));
			}
		}
		return TotalArea;
	}
	template<typename MeshType>
	void SkeletonSimplify(const UE::Geometry::MedialAxis::FSkeletonSimplifier& Simplifier, UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton, const UE::Geometry::TMeshAABBTree3<MeshType>& MeshBVH,
		TConstArrayView<FSQEM> VtxSQEMs, TConstArrayView<float> VtxAreas)
	{
		using namespace UE::Geometry;

		const MeshType& Mesh = *MeshBVH.GetMesh();

		// Optionally pass through skinny triangles and split them to add extra degrees of freedom for simplification
		TArray<FSplitEdgeInfo> SplitEdges;
		if (Simplifier.bSplitThinTriEdges)
		{
			const double UsePosErrorWt = FMath::Clamp(Simplifier.SplitThinTriEdgePosErrorWt, UE_DOUBLE_KINDA_SMALL_NUMBER, 1);
			SplitThinSkeletonTriEdges(Skeleton, SplitEdges, Mesh, VtxSQEMs, VtxAreas, Simplifier.SplitThinTriEdgeAngleThresholdDeg, UsePosErrorWt);
		}

		const int32 InitialNumClusters = Skeleton.Spheres.Num(); // includes clusters added by SplitThinSkeletonTriEdges

		using FNbrTriArray = TArray<FIndex2i, TInlineAllocator<8>>;
		TArray<FNbrTriArray> NbrTris;
		NbrTris.SetNum(InitialNumClusters);
		for (const FIndex3i& Tri : Skeleton.ClusterTriangles)
		{
			for (int32 Idx = 0, Prev = 2, Next = 1; Idx < 3; Next = Prev, Prev = Idx++)
			{
				const int32 V = Tri[Idx];
				NbrTris[V].Add(AsEdge(Tri[Prev], Tri[Next]));
			}
		}

		auto EdgeHasTris = [&NbrTris](const FIndex2i& Edge) -> bool
		{
			for (const FIndex2i& OppE : NbrTris[Edge.A])
			{
				if (OppE.Contains(Edge.B))
				{
					return true;
				}
			}
			return false;
		};

		// Track which cluster each cluster has been merged into so far
		TArray<int32> ClusterMergeParent;
		ClusterMergeParent.SetNumUninitialized(InitialNumClusters);
		for (int32 Idx = 0; Idx < InitialNumClusters; ++Idx)
		{
			ClusterMergeParent[Idx] = Idx; // by convention, self-index == no merge
		}

		// Tracking int incremented when cluster is updated (so we can know if our computed error is stale)
		TArray<int32> ClusterVersion;
		ClusterVersion.SetNumZeroed(InitialNumClusters);

		TArray<FSQEM> ClusterSQEMs;
		ClusterSQEMs.SetNumZeroed(InitialNumClusters);
		TArray<double> ClusterBaseErrors;
		ClusterBaseErrors.SetNumZeroed(InitialNumClusters);
		TArray<double> ClusterAreas;
		ClusterAreas.SetNumZeroed(InitialNumClusters);
		const int32 MeshMaxVID = Mesh.MaxVertexID();

		double UseSkelDistWt = FMath::Clamp(Simplifier.ClusterSkeletonDistanceWt, 0., 1.);
		double UseSphereWt = 1. - UseSkelDistWt;
		double UseRegWt = Simplifier.ClusterRegularizeWeight;
		// If we don't have sphere weights we *must* have regularization to have some error term care about the sphere radii
		if (UseSphereWt < FMathd::ZeroTolerance)
		{
			UseRegWt = FMath::Max(FMathd::ZeroTolerance, UseRegWt);
		}

		for (int32 VID = 0; VID < MeshMaxVID; ++VID)
		{
			int32 ClusterIdx = Skeleton.VIDtoClusterIndex[VID];
			if (ClusterIdx != INDEX_NONE)
			{
				ClusterSQEMs[ClusterIdx].Add(VtxSQEMs[VID], UseSphereWt);
				ClusterAreas[ClusterIdx] += VtxAreas[VID];
			}
		}
		if (UseSkelDistWt > 0.)
		{
			AccumulateSkeletonDistanceQEMs(Skeleton, ClusterAreas, InitialNumClusters, EdgeHasTris, UseSkelDistWt, ClusterSQEMs);
		}
		if (UseRegWt > 0.)
		{
			for (int32 ClusterIdx = 0; ClusterIdx < InitialNumClusters; ++ClusterIdx)
			{
				ClusterSQEMs[ClusterIdx].AddRegularizationToSphere(Skeleton.Spheres[ClusterIdx], UseRegWt);
			}
		}

		// Compute base errors to compare against, so we worry about the error introduced by collapses rather than the total error
		for (int32 ClusterIdx = 0; ClusterIdx < InitialNumClusters; ++ClusterIdx)
		{
			ClusterBaseErrors[ClusterIdx] = ClusterSQEMs[ClusterIdx].Eval(Skeleton.Spheres[ClusterIdx]);
		}

		// track dead-end clusters that we should be allowed to collapse even if bOnlySimplifySurfaces==true
		// because they were created from a collapsed surface (and are often not 'real' features)
		TSet<int32> CollapseInducedDeadEnds;

		auto CollapseEdge = [&Simplifier, &ClusterMergeParent, &ClusterVersion, &ClusterSQEMs, &ClusterBaseErrors, &Skeleton, &NbrTris, &CollapseInducedDeadEnds](FIndex2i Edge, const FSphere& NewSphere)
		{
			ensure(ClusterMergeParent[Edge.A] == Edge.A && ClusterMergeParent[Edge.B] == Edge.B);
			// always sort and remove the higher vertex, so relative ordering from construction is approximately preserved through simplification
			Edge.Sort();
			ClusterMergeParent[Edge.B] = Edge.A;
			ClusterVersion[Edge.A]++;
			ClusterVersion[Edge.B]++;
			ClusterSQEMs[Edge.A].Add(ClusterSQEMs[Edge.B]);
			ClusterBaseErrors[Edge.A] += ClusterBaseErrors[Edge.B];
			Skeleton.Spheres[Edge.A] = NewSphere;

			// flag used for the 'CollapseInducedDeadEnds' tagging, below;
			// heuristically, if the triangle already looked kind of like a dead-end, it's more of a 'real' feature, so
			// we will get better results not tagging it as an 'induced'/collapsable edge.
			bool bAorBWasTerminal = Skeleton.ClusterNeighbors[Edge.A].Num() < 3 || Skeleton.ClusterNeighbors[Edge.B].Num() < 3;
			// Clear any dead-end tags on collapsed edge vertices
			CollapseInducedDeadEnds.Remove(Edge.A);
			CollapseInducedDeadEnds.Remove(Edge.B);

			// update neighbor lists of B's neighbors to (1) remove B, and (2) add A & A's back-ref (unless the neighbor was A itself)
			for (int32 BNbr : Skeleton.ClusterNeighbors[Edge.B])
			{
				Skeleton.ClusterNeighbors[BNbr].Remove(Edge.B);
				if (BNbr != Edge.A)
				{
					Skeleton.ClusterNeighbors[BNbr].AddUnique(Edge.A);
					Skeleton.ClusterNeighbors[Edge.A].AddUnique(BNbr);
				}
			}
			// update NbrTris of B
			for (int32 Idx = 0, Num = NbrTris[Edge.B].Num(); Idx < Num; ++Idx)
			{
				FIndex2i OppEdge = NbrTris[Edge.B][Idx];
				int32 ASubIdx = OppEdge.IndexOf(Edge.A);
				// tri includes AB edge, should be deleted by collapse
				if (ASubIdx != INDEX_NONE)
				{
					int32 OtherV = OppEdge[1 - ASubIdx];
					NbrTris[OtherV].Remove(Edge); // ok b/c edges canonically have vertices in sorted order
					FIndex2i EdgeFromA(Edge.B, OtherV); EdgeFromA.Sort();
					NbrTris[Edge.A].Remove(EdgeFromA); // (note: could do this in a separate pass if there may be a lot of tris on AB ...)
					// If cluster on OtherV has only one neighbor, it'll become a new 'dead end' branch -- conditionally tag it as such
					if (Simplifier.bOnlySimplifySurfaces && !bAorBWasTerminal && Skeleton.ClusterNeighbors[OtherV].Num() == 1)
					{
						CollapseInducedDeadEnds.Add(OtherV);
					}
				}
				else // tri not on AB edge, need to remap B to A and add to A's neighbors
				{
					NbrTris[Edge.A].Add(OppEdge);
					// remap the two copies of tri on edge's vertices
					for (int32 SubIdx = 0; SubIdx < 2; ++SubIdx)
					{
						FIndex2i ToUpdate(Edge.B, OppEdge[1 - SubIdx]);
						ToUpdate.Sort();
						FIndex2i UpdateTo(Edge.A, OppEdge[1 - SubIdx]);
						UpdateTo.Sort();
						for (FIndex2i& SearchEdge : NbrTris[OppEdge[SubIdx]])
						{
							if (SearchEdge == ToUpdate)
							{
								SearchEdge = UpdateTo;
								break;
							}
						}
					}
				}
			}
			Skeleton.ClusterNeighbors[Edge.B].Empty();
			NbrTris[Edge.B].Empty();
		};

		struct FEdgeInfo
		{
			FIndex2i Edge, Version;
			FSphere MergeSphere;
			float Error;

			bool operator<(const FEdgeInfo& Other) const
			{
				return Error < Other.Error;
			}

			FEdgeInfo(FIndex2i InEdge = FIndex2i::Invalid())
				: Edge(InEdge), Version(0, 0) {}
		};

		TArray<FEdgeInfo> EdgeQ;

		auto ComputeMergeError = [&Skeleton, &ClusterSQEMs, &ClusterBaseErrors](const FIndex2i& Edge, FSphere& OutSphere) -> float
		{
			double BaseError = ClusterBaseErrors[Edge.A] + ClusterBaseErrors[Edge.B];
			FSQEM SQEM = ClusterSQEMs[Edge.A];
			SQEM.Add(ClusterSQEMs[Edge.B]);
			if (!SQEM.SolveSphere(OutSphere))
			{
				OutSphere = LerpSphere(Skeleton.Spheres[Edge.A], Skeleton.Spheres[Edge.B], .5);
			}
			return (float)(SQEM.Eval(OutSphere) - BaseError);
		};

		// Helper to add an edge to the EdgeQ, if it passes the error threshold; returns true if added
		auto AddEdgeToQueue = [&Simplifier, &EdgeQ, &ComputeMergeError](const FIndex2i& Edge, bool bHeapPush = false, float ErrorFloor = -FMathf::MaxReal) -> bool
		{
			FEdgeInfo Info(Edge);
			Info.Error = FMath::Max(ComputeMergeError(Info.Edge, Info.MergeSphere), ErrorFloor);
			if (Simplifier.QEMErrorThreshold <= 0 || Info.Error <= Simplifier.QEMErrorThreshold)
			{
				if (bHeapPush)
				{
					EdgeQ.HeapPush(Info);
				}
				else
				{
					EdgeQ.Add(Info);
				}
				return true;
			}
			return false;
		};

		double UseEdgeLengthThresholdSq = Simplifier.EdgeLengthThreshold * Simplifier.EdgeLengthThreshold;
		auto ShouldSkipEdge = [&Simplifier, &Skeleton, UseEdgeLengthThresholdSq, &EdgeHasTris, &CollapseInducedDeadEnds](const FIndex2i& Edge) -> bool
		{
			if (Simplifier.bOnlySimplifySurfaces && !EdgeHasTris(Edge) &&
				// Allow collapse if this dead-end was artificially introduced by a prior triangle collapse.
				!CollapseInducedDeadEnds.Contains(Edge.A) && !CollapseInducedDeadEnds.Contains(Edge.B))
			{
				return true;
			}
			if (Simplifier.EdgeLengthThreshold > 0 &&
				FVector3d::DistSquared(Skeleton.Spheres[Edge.A].Center, Skeleton.Spheres[Edge.B].Center) > UseEdgeLengthThresholdSq)
			{
				return true;
			}
			if (Simplifier.SphereRadiusThreshold > 0 &&
				Skeleton.Spheres[Edge.A].W > Simplifier.SphereRadiusThreshold && Skeleton.Spheres[Edge.B].W > Simplifier.SphereRadiusThreshold)
			{
				return true;
			}
			if (Simplifier.SphereOverlapThreshold > 0)
			{
				double RadiusA = Skeleton.Spheres[Edge.A].W;
				double RadiusB = Skeleton.Spheres[Edge.B].W;
				double Dist = FVector3d::Dist(Skeleton.Spheres[Edge.A].Center, Skeleton.Spheres[Edge.B].Center);
				double OverlapDepth = RadiusA + RadiusB - Dist;
				if (OverlapDepth < 0)
				{
					return true; // Spheres don't touch at all
				}
				double MinRadius = FMath::Min(RadiusA, RadiusB);
				// Normalize by diameter of smaller sphere, or snap to fully-contained if diameter is 0
				double OverlapFraction = (MinRadius > 0) ? (OverlapDepth / (2.0 * MinRadius)) : 1.0;
				if (OverlapFraction < Simplifier.SphereOverlapThreshold)
				{
					return true; // Spheres don't overlap enough
				}
			}
			return false;
		};

		// Pre-compute error floors for split sub-edges: for each sub-edge (AI, IB) of edge AB,
		// inflate priority to Min(CollapseError(AB), min_C(CollapseError(IC)) + Epsilon)
		// so the sub-edges aren't considered 'too soon' before IC would be considered
		TMap<FIndex2i, float> SplitSubEdgeErrorFloors;
		for (const FSplitEdgeInfo& Split : SplitEdges)
		{
			checkSlow(Split.IntermediateNodes.Num() == 1);
			const int32 I = Split.IntermediateNodes[0];
			FSphere TmpSphere;
			float ErrorFloor = ComputeMergeError(FIndex2i(Split.A, Split.B), TmpSphere);
			for (int32 C : Split.OppositeVerts)
			{
				ErrorFloor = FMath::Min(ErrorFloor, ComputeMergeError(AsEdge(I, C), TmpSphere) + UE_KINDA_SMALL_NUMBER);
			}
			SplitSubEdgeErrorFloors.Add(AsEdge(Split.A, I), ErrorFloor);
			SplitSubEdgeErrorFloors.Add(AsEdge(Split.B, I), ErrorFloor);
		}

		for (int32 ClusterIdx = 0; ClusterIdx < InitialNumClusters; ++ClusterIdx)
		{
			for (int32 NbrIdx : Skeleton.ClusterNeighbors[ClusterIdx])
			{
				if (NbrIdx <= ClusterIdx)
				{
					continue;
				}
				FIndex2i Edge(ClusterIdx, NbrIdx);
				if (ShouldSkipEdge(Edge))
				{
					continue;
				}
				float UseErrorFloor = -FMathf::MaxReal;
				if (const float* ErrorFloor = SplitSubEdgeErrorFloors.Find(Edge))
				{
					UseErrorFloor = *ErrorFloor;
				}
				AddEdgeToQueue(Edge, false, UseErrorFloor);
			}
		}

		auto UpdateClusterIndexFromMerge = [&ClusterMergeParent](int32& Parent) -> void
		{
			while (ClusterMergeParent[Parent] != Parent)
			{
				ClusterMergeParent[Parent] = ClusterMergeParent[ClusterMergeParent[Parent]];
				Parent = ClusterMergeParent[Parent];
			}
		};

		EdgeQ.Heapify();
		int32 NumClusters = InitialNumClusters;

		auto ProcessEdgeQueue = [&EdgeQ, &NumClusters, &Simplifier, &ClusterVersion, &ClusterMergeParent, &ShouldSkipEdge, &UpdateClusterIndexFromMerge, &ComputeMergeError, &Skeleton, &MeshBVH, &CollapseEdge]()
		{
			while (!EdgeQ.IsEmpty() && NumClusters > Simplifier.MinSpheres)
			{
				FEdgeInfo EdgeInfo;
				EdgeQ.HeapPop(EdgeInfo, EAllowShrinking::No);

				FIndex2i& Edge = EdgeInfo.Edge;
				FIndex2i& Version = EdgeInfo.Version;
				const bool bIsStale = (ClusterVersion[Edge.A] > Version.A || ClusterVersion[Edge.B] > Version.B);
				if (bIsStale)
				{
					UpdateClusterIndexFromMerge(Edge.A);
					UpdateClusterIndexFromMerge(Edge.B);
					if (Edge.A == Edge.B)
					{
						// edge was merged away, skip
						continue;
					}
					if (ShouldSkipEdge(Edge))
					{
						continue;
					}
					Version.A = ClusterVersion[Edge.A];
					Version.B = ClusterVersion[Edge.B];
					EdgeInfo.Error = ComputeMergeError(Edge, EdgeInfo.MergeSphere);
					if (Simplifier.QEMErrorThreshold > 0 && EdgeInfo.Error > Simplifier.QEMErrorThreshold)
					{
						continue;
					}

					if (!EdgeQ.IsEmpty() && EdgeQ.HeapTop().Error < EdgeInfo.Error)
					{
						// after updating error, there's a cheaper edge on top of heap --
						// need just re-insert this one and continue to pick up the smaller edge instead
						EdgeQ.HeapPush(EdgeInfo);
						continue;
					}
				}
				else
				{
					if (ShouldSkipEdge(Edge))
					{
						continue;
					}
				}

				if (Simplifier.bPreventEdgeSurfaceIntersections)
				{
					bool bFoundHit = false;
					for (int32 SubIdx = 0; !bFoundHit && SubIdx < 2; ++SubIdx)
					{
						int32 SrcClusterIdx = Edge[SubIdx];
						int32 OtherIdx = Edge[1 - SubIdx];
						FVector3d SrcPos = EdgeInfo.MergeSphere.Center;
						for (int32 ToClusterIdx : Skeleton.ClusterNeighbors[SrcClusterIdx])
						{
							if (ToClusterIdx != OtherIdx)
							{
								FVector3d DestPos = Skeleton.Spheres[ToClusterIdx].Center;
								if (EdgeIntersectsMesh(SrcPos, DestPos, MeshBVH))
								{
									bFoundHit = true;
									break;
								}
							}
						}
					}
					if (bFoundHit)
					{
						continue;
					}
				}

				CollapseEdge(EdgeInfo.Edge, EdgeInfo.MergeSphere);
				NumClusters--;
			}
		};
		ProcessEdgeQueue();

		// If we're still above target sphere count, re-add split sub-edges with un-clamped errors for a cleanup pass
		if (NumClusters > Simplifier.MinSpheres)
		{
			bool bAddedEdges = false;
			for (const FSplitEdgeInfo& Split : SplitEdges)
			{
				checkSlow(Split.IntermediateNodes.Num() == 1);
				for (int32 EndV : {Split.A, Split.B})
				{
					int32 InterV = Split.IntermediateNodes[0];
					UpdateClusterIndexFromMerge(EndV);
					UpdateClusterIndexFromMerge(InterV);
					if (EndV == InterV)
					{
						continue;
					}
					FIndex2i EdgeEI = AsEdge(EndV, InterV);
					if (!ShouldSkipEdge(EdgeEI))
					{
						bAddedEdges |= AddEdgeToQueue(EdgeEI, true);
					}
				}
			}
			if (bAddedEdges)
			{
				ProcessEdgeQueue();
			}
		}

		// Do a final pass to compact away the removed cluster spheres/info and remap indices accordingly
		TArray<int32> RemapClusterIDs;
		int32 FinalNumClusters = 0;
		RemapClusterIDs.SetNumUninitialized(InitialNumClusters);
		for (int32 Idx = 0; Idx < InitialNumClusters; ++Idx)
		{
			if (ClusterMergeParent[Idx] == Idx)
			{
				int32 NewIdx = FinalNumClusters++;
				RemapClusterIDs[Idx] = NewIdx;
				if (Idx != NewIdx)
				{
					ensure(Idx > NewIdx);
					Skeleton.Spheres[NewIdx] = Skeleton.Spheres[Idx];
				}
			}
			else
			{
				ensure(ClusterMergeParent[Idx] < Idx); // by construction, parent should always be the smallest index in the merged cluster
				RemapClusterIDs[Idx] = RemapClusterIDs[ClusterMergeParent[Idx]];
			}
		}
		Skeleton.Spheres.SetNum(FinalNumClusters);
		for (int32 Idx = 0; Idx < InitialNumClusters; ++Idx)
		{
			int32 NewIdx = RemapClusterIDs[Idx];
			if (ClusterMergeParent[Idx] != Idx)
			{
				continue;
			}
			if (NewIdx != Idx)
			{
				Skeleton.ClusterNeighbors[NewIdx] = MoveTemp(Skeleton.ClusterNeighbors[Idx]);
			}
			for (int32& Nbr : Skeleton.ClusterNeighbors[NewIdx])
			{
				Nbr = RemapClusterIDs[Nbr];
			}
		}
		Skeleton.ClusterNeighbors.SetNum(FinalNumClusters);
		ParallelFor(MeshMaxVID, [&Skeleton, &RemapClusterIDs](int32 VID)
		{
			int32& ClusterIdx = Skeleton.VIDtoClusterIndex[VID];
			if (ClusterIdx != INDEX_NONE)
			{
				ClusterIdx = RemapClusterIDs[ClusterIdx];
			}
		});
		Skeleton.ClusterTriangles.Empty();
		TSet<FIndex3i> NewTriSet;
		for (int32 Idx = 0; Idx < InitialNumClusters; ++Idx)
		{
			for (const FIndex2i& OppE : NbrTris[Idx])
			{
				NewTriSet.Add(AsTri(
					RemapClusterIDs[OppE.A],
					RemapClusterIDs[OppE.B],
					RemapClusterIDs[Idx])
				);
			}
		}
		Skeleton.ClusterTriangles = NewTriSet.Array();
	}
}

namespace UE::Geometry::MedialAxis
{
	bool FMedialSkeleton::CheckValidity(EValidityCheckFailMode FailMode) const
	{
		bool bIsOk = true;
		TFunction<void(bool)> CheckOrFailF = [&bIsOk](bool b) { bIsOk = bIsOk && b; };
		if (FailMode == EValidityCheckFailMode::Check)
		{
			CheckOrFailF = [&bIsOk](bool b)
			{
				checkf(b, TEXT("FMedialSkeleton::CheckValidity failed!"));
				bIsOk = bIsOk && b;
			};
		}
		else if (FailMode == EValidityCheckFailMode::Ensure)
		{
			CheckOrFailF = [&bIsOk](bool b)
			{
				ensureMsgf(b, TEXT("FMedialSkeleton::CheckValidity failed!"));
				bIsOk = bIsOk && b;
			};
		}

		// ClusterNeighbors must be sized to match Spheres
		CheckOrFailF(ClusterNeighbors.Num() == Spheres.Num());

		// Neighbor arrays must be bidirectional and contain no self-loops
		for (int32 Idx = 0; Idx < ClusterNeighbors.Num(); ++Idx)
		{
			for (int32 Nbr : ClusterNeighbors[Idx])
			{
				CheckOrFailF(Nbr != Idx);
				CheckOrFailF(Nbr >= 0 && Nbr < ClusterNeighbors.Num());
				if (Nbr >= 0 && Nbr < ClusterNeighbors.Num())
				{
					CheckOrFailF(ClusterNeighbors[Nbr].Contains(Idx));
				}
			}
		}

		// Every edge of every triangle must appear in ClusterNeighbors
		for (const FIndex3i& Tri : ClusterTriangles)
		{
			for (int32 Prev = 2, Idx = 0; Idx < 3; Prev = Idx++)
			{
				CheckOrFailF(Tri[Prev] >= 0 && Tri[Prev] < ClusterNeighbors.Num());
				CheckOrFailF(Tri[Idx] >= 0 && Tri[Idx] < ClusterNeighbors.Num());
				if (Tri[Prev] >= 0 && Tri[Prev] < ClusterNeighbors.Num() &&
					Tri[Idx] >= 0 && Tri[Idx] < ClusterNeighbors.Num())
				{
					CheckOrFailF(ClusterNeighbors[Tri[Prev]].Contains(Tri[Idx]));
					CheckOrFailF(ClusterNeighbors[Tri[Idx]].Contains(Tri[Prev]));
				}
			}
		}

		return bIsOk;
	}

	template<typename MeshType>
	FMedialSkeleton FSkeletonViaSampling::ComputeSkeleton(
		const TMeshAABBTree3<MeshType>& MeshBVH, 
		TFunctionRef<FVector3d(int32)> GetVertexNormal, 
		TFunctionRef<int32(int32)> VIDtoWeldVIDFn, 
		const TFastWindingTree<MeshType>* MeshFWTree)
	{
		using namespace MeshMedialAxisSamplingLocals;

		FMedialSkeleton Result;

		const MeshType& Mesh = *MeshBVH.GetMesh();
		if (Mesh.TriangleCount() == 0)
		{
			return Result;
		}

		// Get mapping from vertex IDs to coincident triangles, accounting for any welding
		TArray<TArray<int32, TInlineAllocator<8>>> VIDtoTIDs, VIDtoNbrVIDs;
		VIDtoTIDs.SetNum(Mesh.MaxVertexID());
		VIDtoNbrVIDs.SetNum(Mesh.MaxVertexID());
		for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
		{
			if (Mesh.IsTriangle(TID))
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				Tri.A = VIDtoWeldVIDFn(Tri.A);
				Tri.B = VIDtoWeldVIDFn(Tri.B);
				Tri.C = VIDtoWeldVIDFn(Tri.C);
				for (int32 PrevIdx = 2, Idx = 0; Idx < 3; PrevIdx = Idx++)
				{

					VIDtoTIDs[Tri[Idx]].Add(TID);
					VIDtoNbrVIDs[Tri[Idx]].AddUnique(Tri[PrevIdx]);
					VIDtoNbrVIDs[Tri[PrevIdx]].AddUnique(Tri[Idx]);
				}
			}
		}

		// Simple enum to track which vertices are valid (exist in the mesh and are referenced by triangles) and have non-zero-radius medial spheres
		enum EVertexState : uint8
		{
			// unreferenced / not valid
			Skip = 0,
			// valid but failed to find a plausible medial sphere, don't use as source
			NoMedialSphere,
			// fully valid, incl. with a medial sphere
			Valid
		};
		TArray<EVertexState> VertexState; // track whether a vertex ID is valid and has an associated sphere
		VertexState.SetNumZeroed(Mesh.MaxVertexID());

		TArray<bool> AddedVertexSphere; // track whether we've already used a given seed vertex
		AddedVertexSphere.SetNumZeroed(Mesh.MaxVertexID());

		TArray<FSphere> VtxMedialSpheres; // computed initial/seed spheres per vertex
		double InitialRadius = MeshBVH.GetBoundingBox().MaxDim();

		// Pre-compute a medial sphere per-vertex (or a tag indicating where we couldn't find a sphere)
		VtxMedialSpheres.SetNumUninitialized(Mesh.MaxVertexID());
		const double FailedMedialSphereThreshold = FMath::Max(FMathd::ZeroTolerance, ShrinkMedial.RadiusThreshold);
		ParallelFor(Mesh.MaxVertexID(), 
			[&MeshBVH, &Mesh, &VIDtoWeldVIDFn, &VIDtoNbrVIDs, this, &VtxMedialSpheres, &VertexState, InitialRadius, FailedMedialSphereThreshold, &GetVertexNormal]
			(int32 VID)
		{
			if (Mesh.IsVertex(VID) && VIDtoWeldVIDFn(VID) == VID && !VIDtoNbrVIDs[VID].IsEmpty())
			{
				double LocalFindRadius = InitialRadius;
				FVector3d P = Mesh.GetVertex(VID), N = GetVertexNormal(VID);
				bool bHotStart = ShrinkMedial.FindLocalMinCurvatureRadius(Mesh, P, N, VIDtoNbrVIDs[VID], LocalFindRadius);
				VtxMedialSpheres[VID] = ShrinkMedial.FindWithImplicitWeld(MeshBVH, VID, VIDtoWeldVIDFn, P, N, bHotStart, LocalFindRadius);
				VertexState[VID] = VtxMedialSpheres[VID].W > FailedMedialSphereThreshold ? EVertexState::Valid : EVertexState::NoMedialSphere;
			}
		});

		VIDtoNbrVIDs.Empty(); // Not used after this point

		// Start with the biggest medial sphere
		int32 BiggestSphereIdx = INDEX_NONE;
		double BigSphereRad = -1;
		for (int32 VID = 0; VID < Mesh.MaxVertexID(); ++VID)
		{
			if (VertexState[VID] == EVertexState::Valid && VtxMedialSpheres[VID].W > BigSphereRad)
			{
				BigSphereRad = VtxMedialSpheres[VID].W;
				BiggestSphereIdx = VID;
			}
		}
		// Early out if there were no valid medial spheres found
		if (BiggestSphereIdx == INDEX_NONE)
		{
			return Result;
		}

		// Initialize clustering with the found largest sphere, mapping all vertices to that sphere
		Result.Spheres.SetNum(1);
		Result.Spheres[0] = VtxMedialSpheres[BiggestSphereIdx];
		Result.VIDtoClusterIndex.SetNumUninitialized(Mesh.MaxVertexID());
		for (int32 VID = 0; VID < Result.VIDtoClusterIndex.Num(); ++VID)
		{
			Result.VIDtoClusterIndex[VID] = VertexState[VID] == EVertexState::Skip ? INDEX_NONE : 0;
		}


		/// Arrays for tracking clusters and associated stats, will be re-used across multiple iterations below

		TArray<double> MaxErrInCluster; // indexed by cluster ID (i.e., sphere index)
		TArray<int32> MaxErrVertInCluster; // 1:1 with MaxErrInCluster
		TArray<int32> ClusterSize; // number of vertices in a cluster
		TArray<int32> RemapClusters; // used to remap cluster indices during a cluster deletion pass
		TArray<int32> ClusterVertStarts, VertsByCluster, ClustersToUpdate;
		// used to track which clusters should 'sleep' (not be optimized) b/c they haven't added or removed any vertices recently
		TArray<std::atomic<int8>> FullPassesToUpdateCluster;
		FullPassesToUpdateCluster.SetNum(1);
		FullPassesToUpdateCluster[0] = OptimizeSpheresForIterationsPostClusterChange;
		VertsByCluster.SetNumUninitialized(Mesh.MaxVertexID());
		Result.ClusterNeighbors.SetNum(1);
		TArray<int32> ClusterIndicesToSplitHeap; // tracks candidate cluster indices to split (ordered via heap)
		TArray<bool> HasSplitNbr;
		TArray<TArray<int32, TInlineAllocator<3>>> ShouldSplitClusterEdge;
		TArray<FMedialSkeleton::FNbrArray> ClusterNbrNbrs; // Stores neighbors-of-neighbors per cluster (excluding the cluster itself)


		// Helper to compute cluster sizes based on current VIDtoClusterIndex assignments
		auto UpdateClusterSize = [](const MeshType& Mesh, FMedialSkeleton& Result, TArray<int32>& OutClusterSize)
		{
			OutClusterSize.Reset(); OutClusterSize.SetNumZeroed(Result.Spheres.Num());
			for (int32 VID = 0; VID < Mesh.MaxVertexID(); ++VID)
			{
				// Note: Don't need to check Mesh.IsVertex(VID) here, because invalid vertices will already be mapped to the INDEX_NONE cluster index
				int32 ClusterIdx = Result.VIDtoClusterIndex[VID];
				if (ClusterIdx != INDEX_NONE)
				{
					OutClusterSize[ClusterIdx]++;
				}
			}
		};

		// Helper to transform cluster neighbors to neighbors-of-neighbors
		auto BuildClusterNbrNbrs = [&Result, &ClusterNbrNbrs]()
		{
			ClusterNbrNbrs.Reset(); ClusterNbrNbrs.SetNum(Result.Spheres.Num());
			// Cluster Neighbors must be already computed
			if (!ensure(Result.Spheres.Num() == Result.ClusterNeighbors.Num()))
			{
				return;
			}
			ParallelFor(Result.Spheres.Num(), [&Result, &ClusterNbrNbrs](int32 ClusterIdx)
			{
				ClusterNbrNbrs[ClusterIdx].Append(Result.ClusterNeighbors[ClusterIdx]);
				for (int32 Nbr : Result.ClusterNeighbors[ClusterIdx])
				{
					for (int32 NbrNbr : Result.ClusterNeighbors[Nbr])
					{
						if (NbrNbr != ClusterIdx)
						{
							ClusterNbrNbrs[ClusterIdx].AddUnique(NbrNbr);
						}
					}
				}
			});
		};

		// cache the triangle normals and areas, and vertex areas and SQEMs, for use in optimization below
		TArray<FVector3f> TriNormals;
		TArray<FSQEM> VtxSQEMs;
		TArray<float> TriAreas, VtxAreas;
		// Note: vertex areas are per-triangle sums (not divided by 3); 1/3 factors cancel out in usages below
		const double TotalArea = ComputeVertexAreasAndSQEMs(Mesh, VIDtoWeldVIDFn, TriNormals, TriAreas, VtxSQEMs, VtxAreas);

		// Stop early on a fully degenerate / zero-area mesh
		if (TotalArea == 0)
		{
			return Result;
		}

		PosErrorWt = FMath::Clamp(PosErrorWt, 0., 1.);
		double NormalErrorWt = 1. - PosErrorWt;

		// Area-Weighted Average of face-Normal errors, using FSQEMs
		auto GetVertexAvgSquaredError = [&Mesh, &VtxSQEMs, &VtxAreas, this, NormalErrorWt](int32 VID, const FSphere& Sphere) -> double
		{
			FVector3d V = Mesh.GetVertex(VID);
			double PosErr = (V - Sphere.Center).Length() - Sphere.W;
			double PosErrContrib = PosErr * PosErr * PosErrorWt;
			double NormalErrContrib = VtxAreas[VID] > 0 ? NormalErrorWt * VtxSQEMs[VID].Eval(Sphere) / VtxAreas[VID] : 0.;
			return PosErrContrib + NormalErrContrib;
		};

		// Area-Weighted Average of face-Normal errors, without using FSQEM (should be equivalent to the above GetVertexAvgSquaredError, kept here for comparison -- currently this appears to be slower)
		auto GetVertexAvgSquaredError_NoSQEM = 
			[&Mesh, this, &VtxAreas, &TriNormals, &TriAreas, &VIDtoTIDs, NormalErrorWt]
			(int32 VID, const FSphere& Sphere) -> double
		{
			FVector3d V = Mesh.GetVertex(VID);
			double PosErr = (V - Sphere.Center).Length() - Sphere.W;
			double TotalErr = PosErr * PosErr * PosErrorWt;
			double AreaSum = VtxAreas[VID];
			if (AreaSum > 0)
			{
				double NormalErrContrib = 0;
				for (int32 TID : VIDtoTIDs[VID])
				{
					FVector3d N = (FVector3d)TriNormals[TID];
					double SignedDist = N.Dot(V - Sphere.Center) - Sphere.W;
					NormalErrContrib += TriAreas[TID] * SignedDist * SignedDist;
				}
				TotalErr += NormalErrorWt * NormalErrContrib / AreaSum;
			}
			return TotalErr;
		};

		double LastAvgError = FMathd::MaxReal;
		const double MinClusterErrorToSplitSq = MinClusterErrorToSplit * MinClusterErrorToSplit;


		// Helper to assign a vertex to the lowest-error cluster, via exhaustive search
		auto AssignVertexCluster = [&Result, &Mesh, &VertexState, &GetVertexAvgSquaredError, this, &FullPassesToUpdateCluster](int32 VID)
		{
			if (VertexState[VID] == EVertexState::Skip)
			{
				return;
			}

			int32 OrigCluster = Result.VIDtoClusterIndex[VID];

			double BestErr = GetVertexAvgSquaredError(VID, Result.Spheres[0]);
			int32 BestSphere = 0;
			for (int32 ClusterIdx = 1; ClusterIdx < Result.Spheres.Num(); ++ClusterIdx)
			{
				double Err = GetVertexAvgSquaredError(VID, Result.Spheres[ClusterIdx]);
				if (Err < BestErr)
				{
					BestErr = Err;
					BestSphere = ClusterIdx;
				}
			}
			if (BestSphere != OrigCluster)
			{
				FullPassesToUpdateCluster[BestSphere] = OptimizeSpheresForIterationsPostClusterChange;
				if (OrigCluster != INDEX_NONE)
				{
					FullPassesToUpdateCluster[OrigCluster] = OptimizeSpheresForIterationsPostClusterChange;
				}
			}
			Result.VIDtoClusterIndex[VID] = BestSphere;
		};

		// Helper to assign a vertex to the lowest-error cluster, via local search of the ClusterNbrNbrs
		auto AssignVertexCluster_LocalSearch = [&Result, &Mesh, &VertexState, &GetVertexAvgSquaredError, this, &FullPassesToUpdateCluster, &ClusterNbrNbrs, &AssignVertexCluster](int32 VID)
		{
			checkSlow(ClusterNbrNbrs.Num() == Result.Spheres.Num());
			if (VertexState[VID] == EVertexState::Skip)
			{
				return;
			}

			int32 OrigCluster = Result.VIDtoClusterIndex[VID];
			// fall back to full search for unassigned or isolated clusters
			if (OrigCluster == INDEX_NONE || ClusterNbrNbrs[OrigCluster].IsEmpty())
			{
				AssignVertexCluster(VID);
				return;
			}

			double BestErr = GetVertexAvgSquaredError(VID, Result.Spheres[OrigCluster]);
			int32 BestSphere = OrigCluster;
			for (int32 ClusterIdx : ClusterNbrNbrs[OrigCluster])
			{
				double Err = GetVertexAvgSquaredError(VID, Result.Spheres[ClusterIdx]);
				if (Err < BestErr)
				{
					BestErr = Err;
					BestSphere = ClusterIdx;
				}
			}
			if (BestSphere != OrigCluster)
			{
				FullPassesToUpdateCluster[BestSphere] = OptimizeSpheresForIterationsPostClusterChange;
				if (OrigCluster != INDEX_NONE)
				{
					FullPassesToUpdateCluster[OrigCluster] = OptimizeSpheresForIterationsPostClusterChange;
				}
			}
			Result.VIDtoClusterIndex[VID] = BestSphere;
		};

		// Helper to update cluster assignments, either via exhuastive or local search
		auto UpdateClusterAssignments = 
			[&FullPassesToUpdateCluster, &Result, &Mesh, &AssignVertexCluster_LocalSearch, &AssignVertexCluster, &VIDtoWeldVIDFn, &BuildClusterNbrNbrs, this]
			(bool bStaleClusterNeighbors)
		{
			ensure(FullPassesToUpdateCluster.Num() == Result.Spheres.Num());
			if (UseLocalClusterVertexReassignmentAtClusterCount >= 0 && Result.Spheres.Num() > UseLocalClusterVertexReassignmentAtClusterCount)
			{
				if (bStaleClusterNeighbors)
				{
					Result.ComputeClusterNeighbors(Mesh, VIDtoWeldVIDFn);
				}
				BuildClusterNbrNbrs();
				ParallelFor(Mesh.MaxVertexID(), [&AssignVertexCluster_LocalSearch](int32 VID) { AssignVertexCluster_LocalSearch(VID); });
			}
			else
			{
				ParallelFor(Mesh.MaxVertexID(), [&AssignVertexCluster](int32 VID) { AssignVertexCluster(VID); });
			}
		};

		// Helper to evaluate the cluster error at a given vertex
		// Note: ErrorScaleFactor and ForceMinError are used to force the error to be higher where an edge between clusters intersects the mesh
		//  (when using the bSplitClustersIfEdgesIntersectSurface setting)
		auto UpdateMaxError =
			[&VertexState, &AddedVertexSphere, &Result, &MaxErrInCluster, &MaxErrVertInCluster, &GetVertexAvgSquaredError]
			(int32 VID, double ErrorScaleFactor = 1., double ForcedMinError = 0.)
		{
			// Note this is skipping the vertices that have no associated sphere, 
			// since we are looking for a high-error vertex that we can add a corresponding sphere for
			if (VertexState[VID] != EVertexState::Valid)
			{
				return;
			}
			// Heuristically, we only add the medial sphere of a given vertex once, to help the algorithm avoid 'fixating' on 
			// single bad vertices and to avoid e.g. cases where the same vertex's medial sphere could be repeatedly added and removed.
			// This may not always be ideal, as the optimization could shift spheres s.t. we would want to re-add a vertex's medial sphere.
			// TODO: consider exposing this as a parameter, and/or revisiting the effect of this parameter across more inputs.
			constexpr bool bNeverReAddVertexMedialSpheres = true;
			if (bNeverReAddVertexMedialSpheres && AddedVertexSphere[VID])
			{
				return;
			}
			int32 ClusterIdx = Result.VIDtoClusterIndex[VID];
			double VtxErr = FMath::Max(ForcedMinError, GetVertexAvgSquaredError(VID, Result.Spheres[ClusterIdx]) * ErrorScaleFactor);
			if (VtxErr > MaxErrInCluster[ClusterIdx])
			{
				MaxErrInCluster[ClusterIdx] = VtxErr;
				MaxErrVertInCluster[ClusterIdx] = VID;
			}
		};

		const FAxisAlignedBox3d MaxCenterBounds = MeshBVH.GetBoundingBox();
		const double MaxRadius = MaxCenterBounds.MaxDim();
		
		int32 ItersWithoutIncrease = 0;
		bool bHasSurfaceIntersections = false;
		while (Result.Spheres.Num() < MaxSpheres)
		{
			int32 LastNumSpheres = Result.Spheres.Num();

			MaxErrInCluster.Reset(); MaxErrInCluster.SetNumZeroed(Result.Spheres.Num());
			HasSplitNbr.Reset(); HasSplitNbr.SetNumZeroed(Result.Spheres.Num());

			// Find the highest error vertex per cluster
			MaxErrVertInCluster.Init(INDEX_NONE, Result.Spheres.Num());
			
			for (int32 VID = 0; VID < Mesh.MaxVertexID(); ++VID)
			{
				UpdateMaxError(VID);
			}

			if (bSplitClustersIfEdgesIntersectSurface)
			{
				ShouldSplitClusterEdge.SetNum(Result.Spheres.Num());
				for (int32 Idx = 0; Idx < ShouldSplitClusterEdge.Num(); ++Idx)
				{
					ShouldSplitClusterEdge[Idx].Reset();
				}
				ParallelFor(Result.Spheres.Num(), [&Result, &MeshBVH, &ShouldSplitClusterEdge](int32 ClusterIdx)
				{
					FVector3d ClusterCenter = Result.Spheres[ClusterIdx].Center;
					for (int32 NbrIdx : Result.ClusterNeighbors[ClusterIdx])
					{
						if (NbrIdx > ClusterIdx)
						{
							FVector3d NbrCenter = Result.Spheres[NbrIdx].Center;
							FVector3d Dir = NbrCenter - ClusterCenter;
							double Len = Normalize(Dir);
							if (Len > 0)
							{ 
								FRay3d Ray(ClusterCenter, Dir);
								IMeshSpatial::FQueryOptions Options;
								Options.MaxDistance = Len;
								if (MeshBVH.FindNearestHitTriangle(Ray, Options) != INDEX_NONE)
								{
									ShouldSplitClusterEdge[ClusterIdx].Add(NbrIdx);
								}
							}
						}
					}
				});
				bHasSurfaceIntersections = false;
				for (int32 Idx = 0; Idx < ShouldSplitClusterEdge.Num(); ++Idx)
				{
					if (!ShouldSplitClusterEdge[Idx].IsEmpty())
					{
						bHasSurfaceIntersections = true;
						break;
					}
				}
				if (bHasSurfaceIntersections)
				{
					for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
					{
						if (!Mesh.IsTriangle(TID))
						{
							continue;
						}
						FIndex3i Tri = Mesh.GetTriangle(TID);
						Tri.A = VIDtoWeldVIDFn(Tri.A);
						Tri.B = VIDtoWeldVIDFn(Tri.B);
						Tri.C = VIDtoWeldVIDFn(Tri.C);
						for (int32 Prev = 2, Idx = 0; Idx < 3; Prev = Idx++)
						{
							FIndex2i EdgeV(Tri[Prev], Tri[Idx]);
							FIndex2i EdgeCluster(
								Result.VIDtoClusterIndex[EdgeV.A],
								Result.VIDtoClusterIndex[EdgeV.B]);
							if (EdgeCluster.A == EdgeCluster.B 
								|| !ensure(EdgeCluster.A != INDEX_NONE && EdgeCluster.B != INDEX_NONE))
							{
								continue;
							}
							EdgeCluster.Sort();
							if (ShouldSplitClusterEdge[EdgeCluster.A].Contains(EdgeCluster.B))
							{
								UpdateMaxError(EdgeV.A, ErrorScaleNearEdgeSurfaceIntersection, MinClusterErrorToSplitSq + FMathd::ZeroTolerance);
								UpdateMaxError(EdgeV.B, ErrorScaleNearEdgeSurfaceIntersection, MinClusterErrorToSplitSq + FMathd::ZeroTolerance);
							}
						}
					}
				}
			}

			// For clusters w/ max error above threshold, add the max-error vertex's medial sphere
			// (if no neighboring cluster already added a sphere, and up to a fixed number of splits)
			ClusterIndicesToSplitHeap.Reset();
			auto MaxHeapPred = [&MaxErrInCluster](int32 A, int32 B) -> bool { return MaxErrInCluster[A] > MaxErrInCluster[B]; };
			for (int32 Idx = 0; Idx < MaxErrInCluster.Num(); ++Idx)
			{
				if (MaxErrInCluster[Idx] > MinClusterErrorToSplitSq)
				{
					ClusterIndicesToSplitHeap.Add(Idx);
				}
			}
			if (ClusterIndicesToSplitHeap.IsEmpty())
			{
				break;
			}
			ClusterIndicesToSplitHeap.Heapify(MaxHeapPred);
			int32 NumSplits = 0;
			double MinErrorToSplit = MaxErrInCluster[ClusterIndicesToSplitHeap.HeapTop()] * MinFractionOfMaxErrorToSplit;
			int32 MaxNumSplits = FMath::Max(MaxSplitPerIteration, FMath::CeilToInt32(Result.Spheres.Num() * MaxSplitFractionPerIteration));
			if (MaxNumSplits <= 0) // handle case where both Max parameters were disabled (set to 0)
			{
				MaxNumSplits = ClusterIndicesToSplitHeap.Num();
			}
			while (!ClusterIndicesToSplitHeap.IsEmpty() && NumSplits < MaxNumSplits)
			{
				int32 ClusterToSplit;
				ClusterIndicesToSplitHeap.HeapPop(ClusterToSplit, MaxHeapPred, EAllowShrinking::No);
				if (MaxErrInCluster[ClusterToSplit] < MinErrorToSplit)
				{
					break;
				}
				if (!HasSplitNbr[ClusterToSplit])
				{
					int32 ClusterSourceVID = MaxErrVertInCluster[ClusterToSplit];
					AddedVertexSphere[ClusterSourceVID] = true;
					FSphere ToAdd = VtxMedialSpheres[ClusterSourceVID];
					int32 NewClusterIdx = Result.Spheres.Add(ToAdd);
					FullPassesToUpdateCluster.Emplace(OptimizeSpheresForIterationsPostClusterChange);
					checkSlow(FullPassesToUpdateCluster.Num() == Result.Spheres.Num());
					NumSplits++;
					for (int32 Nbr : Result.ClusterNeighbors[ClusterToSplit])
					{
						HasSplitNbr[Nbr] = true;
					}

					// Add new cluster as neighbor of the split cluster, so local re-clustering can find it
					Result.ClusterNeighbors[ClusterToSplit].Add(NewClusterIdx);
					Result.ClusterNeighbors.Emplace_GetRef().Add(ClusterToSplit);
					check(Result.ClusterNeighbors.Num() == Result.Spheres.Num());
				}
			}

			// If no spheres were added, stop iterating
			if (LastNumSpheres == Result.Spheres.Num())
			{
				break;
			}

			UpdateClusterAssignments(false /*bStaleClusterNeighbors, neighbors are already updated above so aren't stale here*/);

			// Delete clusters that are below a minimum size, and re-assign their vertices
			{
				UpdateClusterSize(Mesh, Result, ClusterSize);
				RemoveSmallClusters(Result, ClusterSize, RemapClusters, FullPassesToUpdateCluster, AssignVertexCluster);

				// Track the number of iterations where we delete as many spheres as we add --
				// if this happens repeatedly over multiple sequential iterations, the algorithm is likely stuck
				if (Result.Spheres.Num() > LastNumSpheres)
				{
					ItersWithoutIncrease = 0;
				}
				else
				{
					ItersWithoutIncrease++;
				}
			}

			// Build a flat array of per-cluster vertex IDs for the updated clusters
			// Transform cluster sizes into 'cluster starts' (indicating the first VID for each cluster, in the combined array)
			// Reset cluster size counts for updated clusters to use for per-cluster index tracking
			ClusterVertStarts.Reset(); ClusterVertStarts.SetNumZeroed(Result.Spheres.Num());
			ClustersToUpdate.Reset();
			for (int32 ClusterIdx = 0, ClusterStart = 0; ClusterIdx < Result.Spheres.Num(); ++ClusterIdx)
			{
				if (FullPassesToUpdateCluster[ClusterIdx] > 0)
				{
					ClustersToUpdate.Add(ClusterIdx);
					ClusterVertStarts[ClusterIdx] = ClusterStart;
					ClusterStart += ClusterSize[ClusterIdx];
					ClusterSize[ClusterIdx] = 0;
				}
			}
			// Distribute the vertex IDs to their corresponding cluster in the VertsByCluster array
			for (int32 VID = 0; VID < Mesh.MaxVertexID(); ++VID)
			{
				int32 ClusterIdx = Result.VIDtoClusterIndex[VID];
				if (ClusterIdx != INDEX_NONE && FullPassesToUpdateCluster[ClusterIdx])
				{
					VertsByCluster[ClusterVertStarts[ClusterIdx] + ClusterSize[ClusterIdx]++] = VID;
				}
			}

			// Optimize the sphere for each cluster
			ParallelFor(ClustersToUpdate.Num(),
				[this, &MeshBVH, &MeshFWTree, &Mesh, &Result, &VIDtoTIDs, &TriAreas, &TriNormals, InitialRadius, &MaxCenterBounds, MaxRadius, &FullPassesToUpdateCluster, &ClustersToUpdate, &VtxSQEMs,
				&ClusterVertStarts, &ClusterSize, &VertsByCluster, NormalErrorWt](int32 ClustersToUpdateIdx)
			{
				int32 ClusterIdx = ClustersToUpdate[ClustersToUpdateIdx];
				if (!ensure(FullPassesToUpdateCluster[ClusterIdx] > 0))
				{
					return;
				}

				// optimization is not reliable if the cluster has very few vertices; better to leave the sphere as-is
				if (ClusterSize[ClusterIdx] < FMath::Max(2, MinClusterSizeToOptimize))
				{
					return;
				}
				// the plane-error part of A is just an area-weighted outer product of [N 1] with itself, same as for the SQEM metric, 
				// and doesn't depend on the sphere parameters, so we can compute it once outside of the newton iterations
				FEigMat4 A_SQEM = FEigMat4::Zero();
				for (int32 Idx = ClusterVertStarts[ClusterIdx], EndIdx = ClusterVertStarts[ClusterIdx] + ClusterSize[ClusterIdx]; Idx < EndIdx; ++Idx)
				{
					int32 VID = VertsByCluster[Idx];
					A_SQEM += VtxSQEMs[VID].A;
				}
				A_SQEM *= NormalErrorWt;
				for (int32 Iter = 0; Iter < MaxGaussNewtonIterations; ++Iter)
				{
					FEigMat4 A = FEigMat4::Zero();
					FEigVec4 B = FEigVec4::Zero();
					for (int32 Idx = ClusterVertStarts[ClusterIdx], EndIdx = ClusterVertStarts[ClusterIdx] + ClusterSize[ClusterIdx]; Idx < EndIdx; ++Idx)
					{
						int32 VID = VertsByCluster[Idx];
						checkSlow(Result.VIDtoClusterIndex[VID] == ClusterIdx);
						FVector3d VertPos = Mesh.GetVertex(VID);
						double AreaSum = 0;
						for (int32 TID : VIDtoTIDs[VID])
						{
							double AreaWt = (double)TriAreas[TID];
							AreaSum += AreaWt;
							FVector3d TriNormal = (FVector3d)TriNormals[TID];
							FEigVec4 N_1(TriNormal.X, TriNormal.Y, TriNormal.Z, 1.);
							// Plane-error contribution to A added below via A_SQEM
							B += N_1 * (NormalErrorWt * AreaWt * ((VertPos - Result.Spheres[ClusterIdx].Center).Dot((FVector3d)TriNormal) - Result.Spheres[ClusterIdx].W));
						}
						FVector3d FromCenterDir = (VertPos - Result.Spheres[ClusterIdx].Center);
						double FromCenterDist = Normalize(FromCenterDir);
						FEigVec4 D_1(FromCenterDir.X, FromCenterDir.Y, FromCenterDir.Z, 1.);
						A += D_1 * (D_1.transpose() * (AreaSum * PosErrorWt));
						B += D_1 * ((AreaSum * PosErrorWt) * (FromCenterDist - Result.Spheres[ClusterIdx].W));
					}
					A += A_SQEM;
					FEigVec4 Delta = A.ldlt().solve(B);
					if (!Delta.hasNaN())
					{
						FVector3d NewCenter = Result.Spheres[ClusterIdx].Center + FVector3d(Delta[0], Delta[1], Delta[2]);
						double NewRadius = Result.Spheres[ClusterIdx].W + Delta[3];
						
						// if the optimization starts sending the sphere off to a tiny/negative radius, a huge size or outside the input shape bounds, bail out of the optimization
						if (NewRadius <= UE_DOUBLE_SMALL_NUMBER || NewRadius > MaxRadius || !MaxCenterBounds.Contains(NewCenter))
						{
							break;
						}

						Result.Spheres[ClusterIdx].Center = NewCenter;
						Result.Spheres[ClusterIdx].W = NewRadius;

						if (Delta.squaredNorm() < GaussNewtonConvergenceThreshold * GaussNewtonConvergenceThreshold)
						{
							break;
						}
					}
					else // solve failed, stop running the optimization
					{
						break;
					}
				}
				// Project the optimized sphere to a medial sphere, via shrinking
				ShrinkMedial.Project(MeshBVH, Result.Spheres[ClusterIdx], InitialRadius, MeshFWTree);

				// Decrement the UpdatedCluster counter, so we can stop updating clusters that haven't changed recently
				if (--FullPassesToUpdateCluster[ClusterIdx] < 0)
				{
					FullPassesToUpdateCluster[ClusterIdx] = 0;
				}
			}, EParallelForFlags::Unbalanced);

			UpdateClusterAssignments(true /*bStaleClusterNeighbors, they can be stale here b/c e.g. clusters could be deleted above*/);

			// Re-compute cluster neighbors
			Result.ComputeClusterNeighbors(Mesh, VIDtoWeldVIDFn);
			if (ItersWithoutIncrease > 3)
			{
				break;
			}


		}

		// Do a final pass of cluster assignment, filtering, and connectivity
		UpdateClusterAssignments(true /*bStaleClusterNeighbors*/);
		UpdateClusterSize(Mesh, Result, ClusterSize);
		RemoveSmallClusters(Result, ClusterSize, RemapClusters, FullPassesToUpdateCluster, AssignVertexCluster);
		Result.ComputeClusterNeighbors(Mesh, VIDtoWeldVIDFn);
		Result.ComputeClusterTriangles(Mesh, VIDtoWeldVIDFn);

		// For unassigned vertices, try to transfer the cluster assignment from the weld vertex
		for (int32 VID = 0; VID < Result.VIDtoClusterIndex.Num(); ++VID)
		{
			if (Result.VIDtoClusterIndex[VID] == INDEX_NONE)
			{
				int32 WeldVID = VIDtoWeldVIDFn(VID);
				if (Result.VIDtoClusterIndex.IsValidIndex(WeldVID) && WeldVID != VID)
				{
					Result.VIDtoClusterIndex[VID] = Result.VIDtoClusterIndex[WeldVID];
				}
			}
		}

		if (Simplifier.IsSet())
		{
			SkeletonSimplify(*Simplifier, Result, MeshBVH, VtxSQEMs, VtxAreas);
		}

		return Result;
	}

	template<typename MeshType>
	FMedialSkeleton FSkeletonViaSampling::ComputeSkeleton(const TMeshAABBTree3<MeshType>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter, double ImplicitVertexNormalWeldThreshold, const TFastWindingTree<MeshType>* MeshFWTree)
	{
		using namespace MeshMedialAxisSamplingLocals;
		const MeshType& Mesh = *MeshBVH.GetMesh();

		TArray<FVector3f> Normals;
		Normals.SetNumUninitialized(Mesh.MaxVertexID());
		auto NormalFn = [&Normals](int32 VID) -> FVector3d { return (FVector3d)Normals[VID]; };
		TArray<int32> WeldIdx = ComputeWeldIdx(Mesh, OptionalBoundaryVertexFilter, ImplicitVertexNormalWeldThreshold);
		if (WeldIdx.IsEmpty())
		{
			MeshNormals::ComputeVertexNormals(Mesh, Normals);
			return ComputeSkeleton(MeshBVH, NormalFn, [](int32 VID)->int32 {return VID;}, MeshFWTree);
		}
		else
		{
			MeshNormals::ComputeWeldedVertexNormals(Mesh, [&WeldIdx](int32 VID)->int32 {return WeldIdx[VID];}, Normals);
			return ComputeSkeleton(MeshBVH, NormalFn, [&WeldIdx](int32 VID)->int32 {return WeldIdx[VID];}, MeshFWTree);
		}
	}

	template<typename MeshType>
	void FSkeletonSimplifier::Simplify(
		FMedialSkeleton& Skeleton,
		const TMeshAABBTree3<MeshType>& MeshBVH,
		TFunctionRef<FVector3d(int32)> GetVertexNormal,
		TFunctionRef<int32(int32)> VIDtoWeldVIDFn)
	{
		using namespace MeshMedialAxisSamplingLocals;
		const MeshType& Mesh = *MeshBVH.GetMesh();
		if (!ensure(Skeleton.IsCompatibleWithMesh(Mesh)))
		{
			return;
		}
		TArray<FVector3f> TriNormals;
		TArray<float> TriAreas;
		TArray<FSQEM> VtxSQEMs;
		TArray<float> VtxAreas;
		ComputeVertexAreasAndSQEMs(Mesh, VIDtoWeldVIDFn, TriNormals, TriAreas, VtxSQEMs, VtxAreas);
		SkeletonSimplify(*this, Skeleton, MeshBVH, VtxSQEMs, VtxAreas);
	}

	template<typename MeshType>
	void FSkeletonSimplifier::Simplify(
		FMedialSkeleton& Skeleton,
		const TMeshAABBTree3<MeshType>& MeshBVH,
		TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter,
		double ImplicitVertexNormalWeldThreshold)
	{
		using namespace MeshMedialAxisSamplingLocals;
		const MeshType& Mesh = *MeshBVH.GetMesh();
		if (!ensure(Skeleton.IsCompatibleWithMesh(Mesh)))
		{
			return;
		}
		TArray<FVector3f> Normals;
		Normals.SetNumUninitialized(Mesh.MaxVertexID());
		auto NormalFn = [&Normals](int32 VID) -> FVector3d { return (FVector3d)Normals[VID]; };
		TArray<int32> WeldIdx = ComputeWeldIdx(Mesh, OptionalBoundaryVertexFilter, ImplicitVertexNormalWeldThreshold);
		if (WeldIdx.IsEmpty())
		{
			MeshNormals::ComputeVertexNormals(Mesh, Normals);
			Simplify(Skeleton, MeshBVH, NormalFn, [](int32 VID) -> int32 { return VID; });
		}
		else
		{
			MeshNormals::ComputeWeldedVertexNormals(Mesh, [&WeldIdx](int32 VID) -> int32 { return WeldIdx[VID]; }, Normals);
			Simplify(Skeleton, MeshBVH, NormalFn, [&WeldIdx](int32 VID) -> int32 { return WeldIdx[VID]; });
		}
	}

	template<typename MeshType>
	void FSkeletonSubdivider::Subdivide(
		FMedialSkeleton& Skeleton,
		const TMeshAABBTree3<MeshType>& MeshBVH,
		TFunctionRef<FVector3d(int32)> GetVertexNormal,
		TFunctionRef<int32(int32)> VIDtoWeldVIDFn,
		const TFastWindingTree<MeshType>* MeshFWTree)
	{
		using namespace MeshMedialAxisSamplingLocals;
		const MeshType& Mesh = *MeshBVH.GetMesh();
		if (!ensure(Skeleton.IsCompatibleWithMesh(Mesh)))
		{
			return;
		}
		check(GetTargetEdgeLength);
		TArray<FVector3f> TriNormals;
		TArray<float> TriAreas;
		TArray<FSQEM> VtxSQEMs;
		TArray<float> VtxAreas;
		ComputeVertexAreasAndSQEMs(Mesh, VIDtoWeldVIDFn, TriNormals, TriAreas, VtxSQEMs, VtxAreas);

		int32 StartNewClusters = 0;
		TSet<int32> ClusterCanRemapVertices;
		if (SubdivideSkeletonEdges(*this, Skeleton, StartNewClusters, ClusterCanRemapVertices))
		{
			if (bReprojectMedialSpheres)
			{
				FShrinkMedialBall ShrinkMedial;
				for (int32 Idx = StartNewClusters; Idx < Skeleton.Spheres.Num(); ++Idx)
				{
					ShrinkMedial.Project(MeshBVH, Skeleton.Spheres[Idx], -1.0, MeshFWTree, &VIDtoWeldVIDFn);
				}
			}
			ReclusterVerticesAfterSplit(Skeleton, Mesh, VtxSQEMs, VtxAreas, ReassignClusterPosErrorWt, StartNewClusters, ClusterCanRemapVertices);
		}
	}

	template<typename MeshType>
	void FSkeletonSubdivider::Subdivide(
		FMedialSkeleton& Skeleton,
		const TMeshAABBTree3<MeshType>& MeshBVH,
		TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter,
		double ImplicitVertexNormalWeldThreshold,
		const TFastWindingTree<MeshType>* MeshFWTree)
	{
		using namespace MeshMedialAxisSamplingLocals;
		const MeshType& Mesh = *MeshBVH.GetMesh();
		if (!ensure(Skeleton.IsCompatibleWithMesh(Mesh)))
		{
			return;
		}
		TArray<FVector3f> Normals;
		Normals.SetNumUninitialized(Mesh.MaxVertexID());
		auto NormalFn = [&Normals](int32 VID) -> FVector3d { return (FVector3d)Normals[VID]; };
		TArray<int32> WeldIdx = ComputeWeldIdx(Mesh, OptionalBoundaryVertexFilter, ImplicitVertexNormalWeldThreshold);
		if (WeldIdx.IsEmpty())
		{
			MeshNormals::ComputeVertexNormals(Mesh, Normals);
			Subdivide(Skeleton, MeshBVH, NormalFn, [](int32 VID) -> int32 { return VID; }, MeshFWTree);
		}
		else
		{
			MeshNormals::ComputeWeldedVertexNormals(Mesh, [&WeldIdx](int32 VID) -> int32 { return WeldIdx[VID]; }, Normals);
			Subdivide(Skeleton, MeshBVH, NormalFn, [&WeldIdx](int32 VID) -> int32 { return WeldIdx[VID]; }, MeshFWTree);
		}
	}

	void FSkeletonSubdivider::Subdivide(FMedialSkeleton& Skeleton)
	{
		using namespace MeshMedialAxisSamplingLocals;
		check(GetTargetEdgeLength);

		int32 StartNewClusters = 0;
		TSet<int32> ClusterCanRemapVertices;
		SubdivideSkeletonEdges(*this, Skeleton, StartNewClusters, ClusterCanRemapVertices);

		Skeleton.VIDtoClusterIndex.Empty();
	}

	void FSkeletonViaSampling::RemoveSmallClusters(FMedialSkeleton& Result, TArray<int32>& InOutClusterSize, 
		TArray<int32>& OutRemapClusters, 
		TArray<std::atomic<int8>>& OutUpdatedClusters,
		TFunctionRef<void(int32)> AssignVertexCluster)
	{
		const int32 MinimumClusterSize = FMath::Max(MinClusterSizeToKeep, 0);
		checkSlow(InOutClusterSize.Num() == Result.Spheres.Num());
		OutRemapClusters.Init(INDEX_NONE, Result.Spheres.Num());
		int32 ValidClusters = 0, LargestClusterIdx = 0, LargestClusterSize = -1;
		for (int32 ClusterIdx = 0; ClusterIdx < Result.Spheres.Num(); ++ClusterIdx)
		{
			if (InOutClusterSize[ClusterIdx] > LargestClusterSize)
			{
				LargestClusterSize = InOutClusterSize[ClusterIdx];
				LargestClusterIdx = ClusterIdx;
			}
			if (InOutClusterSize[ClusterIdx] < MinimumClusterSize)
			{
				OutRemapClusters[ClusterIdx] = INDEX_NONE;
			}
			else
			{
				OutRemapClusters[ClusterIdx] = ValidClusters++;
			}
		}
		// recover from edge case where all clusters were deleted, by keeping the largest cluster
		if (ValidClusters == 0)
		{
			OutRemapClusters[LargestClusterIdx] = ValidClusters++;
		}
		for (int32 ClusterIdx = 0; ClusterIdx < Result.Spheres.Num(); ++ClusterIdx)
		{
			int32 NewIdx = OutRemapClusters[ClusterIdx];
			if (NewIdx != INDEX_NONE)
			{
				Result.Spheres[NewIdx] = Result.Spheres[ClusterIdx];
				InOutClusterSize[NewIdx] = InOutClusterSize[ClusterIdx];
				OutUpdatedClusters[NewIdx] = OutUpdatedClusters[ClusterIdx].load();
			}
		}
		Result.Spheres.SetNum(ValidClusters, EAllowShrinking::No);
		InOutClusterSize.SetNum(ValidClusters, EAllowShrinking::No);
		OutUpdatedClusters.SetNum(ValidClusters, EAllowShrinking::No);
		for (int32 VID = 0; VID < Result.VIDtoClusterIndex.Num(); ++VID)
		{
			int32& ClusterIdx = Result.VIDtoClusterIndex[VID];
			if (ClusterIdx != INDEX_NONE)
			{
				ClusterIdx = OutRemapClusters[ClusterIdx];
				if (ClusterIdx == INDEX_NONE)
				{
					AssignVertexCluster(VID);
					InOutClusterSize[Result.VIDtoClusterIndex[VID]]++;
				}
			}
		}
	}

	template<typename MeshType>
	void FMedialSkeletonToTreeSkeletonOptions::ToHierarchy(const FMedialSkeleton& InMedialSkeleton, const TMeshAABBTree3<MeshType>* AvoidIntersectingMeshBVH, TArray<int32>& OutParent, TArray<FTransform>& OutTransform, FMedialSkeletonToTreeSkeletonOptions Options, int32 InRootIndex,
		TArray<int32>* OutBoneIndexToMedialIndex, TArray<int32>* OutMedialIndexToBoneIndex)
	{
		using namespace UE::Geometry;
		using namespace MeshMedialAxisSamplingLocals;

		const int32 NumClusters = InMedialSkeleton.ClusterNeighbors.Num();
		if (!ensureMsgf(NumClusters == InMedialSkeleton.Spheres.Num(), TEXT("Invalid medial skeleton data; %d spheres does not match %d cluster neighbors"), InMedialSkeleton.Spheres.Num(), NumClusters))
		{
			return;
		}

		// setup/clear output arrays
		TArray<int32> LocalIndexToMedial;
		TArray<int32>* UseIndexToMedial = OutBoneIndexToMedialIndex ? OutBoneIndexToMedialIndex : &LocalIndexToMedial;
		UseIndexToMedial->Empty(NumClusters);
		OutParent.Empty(NumClusters);
		OutTransform.Empty(NumClusters);
		if (OutMedialIndexToBoneIndex)
		{
			OutMedialIndexToBoneIndex->Empty(NumClusters);
		}

		if (NumClusters == 0)
		{
			return;
		}

		TArray<FIndex2i> Edges;
		for (int32 Source = 0; Source < NumClusters; ++Source)
		{
			for (int32 Dest : InMedialSkeleton.ClusterNeighbors[Source])
			{
				if (!ensure(InMedialSkeleton.ClusterNeighbors.IsValidIndex(Dest)))
				{
					continue;
				}
				if (Source < Dest)
				{
					Edges.Emplace(Source, Dest);
				}
			}
		}

		// If we have a BVH to test edges for intersection, move the intersecting edges to the back of the array
		int32 IsIntersectingStart = Edges.Num();
		if (AvoidIntersectingMeshBVH)
		{
			// Note Algo::RemoveIf is actually 'move to back of array if'
			IsIntersectingStart = Algo::RemoveIf(Edges, [&](const FIndex2i& Edge)
			{
				return EdgeIntersectsMesh(
					InMedialSkeleton.Spheres[Edge.A].Center,
					InMedialSkeleton.Spheres[Edge.B].Center,
					*AvoidIntersectingMeshBVH
				);
			});
		}
		
		// Sort the non-intersecting and intersecting ranges separately by the secondary sort criteria
		for (int32 Range = 0; Range < 2; Range++)
		{
			int32 Start = Range * IsIntersectingStart;
			int32 Count = (Range ? Edges.Num() : IsIntersectingStart) - Start;;
			TConstArrayView<FIndex2i> EdgesView(Edges.GetData() + Start, Count);
			if (EdgesView.IsEmpty())
			{
				continue;
			}
			if (Options.EdgeWeightMethod == FMedialSkeletonToTreeSkeletonOptions::EEdgeWeightMethod::ArrayOrder)
			{
				EdgesView.Sort([](const FIndex2i& A, const FIndex2i& B) { return A.A + A.B < B.A + B.B; });
			}
			else if (Options.EdgeWeightMethod == FMedialSkeletonToTreeSkeletonOptions::EEdgeWeightMethod::AvgRadius)
			{
				EdgesView.Sort([&InMedialSkeleton](const FIndex2i& A, const FIndex2i& B)
					{
						return	InMedialSkeleton.Spheres[A.A].W + InMedialSkeleton.Spheres[A.B].W > // note: greater than, so we prefer connections between larger spheres
								InMedialSkeleton.Spheres[B.A].W + InMedialSkeleton.Spheres[B.B].W;
					}
				);
			}
			else // EdgeLength
			{
				EdgesView.Sort([&InMedialSkeleton](const FIndex2i& A, const FIndex2i& B)
					{
						return	FVector::DistSquared(InMedialSkeleton.Spheres[A.A].Center, InMedialSkeleton.Spheres[A.B].Center) <
								FVector::DistSquared(InMedialSkeleton.Spheres[B.A].Center, InMedialSkeleton.Spheres[B.B].Center);
					}
				);
			}
		}

		// Use a disjoint set to track the connectivity as we add one edge at a time
		FSizedDisjointSet MedialForest;
		MedialForest.Init(InMedialSkeleton.ClusterNeighbors.Num());

		// KeptNbrs will hold the subset of the medial skeleton ClusterNeighbors that form a spanning tree
		TArray<TArray<int32, TInlineAllocator<8>>> KeptNbrs;
		KeptNbrs.SetNum(NumClusters);
		for (const FIndex2i& Edge : Edges)
		{
			// add the edge only if the clusters aren't already indirectly connected by a previously-added edge
			if (MedialForest.Find(Edge.A) != MedialForest.Find(Edge.B))
			{
				MedialForest.Union(Edge.A, Edge.B);
				KeptNbrs[Edge.A].Add(Edge.B);
				KeptNbrs[Edge.B].Add(Edge.A);
			}
		}

		int32 RootIdx;
		if (InMedialSkeleton.Spheres.IsValidIndex(InRootIndex))
		{
			RootIdx = InRootIndex;
		}
		else
		{
			// Automatically select a root cluster based on the configured method
			const TArray<FSphere>& Spheres = InMedialSkeleton.Spheres;
			const FSphere* Selected = nullptr;
			switch (Options.SelectRootMethod)
			{
			case ESelectRootMethod::ClosestToPoint:
				Selected = Algo::MinElementBy(Spheres, [&](const FSphere& S) { return FVector3d::DistSquared(S.Center, Options.RootSelectionPoint); });
				break;
			case ESelectRootMethod::FarthestInDirection:
				Selected = Algo::MaxElementBy(Spheres, [&](const FSphere& S) { return S.Center.Dot(Options.RootSelectionDirection); });
				break;
			case ESelectRootMethod::ClosestToBoundsCenter:
			{
				FAxisAlignedBox3d SkeletonBounds = FAxisAlignedBox3d::Empty();
				for (const FSphere& S : Spheres)
				{
					SkeletonBounds.Contain(S.Center + FVector(S.W));
					SkeletonBounds.Contain(S.Center - FVector(S.W));
				}
				FVector3d BoundsCenter = SkeletonBounds.Center();
				Selected = Algo::MinElementBy(Spheres, [&](const FSphere& S) { return FVector3d::DistSquared(S.Center, BoundsCenter); });
				break;
			}
			case ESelectRootMethod::LargestSphere:
				Selected = Algo::MaxElementBy(Spheres, [](const FSphere& S) { return S.W; });
				break;
			case ESelectRootMethod::ArrayOrder:
			default:
				break;
			}
			RootIdx = Selected ? UE_PTRDIFF_TO_INT32(Selected - Spheres.GetData()) : 0;
		}

		if (Options.MergeDisconnectedMethod == EMergeDisconnectedMethod::ConnectClosestBones)
		{
			int32 IterLimit = NumClusters; // sanity check to avoid infinite loop, medial forest may take several iterations to merge to a single tree, but should never take more iterations than there are clusters (and typically, hopefully, much fewer!)
			while (MedialForest.GetSize(0) != NumClusters && ensure(IterLimit-- > 0))
			{
				const int32 RootTree = MedialForest.Find(RootIdx);
				struct FLinkCandidate
				{
					double DistSq;
					FIndex2i Edge;
				};
				TMap<int32, FLinkCandidate> BestLinks;
				for (int32 Idx = 0; Idx < NumClusters; ++Idx)
				{
					const int32 IdxTree = MedialForest.Find(Idx);
					if (IdxTree != RootTree)
					{
						// find the closest neighbor by brute force
						double ClosestDistSq = FMathd::MaxReal;
						if (FLinkCandidate* BestLink = BestLinks.Find(IdxTree))
						{
							ClosestDistSq = BestLink->DistSq;
						}

						int32 Closest = INDEX_NONE;
						for (int32 NbrIdx = 0; NbrIdx < NumClusters; ++NbrIdx)
						{
							// Any disconnected tree is a candidate for a new edge (not just the root tree)
							if (Idx != NbrIdx && MedialForest.Find(NbrIdx) != IdxTree)
							{
								double DSq = FVector3d::DistSquared(InMedialSkeleton.Spheres[Idx].Center, InMedialSkeleton.Spheres[NbrIdx].Center);
								if (DSq < ClosestDistSq)
								{
									ClosestDistSq = DSq;
									Closest = NbrIdx;
								}
							}
						}
						if (Closest != INDEX_NONE)
						{
							BestLinks.Add(IdxTree, FLinkCandidate{ ClosestDistSq, FIndex2i(Idx, Closest) });
						}
					}
				}
				for (const TPair<int32, FLinkCandidate>& Link : BestLinks)
				{
					MedialForest.Union(Link.Value.Edge.A, Link.Value.Edge.B);
					KeptNbrs[Link.Value.Edge.A].Add(Link.Value.Edge.B);
					KeptNbrs[Link.Value.Edge.B].Add(Link.Value.Edge.A);
				}
			}
		}

		
		FVector3d RootCenter = InMedialSkeleton.Spheres[RootIdx].Center;

		TArray<bool> Visited;
		Visited.SetNumZeroed(NumClusters);

		// World position of the virtual/custom root bone, if needed
		bool bHasCustomRoot = Options.CustomRootPosition.IsSet();
		FVector3d VirtualRootPosition = bHasCustomRoot ? *Options.CustomRootPosition : FVector3d::ZeroVector;

		TArray<FIndex2i> MedialQueue;

		const bool bStillDisconnected = (MedialForest.GetSize(RootIdx) != NumClusters);

		if (bStillDisconnected || bHasCustomRoot)
		{
			ensure(bHasCustomRoot || Options.MergeDisconnectedMethod == EMergeDisconnectedMethod::AddTopLevelRoot);

			// add the virtual root
			OutParent.Add(INDEX_NONE);
			OutTransform.Add(FTransform(VirtualRootPosition));
			UseIndexToMedial->Add(INDEX_NONE); // virtual root doesn't correspond to a medial skeleton cluster
		}
		if (bStillDisconnected)
		{
			TSet<int32> SeenTrees;
			for (int32 Idx = 0; Idx < NumClusters; ++Idx)
			{
				int32 TreeIdx = MedialForest.Find(Idx);
				bool bWasInSet = false;
				SeenTrees.FindOrAdd(TreeIdx, &bWasInSet);
				if (!bWasInSet)
				{
					MedialQueue.Emplace(Idx, 0);
				}
			}
		}

		// If above multi-root handling hasn't already added to the queue, add the root
		if (MedialQueue.IsEmpty())
		{
			MedialQueue.Emplace(RootIdx, bHasCustomRoot ? 0 : INDEX_NONE);
		}

		while (!MedialQueue.IsEmpty())
		{
			FIndex2i SrcIdxOutParent = MedialQueue.Pop(EAllowShrinking::No);
			// note: should not have already visited, because we enforced a tree structure on the graph by construction above
			if (!ensure(!Visited[SrcIdxOutParent.A]))
			{
				continue;
			}
			Visited[SrcIdxOutParent.A] = true;
			int32 OutIdx = UseIndexToMedial->Add(SrcIdxOutParent.A);
			OutParent.Add(SrcIdxOutParent.B);
			FVector3d ParentPos = VirtualRootPosition;
			if (SrcIdxOutParent.B != INDEX_NONE && (*UseIndexToMedial)[SrcIdxOutParent.B] != INDEX_NONE)
			{
				ParentPos = InMedialSkeleton.Spheres[(*UseIndexToMedial)[SrcIdxOutParent.B]].Center;
			}
			OutTransform.Add(FTransform(InMedialSkeleton.Spheres[SrcIdxOutParent.A].Center - ParentPos));
			for (int32 Nbr : KeptNbrs[SrcIdxOutParent.A])
			{
				if (!Visited[Nbr])
				{
					MedialQueue.Emplace(Nbr, OutIdx);
				}
			}
		}

		if (OutMedialIndexToBoneIndex)
		{
			OutMedialIndexToBoneIndex->Init(INDEX_NONE, NumClusters);
			for (int32 BoneIdx = 0; BoneIdx < UseIndexToMedial->Num(); ++BoneIdx)
			{
				int32 MedialIdx = (*UseIndexToMedial)[BoneIdx];
				if (OutMedialIndexToBoneIndex->IsValidIndex(MedialIdx))
				{
					(*OutMedialIndexToBoneIndex)[MedialIdx] = BoneIdx;
				}
			}
		}
	}


	// Instantiate templated ComputeSkeleton methods for the supported mesh types
	template GEOMETRYALGORITHMS_API FMedialSkeleton FSkeletonViaSampling::ComputeSkeleton<FDynamicMesh3>(const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn, const TFastWindingTree<FDynamicMesh3>* MeshFWTree);
	template GEOMETRYALGORITHMS_API FMedialSkeleton FSkeletonViaSampling::ComputeSkeleton<FDynamicMesh3>(const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter, double ImplicitVertexNormalWeldThreshold, const TFastWindingTree<FDynamicMesh3>* MeshFWTree);
	template GEOMETRYALGORITHMS_API FMedialSkeleton FSkeletonViaSampling::ComputeSkeleton<FTriangleMeshAdapterd>(const TMeshAABBTree3<FTriangleMeshAdapterd>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn, const TFastWindingTree<FTriangleMeshAdapterd>* MeshFWTree);
	template GEOMETRYALGORITHMS_API FMedialSkeleton FSkeletonViaSampling::ComputeSkeleton<FTriangleMeshAdapterd>(const TMeshAABBTree3<FTriangleMeshAdapterd>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter, double ImplicitVertexNormalWeldThreshold, const TFastWindingTree<FTriangleMeshAdapterd>* MeshFWTree);

	template GEOMETRYALGORITHMS_API void FSkeletonSimplifier::Simplify<FDynamicMesh3>(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn);
	template GEOMETRYALGORITHMS_API void FSkeletonSimplifier::Simplify<FTriangleMeshAdapterd>(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FTriangleMeshAdapterd>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn);
	template GEOMETRYALGORITHMS_API void FSkeletonSimplifier::Simplify<FDynamicMesh3>(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter, double ImplicitVertexNormalWeldThreshold);
	template GEOMETRYALGORITHMS_API void FSkeletonSimplifier::Simplify<FTriangleMeshAdapterd>(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FTriangleMeshAdapterd>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter, double ImplicitVertexNormalWeldThreshold);

	template GEOMETRYALGORITHMS_API void FSkeletonSubdivider::Subdivide<FDynamicMesh3>(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn, const TFastWindingTree<FDynamicMesh3>* MeshFWTree);
	template GEOMETRYALGORITHMS_API void FSkeletonSubdivider::Subdivide<FTriangleMeshAdapterd>(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FTriangleMeshAdapterd>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn, const TFastWindingTree<FTriangleMeshAdapterd>* MeshFWTree);
	template GEOMETRYALGORITHMS_API void FSkeletonSubdivider::Subdivide<FDynamicMesh3>(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter, double ImplicitVertexNormalWeldThreshold, const TFastWindingTree<FDynamicMesh3>* MeshFWTree);
	template GEOMETRYALGORITHMS_API void FSkeletonSubdivider::Subdivide<FTriangleMeshAdapterd>(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FTriangleMeshAdapterd>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter, double ImplicitVertexNormalWeldThreshold, const TFastWindingTree<FTriangleMeshAdapterd>* MeshFWTree);

	template GEOMETRYALGORITHMS_API void FMedialSkeletonToTreeSkeletonOptions::ToHierarchy<FDynamicMesh3>(const FMedialSkeleton& InMedialSkeleton, const TMeshAABBTree3<FDynamicMesh3>* AvoidIntersectingMeshBVH, TArray<int32>& OutParent, TArray<FTransform>& OutTransform, FMedialSkeletonToTreeSkeletonOptions Options, int32 RootIndex, TArray<int32>* OutBoneIndexToMedialIndex, TArray<int32>* OutMedialIndexToBoneIndex);
	template GEOMETRYALGORITHMS_API void FMedialSkeletonToTreeSkeletonOptions::ToHierarchy<FTriangleMeshAdapterd>(const FMedialSkeleton& InMedialSkeleton, const TMeshAABBTree3<FTriangleMeshAdapterd>* AvoidIntersectingMeshBVH, TArray<int32>& OutParent, TArray<FTransform>& OutTransform, FMedialSkeletonToTreeSkeletonOptions Options, int32 RootIndex, TArray<int32>* OutBoneIndexToMedialIndex, TArray<int32>* OutMedialIndexToBoneIndex);



} // namespace UE::Geometry::MedialAxis