// Copyright Epic Games, Inc. All Rights Reserved.


#include "XAtlasWrapper.h"
#include "ThirdParty/xatlas/xatlas.h"

namespace XAtlasWrapper
{
	namespace
	{
		xatlas::ChartOptions ConvertChartOptions( const XAtlasChartOptions& InOptions )
		{
			xatlas::ChartOptions OutOptions;

			OutOptions.paramFunc = InOptions.ParamFunc;
			OutOptions.maxChartArea = InOptions.MaxChartArea;
			OutOptions.maxBoundaryLength = InOptions.MaxBoundaryLength;
			OutOptions.normalDeviationWeight = InOptions.NormalDeviationWeight;
			OutOptions.roundnessWeight = InOptions.RoundnessWeight;
			OutOptions.straightnessWeight = InOptions.StraightnessWeight;
			OutOptions.normalSeamWeight = InOptions.NormalSeamWeight;
			OutOptions.textureSeamWeight = InOptions.TextureSeamWeight;
			OutOptions.maxCost = InOptions.MaxCost;
			OutOptions.maxIterations = InOptions.MaxIterations;
			OutOptions.useInputMeshUvs = InOptions.bUseInputMeshUvs;
			OutOptions.fixWinding = InOptions.bFixWinding;

			return OutOptions;
		}

		xatlas::PackOptions ConvertPackOptions( const XAtlasPackOptions& InOptions )
		{
			xatlas::PackOptions OutOptions;

			OutOptions.maxChartSize = InOptions.MaxChartSize;
			OutOptions.padding = InOptions.Padding;
			OutOptions.texelsPerUnit = InOptions.TexelsPerUnit;
			OutOptions.resolution = InOptions.Resolution;
			OutOptions.bilinear = InOptions.bBilinear;
			OutOptions.blockAlign = InOptions.bBlockAlign;
			OutOptions.bruteForce = InOptions.bBruteForce;
			OutOptions.createImage = InOptions.bCreateImage;
			OutOptions.rotateChartsToAxis = InOptions.bRotateChartsToAxis;
			OutOptions.rotateCharts = InOptions.bRotateCharts;

			return OutOptions;
		}
	}

	bool ComputeUVs(const TArray<int32>& IndexBuffer,
					const TArray<FVector3f>& VertexBuffer,
					const XAtlasChartOptions& InChartOptions,
					const XAtlasPackOptions& InPackOptions,
					TArray<FVector2D>& UVVertexBuffer,
					TArray<int32>& UVIndexBuffer,
					TArray<int32>& VertexRemapArray)
	{
		xatlas::ChartOptions ChartOptions = ConvertChartOptions(InChartOptions);
		xatlas::PackOptions PackOptions = ConvertPackOptions(InPackOptions);		

		xatlas::Atlas* Atlas = xatlas::Create();
		xatlas::MeshDecl XAtlasMesh;

		XAtlasMesh.vertexCount = VertexBuffer.Num();
		XAtlasMesh.vertexPositionStride = sizeof(float) * 3;
		XAtlasMesh.vertexPositionData = (float*)VertexBuffer.GetData();

		// TODO: Add normal information
		//if (NormalBuffer.Num() == NumVertices)
		//{
		//	XAtlasMesh.vertexNormalStride = sizeof(float) * 3;
		//	XAtlasMesh.vertexNormalData = NormalBuffer.GetData();
		//}
		  
		XAtlasMesh.indexCount = IndexBuffer.Num();
		XAtlasMesh.indexFormat = xatlas::IndexFormat::UInt32;	// TODO: We are actually passing in signed int32s. Does that matter?
		XAtlasMesh.indexData = IndexBuffer.GetData();

		xatlas::AddMeshError Error = xatlas::AddMesh(Atlas, XAtlasMesh);
		xatlas::AddMeshJoin(Atlas);
		check(Error == xatlas::AddMeshError::Success);

		xatlas::Generate(Atlas, ChartOptions, PackOptions);

		check(Atlas->meshCount == 1);

		UVVertexBuffer.Empty();
		UVIndexBuffer.Empty();
		VertexRemapArray.Empty();

		const double W = Atlas->width;
		const double H = Atlas->height;

		for (uint32 OutMeshID = 0; OutMeshID < Atlas->meshCount; ++OutMeshID)
		{
			const xatlas::Mesh& CurrentMesh = Atlas->meshes[OutMeshID];

			for (uint32 VertexID = 0; VertexID < CurrentMesh.vertexCount; ++VertexID)
			{
				xatlas::Vertex UVVert = CurrentMesh.vertexArray[VertexID];
				UVVertexBuffer.Add(FVector2D{ UVVert.uv[0] / W, UVVert.uv[1] / H});

				uint32 OriginalVertexIndex = UVVert.xref;
				VertexRemapArray.Add(OriginalVertexIndex);
			}

			for (uint32 IndexID = 0; IndexID < CurrentMesh.indexCount; ++IndexID)
			{
				uint32_t Index = CurrentMesh.indexArray[IndexID];
				UVIndexBuffer.Add((int32)Index);
			}
		}

		xatlas::Destroy(Atlas);

		return true;
	}

}	// namespace XAtlasWrapper
