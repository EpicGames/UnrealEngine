// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp Arrangement2d

#pragma once

#include "BoxTypes.h"
#include "Curve/DynamicGraph2.h"
#include "Intersection/IntrSegment2Segment2.h"
#include "Polygon2.h"
#include "Spatial/PointHashGrid2.h"
#include "Util/GridIndexing2.h"

#include "CoreMinimal.h"

namespace UE {
namespace Geometry {


/**
 * Arrangement2d constructs a planar arrangement of a set of 2D line segments.
 * When a segment is inserted, existing edges are split, and the inserted
 * segment becomes multiple graph edges. So, the resulting FDynamicGraph2d should
 * not have any edges that intersect.
 * 
 * Calculations are performed in double-precision, so there is no guarantee
 * of correctness. 
 * 
 * 
 * [TODO] multi-level segment has to accelerate find_intersecting_edges()
 * [TODO] maybe smarter handling
 * 
 */
struct FArrangement2d
{
	// graph of arrangement
	FDynamicGraph2d Graph;

	// PointHash for vertices of graph
	TPointHashGrid2d<int> PointHash;

	// points within this tolerance are merged
	double VertexSnapTol = 0.00001;

	FArrangement2d(const FAxisAlignedBox2d& BoundsHint)
		: PointHash(FMath::Max(FMathd::ZeroTolerance, BoundsHint.MaxDim() / 64), -1)
	{
	}

	FArrangement2d(double PointHashCellSize)
		: PointHash(FMath::Max(FMathd::ZeroTolerance, PointHashCellSize), -1)
	{
	}

	/**
	 * Attempts to triangulates the arrangement with a constrained Delaunay triangulation
	 * NOTE: Will return all triangles if no edges found with the BoundaryEdgeGroupID
	 * NOTE: May fail if arrangement has self-intersections
	 * 
	 * Triangles: Output triangles (as indices into Graph vertices)
	 * SkippedEdges: Output indices of edges that the algorithm failed to insert
	 * BoundaryEdgeGroupID: ID of edges corresponding to a boundary; if we have a closed loop of these boundary edges on output triangulation, will discard triangles outside this
	 * return: false if triangulation algo knows it failed (note Triangles may still have some triangulation of the input in this case; for example it may just be missing some required edges)
	 */
	UE_DEPRECATED(5.1, "Please use the Triangulate or TriangulateWithBoundary functions instead, which are explicit about whether a BoundaryEdgeGroupID should be present")
	bool GEOMETRYALGORITHMS_API AttemptTriangulate(TArray<FIndex3i>& Triangles, TArray<int32>& SkippedEdges, int32 BoundaryEdgeGroupID);

	// Variant of AttemptTriangulate using FIntVector instead of FIndex3i; Note this incurs an extra copy of the triangle array
	UE_DEPRECATED(5.1, "Please use the Triangulate or TriangulateWithBoundary functions instead, which are explicit about whether a BoundaryEdgeGroupID should be present")
	bool GEOMETRYALGORITHMS_API AttemptTriangulate(TArray<FIntVector>& Triangles, TArray<int32>& SkippedEdges, int32 BoundaryEdgeGroupID);

	/**
	 * Attempts to triangulate the arrangement with a constrained Delaunay triangulation
	 * NOTE: May fail if arrangement has self-intersections
	 * 
	 * @param Triangles				Output triangles (as indices into Graph vertices)
	 * @param BoundaryEdgeGroupID	ID of edges corresponding to a boundary: triangles outside these edges will be removed
	 * @param HoleEdgeGroupID		ID of edges corresponding to internal holes: triangles inside these edges will be removed
	 * @return						false if triangulation algo knows it failed; will still likely have some triangulation even in this case
	 */
	bool GEOMETRYALGORITHMS_API TriangulateWithBoundaryAndHoles(TArray<FIndex3i>& Triangles, int32 BoundaryEdgeGroupID, int32 HoleEdgeGroupID);

	/**
	 * Attempts to triangulate the arrangement with a constrained Delaunay triangulation
	 * NOTE: May fail if arrangement has self-intersections
	 *
	 * @param Triangles				Output triangles (as indices into Graph vertices)
	 * @param BoundaryEdgeGroupID	ID of edges corresponding to a boundary: triangles outside these edges will be removed
	 * @return						false if triangulation algo knows it failed; will still likely have some triangulation even in this case
	 */
	bool GEOMETRYALGORITHMS_API TriangulateWithBoundary(TArray<FIndex3i>& Triangles, int32 BoundaryEdgeGroupID);

	/**
	 * Attempts to triangulate the arrangement with a constrained Delaunay triangulation
	 * NOTE: May fail if arrangement has self-intersections
	 *
	 * @param Triangles				Output triangles (as indices into Graph vertices)
	 * @return						false if triangulation algo knows it failed; will still likely have some triangulation even in this case
	 */
	bool GEOMETRYALGORITHMS_API Triangulate(TArray<FIndex3i>& Triangles);

	/**
	 * Check if current Graph has self-intersections; not optimized, only for debugging
	 */
	bool HasSelfIntersections()
	{
		for (const FDynamicGraph::FEdge e : Graph.Edges())
		{
			TArray<FIntersection> Hits;
			int HitCount = find_intersecting_edges(Graph.GetVertex(e.A), Graph.GetVertex(e.B), Hits, 0.0);
			for (const FIntersection& Intersect : Hits)
			{
				FDynamicGraph::FEdge o = Graph.GetEdge(Intersect.EID);
				if (o.A != e.A && o.A != e.B && o.B != e.A && o.B != e.B)
				{
					return true;
				}
			}
		}
		return false;
	}

	/**
	 * Subdivide edge at a given position
	 */
	FIndex2i SplitEdgeAtPoint(int EdgeID, FVector2d Point)
	{
		FDynamicGraph2d::FEdgeSplitInfo splitInfo;
		EMeshResult result = Graph.SplitEdge(EdgeID, splitInfo);
		ensureMsgf(result == EMeshResult::Ok, TEXT("SplitEdgeAtPoint: edge split failed?"));
		Graph.SetVertex(splitInfo.VNew, Point);
		PointHash.InsertPointUnsafe(splitInfo.VNew, Point);
		return FIndex2i(splitInfo.VNew, splitInfo.ENewBN);
	}

	/**
	 * Check if vertex exists in region
	 */
	bool HasVertexNear(FVector2d Point, double SearchRadius)
	{
		return find_nearest_vertex(Point, SearchRadius) > -1;
	}

	/**
	 * Insert isolated point P into the arrangement
	 */
	int Insert(const FVector2d& Pt)
	{
		return insert_point(Pt, VertexSnapTol);
	}

	/**
	 * Insert isolated point P into the arrangement when you know by construction it's not too close to any vertex or edge
	 * Much faster, but will break things if you use it to insert a point that is on top of any existing element!
	 */
	int32 InsertNewIsolatedPointUnsafe(const FVector2d& Pt)
	{
		int32 VID = Graph.AppendVertex(Pt);
		PointHash.InsertPointUnsafe(VID, Pt);
		return VID;
	}

	/**
	 * insert segment [A,B] into the arrangement
	 */
	void Insert(const FVector2d& A, const FVector2d& B, int GID = -1)
	{
		insert_segment(A, B, GID, VertexSnapTol);
	}

	/**
	 * insert segment into the arrangement
	 */
	void Insert(const FSegment2d& Segment, int GID = -1)
	{
		insert_segment(Segment.StartPoint(), Segment.EndPoint(), GID, VertexSnapTol);
	}

	///**
	// * sequentially insert segments of polyline
	// */
	//void Insert(PolyLine2d pline, int GID = -1)
	//{
	//	int N = pline.VertexCount - 1;
	//	for (int i = 0; i < N; ++i) {
	//		FVector2d A = pline[i];
	//		FVector2d B = pline[i + 1];
	//		insert_segment(A, B, GID);
	//	}
	//}

	///**
	// * sequentially insert segments of polygon
	// */
	void Insert(const FPolygon2d& Poly, int GID = -1)
	{
		int N = Poly.VertexCount();
		for (int i = 0; i < N; ++i)
		{
			insert_segment(Poly[i], Poly[(i + 1) % N], GID, VertexSnapTol);
		}
	}

	/*
	*  Graph improvement
	*/

	/**
	 * connect open boundary vertices within DistThresh, by inserting new segments
	 */
	void ConnectOpenBoundaries(double DistThresh)
	{
		int max_vid = Graph.MaxVertexID();
		for (int VID = 0; VID < max_vid; ++VID)
		{
			if (Graph.IsBoundaryVertex(VID) == false)
			{
				continue;
			}

			FVector2d v = Graph.GetVertex(VID);
			int snap_with = find_nearest_boundary_vertex(v, DistThresh, VID);
			if (snap_with != -1)
			{
				FVector2d v2 = Graph.GetVertex(snap_with);
				Insert(v, v2);
			}
		}
	}

protected:
	struct FSegmentPoint
	{
		double T;
		int VID;
	};

	/**
	 * insert pt P into the arrangement, splitting existing edges as necessary
	 */
	int insert_point(const FVector2d& P, double Tol = 0)
	{
		int PIdx = find_existing_vertex(P);
		if (PIdx > -1)
		{
			return -1;
		}

		// TODO: currently this tries to add the vertex on the closest edge below tolerance; we should instead insert at *every* edge below tolerance!  ... but that is more inconvenient to write
		FVector2d x = FVector2d::Zero(), y = FVector2d::Zero();
		double ClosestDistSq = Tol*Tol;
		int FoundEdgeToSplit = -1;
		for (int EID = 0, ExistingEdgeMax = Graph.MaxEdgeID(); EID < ExistingEdgeMax; EID++)
		{
			if (!Graph.IsEdge(EID))
			{
				continue;
			}

			Graph.GetEdgeV(EID, x, y);
			FSegment2d Seg(x, y);
			double DistSq = Seg.DistanceSquared(P);
			if (DistSq < ClosestDistSq)
			{
				ClosestDistSq = DistSq;
				FoundEdgeToSplit = EID;
			}
		}
		if (FoundEdgeToSplit > -1)
		{
			FDynamicGraph2d::FEdgeSplitInfo splitInfo;
			EMeshResult result = Graph.SplitEdge(FoundEdgeToSplit, splitInfo);
			ensureMsgf(result == EMeshResult::Ok, TEXT("insert_into_segment: edge split failed?"));
			Graph.SetVertex(splitInfo.VNew, P);
			PointHash.InsertPointUnsafe(splitInfo.VNew, P);
			return splitInfo.VNew;
		}

		int VID = Graph.AppendVertex(P);
		PointHash.InsertPointUnsafe(VID, P);
		return VID;
	}

	/**
	 * insert edge [A,B] into the arrangement, splitting existing edges as necessary
	 */
	bool insert_segment(FVector2d A, FVector2d B, int GID = -1, double Tol = 0)
	{
		// handle degenerate edges
		int a_idx = find_existing_vertex(A);
		int b_idx = find_existing_vertex(B);
		if (a_idx == b_idx && a_idx >= 0)
		{
			return false;
		}
		// snap input vertices
		if (a_idx >= 0)
		{
			A = Graph.GetVertex(a_idx);
		}
		if (b_idx >= 0)
		{
			B = Graph.GetVertex(b_idx);
		}

		// handle tiny-segment case
		double SegLenSq = DistanceSquared( A, B );
		if (SegLenSq <= VertexSnapTol*VertexSnapTol)
		{
			// seg is too short and was already on an existing vertex; just consider that vertex to be the inserted segment
			if (a_idx >= 0 || b_idx >= 0)
			{
				return false;
			}
			// seg is too short and wasn't on an existing vertex; add it as an isolated vertex
			return insert_point(A, Tol) != -1;
		}

		// ok find all intersections
		TArray<FIntersection> Hits;
		find_intersecting_edges(A, B, Hits, Tol);		

		// we are going to construct a list of <T,vertex_id> values along segment AB
		TArray<FSegmentPoint> points;
		FSegment2d segAB = FSegment2d(A, B);

		find_intersecting_floating_vertices(segAB, a_idx, b_idx, points, Tol);

		// insert intersections into existing segments
		for (int i = 0, N = Hits.Num(); i < N; ++i)
		{
			FIntersection Intr = Hits[i];
			int EID = Intr.EID;
			double t0 = Intr.Intr.Parameter0, t1 = Intr.Intr.Parameter1;

			// insert first point at t0
			int new_eid = -1;
			if (Intr.Intr.Type == EIntersectionType::Point || Intr.Intr.Type == EIntersectionType::Segment)
			{
				FIndex2i new_info = split_segment_at_t(EID, t0, VertexSnapTol);
				new_eid = new_info.B;
				FVector2d v = Graph.GetVertex(new_info.A);
				points.Add(FSegmentPoint{segAB.Project(v), new_info.A});
			}

			// if intersection was on-segment, then we have a second point at t1
			if (Intr.Intr.Type == EIntersectionType::Segment)
			{
				if (new_eid == -1)
				{
					// did not actually split edge for t0, so we can still use EID
					FIndex2i new_info = split_segment_at_t(EID, t1, VertexSnapTol);
					FVector2d v = Graph.GetVertex(new_info.A);
					points.Add(FSegmentPoint{segAB.Project(v), new_info.A});
				}
				else
				{
					// find t1 was in EID, rebuild in new_eid
					FSegment2d new_seg = Graph.GetEdgeSegment(new_eid);
					FVector2d p1 = Intr.Intr.GetSegment1().PointAt(t1);
					double new_t1 = new_seg.Project(p1);
					// note: new_t1 may be outside of new_seg due to snapping; in this case the segment will just not be split

					FIndex2i new_info = split_segment_at_t(new_eid, new_t1, VertexSnapTol);
					FVector2d v = Graph.GetVertex(new_info.A);
					points.Add(FSegmentPoint{segAB.Project(v), new_info.A});
				}
			}
		}

		// find or create start and end points
		if (a_idx == -1)
		{
			a_idx = find_existing_vertex(A);
		}
		if (a_idx == -1)
		{
			a_idx = Graph.AppendVertex(A);
			PointHash.InsertPointUnsafe(a_idx, A);
		}
		if (b_idx == -1)
		{
			b_idx = find_existing_vertex(B);
		}
		if (b_idx == -1)
		{
			b_idx = Graph.AppendVertex(B);
			PointHash.InsertPointUnsafe(b_idx, B);
		}

		// add start/end to points list. These may be duplicates but we will sort that out after
		points.Add(FSegmentPoint{-segAB.Extent, a_idx});
		points.Add(FSegmentPoint{segAB.Extent, b_idx});
		// sort by T
		points.Sort([](const FSegmentPoint& pa, const FSegmentPoint& pb) { return pa.T < pb.T; });

		// connect sequential points, as long as they aren't the same point,
		// and the segment doesn't already exist
		for (int k = 0; k < points.Num() - 1; ++k)
		{
			int v0 = points[k].VID;
			int v1 = points[k + 1].VID;
			if (v0 == v1)
			{
				continue;
			}

			if (Graph.FindEdge(v0, v1) == FDynamicGraph2d::InvalidID)
			{
				// sanity check; technically this can happen and still be correct but it's more likely an error case
				ensureMsgf(FMath::Abs(points[k].T - points[k + 1].T) >= std::numeric_limits<float>::epsilon(), TEXT("insert_segment: different points have same T??"));

				Graph.AppendEdge(v0, v1, GID);
			}
		}

		return true;
	}

	/**
	 * insert new point into segment EID at parameter value T
	 * If T is within Tol of endpoint of segment, we use that instead.
	 */
	FIndex2i split_segment_at_t(int EID, double T, double Tol)
	{
		FIndex2i ev = Graph.GetEdgeV(EID);
		FSegment2d seg = FSegment2d(Graph.GetVertex(ev.A), Graph.GetVertex(ev.B));

		int use_vid = -1;
		int new_eid = -1;
		if (T < -(seg.Extent - Tol))
		{
			use_vid = ev.A;
		}
		else if (T > (seg.Extent - Tol))
		{
			use_vid = ev.B;
		}
		else
		{
			FVector2d Pt = seg.PointAt(T);
			FDynamicGraph2d::FEdgeSplitInfo splitInfo;
			EMeshResult result;
			int CrossingVert = find_existing_vertex(Pt);
			if (CrossingVert == -1)
			{
				result = Graph.SplitEdge(EID, splitInfo);
			}
			else
			{
				result = Graph.SplitEdgeWithExistingVertex(EID, CrossingVert, splitInfo);
			}
			ensureMsgf(result == EMeshResult::Ok, TEXT("insert_into_segment: edge split failed?"));
			use_vid = splitInfo.VNew;
			new_eid = splitInfo.ENewBN;
			if (CrossingVert == -1)
			{	// position + track added vertex
				Graph.SetVertex(use_vid, Pt);
				PointHash.InsertPointUnsafe(splitInfo.VNew, Pt);
			}
		}
		return FIndex2i(use_vid, new_eid);
	}

	/**
	 * find existing vertex at point, if it exists
	 */
	int find_existing_vertex(FVector2d Pt)
	{
		return find_nearest_vertex(Pt, VertexSnapTol);
	}
	/**
	 * find closest vertex, within SearchRadius
	 */
	int find_nearest_vertex(FVector2d Pt, double SearchRadius, int IgnoreVID = -1)
	{
		auto FuncDistSq = [&](int B) { return DistanceSquared(Pt, Graph.GetVertex(B)); };
		auto FuncIgnore = [&](int VID) { return VID == IgnoreVID; };
		TPair<int, double> found = (IgnoreVID == -1) ? PointHash.FindNearestInRadius(Pt, SearchRadius, FuncDistSq)
													 : PointHash.FindNearestInRadius(Pt, SearchRadius, FuncDistSq, FuncIgnore);
		if (found.Key == PointHash.GetInvalidValue())
		{
			return -1;
		}
		return found.Key;
	}

	/**
	 * find nearest boundary vertex, within SearchRadius
	 */
	int find_nearest_boundary_vertex(FVector2d Pt, double SearchRadius, int IgnoreVID = -1)
	{
		auto FuncDistSq = [&](int B) { return DistanceSquared(Pt, Graph.GetVertex(B)); };
		auto FuncIgnore = [&](int VID) { return Graph.IsBoundaryVertex(VID) == false || VID == IgnoreVID; };
		TPair<int, double> found =
			PointHash.FindNearestInRadius(Pt, SearchRadius, FuncDistSq, FuncIgnore);
		if (found.Key == PointHash.GetInvalidValue())
		{
			return -1;
		}
		return found.Key;
	}

	struct FIntersection
	{
		int EID;
		int SideX;
		int SideY;
		FIntrSegment2Segment2d Intr;
	};

	/**
	 * find set of edges in graph that intersect with edge [A,B]
	 */
	bool find_intersecting_edges(FVector2d A, FVector2d B, TArray<FIntersection>& Hits, double Tol = 0)
	{
		int num_hits = 0;
		FVector2d x = FVector2d::Zero(), y = FVector2d::Zero();
		FVector2d EPerp = PerpCW(B - A);
		Normalize(EPerp);
		for (int EID : Graph.EdgeIndices())
		{
			Graph.GetEdgeV(EID, x, y);
			// inlined version of WhichSide with pre-normalized EPerp, to ensure Tolerance is consistent for different edge lengths
			double SignX = EPerp.Dot(x - A);
			double SignY = EPerp.Dot(y - A);
			int SideX = (SignX > Tol ? +1 : (SignX < -Tol ? -1 : 0)); 
			int SideY = (SignY > Tol ? +1 : (SignY < -Tol ? -1 : 0)); 
			if (SideX == SideY && SideX != 0)
			{
				continue; // both pts on same side
			}

			FIntrSegment2Segment2d Intr(FSegment2d(x, y), FSegment2d(A, B));
			Intr.SetIntervalThreshold(Tol);
			// set a loose DotThreshold as well so almost-parallel segments are treated as parallel;
			//  otherwise we're more likely to hit later problems when an edge intersects near-overlapping edges at almost the same point
			// (TODO: detect + handle that case!)
			Intr.SetDotThreshold(1e-4);
			if (Intr.Find())
			{
				Hits.Add(FIntersection{EID, SideX, SideY, Intr});
				num_hits++;
			}
		} 
		
		return (num_hits > 0);
	}

	bool find_intersecting_floating_vertices(const FSegment2d &SegAB, int32 AID, int32 BID, TArray<FSegmentPoint>& Hits, double Tol = 0)
	{
		int num_hits = 0;

		for (int VID : Graph.VertexIndices())
		{
			if (Graph.GetVtxEdgeCount(VID) > 0 || VID == AID || VID == BID) // if it's an existing edge or on the currently added edge, it's not floating so skip it
			{
				continue;
			}

			FVector2d V = Graph.GetVertex(VID);
			double T;
			double DSQ = SegAB.DistanceSquared(V, T);
			if (DSQ < Tol*Tol)
			{
				Hits.Add(FSegmentPoint{ T, VID });
				num_hits++;
			}
		}

		return num_hits > 0;
	}

private:
	// Full-featured implementation for all Triangulate function variants to call
	bool GEOMETRYALGORITHMS_API TriangulateInternal(TArray<FIndex3i>& Triangles, bool bHasBoundaryEdgeGroupID, int32 BoundaryEdgeGroupID, bool bHasHoleGroupID, int32 HoleEdgeGroupID, bool bLegacyKeepTrianglesIfBoundaryNotFound, TArray<int32>* SkippedEdges);
};



} // end namespace UE::Geometry
} // end namespace UE
