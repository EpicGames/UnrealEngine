// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "IndexTypes.h"
#include "Polygon2.h"
#include "Curve/DynamicGraph2.h"
#include "Curve/GeneralPolygon2.h"

#include "CoreMinimal.h"


namespace UE {
namespace Geometry {

using namespace UE::Math;

/**
 * This is a version of FDelaunay (which can also do Constrained Delaunay triangulation), but with added support for:
 *  1. Explicit "Hole" edges that must be cut out of the result (FDelaunay relies only on winding rules to create holes)
 *  2. Option to duplicate vertices to split any 'bowtie' vertices
 * If you do not need either of these features, FDelaunay may be the faster option.
 */
template<typename RealType>
struct TConstrainedDelaunay2
{
	//
	// inputs
	//
	TArray<TVector2<RealType>> Vertices;

	// Edges and HoleEdges must not be intersecting; use Arrangment2D to pre-process any input w/ intersecting edges
	TArray<FIndex2i> Edges;	// Edges can be boundaries or not based on the EFillRule setting
	TArray<FIndex2i> HoleEdges; // Any triangles inside 'hole' edges *must* be cut out

	//TODO: also support FeatureEdges?
	//TArray<FIndex2i> FeatureEdges; // Edges that should be preserved in mesh but do not correspond to a boundary and should not affect inside/outside classification at all

	bool bOrientedEdges = true;
	bool bOutputCCW = false;
	bool bSplitBowties = false;

	enum class EFillRule {
		Odd = 0,
		// bOrientedEdges must be true for the below
		NonZero,
		Positive,
		Negative
	};
	EFillRule FillRule = EFillRule::Odd;

	inline bool ClassifyFromRule(int Winding)
	{
		switch (FillRule)
		{
		case EFillRule::Odd:
			return Winding % 2 != 0;
		case EFillRule::NonZero:
			return Winding != 0;
		case EFillRule::Positive:
			return Winding > 0;
		case EFillRule::Negative:
			return Winding < 0;
		default:
			check(false);
			return false;
		}
	}

	template<class InputRealType>
	void Add(const FDynamicGraph2<InputRealType>& Graph)
	{
		int32 VertexStart = Vertices.Num();
		int32 GMaxVertID = Graph.MaxVertexID();
		TArray<int32> GraphToDTVertIdxMap; GraphToDTVertIdxMap.SetNum(GMaxVertID);
		for (int32 Idx = 0; Idx < GMaxVertID; Idx++)
		{
			if (Graph.IsVertex(Idx))
			{
				GraphToDTVertIdxMap[Idx] = Vertices.Num();
				Vertices.Add((TVector2<RealType>)Graph.GetVertex(Idx));
			}
			else
			{
				GraphToDTVertIdxMap[Idx] = -1;
			}
		}
		for (const FDynamicGraph::FEdge Edge : Graph.Edges())
		{
			Edges.Add(FIndex2i(GraphToDTVertIdxMap[Edge.A], GraphToDTVertIdxMap[Edge.B]));
		}
	}
	template<class InputRealType>
	void Add(const TPolygon2<InputRealType>& Polygon, bool bIsHole = false)
	{
		int32 VertexStart = Vertices.Num();
		int32 VertexEnd = VertexStart + Polygon.VertexCount();
		for (const TVector2<InputRealType> &Vertex : Polygon.GetVertices())
		{
			Vertices.Add((TVector2<RealType>)Vertex);
		}

		TArray<FIndex2i>* EdgeArr;
		if (bIsHole)
		{
			EdgeArr = &HoleEdges;
		}
		else
		{
			EdgeArr = &Edges;
		}
		for (int32 A = VertexEnd - 1, B = VertexStart; B < VertexEnd; A = B++)
		{
			EdgeArr->Add(FIndex2i(A, B));
		}
	}
	template<class InputRealType>
	void Add(const TGeneralPolygon2<InputRealType>& GPolygon)
	{
		Add(GPolygon.GetOuter(), false);
		const TArray<TPolygon2<InputRealType>>& Holes = GPolygon.GetHoles();
		for (int HoleIdx = 0, HolesNum = Holes.Num(); HoleIdx < HolesNum; HoleIdx++)
		{
			Add(Holes[HoleIdx], true);
		}
	}

	// Add Polygon with self-intersections resolved. Returns false if resolving self-intersections failed.
	bool GEOMETRYALGORITHMS_API AddWithIntersectionResolution(const TPolygon2<RealType>& Polygon);
	// Add General Polygon with self-intersections resolved. Returns false if resolving self-intersections failed.
	bool GEOMETRYALGORITHMS_API AddWithIntersectionResolution(const TGeneralPolygon2<RealType>& GeneralPolygon);

	//
	// outputs
	//
	TArray<FIndex3i> Triangles;

	/** If vertices were added to output (e.g. to split bowties), this is set to the index of the first added vertex */
	int AddedVerticesStartIndex = -1;

	/**
	 * Populate Triangles
	 *
	 * @return false if Triangulation failed
	 */
	bool GEOMETRYALGORITHMS_API Triangulate();

	/**
	 * Populate Triangles with override function to determine which triangles are in or out.
	 * Note that boundary edges and hole edges are treated as equivalent by this function; only the KeepTriangle function determines what triangles are excluded.
	 *
	 * @param KeepTriangle Function to check whether the given triangle should be kept in the output
	 * @return false if Triangulation failed
	 */
	bool GEOMETRYALGORITHMS_API Triangulate(TFunctionRef<bool(const TArray<TVector2<RealType>>&, const FIndex3i&)> KeepTriangle);
};

/**
 * Convenience function; invokes TConstrainedDelaunay2::Triangulate with most-often-used options
 * 
 * @param GeneralPolygon A general polygon, which may include holes.
 * @return An array of triangle indices.  Indices refer to a flat array w/ the outer polygon vertices first, and the hole vertices appended after.
 */
template<typename RealType>
TArray<FIndex3i> GEOMETRYALGORITHMS_API ConstrainedDelaunayTriangulate(const TGeneralPolygon2<RealType>& GeneralPolygon);

/**
 * Convenience function; invokes TConstrainedDelaunay2::Triangulate with most-often-used options.
 *
 * @param GeneralPolygon A general polygon, which may include holes.
 * @param OutVertices Output array of vertices used in the triangulation.
 * @return An array of triangle indices.
 */
template<typename RealType>
TArray<FIndex3i> GEOMETRYALGORITHMS_API ConstrainedDelaunayTriangulateWithVertices(const TGeneralPolygon2<RealType>& GeneralPolygon, TArray<TVector2<RealType>>& OutVertices);

typedef TConstrainedDelaunay2<float> FConstrainedDelaunay2f;
typedef TConstrainedDelaunay2<double> FConstrainedDelaunay2d;



} // end namespace UE::Geometry
} // end namespace UE