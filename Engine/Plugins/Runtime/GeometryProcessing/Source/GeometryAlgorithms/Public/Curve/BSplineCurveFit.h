// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::Geometry
{
using namespace UE::Math;

/**
*  Compute the control points that result from a best fit of the provided 2d data points when using b-splines of the specified degree.
* 
*  This fit assumes the sample points are evenly spaced in 't' over the interval [0, 1].
*        and the knot vector has unique knots at  j / (NumControlPoints - SplineDegree) for j in [0, (NumControlPoints - Degree)].
*        the knots at 0 and 1 each have multiplicity SplineDegree + 1, while the internal knots just have multiplicity 1.
*
*  @param DataPoints       - an array of 2d points the b-spline will attempt to fit
*  @param SplineDegree     - the degree of the underling b-spline.  Must be greater that 0.
*  @param NumControlPoints - requested number of control points solved for.  
*                            Note: requires SplineDegree < NumControlPoints <= DataPoints.Num()
*  @param ControlPointsOut - on successful return, an array of 2d control points of the specified length ( on failure the array is unchanged ).
*
*  @return true on success. 
* 
*/
bool GEOMETRYALGORITHMS_API BSplineCurveFit(const TArray<FVector2f>& DataPoints,  const int32 SplineDegree, const int32 NumControlPoints, TArray<FVector2f>& ControlPointsOut);


/**
*  Compute the control points that result from a best fit of the provided 3d data points when using b-splines of the specified degree.
* 
*  This fit assumes the sample points are evenly spaced in 't' over the interval [0, 1].
*        and the knot vector has unique knots at  j / (NumControlPoints - SplineDegree) for j in [0, (NumControlPoints - Degree)].
*        the knots at 0 and 1 each have multiplicity SplineDegree + 1, while the internal knots just have multiplicity 1.
*
*  @param DataPoints       - an array of 3d points the b-spline will attempt to fit
*  @param SplineDegree     - the degree of the underling b-spline.  Must be greater that 0.
*  @param NumControlPoints - requested number of control points solved for.
*                            Note: requires SplineDegree < NumControlPoints <= DataPoints.Num()
*  @param ControlPointsOut - on successful return, an array of 3d control points of the specified length  ( on failure the array is unchanged ).
*
*  @return true on success.
* 
*/
bool GEOMETRYALGORITHMS_API BSplineCurveFit(const TArray<FVector3f>& DataPoints, const int32 SplineDegree, const int32 NumControlPoints, TArray<FVector3f>& ControlPointsOut);

} // end namespace