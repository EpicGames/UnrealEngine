// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curve/GeneralPolygon2.h"

namespace UE::Geometry
{
	enum class EPolygonBooleanOp : uint8
	{
		Union,
		Difference,
		Intersect,
		ExclusiveOr
	};

	/** Produces the boolean'd result of PolygonA and PolygonB depending on the OperationType. Both polygons must be closed. */
	template <EPolygonBooleanOp OperationType, typename GeometryType, typename RealType>
	class TBooleanPolygon2Polygon2
	{
	public:
		// Input
		GeometryType PolygonA;
		GeometryType PolygonB;

		// Output
		TArray<UE::Geometry::TGeneralPolygon2<RealType>> Result;

		TBooleanPolygon2Polygon2(
			const GeometryType& InPolygonA,
			const GeometryType& InPolygonB);

		bool ComputeResult();
	};
 
	template <EPolygonBooleanOp OperationType>
	using TBooleanGeneralPolygon2GeneralPolygon2f = TBooleanPolygon2Polygon2<OperationType, TGeneralPolygon2<float>, float>;

	template <EPolygonBooleanOp OperationType>
	using TBooleanGeneralPolygon2GeneralPolygon2d = TBooleanPolygon2Polygon2<OperationType, TGeneralPolygon2<double>, double>;

	/** Produces the combined result of PolygonA and PolygonB. Both must be closed. */
	template <typename GeometryType, typename RealType>
	using TUnionPolygon2Polygon2 = TBooleanPolygon2Polygon2<EPolygonBooleanOp::Union, GeometryType, RealType>;

	typedef TUnionPolygon2Polygon2<TGeneralPolygon2<float>, float> FUnionGeneralPolygon2GeneralPolygon2f;
	typedef TUnionPolygon2Polygon2<TGeneralPolygon2<double>, double> FUnionGeneralPolygon2GeneralPolygon2d;
	
	/** Produces the result of PolygonA with the shape of PolygonB removed. Both must be closed. */
	template <typename GeometryType, typename RealType>
	using TDifferencePolygon2Polygon2 = TBooleanPolygon2Polygon2<EPolygonBooleanOp::Difference, GeometryType, RealType>;

	typedef TDifferencePolygon2Polygon2<TGeneralPolygon2<float>, float> FDifferenceGeneralPolygon2GeneralPolygon2f;
	typedef TDifferencePolygon2Polygon2<TGeneralPolygon2<double>, double> FDifferenceGeneralPolygon2GeneralPolygon2d;
	
	/** Produces the areas of each polygon that are also present in the other as a combined result. Both must be closed. */
	template <typename GeometryType, typename RealType>
	using TIntersectPolygon2Polygon2 = TBooleanPolygon2Polygon2<EPolygonBooleanOp::Intersect, GeometryType, RealType>;
	
	typedef TIntersectPolygon2Polygon2<TGeneralPolygon2<float>, float> FIntersectGeneralPolygon2GeneralPolygon2f;
	typedef TIntersectPolygon2Polygon2<TGeneralPolygon2<double>, double> FIntersectGeneralPolygon2GeneralPolygon2d;

	/** Produces the areas of each polygon not present in the other as a combined result. Both must be closed. */
	template <typename GeometryType, typename RealType>
	using TExclusiveOrPolygon2Polygon2 = TBooleanPolygon2Polygon2<EPolygonBooleanOp::ExclusiveOr, GeometryType, RealType>;

	typedef TExclusiveOrPolygon2Polygon2<TGeneralPolygon2<float>, float> FExclusiveOrGeneralPolygon2GeneralPolygon2f;
	typedef TExclusiveOrPolygon2Polygon2<TGeneralPolygon2<double>, double> FExclusiveOrGeneralPolygon2GeneralPolygon2d;

	// Array-based interfaces to support cases where we have different numbers of polygons to operate on
	
	/// Populate ResultOut with the union of the input Polygons
	/// @param bCopyInputOnFailure		If true, copy the input Polygons to ResultOut if the union is not successfully computed. Otherwise, ResultOut is left empty on failure.
	GEOMETRYALGORITHMS_API bool PolygonsUnion(TArrayView<const FGeneralPolygon2d> Polygons, TArray<FGeneralPolygon2d>& ResultOut, bool bCopyInputOnFailure);

	/// Populate ResultOut with the difference of PosPolygons minus NegPolygons
	GEOMETRYALGORITHMS_API bool PolygonsDifference(TArrayView<const FGeneralPolygon2d> PosPolygons, TArrayView<const FGeneralPolygon2d> NegPolygons, TArray<FGeneralPolygon2d>& ResultOut);

	/// Populate ResultOut with the intersection of SubjPolygons with ClipPolygons
	GEOMETRYALGORITHMS_API bool PolygonsIntersection(TArrayView<const FGeneralPolygon2d> SubjPolygons, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<FGeneralPolygon2d>& ResultOut);

	/// Populate ResultOut with the 'exclusive or' of SubjPolygons with ClipPolygons
	GEOMETRYALGORITHMS_API bool PolygonsExclusiveOr(TArrayView<const FGeneralPolygon2d> SubjPolygons, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<FGeneralPolygon2d>& ResultOut);

	/// Populate PathsOut with the intersected SubjPaths with ClipPolygons
	GEOMETRYALGORITHMS_API bool PathsIntersection(const TArray<TArrayView<FVector2d>>& Paths, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<TArray<FVector2d>>& PathsOut);

	/// Populate PathsOut with the clipped SubjPaths with ClipPolygons
	GEOMETRYALGORITHMS_API bool PathsDifference(const TArray<TArrayView<FVector2d>>& Paths, TArrayView<const FGeneralPolygon2d> ClipPolygons, TArray<TArray<FVector2d>>& PathsOut);
}
