// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curve/GeneralPolygon2.h"

namespace UE::Geometry
{
	enum class EPolygonOffsetJoinType : uint8
	{
		Square,		/* Uniform squaring on all convex edge joins. */
		Round,		/* Arcs on all convex edge joins. */
		Miter,		/* Squaring of convex edge joins with acute angles ("spikes"). Use in combination with MiterLimit. */
	};

	enum class EPolygonOffsetEndType : uint8
	{
		Polygon,	/* Offsets only one side of a closed path */
		Joined,		/* Offsets both sides of a path, with joined ends */
		Butt,		/* Offsets both sides of a path, with square blunt ends */
		Square,		/* Offsets both sides of a path, with square extended ends */
		Round,		/* Offsets both sides of a path, with round extended ends */
	};
	
	/** Offsets a given polygon (a polyline or closed polygon), determined by EndType by a  given amount.  */
	template <typename GeometryType, typename RealType>
	class TOffsetPolygon2
	{
	public:
		// Input
		TArray<GeometryType> Polygons;
		RealType Offset = 1.0;
		RealType MiterLimit = 2.0; // Minimum value is clamped to 2.0
		double MaxStepsPerRadian = -1; // If > 0 and JoinType is Round, limit the maximum steps per radian to this value
		double DefaultStepsPerRadianScale = 1.0; // If JoinType is Round, multiply the default-computed steps per radian by this amount.
		EPolygonOffsetJoinType JoinType = EPolygonOffsetJoinType::Square;
		EPolygonOffsetEndType EndType = EPolygonOffsetEndType::Polygon;

		// Output
		TArray<UE::Geometry::TGeneralPolygon2<RealType>> Result;

		explicit TOffsetPolygon2(const TArray<GeometryType>& InPolygons);
		TOffsetPolygon2() = default;

		bool ComputeResult();
	};

	typedef TOffsetPolygon2<TArrayView<TVector2<float>>, float> FOffsetPolygon2f;
	typedef TOffsetPolygon2<TArrayView<TVector2<double>>, double> FOffsetPolygon2d;


	// Offset an array of general polygons.
	// Note: Input polygons should not overlap; to process overlapping polygons, call PolygonsUnion first.
	// @param Offset		The amount to offset the polygons.
	// @param Polygons		The polygons to offset
	// @param ResultOut		On success, will be filled with the resulting polygons. Note: It is ok to pass the same array as both the input and output.
	// @param bCopyInputOnFailure Whether to copy the input polygons to the result on failure.
	// @param MiterLimit	If JoinType is Miter, this limits how far each miter join can extend
	// @param JoinType		How to join/extend corners between two edges
	// @param EndType		Should generally be Polygon. Otherwise, input is interpreted as a path, and offset is a 'thickening' applied to each side.
	// @param MaxStepsPerRadian				For Round JoinType, sets a maximum number of vertices to add per radian of curve. If <= 0, no maximum is used.
	// @param DefaultStepsPerRadianScale	For Round JoinType, scales the default number of vertices to add per radian of curve. The default number is extremely large, so a value near 1e-3 is typical here.
	GEOMETRYALGORITHMS_API bool PolygonsOffset(
		double Offset,
		TArrayView<const FGeneralPolygon2d> Polygons, TArray<FGeneralPolygon2d>& ResultOut,
		bool bCopyInputOnFailure,
		double MiterLimit,
		EPolygonOffsetJoinType JoinType = EPolygonOffsetJoinType::Square,
		EPolygonOffsetEndType EndType = EPolygonOffsetEndType::Polygon,
		double MaxStepsPerRadian = -1.0,
		double DefaultStepsPerRadianScale = 1.0);

	// Perform two offsets of the input polygons, intended to support "opening" or "closing" operations
	// Note: Input polygons should not overlap; to process overlapping polygons, call PolygonsUnion first.
	// @param FirstOffset	The initial amount to offset the polygons.
	// @param SecondOffset	The second amount to offset the polygons.
	// @param Polygons		The polygons to offset
	// @param ResultOut		On success, will be filled with the resulting polygons. Note: It is ok to pass the same array as both the input and output.
	// @param bCopyInputOnFailure Whether to copy the input polygons to the result on failure.
	// @param MiterLimit	If JoinType is Miter, this limits how far each miter join can extend
	// @param JoinType		How to join/extend corners between two edges
	// @param EndType		Should generally be Polygon. Otherwise, input is interpreted as a path, and offset is a 'thickening' applied to each side.
	// @param MaxStepsPerRadian				For Round JoinType, sets a maximum number of vertices to add per radian of circular join. If <= 0, no maximum is used.
	// @param DefaultStepsPerRadianScale	For Round JoinType, scales the default number of vertices to add per radian of curve. The default number is extremely large, so a value near 1e-3 is typical here.
	GEOMETRYALGORITHMS_API bool PolygonsOffsets(
		double FirstOffset,
		double SecondOffset,
		TArrayView<const FGeneralPolygon2d> Polygons, TArray<FGeneralPolygon2d>& ResultOut,
		bool bCopyInputOnFailure,
		double MiterLimit,
		EPolygonOffsetJoinType JoinType = EPolygonOffsetJoinType::Square,
		EPolygonOffsetEndType EndType = EPolygonOffsetEndType::Polygon,
		double MaxStepsPerRadian = -1.0,
		double DefaultStepsPerRadianScale = 1.0);
}
