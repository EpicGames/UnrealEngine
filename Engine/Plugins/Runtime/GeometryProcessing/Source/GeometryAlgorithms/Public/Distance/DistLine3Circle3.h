// Copyright Epic Games, Inc. All Rights Reserved.

// Port of geometry3Sharp DistLine3Circle3
// which was ported from WildMagic 5 

#pragma once

#include "VectorTypes.h"
#include "CircleTypes.h"
#include "LineTypes.h"

namespace UE {
namespace Geometry {

using namespace UE::Math;

/**
* Compute unsigned distance and closest-points between 3D line and 3D circle
*/
template <typename Real>
class TDistLine3Circle3
{
public:
	// Input
	TLine3<Real> Line;
	TCircle3<Real> Circle;

	// Output
	Real DistanceSquared = -1.0;

	int NumClosestPairs = 0;
	TVector<Real> LineClosest[2];
	TVector<Real> CircleClosest[2];
	bool bIsEquiDistant;

	TDistLine3Circle3(const TLine3<Real>& LineIn, const TCircle3<Real>& CircleIn) : Line(LineIn), Circle(CircleIn)
	{
	}

	Real Get()
	{
		return TMathUtil<Real>::Sqrt(ComputeResult());
	}
	Real GetSquared()
	{
		return ComputeResult();
	}

	Real ComputeResult();
};

typedef TDistLine3Circle3<float> FDistLine3Circle3f;
typedef TDistLine3Circle3<double> FDistLine3Circle3d;



} // end namespace UE::Geometry
} // end namespace UE
