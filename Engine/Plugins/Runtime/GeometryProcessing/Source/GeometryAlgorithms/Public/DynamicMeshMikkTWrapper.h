// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/MeshTangents.h"

namespace DynamicMeshMikkTWrapper
{	
	/** @return true if the current platform supports MikkT tangent calculations */
	bool GEOMETRYALGORITHMS_API IsSupported();

	/**
	 * Compute MikkT tangents for the given MeshTangents
	 * 
	 * @param MeshTangents Tangents to fill in. Must have a valid target mesh set.
	 * @param UVLayer The UV layer tangents should reference
	 * @return true if tangents were successfully computed (i.e. platform supported mikkt + requested mesh and UV layer were valid)
	 */
	bool GEOMETRYALGORITHMS_API ComputeTangents(UE::Geometry::FMeshTangentsd& MeshTangents, int32 UVLayer);
}

