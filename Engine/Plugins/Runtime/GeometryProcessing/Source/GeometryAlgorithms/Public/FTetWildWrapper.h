// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"

namespace UE::Geometry::FTetWild
{
	enum class EFilterOutsideMethod
	{
		None, // Do not remove outside tets from the result
		FloodFill, // Floodfill from tagged boundary faces of the active tet mesh
		InputSurface, // Use the winding number of the input surface mesh
		TrackedSurface, // Use the winding number of tagged boundary faces on the active tet mesh
		SmoothOpenBoundary // Smooth the non-input faces that separate inside from outside, before filtering
	};

	struct FTetMeshParameters
	{
		EFilterOutsideMethod OutsideFilterMethod = EFilterOutsideMethod::TrackedSurface;
		
		// Whether to coarsen the tet mesh result
		bool bCoarsen = false;
		// Whether to enforce that the output boundary surface should be manifold. (Only relevant if a final surface is extracted)
		bool bExtractManifoldBoundarySurface = false;
		// Whether to skip the initial simplification step
		bool bSkipSimplification = false;

		// Whether to apply a sizing field, interpolated over a background tet mesh, to control edge lengths
		bool bApplySizing = false;
		TArray<FVector> SizingFieldVertices;
		TArray<FIntVector4> SizingFieldTets;
		TArray<double> SizingFieldValues;

		// Ideal edge length relative to bounding box size
		double IdealEdgeLengthRel = 0.05;

		UE_DEPRECATED(5.6, "Use IdealEdgeLengthRel instead.")
		double IdealEdgeLength = 0.05;

		// Relative tolerance, controlling how closely the mesh must follow the input surface
		double EpsRel = 1e-3;

		// Maximum number of optimization iterations
		int32 MaxIts = 80;
		// Energy at which to stop optimizing tet quality and accept the result
		double StopEnergy = 10;

		// Whether to invert tets relative to what UE typically expects in the final output
		bool bInvertOutputTets = false;
	};


	/**
	 * Use the fTetWild algorithm to compute a tetrahedral mesh for the input surface mesh
	 * 
	 * @param Params		fTetWild algorithm parameters
	 * @param InVertices	Surface mesh vertices as a vertex buffer
	 * @param InFaces		Surface mesh faces as indices into InVertices
	 * @param OutVertices	Tet mesh vertices
	 * @param OutTets		Tetrahedra as indices into OutVertices
	 * @param Progress		Optional FProgressCancel object to support cancelling the computation and reporting progress
	 * @return true if algorithm succeeded, false if it failed or was cancelled
	 */
	bool GEOMETRYALGORITHMS_API ComputeTetMesh(
		const FTetMeshParameters& Params,
		const TArray<FVector>& InVertices,
		const TArray<FIntVector3>& InFaces,
		TArray<FVector>& OutVertices,
		TArray<FIntVector4>& OutTets,
		FProgressCancel* Progress = nullptr);

}

