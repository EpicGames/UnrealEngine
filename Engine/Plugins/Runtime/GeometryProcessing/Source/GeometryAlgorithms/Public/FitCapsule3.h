// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "CapsuleTypes.h"

namespace UE {
namespace Geometry {

using namespace UE::Math;

/**
 * Fit a Capsule to a set of 3D points.
 * Currently solved by using least-squares to fit a line to the point set.
 * This does not guarantee that the capsule is the minimal possible volume.
 */
template<typename RealType>
class TFitCapsule3
{
public:
	// Outputs calculated by Solve() function

	/** Set to true in Solve() on success */
	bool bResultValid = false;
	/** The capsule computed in Solve() */
	TCapsule3<RealType> Capsule;


	/**
	 * Solve variants
	 */
	
	 /**
	  * Calculate a Capsule that contains the given Point Set and store in Output variables
	  * @return true if capsule was found
	  */
	bool Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc);

	/**
	 * Calculate a Capsule that contains the given Point Set and return in CapsuleOut
	 * @return true if capsule was found
	 */
	static bool Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc,
		TCapsule3<RealType>& CapsuleOut)
	{
		TFitCapsule3<RealType> Compute;
		Compute.Solve(NumPoints, GetPointFunc);
		CapsuleOut = Compute.Capsule;
		return Compute.bResultValid;
	}

};

typedef TFitCapsule3<float> FFitCapsule3f;
typedef TFitCapsule3<double> FFitCapsule3d;


} // end namespace UE::Geometry
} // end namespace UE
