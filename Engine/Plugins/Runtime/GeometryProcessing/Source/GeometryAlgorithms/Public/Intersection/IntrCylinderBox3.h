// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"

namespace UE
{
namespace Geometry
{

/**
 * Compute intersection between a cylinder and a 3D axis-aligned box.
 */
template <typename RealType>
bool DoesCylinderIntersectBox(
	const TAxisAlignedBox3<RealType>& Box,
	const TVector<RealType>& CylinderCenter,
	const TVector<RealType>& CylinderDirection,
	RealType CylinderRadius,
	RealType CylinderHeight);

} // end namespace UE::Geometry
} // end namespace UE