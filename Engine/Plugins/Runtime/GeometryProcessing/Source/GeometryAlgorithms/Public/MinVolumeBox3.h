// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "OrientedBoxTypes.h"
#include "Templates/PimplPtr.h"

class FProgressCancel;

namespace UE {
namespace Geometry {

using namespace UE::Math;

template <typename RealType> struct TMinVolumeBox3Internal;

/**
 * Calculate a Minimal-Volume Oriented Box for a set of 3D points.
 * This internally first computes the Convex Hull of the point set. 
 * The minimal box is then guaranteed to be aligned with one of the faces of the convex hull.
 * Note that this is increasingly expensive as the Convex Hull face count increases.
 */
template<typename RealType>
class TMinVolumeBox3
{
public:
	/**
	 * Calculate the minimal box for the given point set.
	 * @param NumPoints number of points in the set, ie GetPointFunc can be called for any integer in range [0...NumPoints)
	 * @param GetPointFunc function that returns a 3D point for a valid Index
	 * @param bMostAccurateFit if true, use the most expensive method to get the best-possible fit 
	 * @param Progress optionally allow early-cancel of the box fit operation
	 * @return true if minimal box was found
	 */
	bool Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, bool bMostAccurateFit = false, FProgressCancel* Progress = nullptr);

	/**
	 * Calculate the minimal box for a Subsampling of MaxPoints points of a point set
	 * @param NumPoints number of points in the set, ie GetPointFunc can be called for any integer in range [0...NumPoints)
	 * @param NumSamplePoints maximum number of points to sample from NumPoints. Currently a random-ish subset is selected.
	 * @param GetPointFunc function that returns a 3D point for a valid Index
	 * @param bMostAccurateFit if true, use the most expensive method to get the best-possible fit
	 * @param Progress optionally allow early-cancel of the box fit operation
	 * @return true if minimal box was found
	 */
	bool SolveSubsample(int32 NumPoints, int32 NumSamplePoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, bool bMostAccurateFit = false, FProgressCancel* Progress = nullptr);


	/** @return true if minimal box is available */
	bool IsSolutionAvailable() const;

	/** @return minimal box in BoxOut */
	void GetResult(TOrientedBox3<RealType>& BoxOut);

protected:
	void Initialize(bool bMostAccurateFit);

	TPimplPtr<TMinVolumeBox3Internal<RealType>> Internal;
};

typedef TMinVolumeBox3<float> FMinVolumeBox3f;
typedef TMinVolumeBox3<double> FMinVolumeBox3d;


} // end namespace UE::Geometry
} // end namespace UE
