// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "SphereTypes.h"
#include "Templates/PimplPtr.h"

namespace UE {
namespace Geometry {

using namespace UE::Math;

template <typename RealType> struct TMinVolumeSphere3Internal;

/**
 * Calculate a Minimal-Volume Sphere for a set of 3D points.
 *
 * @warning Currently this processes input points in a randomized order so the results are not strictly deterministic. In particular if it fails you might try calling it again...
 */
template<typename RealType>
class TMinVolumeSphere3
{
public:
	/**
	 * Calculate the minimal sphere for the given point set.
	 * @param bUseExactComputation If true, high-precision Rational number types are used for the calculation, rather than doubles. This is slower but more precise.
	 * @return true if minimal sphere was found
	 */
	bool Solve(int32 NumPoints, TFunctionRef<TVector<RealType>(int32)> GetPointFunc, bool bUseExactComputation = false);

	/** @return true if minimal box is available */
	bool IsSolutionAvailable() const;

	/** @return minimal sphere in SphereOut */
	void GetResult(TSphere3<RealType>& SphereOut);

protected:
	void Initialize(int32 NumPoints, bool bUseExactComputation);

	TPimplPtr<TMinVolumeSphere3Internal<RealType>> Internal;
};

typedef TMinVolumeSphere3<float> FMinVolumeSphere3f;
typedef TMinVolumeSphere3<double> FMinVolumeSphere3d;

} // end namespace UE::Geometry
} // end namespace UE