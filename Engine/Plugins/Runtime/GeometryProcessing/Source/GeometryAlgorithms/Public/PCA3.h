// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "VectorTypes.h"

class FProgressCancel;

namespace UE::Geometry {

struct FComputePCA3Options
{
	// Whether to sort the eigenvalues in decreasing order, so the largest components are first
	bool bSortEigenvalues = true;
	// Whether to uniformly re-scale the data to fit in a unit cube before computing PCA. Note the scale factor applied will be stored in TPCA3::ScaleFactor.
	bool bScaleDataToUnitCube = true;
};

template<typename RealType>
class TPCA3
{
public:
	bool Compute(TConstArrayView<UE::Math::TVector<RealType>> Points, const FComputePCA3Options& Options = FComputePCA3Options());

	//
	// PCA results
	//

	UE::Math::TVector<RealType> Mean;
	TStaticArray<UE::Math::TVector<RealType>, 3> Eigenvectors;
	UE::Math::TVector<RealType> Eigenvalues;

	// Any scale factor applied to the input data will be stored here
	UE::Math::TVector<RealType> ScaleFactor = UE::Math::TVector<RealType>::One();
};

typedef TPCA3<float> FPCA3f;
typedef TPCA3<double> FPCA3d;

} // end namespace UE::Geometry
