// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#define UE_API GEOMETRYALGORITHMS_API

namespace UE {
namespace Geometry {

	class FDynamicMesh3;

	class FUVMetrics
	{
		public:
			static UE_API double ReedBeta(const UE::Geometry::FDynamicMesh3& Mesh, int32 UVChannel, int32 Tid);
			static UE_API double Sander(const UE::Geometry::FDynamicMesh3& Mesh, int32 UVChannel, int32 Tid, bool bUseL2);
			static UE_API double TexelDensity(const UE::Geometry::FDynamicMesh3& Mesh, int32 UVChannel, int32 Tid, int32 MapSize);		
	};
}
}

#undef UE_API
