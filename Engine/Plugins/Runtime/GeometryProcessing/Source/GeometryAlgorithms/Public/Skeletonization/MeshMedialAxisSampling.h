// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MathUtil.h"
#include "VectorTypes.h"
#include "FrameTypes.h"
#include "Util/ProgressCancel.h"

#include "Async/ParallelFor.h"
#include "Distance/DistPoint3Triangle3.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Spatial/FastWinding.h"
#include "Spatial/MeshAABBTree3.h"


class FProgressCancel;

namespace UE::Geometry
{

namespace MedialAxis
{
	// Algorithm adapted from "3D medial axis point approximation using nearest neighbors and the normal field" by Ma et al.
	// Starting from a surface point + normal, aims to find a "medial axis ball" touching that point and one other point
	// w/ center along Point,-Normal ray, via iteratively shrinking an estimated radius to touch the surface point + a nearest point
	// 
	// Note that the returned sphere may not touch the input surface point, esp. in corner cases where the actual medial ball would have zero radius.
	struct FShrinkMedialBall
	{
		// Max iterations of shrinking to apply
		int32 MaxIters = 20;
		// Threshold to stop iterating, if radius or center changes less than this
		double MinDelta = UE_DOUBLE_KINDA_SMALL_NUMBER;
		// Stop iterating below this radius (if > 0). Returned radius may still be smaller than this.
		double RadiusThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER;

		// Method to 'hot start' the search via local vertex neighborhood, using a provided neighborhood
		template<typename MeshType>
		bool FindLocalMinCurvatureRadius(const MeshType& Mesh, FVector3d& SurfacePoint, FVector3d& SurfaceNormal, TConstArrayView<int32> NbrVIDs, double& OutFoundRadius)
		{
			bool bFound = false;
			for (int32 NbrVID : NbrVIDs)
			{
				bFound |= ShrinkFromLocalVertex(SurfacePoint, SurfaceNormal, Mesh.GetVertex(NbrVID), OutFoundRadius);
			}
			return bFound;
		}
		// Method to 'hot start' the search via local vertex neighborhood, using the neighborhood information in the dynamic mesh
		bool FindLocalMinCurvatureRadius(const FDynamicMesh3& Mesh, FVector3d& SurfacePoint, FVector3d& SurfaceNormal, int32 SurfaceVID, double& OutFoundRadius)
		{
			bool bFound = false;
			for (int32 NbrVID : Mesh.VtxVerticesItr(SurfaceVID))
			{
				bFound |= ShrinkFromLocalVertex(SurfacePoint, SurfaceNormal, Mesh.GetVertex(NbrVID), OutFoundRadius);
			}
			return bFound;
		}


		/**
		 * Shrink a sphere touching a given SurfacePoint, with center along the line through its SurfaceNormal
		 * until the sphere approximately touches two closest surface points including the input point
		 * Note: Sphere will often not actually touch the input surface point due to discretization error, since we set the radius s.t. the sphere remains inside the input surface
		 * 
		 * @param MeshBVH BVH of the surface
		 * @param SurfacePoint Reference point that the medial ball should approximately touch
		 * @param SurfaceNormal Normal at the SurfacePoint; note for vertices/edges this should be a smooth normal
		 * @param bRadiusHotStart If true, the InitialRadius should be a more accurate initial guess e.g. provided via one of the FindLocalMinCurvatureRadius methods
		 * @param InitialRadius If non-negative, initial radius to use for the sphere-shrinking search. Otherwise, will be automatically set from the surface bounding box's max dimension
		 * @param EarlyStopAtTIDFn Use when starting from a vertex/edge to stop in the discrete neighborhood of that source vertex/edge. Passed the current closest triangle ID.
		 */
		template<typename MeshType>
		FSphere Find(const TMeshAABBTree3<MeshType>& MeshBVH, const FVector3d& SurfacePoint, const FVector3d& SurfaceNormal, bool bRadiusHotStart,
			double InitialRadius, TFunctionRef<bool(int32)> EarlyStopAtTIDFn)
		{
			const MeshType& Mesh = *MeshBVH.GetMesh();

			if (SurfaceNormal.IsZero())
			{
				return FSphere(SurfacePoint, 0.);
			}

			if (InitialRadius < 0)
			{
				InitialRadius = MeshBVH.GetBoundingBox().MaxDim();
			}

			double CurRadius = InitialRadius;
			FVector3d CurCenter = SurfacePoint - SurfaceNormal * CurRadius;
			double SafeDist = CurRadius;
			for (int32 Iter = (int32)bRadiusHotStart; /*note: we break at MaxIters only after updating SafeDist*/; ++Iter)
			{
				IMeshSpatial::FQueryOptions Options;
				Options.MaxDistance = CurRadius;
				double NearDistSq;
				int32 NearTriID = MeshBVH.FindNearestTriangle(CurCenter, NearDistSq, Options);
				// No nearest point found; either mesh was empty or last iteration converged & nothing else was within MaxDistance of search
				if (NearTriID == INDEX_NONE)
				{
					SafeDist = CurRadius;
					break;
				}
				SafeDist = FMath::Sqrt(NearDistSq);
				
				// Note: All the stopping conditions are tested after updating SafeDist so that if we stop, the returned sphere 
				// (which uses SafeDist for radius) doesn't extend past the closest point
				
				// Radius below stopping threshold
				// Note: it's important that we test vs CurRadius and not SafeDist, because SafeDist can be spuriously small 
				// when the CurRadius is still large (as the sphere center moves past another surface)
				if (CurRadius <= RadiusThreshold)
				{
					break;
				}

				if (Iter >= MaxIters)
				{
					break;
				}

				if (EarlyStopAtTIDFn(NearTriID))
				{
					break;
				}

				FDistPoint3Triangle3d Query = TMeshQueries<MeshType>::TriangleDistance(Mesh, NearTriID, CurCenter);
				FVector3d ClosePoint = Query.ClosestTrianglePoint;

				FVector3d SmC = (SurfacePoint - ClosePoint);
				double SampleDistsSq = SmC.SquaredLength();
				// If closest point is (effectively) the source point, we can't shrink further
				if (SampleDistsSq < FMathd::ZeroTolerance)
				{
					break;
				}
				double SmCdN = SmC.Dot(SurfaceNormal);
				// closest point effectively co-planar, likely converged + would have unstable radius
				if (SmCdN < FMathd::ZeroTolerance)
				{
					break;
				}
				double OldRadius = CurRadius;
				CurRadius = SampleDistsSq / (2 * SmCdN);
				// radius converged (note OldRadius should always be larger than CurRadius by construction)
				if (OldRadius - CurRadius < MinDelta)
				{
					break;
				}
				CurCenter = SurfacePoint - SurfaceNormal * CurRadius;
			}

			// Note we use SafeDist instead of CurRadius to push the sphere inside
			// the target surface, especially in cases where the algorithm terminates early ...
			// On converged cases, this should be approximately the same as using CurRadius.
			return FSphere(CurCenter, SafeDist);
		}

		template<typename MeshType>
		FSphere FindWithImplicitWeld(const TMeshAABBTree3<MeshType>& MeshBVH, int32 WeldedSurfacePointVID, TFunctionRef<int32(int32)> VIDtoWeldVIDFn, 
			const FVector3d& SurfacePoint, const FVector3d& SurfaceNormal, bool bRadiusHotStart,
			double InitialRadius)
		{
			return Find(MeshBVH, SurfacePoint, SurfaceNormal, bRadiusHotStart, InitialRadius, [&MeshBVH, &VIDtoWeldVIDFn, WeldedSurfacePointVID](int32 TID)
			{
				FIndex3i Tri = MeshBVH.GetMesh()->GetTriangle(TID);
				Tri.A = VIDtoWeldVIDFn(Tri.A);
				Tri.B = VIDtoWeldVIDFn(Tri.B);
				Tri.C = VIDtoWeldVIDFn(Tri.C);
				return Tri.Contains(WeldedSurfacePointVID);
			});
		}

		/**
		 * Project an existing sphere to a surface-contained sphere, searching along the line through the initial sphere center and corresponding closest surface point
		 * @param MeshBVH BVH of the surface
		 * @param InOutToProject Sphere to project
		 * @param InitialSearchRadius If non-negative, initial radius to use for the sphere-shrinking search. Otherwise, will be automatically set from the surface bounding box's max dimension
		 * @param MeshFWTree If non-null, test the fast winding of the center before projecting, and reverse the search direction if it's outside. Otherwise, a (less robust) normal test will be used instead.
		 * @param VIDtoWeldVIDFn If provided, early-stop the search when it hits a triangle sharing a welded vertex with the starting triangle. Can help prevent over-shrinking from discretization.
		 * @return true if the sphere was updated via projection, false otherwise (e.g. if the BVH was empty, or if we couldn't get a valid search normal)
		 */
		template<typename MeshType>
		bool Project(const TMeshAABBTree3<MeshType>& MeshBVH,
			FSphere& InOutToProject,
			double InitialSearchRadius = -1,
			const TFastWindingTree<MeshType>* MeshFWTree = nullptr,
			TFunctionRef<int32(int32)>* VIDtoWeldVIDFn = nullptr
		)
		{
			double InitialCenterDistSq;
			int32 NearTriID = MeshBVH.FindNearestTriangle(InOutToProject.Center, InitialCenterDistSq);
			if (NearTriID == INDEX_NONE)
			{
				return false;
			}

			const MeshType& Mesh = *MeshBVH.GetMesh();
			FTriangle3d Tri;
			Mesh.GetTriVertices(NearTriID, Tri.V[0], Tri.V[1], Tri.V[2]);
			FDistPoint3Triangle3d Query(InOutToProject.Center, Tri);
			Query.GetSquared();
			FVector3d ClosePoint = Query.ClosestTrianglePoint;
			FVector3d Normal = Tri.Normal();
			FVector3d RadiusSearchDir = ClosePoint - InOutToProject.Center;
			if (!RadiusSearchDir.Normalize())
			{
				// the center was almost exactly on the surface; try falling back to the surface normal
				// TODO: if we're on a vertex or edge of the triangle, would ideally use the smooth normal
				RadiusSearchDir = Normal;
				if (Normal.IsZero())
				{
					return false;
				}
			}
			// if the center is outside the surface, we want to flip the search normal; optionally use the fast winding for this. 
			// If no fast winding available, test the closest normal (but this can fail near sharp features!)
			else if (((MeshFWTree && !MeshFWTree->IsInside(InOutToProject.Center)) || (!MeshFWTree && Normal.Dot(RadiusSearchDir) < 0.)))
			{
				RadiusSearchDir *= -1.;
			}

			// If VIDtoWeldVIDFn is provided, early-stop when Find hits a triangle sharing a welded vert (similar to FindWithImplicitWeld)
			if (VIDtoWeldVIDFn)
			{
				FIndex3i StartVerts = Mesh.GetTriangle(NearTriID);
				FIndex3i WeldedStart((*VIDtoWeldVIDFn)(StartVerts.A), (*VIDtoWeldVIDFn)(StartVerts.B), (*VIDtoWeldVIDFn)(StartVerts.C));
				InOutToProject = Find(MeshBVH, ClosePoint, RadiusSearchDir, false, InitialSearchRadius,
					[&Mesh, VIDtoWeldVIDFn, WeldedStart](int32 TID)
					{
						FIndex3i T = Mesh.GetTriangle(TID);
						T.A = (*VIDtoWeldVIDFn)(T.A);
						T.B = (*VIDtoWeldVIDFn)(T.B);
						T.C = (*VIDtoWeldVIDFn)(T.C);
						return WeldedStart.Contains(T.A) || WeldedStart.Contains(T.B) || WeldedStart.Contains(T.C);
					});
			}
			else
			{
				InOutToProject = Find(MeshBVH, ClosePoint, RadiusSearchDir, false, InitialSearchRadius, [](int32 TID) {return false;});
			}
			return true;
		}

	private:
		// Update curvature estimate from local vertex in neighborhood of SurfacePoint
		bool ShrinkFromLocalVertex(const FVector3d& SurfacePoint, const FVector3d& SurfaceNormal, const FVector3d& LocalVertex, double& InOutRadius)
		{
			FVector3d LocalToSurface = SurfacePoint - LocalVertex;
			double LocalDistSq = LocalToSurface.SquaredLength();
			// ignore near-duplicate neighbors
			if (LocalDistSq < FMathd::ZeroTolerance)
			{
				return false;
			}
			double LocalDistCosTheta = LocalToSurface.Dot(SurfaceNormal);
			if (LocalDistCosTheta <= FMathd::ZeroTolerance)
			{
				return false;
			}

			InOutRadius = FMath::Min(InOutRadius, LocalDistSq / (2 * LocalDistCosTheta));
			return true;
		}
	};

	/**
	 * Simple representation of a surface's medial skeleton, as a set of spheres + connectivity on those spheres, and a mapping to the surface vertices
	 */
	struct FMedialSkeleton
	{
		// Medial spheres, 1 per cluster, indexed by cluster index
		TArray<FSphere> Spheres;
		// Mapping from source mesh vertex IDs to cluster Index
		TArray<int32> VIDtoClusterIndex;

		// Inline-allocated array used for representing the typically-small neighborhoods in the cluster connectivity graph
		using FNbrArray = TArray<int32, TInlineAllocator<16>>;
		// Indices of neighbors of clusters, for each cluster
		TArray<FNbrArray> ClusterNeighbors;
		// Triangles of the medial skeleton, where the skeleton is a surface rather than a chain. Note: Not oriented, and generally not manifold -- so cannot be directly appended to an FDynamicMesh3
		TArray<FIndex3i> ClusterTriangles;

		// Check that the skeleton data structures are internally consistent
		// Note: Relatively expensive, intended for debugging / testing
		GEOMETRYALGORITHMS_API bool CheckValidity(EValidityCheckFailMode FailMode = EValidityCheckFailMode::Ensure) const;

		template<typename TriangleMeshType>
		bool IsCompatibleWithMesh(const TriangleMeshType& Mesh) const
		{
			if (Mesh.MaxVertexID() != VIDtoClusterIndex.Num())
			{
				return false;
			}
			for (int32 Idx = 0; Idx < VIDtoClusterIndex.Num(); ++Idx)
			{
				if (VIDtoClusterIndex[Idx] != INDEX_NONE && !Mesh.IsVertex(Idx))
				{
					return false;
				}
			}
			return true;
		}


		// Compute the ClusterTriangles array for the current FMedialSkeleton, given a Mesh and assuming VIDtoClusterIndex is already computed
		template<typename TriangleMeshType>
		void ComputeClusterTriangles(const TriangleMeshType& Mesh, TFunctionRef<int32(int32)> VIDtoWeldVIDFn)
		{
			TSet<FIndex3i> Tris;
			for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
			{
				if (!Mesh.IsTriangle(TID))
				{
					continue;
				}

				FIndex3i Tri = Mesh.GetTriangle(TID);
				Tri.A = VIDtoClusterIndex[VIDtoWeldVIDFn(Tri.A)];
				Tri.B = VIDtoClusterIndex[VIDtoWeldVIDFn(Tri.B)];
				Tri.C = VIDtoClusterIndex[VIDtoWeldVIDFn(Tri.C)];
				// Only consider triangles that span three different, valid clusters
				if (Tri.Contains(INDEX_NONE) || (Tri.A == Tri.B || Tri.B == Tri.C || Tri.A == Tri.C))
				{
					continue;
				}
				// Since the cluster tris are not oriented, sort the triangle indices to get a unique handle
				Tri.Sort();
				Tris.Add(Tri);
			}
			ClusterTriangles = Tris.Array();
		}

		// Compute the ClusterNeighbors array for the current FMedialSkeleton, given a Mesh and assuming Spheres and VIDtoClusterIndex are already computed
		template<typename TriangleMeshType>
		void ComputeClusterNeighbors(const TriangleMeshType& Mesh, TFunctionRef<int32(int32)> VIDtoWeldVIDFn)
		{
			ClusterNeighbors.SetNum(Spheres.Num());
			for (FMedialSkeleton::FNbrArray& Nbrs : ClusterNeighbors)
			{
				Nbrs.Reset();
			}
			for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
			{
				if (!Mesh.IsTriangle(TID))
				{
					continue;
				}

				FIndex3i Tri = Mesh.GetTriangle(TID);
				for (int32 Prev = 2, Idx = 0; Idx < 3; Prev = Idx++)
				{
					int32 ClusterA = VIDtoClusterIndex[VIDtoWeldVIDFn(Tri[Idx])];
					int32 ClusterB = VIDtoClusterIndex[VIDtoWeldVIDFn(Tri[Prev])];
					if (ClusterA != INDEX_NONE && ClusterB != INDEX_NONE && ClusterA != ClusterB)
					{
						// note: AddUnique to arrays b/c we expect neighbor arrays to be very small
						ClusterNeighbors[ClusterA].AddUnique(ClusterB);
						ClusterNeighbors[ClusterB].AddUnique(ClusterA);
					}
				}
			}
		}
	};



	struct FMedialSkeletonToTreeSkeletonOptions
	{
		// Method to select which edges of the medial skeleton are prioritized for inclusion in the hierarchical skeleton
		enum class EEdgeWeightMethod
		{
			// Favor adding shorter edges first
			EdgeLength,
			// Favor edges between clusters that are earlier in the medial skeleton array. Because medial spheres are added incrementally in the highest error locations, their order can approximate importance.
			ArrayOrder,
			// Favor adding edges between larger medial spheres first
			AvgRadius
		};

		// Method to merge disconnected components of the medial skeleton into a single hierarchy 
		enum class EMergeDisconnectedMethod
		{
			// Connect closest bones across disconnected components until there is a single hierarchy
			ConnectClosestBones,
			// Add a top-level root node, and connect the selected root and all disconnected components to this node
			AddTopLevelRoot
		};

		// Method to automatically select the medial cluster to use as animation skeleton root, if a root cluster index is not specified
		enum class ESelectRootMethod
		{
			ClosestToPoint,
			FarthestInDirection,
			ClosestToBoundsCenter,
			LargestSphere,
			ArrayOrder
		};

		EEdgeWeightMethod EdgeWeightMethod = EEdgeWeightMethod::ArrayOrder;
		EMergeDisconnectedMethod MergeDisconnectedMethod = EMergeDisconnectedMethod::ConnectClosestBones;
		ESelectRootMethod SelectRootMethod = ESelectRootMethod::ArrayOrder;

		// When SelectRootMethod is ClosestToPoint, the point to use
		FVector3d RootSelectionPoint = FVector3d::ZeroVector;
		// When SelectRootMethod is FarthestInDirection, the direction to use
		FVector3d RootSelectionDirection = -FVector3d::UnitZ();

		// If set, always add a top-level root bone at this position (not corresponding to a medial skeleton cluster)
		// Note: When MergeDisconnectedMethod==AddTopLevelRoot, this will also be used as that top level root bone.
		TOptional<FVector3d> CustomRootPosition;

		/**
		 * Remap a medial skeleton to a hierarchical skeleton suitable for using in a URefSkeleton for animation
		 * Note the indices of the hierarchical skeleton bones will be remapped from the medial skeleton, because URefSkeleton requires that parent bone indices be smaller than child bone indices
		 * 
		 * @param InMedialSkeleton Input medial skeleton
		 * @param AvoidIntersectingMeshBVH Surface that the output skeleton should prefer not to cross. (Surface-intersecting edges will still be added as-needed, if the non-intersecting edges are not sufficient to create a spanning tree.)
		 * @param OutParent Output hierarchy, as parent indices
		 * @param OutTransform Relative transforms for each bone. 1:1 with OutParent.
		 * @param Options Additional options to control the skeleton conversion
		 * @param RootIndex Specify the medial skeleton cluster to prefer as the root bone. If INDEX_NONE, a default root will be selected automatically.
		 * @param OutBoneIndexToMedialIndex Optional mapping from hierarchical skeleton bone indices to source medial skeleton cluster indices
		 * @param OutMedialIndexToBoneIndex Optional mapping from source medial skeleton cluster indices to hierarchical skeleton bone indices
		 */
		template<typename MeshType>
		GEOMETRYALGORITHMS_API static void ToHierarchy(const FMedialSkeleton& InMedialSkeleton, const TMeshAABBTree3<MeshType>* AvoidIntersectingMeshBVH, TArray<int32>& OutParent, TArray<FTransform>& OutTransform, FMedialSkeletonToTreeSkeletonOptions Options, int32 RootIndex = INDEX_NONE,
			TArray<int32>* OutBoneIndexToMedialIndex = nullptr, TArray<int32>* OutMedialIndexToBoneIndex = nullptr);

		// Same as above ToHierarchy method, but without the AvoidIntersectingMeshBVH parameter
		static void ToHierarchy(const FMedialSkeleton& InMedialSkeleton, TArray<int32>& OutParent, TArray<FTransform>& OutTransform, FMedialSkeletonToTreeSkeletonOptions Options, int32 RootIndex = INDEX_NONE,
			TArray<int32>* OutBoneIndexToMedialIndex = nullptr, TArray<int32>* OutMedialIndexToBoneIndex = nullptr)
		{
			return ToHierarchy<FDynamicMesh3>(InMedialSkeleton, nullptr, OutParent, OutTransform, Options, RootIndex, OutBoneIndexToMedialIndex, OutMedialIndexToBoneIndex);
		}
	};

	struct FSkeletonSubdivider
	{
		// Function to compute a new target edge length from the medial spheres on the current edge
		TFunction<double(const FSphere& A, const FSphere& B)> GetTargetEdgeLength;

		// If true, allow subdividing edges that have associated triangles (also splitting these triangles)
		bool bSubdivideOnSurfaces = false;

		// When re-clustering for subdivided edges, weight to use for the position error term. Equivalent in meaning to PosErrorWt in FSkeletonViaSampling.
		double ReassignClusterPosErrorWt = .2;

		// If true, project newly-added spheres to the medial axis. Otherwise, new spheres will linearly interpolate their source edge spheres.
		bool bReprojectMedialSpheres = false;


		template<typename MeshType>
		GEOMETRYALGORITHMS_API void Subdivide(FMedialSkeleton& Skeleton, const TMeshAABBTree3<MeshType>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn, const TFastWindingTree<MeshType>* MeshFWTree = nullptr);

		template<typename MeshType>
		GEOMETRYALGORITHMS_API void Subdivide(FMedialSkeleton& Skeleton, const TMeshAABBTree3<MeshType>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter = [](int32)->bool { return false; }, double ImplicitVertexNormalWeldThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER, const TFastWindingTree<MeshType>* MeshFWTree = nullptr);

		void Subdivide(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, double ImplicitVertexNormalWeldThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER, const TFastWindingTree<FDynamicMesh3>* MeshFWTree = nullptr)
		{
			const FDynamicMesh3& Mesh = *MeshBVH.GetMesh();
			Subdivide(Skeleton, MeshBVH, [&Mesh](int32 VID)->bool {return Mesh.IsBoundaryVertex(VID);}, ImplicitVertexNormalWeldThreshold, MeshFWTree);
		}

		// Subdivide a skeleton without referencing an associated mesh. Will clear the medial skeleton's VIDtoClusterIndex map, and will ignore bReprojectMedialSpheres
		GEOMETRYALGORITHMS_API void Subdivide(FMedialSkeleton& Skeleton);
	};

	struct FSkeletonSimplifier
	{

		/// Common settings

		// Stop simplifying at this sphere count
		int32 MinSpheres = 1;

		// If positive, will not collapse edges if it would introduce more error than this threshold. Note: Error value is defined by the internal accumulated quadric metric, *not* a straightforward distance.
		double QEMErrorThreshold = -1;

		// If positive, the maximum edge length to consider for collapse
		double EdgeLengthThreshold = -1;
		
		// If positive, the maximum medial sphere radius to consider for collapse (will not collapse if both cluster's medial sphere radii on an edge are larger than this)
		double SphereRadiusThreshold = -1;

		// If positive, will only collapse edges with medial spheres whose overlap depth is at least this fraction of the smaller sphere's diameter --
		//  so at 1.0 we will only simplify an edge if the smaller sphere is fully contained inside the larger
		// (A degenerate zero-radius sphere is considered to fully overlap any sphere it touches.)
		double SphereOverlapThreshold = -1;

		// If true, only simplify edges that have triangles associated
		bool bOnlySimplifySurfaces = false;

		// How much the simplifier should attempt to remain close to the initial skeleton, vs attempting to keep medial spheres close to the mesh surface.
		// Relative term, should be in the [0,1] range. At 0, the original skeleton will not be considered; at 1, the mesh surface will not be considered.
		double ClusterSkeletonDistanceWt = .5;

		// If true, prevent collapses that would introduce intersections between skeleton edges and the mesh
		bool bPreventEdgeSurfaceIntersections = true;



		/// Advanced settings

		// Regularization weight encouraging skeleton clusters to keep their original positions and radii.
		double ClusterRegularizeWeight = FMathd::ZeroTolerance;

		// Whether to split edges attached to skinny triangles, as a pre-process before simplifying, which can help add useful degrees of freedom to the simplifier
		bool bSplitThinTriEdges = true;
		// Minimum angle of triangle opposite an edge at which to potentially split the edge
		double SplitThinTriEdgeAngleThresholdDeg = 120;
		// When re-clustering for split edges, weight to use for the position error term. Equivalent in meaning to PosErrorWt in FSkeletonViaSampling.
		double SplitThinTriEdgePosErrorWt = .2;


		template<typename MeshType>
		GEOMETRYALGORITHMS_API void Simplify(FMedialSkeleton& Skeleton, const TMeshAABBTree3<MeshType>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn);

		template<typename MeshType>
		GEOMETRYALGORITHMS_API void Simplify(FMedialSkeleton& Skeleton, const TMeshAABBTree3<MeshType>& MeshBVH, TFunctionRef<bool(int32)> OptionalBoundaryVertexFilter = [](int32)->bool { return false; }, double ImplicitVertexNormalWeldThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER);

		void Simplify(FMedialSkeleton& Skeleton, const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, double ImplicitVertexNormalWeldThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER)
		{
			const FDynamicMesh3& Mesh = *MeshBVH.GetMesh();
			Simplify(Skeleton, MeshBVH, [&Mesh](int32 VID)->bool {return Mesh.IsBoundaryVertex(VID);}, ImplicitVertexNormalWeldThreshold);
		}
	};


	// Implementation/adaptation of "Dynamic Skeletonization via Variational Medial Skeletal Sampling" by Huang et al
	// with some surface-focused additions -- optionally uses raycasts to guide refinement, and fast winding to guide medial-sphere projection
	struct FSkeletonViaSampling
	{
		/// Main user parameters

		// Stop at this sphere count
		int32 MaxSpheres = 10;
		// Do not split a cluster if its max error is below this threshold
		double MinClusterErrorToSplit = .1;
		// Weight for the position error term. Relative to plane error term, so normal and position error weights sum to 1. Should be in the (0,1] range.
		double PosErrorWt = .2;
		// Whether to test medial skeleton edges for intersection w/ the input surface, and try to refine where these intersections are found
		bool bSplitClustersIfEdgesIntersectSurface = true;
		// Factor to scale vertex error near skeleton-edge/input-surface intersections
		double ErrorScaleNearEdgeSurfaceIntersection = 10.;
		// Clusters with fewer vertices than this will be discarded
		int32 MinClusterSizeToKeep = 4;

		// Settings for the medial ball shrinking, used to initialize medial spheres and to project spheres into the surface
		FShrinkMedialBall ShrinkMedial;

		// Optional simplifier, to be run after generating the initial skeleton. Saves re-computing some values compared to running the simplifier separately after.
		TOptional<FSkeletonSimplifier> Simplifier;

		/// Additional parameters -- most users should not need to change these

		// Maximum cluster splits to apply per iteration (if > 0) --
		// Setting this to a low value can force the algorithm to target higher error more aggressively, at the cost of 
		// requiring many more full iterations.
		// Prefer using MaxSplitFractionPerIteration and MinFractionOfMaxErrorToSplit instead, which scale better with the number of clusters.
		int32 MaxSplitPerIteration = -1;
		// Maximum fraction of current cluster count to split per iteration (if > 0) --
		// Functionally similar to MaxSplitPerIteration but scales better with larger cluster counts
		double MaxSplitFractionPerIteration = .2;
		// Minimum vertex error to allow a cluster split, as fraction of current maximum vertex error --
		// This helps to avoid splitting relatively low-error regions when there is significantly-higher error elsewhere since
		// the higher error region will often split into multiple clusters that should all be split before the low error region
		double MinFractionOfMaxErrorToSplit = .75;

		// Number of full passes (top-level skeleton building iterations) to keep optimizing spheres after the cluster vertex assignments have changed
		// This allows us to 'sleep' the spheres for clusters where vertices have not changed.
		// Note: Should generally be at least 1 so that at least one optimization pass is applied.
		// Higher values will allow for the optimization to smooth out the result of the sphere projection,
		// (since projection happens separately after the optimization) and potentially amortize gauss-newton iterations
		// over multiple passes -- i.e., one could try raising this value and lowering MaxGaussNewtonIterations.
		int8 OptimizeSpheresForIterationsPostClusterChange = 2;
		// Maximum number of gauss-newton iterations to apply to improve sphere fit
		int32 MaxGaussNewtonIterations = 10;
		// Stop optimization if the norm of the change in sphere parameters is less than this
		double GaussNewtonConvergenceThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER;
		// Minimum number of vertices in a cluster below which we will not optimize (because the optimization may not capture a good medial sphere if it has too few samples)
		int32 MinClusterSizeToOptimize = 4;
		// Number of clusters at which to switch to local cluster assignment updates, rather than global searches. If < 0, always use global search.
		int32 UseLocalClusterVertexReassignmentAtClusterCount = 100;

		
		/**
		 * Compute mesh skeleton
		 * Current Supported Mesh Types: FDynamicMesh3, FTriangleMeshAdapterd
		 *
		 * @param MeshBVH BVH of the surface
		 * @param GetVertexNormal Helper to provide per-vertex surface normals
		 * @param VIDtoWeldVIDFn Mapping from vertex ID to implicit 'welded' vertex ID, to handle cases where multiple vertices should be treated as one
		 * @param MeshFWTree Optional fast winding tree; if provided will be used for more-robust medial sphere projection
		 */
		template<typename MeshType>
		GEOMETRYALGORITHMS_API FMedialSkeleton ComputeSkeleton(const TMeshAABBTree3<MeshType>& MeshBVH, TFunctionRef<FVector3d(int32)> GetVertexNormal, TFunctionRef<int32(int32)> VIDtoWeldVIDFn, const TFastWindingTree<MeshType>* MeshFWTree = nullptr);

		/**
		 * Compute mesh skeleton
		 * This version will compute vertex normals for you, and will optionally consider near-coincident vertices as welded
		 * Current Supported Mesh Types: FDynamicMesh3, FTriangleMeshAdapterd
		 *
		 * @param MeshBVH BVH of the surface
		 * @param OptionalWeldVertexFilter Optionally filter what vertices are candidates to be welded. Return true if the vertex is a candidate to weld.
		 * @param ImplicitVertexNormalWeldThreshold If > 0, when computing vertex normals will consider coincident boundary vertices as implicitly welded if they're closer than this threshold distance
		 * @param MeshFWTree Optional fast winding tree; if provided will be used for more-robust medial sphere projection
		 */
		template<typename MeshType>
		GEOMETRYALGORITHMS_API FMedialSkeleton ComputeSkeleton(const TMeshAABBTree3<MeshType>& MeshBVH, TFunctionRef<bool(int32)> OptionalWeldVertexFilter = [](int32)->bool {return true;}, double ImplicitVertexNormalWeldThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER, const TFastWindingTree<MeshType>* MeshFWTree = nullptr);

		/**
		 * Compute mesh skeleton
		 * This version will compute vertex normals for you, and will optionally consider near-coincident boundary vertices as welded
		 *
		 * @param MeshBVH BVH of the surface
		 * @param ImplicitVertexNormalWeldThreshold If > 0, when computing vertex normals will consider coincident boundary vertices as implicitly welded if they're closer than this threshold distance
		 * @param MeshFWTree Optional fast winding tree; if provided will be used for more-robust medial sphere projection
		 */
		FMedialSkeleton ComputeSkeleton(const TMeshAABBTree3<FDynamicMesh3>& MeshBVH, double ImplicitVertexNormalWeldThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER, const TFastWindingTree<FDynamicMesh3>* MeshFWTree = nullptr)
		{
			const FDynamicMesh3& Mesh = *MeshBVH.GetMesh();
			return ComputeSkeleton(MeshBVH, [&Mesh](int32 VID)->bool {return Mesh.IsBoundaryVertex(VID);}, ImplicitVertexNormalWeldThreshold, MeshFWTree);
		}

	private:

		// Helper to remove clusters smaller than MinClusterSizeToKeep
		void RemoveSmallClusters(FMedialSkeleton& Result, TArray<int32>& InOutClusterSize, 
			TArray<int32>& OutRemapClusters, TArray<std::atomic<int8>>& OutUpdatedClusters,
			TFunctionRef<void(int32)> AssignVertexCluster);


	};
}


} // end namespace UE::Geometry

