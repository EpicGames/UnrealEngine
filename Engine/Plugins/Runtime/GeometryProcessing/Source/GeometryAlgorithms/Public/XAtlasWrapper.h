// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace XAtlasWrapper
{
	typedef void (*XAtlasParameterizeFunc)(const float* Positions, float* TexCoords, uint32_t VertexCount, const uint32_t* Indices, uint32_t IndexCount);

	struct XAtlasChartOptions
	{
		XAtlasParameterizeFunc ParamFunc = nullptr;

		float MaxChartArea = 0.0f; // Don't grow charts to be larger than this. 0 means no limit.
		float MaxBoundaryLength = 0.0f; // Don't grow charts to have a longer boundary than this. 0 means no limit.

		// Weights determine chart growth. Higher weights mean higher cost for that metric.
		float NormalDeviationWeight = 2.0f; // Angle between face and average chart normal.
		float RoundnessWeight = 0.01f;
		float StraightnessWeight = 6.0f;
		float NormalSeamWeight = 4.0f; // If > 1000, normal seams are fully respected.
		float TextureSeamWeight = 0.5f;

		float MaxCost = 2.0f; // If total of all metrics * weights > maxCost, don't grow chart. Lower values result in more charts.
		uint32_t MaxIterations = 1; // Number of iterations of the chart growing and seeding phases. Higher values result in better charts.

		bool bUseInputMeshUvs = false; // Use MeshDecl::vertexUvData for charts.
		bool bFixWinding = false; // Enforce consistent texture coordinate winding.
	};

	struct XAtlasPackOptions
	{
		// Charts larger than this will be scaled down. 0 means no limit.
		uint32_t MaxChartSize = 0;

		// Number of pixels to pad charts with.
		uint32_t Padding = 0;

		// Unit to texel scale. e.g. a 1x1 quad with texelsPerUnit of 32 will take up approximately 32x32 texels in the atlas.
		// If 0, an estimated value will be calculated to approximately match the given resolution.
		// If resolution is also 0, the estimated value will approximately match a 1024x1024 atlas.
		float TexelsPerUnit = 0.0f;

		// If 0, generate a single atlas with texelsPerUnit determining the final resolution.
		// If not 0, and texelsPerUnit is not 0, generate one or more atlases with that exact resolution.
		// If not 0, and texelsPerUnit is 0, texelsPerUnit is estimated to approximately match the resolution.
		uint32_t Resolution = 0;

		// Leave space around charts for texels that would be sampled by bilinear filtering.
		bool bBilinear = true;

		// Align charts to 4x4 blocks. Also improves packing speed, since there are fewer possible chart locations to consider.
		bool bBlockAlign = false;

		// Slower, but gives the best result. If false, use random chart placement.
		bool bBruteForce = false;

		// Create Atlas::image
		bool bCreateImage = false;

		// Rotate charts to the axis of their convex hull.
		bool bRotateChartsToAxis = true;

		// Rotate charts to improve packing.
		bool bRotateCharts = true;
	};


	bool GEOMETRYALGORITHMS_API ComputeUVs(const TArray<int32>& IndexBuffer,
										   const TArray<FVector3f>& VertexBuffer,
										   const XAtlasChartOptions& ChartOptions,
										   const XAtlasPackOptions& PackOptions,
										   TArray<FVector2D>& UVVertexBuffer,
										   TArray<int32>& UVIndexBuffer,
										   TArray<int32>& VertexRemapArray);

}

