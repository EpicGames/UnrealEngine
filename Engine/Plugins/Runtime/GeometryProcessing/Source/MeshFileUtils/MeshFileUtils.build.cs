// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshFileUtils : ModuleRules
{	
	public MeshFileUtils(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// Note: The module purposefully doesn't have a dependency on CoreUObject.
		// If possible, we would like avoid having UObjects in GeometryProcessing
		// modules to keep the door open for writing standalone command-line programs
		// (which won't have UObject garbage collection).
        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"GeometryCore"
			}
		);
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
