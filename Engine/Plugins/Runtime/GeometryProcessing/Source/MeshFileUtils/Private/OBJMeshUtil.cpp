// Copyright Epic Games, Inc. All Rights Reserved.

#include "OBJMeshUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include <fstream>
#include <string>
#include <sstream>
#include <iomanip>
#include <iostream>

using namespace UE::Geometry;

namespace UE::Private::MeshFileUtilsLocals
{

/**
 * Write obj element: vertex, UV, normal or polygonal face (with only vertex indices).
 * 
 * @param Out    File stream we are writing into.
 * @param Token  One of "v", "vn", "vt", "f".
 * @param Value  Vector type containing the data to write. ith entry will be accessed with the [] operator.
 * 
 * @note Assumes Out is open.
 */
template<int Dim, class VectorType> 
void ObjWriteElement(std::ofstream& Out, const char* Token, const VectorType& Value) 
{
	Out << Token << ' ';
	for (int32 Idx = 0; Idx < Dim; ++Idx) 
	{
		Out << Value[Idx];
		if (Idx < Dim - 1) 
		{
			Out << ' ';
		}
	}
	Out << '\n';
}

void ObjWriteString(std::ofstream& Out, const FString& Line) 
{ 
	Out << TCHAR_TO_ANSI(*Line);
}

} // end namespace MeshFileUtilsLocals

namespace UE::MeshFileUtils
{
	ELoadOBJStatus ReadOBJ(std::istream& FileStream, FDynamicMesh3& Mesh, const FLoadOBJSettings& Settings)
	{
		auto ExtractTripletFromGroup = [](std::stringstream& InLineStream,
			bool bInParseNormalsAndTextures,
			int& OutVertexID,
			int& OutTextureID,
			int& OutNormalID)
		{
			std::string Blob;

			OutVertexID = -1;
			OutTextureID = -1;
			OutNormalID = -1;

			if (InLineStream >> Blob)
			{
				const char* BlobCStr = Blob.c_str();
				const char** Token = &BlobCStr;

				OutVertexID = FCStringAnsi::Atoi((*Token));
				(*Token) += FCStringAnsi::Strcspn((*Token), "/");
				if ((*Token)[0] != '/')
				{
					return true;
				}

				if (!bInParseNormalsAndTextures)
				{
					return true;
				}

				(*Token)++;

				// v//n case
				if ((*Token)[0] == '/')
				{
					(*Token)++;
					OutNormalID = FCStringAnsi::Atoi((*Token));
					(*Token) += FCStringAnsi::Strcspn((*Token), "/");
					return true;
				}

				// v/t/n or v/t case
				OutTextureID = FCStringAnsi::Atoi((*Token));
				(*Token) += FCStringAnsi::Strcspn((*Token), "/");
				if ((*Token)[0] != '/')
				{
					return true;
				}

				// v/t/n case
				(*Token)++;  // skip '/'
				OutNormalID = FCStringAnsi::Atoi((*Token));
				(*Token) += FCStringAnsi::Strcspn((*Token), "/");

				return true;
			}

			return false;
		};

		Mesh.Clear();

		struct FTriToAdd
		{
			FIndex3i Tri, UV, Normal;
			bool bHasUV = false;
			bool bHasNormal = false;
			int32 GroupID = 0;
			FTriToAdd() = default;
			FTriToAdd(FIndex3i Tri, int32 GroupID) : Tri(Tri), GroupID(GroupID)
			{
			}
			void SetUV(FIndex3i InUV)
			{
				UV = InUV;
				bHasUV = true;
			}
			void SetNormal(FIndex3i InNormal)
			{
				Normal = InNormal;
				bHasNormal = true;
			}
		};

		// buffer the triangles in case the faces precede the vertices they reference in the file
		TArray<FTriToAdd> Triangles;

		const bool bLoadNormalsOrUVs = Settings.bLoadNormals || Settings.bLoadUVs;

		if (bLoadNormalsOrUVs)
		{
			Mesh.EnableAttributes();
		}

		FDynamicMeshNormalOverlay* Normals = (Settings.bLoadNormals) ? Mesh.Attributes()->PrimaryNormals() : nullptr;
		FDynamicMeshUVOverlay* UVs = (Settings.bLoadUVs) ? Mesh.Attributes()->PrimaryUV() : nullptr;


		int32 GroupID = 1;
		bool bHasGroups = false;
		std::string Line;
		while (std::getline(FileStream, Line))
		{
			if (Line.empty())
			{
				continue;
			}

			std::stringstream LineStream(Line);
			std::string Command;

			if (LineStream >> Command)
			{
				if (Command.empty() || Command[0] == '#') // skip comment lines
				{
					continue;
				}

				if (Command == "g" || Command == "o")
				{
					bHasGroups = true;
					GroupID++;
				}
				else if (Command == "v") // vertex
				{
					FVector3d Vert;
					LineStream >> Vert.X >> Vert.Y >> Vert.Z;
					Mesh.AppendVertex(Vert);
				}
				else if (Settings.bLoadNormals && Command == "vn")
				{
					FVector3f Normal;
					LineStream >> Normal.X >> Normal.Y >> Normal.Z;
					Normalize(Normal);
					Normals->AppendElement(Normal);
				}
				else if (Settings.bLoadUVs && Command == "vt")
				{
					FVector2f UV = FVector2f::Zero(); // v is optional so default to zero
					LineStream >> UV.X >> UV.Y;
					UVs->AppendElement(UV);
				}
				else if (Command == "f")
				{
					int V0, VN_1, VN; // vertices 0, N-1, and N (the last two are reused to make a triangle fan out of any non-triangle faces)
					int T0, TN_1, TN; // UVs
					int N0, NN_1, NN; // normals

					if (!ExtractTripletFromGroup(LineStream, bLoadNormalsOrUVs, V0, T0, N0))
					{
						continue;
					}
					if (!ExtractTripletFromGroup(LineStream, bLoadNormalsOrUVs, VN_1, TN_1, NN_1))
					{
						continue;
					}
					while (ExtractTripletFromGroup(LineStream, bLoadNormalsOrUVs, VN, TN, NN))
					{
						FTriToAdd& ToAdd = Triangles.Emplace_GetRef(FIndex3i(V0 - 1, VN_1 - 1, VN - 1), GroupID);
						if (Normals)
						{
							ToAdd.SetNormal(FIndex3i(N0 - 1, NN_1 - 1, NN - 1));
						}
						if (UVs)
						{
							ToAdd.SetUV(FIndex3i(T0 - 1, TN_1 - 1, TN - 1));
						}

						VN_1 = VN;
						TN_1 = TN;
						NN_1 = NN;
					}
				}
			}
		}

		if (bHasGroups)
		{
			Mesh.EnableTriangleGroups();
		}
		for (const FTriToAdd& ToAdd : Triangles)
		{
			int32 TID = Mesh.AppendTriangle(ToAdd.Tri, ToAdd.GroupID);
			if (Settings.bAddSeparatedTriForNonManifold && TID == FDynamicMesh3::NonManifoldID)
			{
				FIndex3i DupeVertsTri;
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					DupeVertsTri[SubIdx] = Mesh.AppendVertex(Mesh.GetVertex(ToAdd.Tri[SubIdx]));
				}
				TID = Mesh.AppendTriangle(DupeVertsTri, ToAdd.GroupID);
			}
			if (TID >= 0)
			{
				if (Normals && ToAdd.bHasNormal)
				{
					Normals->SetTriangle(TID, ToAdd.Normal);
				}
				if (UVs && ToAdd.bHasUV)
				{
					UVs->SetTriangle(TID, ToAdd.UV);
				}
			}
		}

		if (Settings.bReverseOrientation)
		{
			Mesh.ReverseOrientation();
		}

		return ELoadOBJStatus::Success;
	}

	ELoadOBJStatus LoadOBJ(const char* Path, FDynamicMesh3& Mesh, const FLoadOBJSettings& Settings)
	{
		std::ifstream FileStream(Path);
		if (!FileStream)
		{
			return ELoadOBJStatus::InvalidPath;
		}

		ELoadOBJStatus Status = ReadOBJ(FileStream, Mesh, Settings);
		FileStream.close();

		return Status;
	}

	FDynamicMesh3 LoadOBJChecked(const char* Path, const FLoadOBJSettings& Settings)
	{
		FDynamicMesh3 Mesh;
		ELoadOBJStatus Status = LoadOBJ(Path, Mesh, Settings);
		check(Status == ELoadOBJStatus::Success);
		return Mesh;
	}

	bool WriteOBJ(const char* Path, const FDynamicMesh3& InMesh, const FWriteOBJSettings& Settings)
	{
		// We compact the mesh to make sure the order that we write the vertex/normal/uv indices into obj is 
		// consistent with the storage order 
		FDynamicMesh3 Mesh;
		Mesh.CompactCopy(InMesh);

		if (Settings.bReverseOrientation)
		{
			Mesh.ReverseOrientation();
		}

		std::ofstream FileStream;
		FileStream.precision(std::numeric_limits<double>::digits10);
		FileStream.open(Path, std::ofstream::out | std::ofstream::trunc);
		if (!ensure(FileStream))
		{
			return false;
		}

		bool bHasVertexNormals = Settings.bWritePerVertexValues && Mesh.HasVertexNormals();
		bool bHasVertexUVs = Settings.bWritePerVertexValues && Mesh.HasVertexUVs();

		for (int32 VID = 0; VID < Mesh.VertexCount(); ++VID)
		{
			check(Mesh.IsVertex(VID)); // mesh is not compact

			FVector3d Pos = Mesh.GetVertex(VID);
			if (!Settings.bWritePerVertexColors || !Mesh.HasVertexColors())
			{
				UE::Private::MeshFileUtilsLocals::ObjWriteElement<3>(FileStream, "v", Pos);
			}
			else
			{
				FVector3f Color = Mesh.GetVertexColor(VID);
				FileStream << "v " << Pos[0] << " " << Pos[1] << " " << Pos[2] << " " << Color[0] << " " << Color[1] << " " << Color[2] << "\n";
			}

			if (bHasVertexNormals)
			{
				FVector3f Normal = Mesh.GetVertexNormal(VID);
				UE::Private::MeshFileUtilsLocals::ObjWriteElement<3>(FileStream, "vn", Normal);
			}

			if (bHasVertexUVs)
			{
				FVector2f UV = Mesh.GetVertexUV(VID);
				UE::Private::MeshFileUtilsLocals::ObjWriteElement<2>(FileStream, "vt", UV);
			}
		}

		const FDynamicMeshUVOverlay* UVs = nullptr;
		const FDynamicMeshNormalOverlay* Normals = nullptr;

		if (Settings.bWritePerVertexValues == false && Mesh.Attributes())
		{
			UVs = Mesh.Attributes()->PrimaryUV();
			if (UVs)
			{
				for (int32 UI = 0; UI < UVs->ElementCount(); ++UI)
				{
					check(UVs->IsElement(UI))
						FVector2f UV = UVs->GetElement(UI);
					UE::Private::MeshFileUtilsLocals::ObjWriteElement<2>(FileStream, "vt", UV);
				}
			}

			Normals = Mesh.Attributes()->PrimaryNormals();
			if (Normals)
			{
				for (int32 NI = 0; NI < Normals->ElementCount(); ++NI)
				{
					check(Normals->IsElement(NI));
					FVector3f Normal = Normals->GetElement(NI);
					UE::Private::MeshFileUtilsLocals::ObjWriteElement<3>(FileStream, "vn", Normal);
				}
			}
		}

		for (int32 TID = 0; TID < Mesh.TriangleCount(); ++TID)
		{
			check(Mesh.IsTriangle(TID));

			FIndex3i TriVertices = Mesh.GetTriangle(TID);

			if (Settings.bWritePerVertexValues)
			{
				if (bHasVertexNormals == false && bHasVertexUVs == false)
				{
					UE::Private::MeshFileUtilsLocals::ObjWriteString(FileStream, FString::Printf(TEXT("f %d %d %d\n"), TriVertices.A + 1, TriVertices.B + 1, TriVertices.C + 1));
				}
				else if (bHasVertexNormals == true && bHasVertexUVs == false)
				{
					UE::Private::MeshFileUtilsLocals::ObjWriteString(FileStream, FString::Printf(TEXT("f %d//%d %d//%d %d//%d\n"),
						TriVertices.A + 1, TriVertices.A + 1,
						TriVertices.B + 1, TriVertices.B + 1,
						TriVertices.C + 1, TriVertices.C + 1));
				}
				else if (bHasVertexNormals == false && bHasVertexUVs == true)
				{
					UE::Private::MeshFileUtilsLocals::ObjWriteString(FileStream, FString::Printf(TEXT("f %d/%d %d/%d %d/%d\n"),
						TriVertices.A + 1, TriVertices.A + 1,
						TriVertices.B + 1, TriVertices.B + 1,
						TriVertices.C + 1, TriVertices.C + 1));
				}
				else
				{
					UE::Private::MeshFileUtilsLocals::ObjWriteString(FileStream, FString::Printf(TEXT("f %d/%d/%d %d/%d/%d %d/%d/%d\n"),
						TriVertices.A + 1, TriVertices.A + 1, TriVertices.A + 1,
						TriVertices.B + 1, TriVertices.B + 1, TriVertices.B + 1,
						TriVertices.C + 1, TriVertices.C + 1, TriVertices.C + 1));
				}
			}
			else
			{
				bool bHaveUV = UVs != nullptr && UVs->IsSetTriangle(TID);
				bool bHaveNormal = Normals != nullptr && Normals->IsSetTriangle(TID);

				FIndex3i TriUVs = bHaveUV ? UVs->GetTriangle(TID) : FIndex3i::Invalid();
				FIndex3i TriNormals = bHaveNormal ? Normals->GetTriangle(TID) : FIndex3i::Invalid();

				if (bHaveUV && bHaveNormal)
				{
					UE::Private::MeshFileUtilsLocals::ObjWriteString(FileStream, FString::Printf(TEXT("f %d/%d/%d %d/%d/%d %d/%d/%d\n"),
						TriVertices.A + 1, TriUVs.A + 1, TriNormals.A + 1,
						TriVertices.B + 1, TriUVs.B + 1, TriNormals.B + 1,
						TriVertices.C + 1, TriUVs.C + 1, TriNormals.C + 1));
				}
				else if (bHaveUV)
				{
					UE::Private::MeshFileUtilsLocals::ObjWriteString(FileStream, FString::Printf(TEXT("f %d/%d %d/%d %d/%d\n"),
						TriVertices.A + 1, TriUVs.A + 1,
						TriVertices.B + 1, TriUVs.B + 1,
						TriVertices.C + 1, TriUVs.C + 1));
				}
				else if (bHaveNormal)
				{
					UE::Private::MeshFileUtilsLocals::ObjWriteString(FileStream, FString::Printf(TEXT("f %d//%d %d//%d %d//%d\n"),
						TriVertices.A + 1, TriNormals.A + 1,
						TriVertices.B + 1, TriNormals.B + 1,
						TriVertices.C + 1, TriNormals.C + 1));
				}
				else
				{
					UE::Private::MeshFileUtilsLocals::ObjWriteString(FileStream, FString::Printf(TEXT("f %d %d %d\n"), TriVertices.A + 1, TriVertices.B + 1, TriVertices.C + 1));
				}
			}
		}

		FileStream.close();

		return true;
	}

}

