// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"

// Developer utilities for managing Wavefront OBJ format mesh files
// Intended for debugging and for use in tests and utilities; may not support every OBJ file format feature
namespace UE::MeshFileUtils
{
	enum class ELoadOBJStatus
	{
		InvalidPath,
		Success
	};

	struct FLoadOBJSettings
	{

		// Dynamic mesh doesn't support non-manifold edges (edges with more than two triangles), but OBJ does
		// If this setting is true, non-manifold triangles will be added as fully separated triangles;
		// otherwise, such triangles will be skipped entirely
		bool bAddSeparatedTriForNonManifold = true;

		// If true, reverses the orientation of the faces
		bool bReverseOrientation = true;

		// If true and obj contains normal information will compute normal overlay
		bool bLoadNormals = false;

		// If true and obj contains UVs information will compute UV overlay.
		bool bLoadUVs = false;
	};

	struct FWriteOBJSettings
	{
		// If true reverses the orientation of the faces
		bool bReverseOrientation = true;

		// If true will attempt to write the per-vertex normals and UVs to the OBJ instead of the per-element values
		bool bWritePerVertexValues = true;

		// Whether to write per-vertex colors (when available)
		bool bWritePerVertexColors = false;
	};

	// Attempt to load an OBJ file into a Dynamic Mesh
	MESHFILEUTILS_API ELoadOBJStatus LoadOBJ(const char* Path, UE::Geometry::FDynamicMesh3& Mesh, const FLoadOBJSettings& Settings = FLoadOBJSettings());
	// Load an OBJ file into a Dynamic Mesh, or fail a check() if it cannot be loaded
	MESHFILEUTILS_API UE::Geometry::FDynamicMesh3 LoadOBJChecked(const char* Path, const FLoadOBJSettings& Settings = FLoadOBJSettings());
	// Write an OBJ file to the target path; returns true on success
	MESHFILEUTILS_API bool WriteOBJ(const char* Path, const UE::Geometry::FDynamicMesh3& InMesh, const FWriteOBJSettings& Settings = FWriteOBJSettings());
}

